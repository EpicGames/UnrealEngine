// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "ControlPointMeshActor.generated.h"

/**
 * ControlPointMeshActor is an actor with a ControlPointMeshComponent.
 *
 * @see UControlPointMeshComponent
 */
UCLASS(NotPlaceable, hideCategories = (Input), showCategories = ("Input|MouseInput", "Input|TouchInput"), ConversionRoot, ComponentWrapperClass, meta = (ChildCanTick), MinimalAPI)
class AControlPointMeshActor : public AActor
{
	GENERATED_UCLASS_BODY()

private:
	UPROPERTY(Category = ControlPointMeshActor, VisibleAnywhere, BlueprintReadOnly, meta = (ExposeFunctionCategories = "Mesh,Rendering,Physics,Components|StaticMesh,Components|ControlPointMesh", AllowPrivateAccess = "true"))
	TObjectPtr<class UControlPointMeshComponent> ControlPointMeshComponent;

public:

	/** Function to change mobility type */
	LANDSCAPE_API void SetMobility(EComponentMobility::Type InMobility);

#if WITH_EDITOR
	//~ Begin AActor Interface
	LANDSCAPE_API virtual void CheckForErrors() override;
	LANDSCAPE_API virtual bool GetReferencedContentObjects(TArray<UObject*>& Objects) const override;
	//~ End AActor Interface
#endif // WITH_EDITOR	

protected:
	//~ Begin UObject Interface.
	LANDSCAPE_API virtual FString GetDetailedInfoInternal() const override;
	//~ End UObject Interface.

public:
	/** Returns ControlPointMeshComponent subobject **/
	LANDSCAPE_API class UControlPointMeshComponent* GetControlPointMeshComponent() const;
};



