// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/StaticMeshComponent.h"
#include "ControlPointMeshComponent.generated.h"

UCLASS(MinimalAPI)
class UControlPointMeshComponent : public UStaticMeshComponent
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY(transient)
	uint32 bSelected:1;

	UPROPERTY(EditAnywhere, Category = HLOD, AdvancedDisplay)
	TObjectPtr<UHLODLayer> HLODLayerOverrideForSplinePartitioning;
#endif

	/** 
	 * The max draw distance to use in the main pass when also rendering to a runtime virtual texture. 
	 * This is only exposed to the user through the same setting on ULandscapeSplineControlPoint. 
	 */
	UPROPERTY()
	float VirtualTextureMainPassMaxDrawDistance = 0.f;

	//Begin UPrimitiveComponent Interface
#if WITH_EDITOR
	virtual bool ShouldRenderSelected() const override
	{
		return Super::ShouldRenderSelected() || bSelected;
	}
	virtual bool IsEditorOnly() const override;
#endif
	virtual float GetVirtualTextureMainPassMaxDrawDistance() const override { return VirtualTextureMainPassMaxDrawDistance; }
	//End UPrimitiveComponent Interface

#if WITH_EDITOR
	LANDSCAPE_API bool IsRelevantForSplinePartitioning() const;
	LANDSCAPE_API UHLODLayer* GetHLODLayerForSplinePartitioning() const;
#endif
};
