// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"
#include "ILandscapeSplineInterface.generated.h"

UINTERFACE(MinimalAPI)
class ULandscapeSplineInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class ULandscapeSplinesComponent;
class ULandscapeInfo;

class ILandscapeSplineInterface
{
	GENERATED_IINTERFACE_BODY()
public:
	virtual FGuid GetLandscapeGuid() const PURE_VIRTUAL(ILandscapeSplineInterface::GetLandscapeGuid, return FGuid(););
	virtual ULandscapeInfo* GetLandscapeInfo() const PURE_VIRTUAL(ILandscapeSplineInterface::GetLandscapeInfo, return nullptr;);
	virtual FTransform LandscapeActorToWorld() const PURE_VIRTUAL(ILandscapeSplineInterface::LandscapeActorToWorld, return FTransform::Identity;);
	virtual ULandscapeSplinesComponent* GetSplinesComponent() const PURE_VIRTUAL(ILandscapeSplineInterface::GetSplineComponent, return nullptr;);
	virtual void UpdateSharedProperties(ULandscapeInfo* InLandscapeInfo) PURE_VIRTUAL(ILandscapeSplineInterface::UpdateSharedProperties);
	virtual bool IsSplineOwnerValid() const PURE_VIRTUAL(ILandscapeSplineInterface::IsSplineOwnerValid, return true;);
	 
#if WITH_EDITOR
	virtual bool SupportsForeignSplineMesh() const PURE_VIRTUAL(ILandscapeSplineInterface::SupportsForeignSplineMesh, return false;);
	virtual void CreateSplineComponent() PURE_VIRTUAL(ILandscapeSplineInterface::CreateSplineComponent);
	virtual void CreateSplineComponent(const FVector& Scale3D) PURE_VIRTUAL(ILandscapeSplineInterface::CreateSplineComponent);
#endif
};
