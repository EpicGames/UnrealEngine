// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LandscapeComponent.h"
#include "UObject/ObjectMacros.h"
#include "LandscapeProxy.h"
#include "LandscapeBlueprintBrushBase.h"
#include "LandscapeEditTypes.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "LandscapeEditLayerRenderer.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6

#include "Delegates/DelegateCombinations.h"
#include "Templates/SubclassOf.h"
#include "Algo/Transform.h"
#include "LandscapeLayerInfoObject.h"

#include "Landscape.generated.h"


// ----------------------------------------------------------------------------------
// Forward declarations

class FTextureRenderTargetResource;
class ULandscapeComponent;
class ILandscapeEdModeInterface;
class SNotificationItem;
class UStreamableRenderAsset;
class UTextureRenderTarget;
class ULandscapeEditLayerBase;
class FMaterialResource;
struct FLandscapeEditLayerComponentReadbackResult;
struct FLandscapeNotification;
struct FTextureToComponentHelper;
struct FUpdateLayersContentContext;
struct FEditLayersHeightmapMergeParams;
struct FEditLayersWeightmapMergeParams;
struct FOnLandscapeEditLayerDataChangedParams;
enum class ELandscapeNotificationType;

namespace EditLayersHeightmapLocalMerge_RenderThread
{
	struct FMergeInfo;
}

namespace EditLayersWeightmapLocalMerge_RenderThread
{
	struct FMergeInfo;
}

namespace UE::Landscape::EditLayers
{
	struct FMergeRenderParams;
}

#if WITH_EDITOR
extern LANDSCAPE_API TAutoConsoleVariable<int32> CVarLandscapeSplineFalloffModulation;
#endif

UENUM()
enum class ERTDrawingType : uint8
{
	RTAtlas,
	RTAtlasToNonAtlas,
	RTNonAtlasToAtlas,
	RTNonAtlas,
	RTMips
};

UENUM()
enum class EHeightmapRTType : uint8
{
	HeightmapRT_CombinedAtlas,
	HeightmapRT_CombinedNonAtlas,
	HeightmapRT_Scratch1,
	HeightmapRT_Scratch2,
	HeightmapRT_Scratch3,
	// Mips RT
	HeightmapRT_Mip1,
	HeightmapRT_Mip2,
	HeightmapRT_Mip3,
	HeightmapRT_Mip4,
	HeightmapRT_Mip5,
	HeightmapRT_Mip6,
	HeightmapRT_Mip7,
	HeightmapRT_Count
};

UENUM()
enum class EWeightmapRTType : uint8
{
	WeightmapRT_Scratch_RGBA,
	WeightmapRT_Scratch1,
	WeightmapRT_Scratch2,
	WeightmapRT_Scratch3,

	// Mips RT
	WeightmapRT_Mip0,
	WeightmapRT_Mip1,
	WeightmapRT_Mip2,
	WeightmapRT_Mip3,
	WeightmapRT_Mip4,
	WeightmapRT_Mip5,
	WeightmapRT_Mip6,
	WeightmapRT_Mip7,
	
	WeightmapRT_Count
};

#if WITH_EDITOR
enum ELandscapeLayerUpdateMode : uint32;
#endif

USTRUCT()
struct FLandscapeLayerBrush
#if CPP && WITH_EDITOR // UHT doesn't support inheriting from namespaced class
	: public UE::Landscape::EditLayers::IEditLayerRendererProvider
#endif // CPP && WITH_EDITOR
{
	GENERATED_USTRUCT_BODY()

	FLandscapeLayerBrush() = default;

	FLandscapeLayerBrush(ALandscapeBlueprintBrushBase* InBlueprintBrush)
#if WITH_EDITORONLY_DATA
		: BlueprintBrush(InBlueprintBrush)
#endif // WITH_EDITORONLY_DATA
	{}

	virtual ~FLandscapeLayerBrush() = default;

#if WITH_EDITOR
	LANDSCAPE_API ALandscapeBlueprintBrushBase* GetBrush() const;
	bool AffectsHeightmap() const;
	bool AffectsWeightmapLayer(const FName& InWeightmapLayerName) const;
	bool AffectsVisibilityLayer() const;
	void SetOwner(ALandscape* InOwner);

	//~ Begin UE::Landscape::EditLayers::IEditLayerRendererProvider implementation
	LANDSCAPE_API virtual TArray<UE::Landscape::EditLayers::FEditLayerRendererState> GetEditLayerRendererStates(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) override;
	//~ End UE::Landscape::EditLayers::IEditLayerRendererProvider implementation
#endif // WITH_EDITOR

private:

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<ALandscapeBlueprintBrushBase> BlueprintBrush;
#endif // WITH_EDITORONLY_DATA
};

// TODO [jonathan.bard] : deprecate this
UENUM()
enum ELandscapeBlendMode : int
{
	LSBM_AdditiveBlend,
	LSBM_AlphaBlend,
	LSBM_MAX,
};

USTRUCT()
struct FLandscapeLayer
#if CPP && WITH_EDITOR // UHT doesn't support inheriting from namespaced class
	: public UE::Landscape::EditLayers::IEditLayerRendererProvider
#endif // CPP && WITH_EDITOR
{
	GENERATED_USTRUCT_BODY()

	FLandscapeLayer()
	{}

	virtual ~FLandscapeLayer() = default;

#if WITH_EDITOR
	//~ Begin UE::Landscape::EditLayers::IEditLayerRendererProvider implementation
	LANDSCAPE_API virtual TArray<UE::Landscape::EditLayers::FEditLayerRendererState> GetEditLayerRendererStates(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) override;
	//~ End UE::Landscape::EditLayers::IEditLayerRendererProvider implementation
#endif // WITH_EDITOR
public:
	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	FGuid Guid_DEPRECATED = FGuid::NewGuid();

	UPROPERTY()
	FName Name_DEPRECATED = NAME_None;

	UPROPERTY(Transient)
	bool bVisible_DEPRECATED = true;

	UPROPERTY()
	bool bLocked_DEPRECATED = false;

	UPROPERTY()
	float HeightmapAlpha_DEPRECATED = 1.0f;

	UPROPERTY()
	float WeightmapAlpha_DEPRECATED = 1.0f;

	UPROPERTY()
	TEnumAsByte<enum ELandscapeBlendMode> BlendMode_DEPRECATED = ELandscapeBlendMode::LSBM_AdditiveBlend;

	UPROPERTY()
	TArray<FLandscapeLayerBrush> Brushes;

	UPROPERTY()
	TMap<TObjectPtr<ULandscapeLayerInfoObject>, bool> WeightmapLayerAllocationBlend_DEPRECATED; // True -> Substractive, False -> Additive

	UPROPERTY(Instanced)
	TObjectPtr<ULandscapeEditLayerBase> EditLayer;
};


enum class UE_DEPRECATED(5.7, "Edit layers are now fully processed via BatchedMerge") ELandscapeEditLayersMergeMode : uint8
{
	GlobalMerge = 0,
	LocalMerge, 
	BatchedMerge,
	Invalid 
};

// TODO [jonathan.bard] : When global merge is a thing of the past, we should pass FOnRenderBatchTargetGroupDoneParams here, or extract some of its higher-level information, because there
//  is a lot more useful information accessible in there (components being rendered, target layers, validity render targets, etc.)
struct FOnLandscapeEditLayersMergedParams
{
	FOnLandscapeEditLayersMergedParams(UTextureRenderTarget* InRenderTarget, const FIntPoint& InRenderAreaResolution, bool bInIsHeightmapMerge)
		: RenderTarget(InRenderTarget)
		, RenderAreaResolution(InRenderAreaResolution)
		, bIsHeightmapMerge(bInIsHeightmapMerge)
	{}

	/**
	* Render target of the section of landscape that was rendered (important note: the render target's resolution can be larger than the actual landscape resolution, so RenderAreaResolution must be used.
	* In the case of weightmaps, the render target will actually be a UTextureRenderTarget2DArray
	*/
	UTextureRenderTarget* RenderTarget = nullptr;

	/** Actual resolution of this render : render targets are usually sized larger than the effective resolution at this point, so this needs to be used instead.
	 *  Note : it's the true resolution of the landscape : at this point, there are no duplicate vertices in the render target. */
	FIntPoint RenderAreaResolution = FIntPoint(ForceInit);

	bool bIsHeightmapMerge = false;
};

struct FLandscapeEditLayerRenderCommonParams
{
	// Region to render, in Landscape coordinates
	FIntRect Bounds;

	// Which edit layers to include.  Order matches ALandscape::GetEditLayers().
	TBitArray<> ActiveEditLayers;

	bool bRenderBrushes = true;
};

// Parameters to describe a subset of a landscape heightmap to render.  Can select a subset of edit layers and a subregion of the landscape coordinates.
struct FLandscapeEditLayerRenderHeightParams : public FLandscapeEditLayerRenderCommonParams
{
	// Provide either a cpu buffer or render target to receive the data.  Buffer should be allocated large enough to receive the samples in Bounds.
	TArrayView<uint16> CpuResult;
	UTextureRenderTarget2D* RTResult = nullptr;
};

// Parameters to describe a subset of a landscape weightmaps to render.  Can select a subset of edit layers and a subregion of the landscape coordinates, for a list of weightmaps.
struct FLandscapeEditLayerRenderWeightParams : public FLandscapeEditLayerRenderCommonParams
{
	TArrayView<FName> WeightmapNames;

	// Provide either a cpu buffer or render target to receive the data.  Buffer should be allocated large enough to receive the samples in Bounds.
	TArrayView<uint8> CpuResult;
	UTextureRenderTarget2D* RTResult = nullptr;
};

UCLASS(MinimalAPI, showcategories=(Display, Movement, Collision, Lighting, LOD, Input), hidecategories=(Mobility))
class ALandscape : public ALandscapeProxy
#if CPP && WITH_EDITOR // UHT doesn't support inheriting from namespaced class
	, public UE::Landscape::EditLayers::IEditLayerRendererProvider
#endif // CPP && WITH_EDITOR
{
	GENERATED_BODY()

	friend class FLandscapeConfigHelper; // For copying/manipulating internal data

public:
	ALandscape(const FObjectInitializer& ObjectInitializer);

	//~ Begin ALandscapeProxy Interface
	LANDSCAPE_API virtual ALandscape* GetLandscapeActor() override;
	LANDSCAPE_API virtual const ALandscape* GetLandscapeActor() const override;
	//~ End ALandscapeProxy Interface

#if WITH_EDITOR
	static LANDSCAPE_API FName AffectsLandscapeActorDescProperty;

	LANDSCAPE_API bool HasAllComponent(); // determine all component is in this actor
	
	// Include Components with overlapped vertices
	// X2/Y2 Coordinates are "inclusive" max values
	LANDSCAPE_API static void CalcComponentIndicesOverlap(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, const int32 ComponentSizeQuads, 
		int32& ComponentIndexX1, int32& ComponentIndexY1, int32& ComponentIndexX2, int32& ComponentIndexY2);

	// Exclude Components with overlapped vertices
	// X2/Y2 Coordinates are "inclusive" max values
	LANDSCAPE_API static void CalcComponentIndicesNoOverlap(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, const int32 ComponentSizeQuads,
		int32& ComponentIndexX1, int32& ComponentIndexY1, int32& ComponentIndexX2, int32& ComponentIndexY2);

	LANDSCAPE_API static void SplitHeightmap(ULandscapeComponent* Comp, ALandscapeProxy* TargetProxy = nullptr, class FMaterialUpdateContext* InOutUpdateContext = nullptr, TArray<class FComponentRecreateRenderStateContext>* InOutRecreateRenderStateContext = nullptr, bool InReregisterComponent = true);
	
	//~ Begin APartitionActor Interface
	virtual bool IsPartitionActorNameAffectedByDataLayers() const { return false; }
	//~ End APartitionActor Interface

	//~ Begin UObject Interface.
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditMove(bool bFinished) override;
	virtual void PostEditUndo() override;
	virtual void UnregisterAllComponents(bool bForReregister = false) override;
	virtual void PostRegisterAllComponents() override;
	virtual void PostActorCreated() override;
	virtual bool ShouldImport(FStringView ActorPropString, bool IsMovingLevel) override;
	virtual void PostEditImport() override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	virtual bool CanDeleteSelectedActor(FText& OutReason) const override;
	virtual bool CanChangeIsSpatiallyLoadedFlag() const override { return false; }
#endif // WITH_EDITOR

	virtual void PostLoad() override;
	//~ End UObject Interface

	/** Computes & returns bounds containing all currently loaded landscape proxies (if any) or this landscape's bounds otherwise */
	LANDSCAPE_API FBox GetLoadedBounds() const;

	LANDSCAPE_API bool IsUpToDate() const;
	LANDSCAPE_API void TickLayers(float DeltaTime);

	LANDSCAPE_API void SetLODGroupKey(uint32 InLODGroupKey);
	LANDSCAPE_API uint32 GetLODGroupKey();

	UFUNCTION(BlueprintCallable, Category = "Landscape")
	bool GetGrassEnabled() const { return bGrassEnabled; }

	UFUNCTION(BlueprintCallable, Category = "Landscape")
	LANDSCAPE_API void SetGrassEnabled(bool bEnabled, bool bFlushGrassMaps = true);

	/**
	* Render the final heightmap in the requested top-down window as one -atlased- texture in the provided render target 2D
	*  Can be called at runtime.
	* @param InWorldTransform World transform of the area where the texture should be rendered
	* @param InExtents Extents of the area where the texture should be rendered (local to InWorldTransform). If size is zero, then the entire loaded landscape will be exported.
	* @param OutRenderTarget Render target in which the texture will be rendered. The size/format of the render target will be respected.
	* @return false in case of failure (e.g. invalid inputs, incompatible render target format...)
	*/
	UFUNCTION(BlueprintCallable, Category = "Landscape|Runtime")
	LANDSCAPE_API bool RenderHeightmap(FTransform InWorldTransform, FBox2D InExtents, UTextureRenderTarget2D* OutRenderTarget);

	/**
	* Render the final weightmap for the requested layer, in the requested top-down window, as one -atlased- texture in the provided render target 2D
	*  Can be called at runtime.
	* @param InWorldTransform World transform of the area where the texture should be rendered
	* @param InExtents Extents of the area where the texture should be rendered (local to InWorldTransform). If size is zero, then the entire loaded landscape will be exported.
	* @param InWeightmapLayerName Weightmap layer that is being requested to render
	* @param OutRenderTarget Render target in which the texture will be rendered. The size/format of the render target will be respected.
	* @return false in case of failure (e.g. invalid inputs, incompatible render target format...)
	*/
	UFUNCTION(BlueprintCallable, Category = "Landscape|Runtime")
	LANDSCAPE_API bool RenderWeightmap(FTransform InWorldTransform, FBox2D InExtents, FName InWeightmapLayerName, UTextureRenderTarget2D* OutRenderTarget);

	/**
	* Render the final weightmaps for the requested layers, in the requested top-down window, as one -atlased- texture in the provided render target (2D or 2DArray) 
	*  Can be called at runtime.
	* @param InWorldTransform World transform of the area where the texture should be rendered
	* @param InExtents Extents of the area where the texture should be rendered (local to InWorldTransform). If size is zero, then the entire loaded landscape will be exported.
	* @param InWeightmapLayerNames List of weightmap layers that are being requested to render
	* @param OutRenderTarget Render target in which the texture will be rendered. The size/format of the render target will be respected.
	*  - If a UTextureRenderTarget2D is passed, the requested layers will be packed in the RGBA channels in order (up to the number of channels available with the render target's format).
	*  - If a UTextureRenderTarget2DArray is passed, the requested layers will be packed in the RGBA channels of each slice (up to the number of channels * slices available with the render target's format and number of slices).
	* @return false in case of failure (e.g. invalid inputs, incompatible render target format...)
	*/
	UFUNCTION(BlueprintCallable, Category = "Landscape|Runtime")
	LANDSCAPE_API bool RenderWeightmaps(FTransform InWorldTransform, FBox2D InExtents, const TArray<FName>& InWeightmapLayerNames, UTextureRenderTarget* OutRenderTarget);

	/** 
	* Retrieves the names of valid paint layers on this landscape (editor-only : returns nothing at runtime) 
	* @Param bInIncludeVisibilityLayer whether the visibility layer's name should be included in the list or not
	* @return the list of paint layer names
	*/
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category = "Landscape|Editor", meta=(DevelopmentOnly))
	TArray<FName> GetTargetLayerNames(bool bInIncludeVisibilityLayer = false) const;

	/**
	* @return the list of edit layers on this landscape (editor-only : returns nothing at runtime)
	*/
	UFUNCTION(BlueprintCallable, DisplayName = GetEditLayers, Category = "Landscape|Editor", meta = (DevelopmentOnly))
	const TArray<ULandscapeEditLayerBase*> GetEditLayersBP() const;

	/**
	* @return the edit layer whose name matches the parameter on this landscape (editor-only : returns nothing at runtime)
	*/
	UFUNCTION(BlueprintCallable, DisplayName = GetEditLayer, Category = "Landscape|Editor", meta = (DevelopmentOnly))
	const ULandscapeEditLayerBase* GetEditLayerByNameBP(const FName& InLayerName) const;

	bool IsValidRenderTargetFormatHeightmap(EPixelFormat InRenderTargetFormat, bool& bOutCompressHeight);
	bool IsValidRenderTargetFormatWeightmap(EPixelFormat InRenderTargetFormat, int32& OutNumChannels);

#if WITH_EDITOR

	/** Computes & returns bounds containing all landscape proxies (if any) or this landscape's bounds otherwise. Note that in non-WP worlds this will call GetLoadedBounds(). */
	LANDSCAPE_API FBox GetCompleteBounds() const;
	void RegisterLandscapeEdMode(ILandscapeEdModeInterface* InLandscapeEdMode) { LandscapeEdMode = InLandscapeEdMode; }
	void UnregisterLandscapeEdMode() { LandscapeEdMode = nullptr; }
	bool HasLandscapeEdMode() const { return LandscapeEdMode != nullptr; }
	UE_DEPRECATED(5.7, "Non-edit layer landscapes are deprecated, all landscapes use the edit layer system now.")
	LANDSCAPE_API virtual bool HasLayersContent() const override;
	UE_DEPRECATED(5.7, "Non-edit layer landscapes are deprecated, all landscapes use the edit layer system now.")
	LANDSCAPE_API virtual void UpdateCachedHasLayersContent(bool bInCheckComponentDataIntegrity) override;
	LANDSCAPE_API void RequestSplineLayerUpdate();
	LANDSCAPE_API void RequestLayersInitialization(bool bInRequestContentUpdate = true, bool bInForceLayerResourceReset = true);
	LANDSCAPE_API void RequestLayersContentUpdateForceAll(ELandscapeLayerUpdateMode InModeMask = ELandscapeLayerUpdateMode::Update_All, bool bInUserTriggered = false);
	LANDSCAPE_API void RequestLayersContentUpdate(ELandscapeLayerUpdateMode InModeMask);
	LANDSCAPE_API bool ReorderLayer(int32 InStartingLayerIndex, int32 InDestinationLayerIndex);
	LANDSCAPE_API FLandscapeLayer* DuplicateLayerAndMoveBrushes(const FLandscapeLayer& InOtherLayer);

	/** 
	* Creates a new edit layer
	* @Param InName is the name of the new edit layer
	* @Param InEditLayerClass is the class of the edit layer to create. Passing null will create a standard layer of type ULandscapeEditLayer
	* @Param bInIgnoreLayerCountLimit create the layer even if this would exceed MaxNumberOfLayers
	* @return the index of the newly-created layer
	*/
	LANDSCAPE_API int32 CreateLayer(FName InName = NAME_None, const TSubclassOf<ULandscapeEditLayerBase>& InEditLayerClass = TSubclassOf<ULandscapeEditLayerBase>(), bool bInIgnoreLayerCountLimit = false);
	LANDSCAPE_API void CreateDefaultLayer();

	LANDSCAPE_API void CopyOldDataToDefaultLayer();
	LANDSCAPE_API void CopyOldDataToDefaultLayer(ALandscapeProxy* InProxy);

	/**
	 * Copies edit layer data from each of the InProxy's landscape components to the given edit layer. Overwrites existing data in the destination edit layer
	 * @param InProxy is the Proxy to copy Landscape Component data from
	 * @param InSourceEditLayerGuid is the source edit layer Guid. When valid, copies data from the given edit layer. If invalid, copies each components final merged data
	 * @param InDestEditLayerGuid is the destination edit layer Guid. When valid, must represent a ULandscapeEditLayerPersistent on the ALandscape. If invalid, the default edit layer is used
	 * @param bInUseObsoleteLayerData When true, uses obsolete or duplicated final layer data stored during initial registration. Landscape components only duplicate final data when they have >= 1 obsolete layer on load
	 */
	LANDSCAPE_API void CopyDataToEditLayer(ALandscapeProxy* InProxy, const FGuid& InSourceEditLayerGuid = FGuid(), const FGuid& InDestEditLayerGuid = FGuid(), bool bInUseObsoleteLayerData = false);

	LANDSCAPE_API void AddLayersToProxy(ALandscapeProxy* InProxy);
	LANDSCAPE_API FIntPoint ComputeComponentCounts() const;
	LANDSCAPE_API bool IsLayerNameUnique(const FName& InName) const;

	// FLandscapeLayer accessors : only the const version is provided because we don't want to let them be mutated freely without the landscape being aware
	LANDSCAPE_API TArrayView<const FLandscapeLayer> GetLayersConst() const;
	LANDSCAPE_API const FLandscapeLayer* GetLayerConst(int32 InLayerIndex) const;
	LANDSCAPE_API const FLandscapeLayer* GetLayerConst(const FGuid& InLayerGuid) const;
	LANDSCAPE_API const FLandscapeLayer* GetLayerConst(const FName& InLayerName) const;
	LANDSCAPE_API const FLandscapeLayer* FindLayerOfTypeConst(const TSubclassOf<ULandscapeEditLayerBase>& InLayerClass) const;
	LANDSCAPE_API TArray<const FLandscapeLayer*> GetLayersOfTypeConst(const TSubclassOf<ULandscapeEditLayerBase>& InLayerClass) const;
	LANDSCAPE_API int32 GetLayerIndex(const FGuid& InLayerGuid) const;
	LANDSCAPE_API int32 GetLayerIndex(FName InLayerName) const;

	// ULandscapeEditLayerBase accessors : both const and non-const versions are provided because the landscape listens to data changes on the edit layer object 
	//  and can therefore react to any change
	LANDSCAPE_API const TArray<const ULandscapeEditLayerBase*> GetEditLayersConst() const;
	LANDSCAPE_API const TArray<ULandscapeEditLayerBase*> GetEditLayers() const;
	LANDSCAPE_API const ULandscapeEditLayerBase* GetEditLayerConst(int32 InLayerIndex) const;
	LANDSCAPE_API ULandscapeEditLayerBase* GetEditLayer(int32 InLayerIndex) const;
	LANDSCAPE_API const ULandscapeEditLayerBase* GetEditLayerConst(const FGuid& InLayerGuid) const;
	LANDSCAPE_API ULandscapeEditLayerBase* GetEditLayer(const FGuid& InLayerGuid) const;
	LANDSCAPE_API const ULandscapeEditLayerBase* GetEditLayerConst(const FName& InLayerName) const;
	LANDSCAPE_API ULandscapeEditLayerBase* GetEditLayer(const FName& InLayerName) const;

	LANDSCAPE_API const ULandscapeEditLayerBase* FindEditLayerOfTypeConst(const TSubclassOf<ULandscapeEditLayerBase>& InLayerClass) const;
	LANDSCAPE_API ULandscapeEditLayerBase* FindEditLayerOfType(const TSubclassOf<ULandscapeEditLayerBase>& InLayerClass) const;
	LANDSCAPE_API TArray<const ULandscapeEditLayerBase*> GetEditLayersOfTypeConst(const TSubclassOf<ULandscapeEditLayerBase>& InLayerClass) const;
	LANDSCAPE_API TArray<ULandscapeEditLayerBase*> GetEditLayersOfType(const TSubclassOf<ULandscapeEditLayerBase>& InLayerClass) const;

	/**
	 * Runs the given function on each edit layer, with the possibility of early exit
	 * Most easily used with a lambda as follows:
	 * ForEachLayerConst([](const FLandscapeLayer& InLayer) -> bool
	 * {
	 *     return continueLoop ? true : false;
	 * });
	 */
	LANDSCAPE_API void ForEachLayerConst(TFunctionRef<bool(const FLandscapeLayer&)> Fn);
	LANDSCAPE_API void ForEachEditLayerConst(TFunctionRef<bool(const ULandscapeEditLayerBase*)> Fn);

	LANDSCAPE_API void GetUsedPaintLayers(int32 InLayerIndex, TArray<ULandscapeLayerInfoObject*>& OutUsedLayerInfos) const;
	LANDSCAPE_API void GetUsedPaintLayers(const FGuid& InLayerGuid, TArray<ULandscapeLayerInfoObject*>& OutUsedLayerInfos) const;
	LANDSCAPE_API void ClearPaintLayer(int32 InLayerIndex, ULandscapeLayerInfoObject* InLayerInfo);
	LANDSCAPE_API void ClearPaintLayer(const FGuid& InLayerGuid, ULandscapeLayerInfoObject* InLayerInfo);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.7, "ELandscapeClearMode is deprecated, use ClearEditLayer with ELandscapeToolTargetType instead.")
	LANDSCAPE_API void ClearLayer(int32 InLayerIndex, TSet<TObjectPtr<ULandscapeComponent>>* InComponents = nullptr, ELandscapeClearMode InClearMode = ELandscapeClearMode::Clear_All);
	UE_DEPRECATED(5.7, "ELandscapeClearMode is deprecated, use ClearEditLayer with ELandscapeToolTargetType instead.")
	LANDSCAPE_API void ClearLayer(const FGuid& InLayerGuid, TSet<TObjectPtr<ULandscapeComponent>>* InComponents = nullptr, ELandscapeClearMode InClearMode = ELandscapeClearMode::Clear_All, bool bMarkPackageDirty = true);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	LANDSCAPE_API void ClearEditLayer(int32 InLayerIndex, TSet<TObjectPtr<ULandscapeComponent>>* InComponents = nullptr, ELandscapeToolTargetTypeFlags InClearMode = ELandscapeToolTargetTypeFlags::All);
	LANDSCAPE_API void ClearEditLayer(const FGuid& InLayerGuid, TSet<TObjectPtr<ULandscapeComponent>>* InComponents = nullptr, ELandscapeToolTargetTypeFlags InClearMode = ELandscapeToolTargetTypeFlags::All, bool bMarkPackageDirty = true);
	
	LANDSCAPE_API bool DeleteLayer(int32 InLayerIndex);
	LANDSCAPE_API void CollapseLayer(int32 InLayerIndex);
	UE_DEPRECATED(5.7, "Non-edit layer landscapes are deprecated. Use CollapseAllEditLayers to flatten data to the base edit layer.")
	LANDSCAPE_API void DeleteLayers();
	LANDSCAPE_API void CollapseAllEditLayers();
	LANDSCAPE_API void SetEditingLayer(const FGuid& InLayerGuid = FGuid());
	LANDSCAPE_API void SetGrassUpdateEnabled(bool bInGrassUpdateEnabled);
	LANDSCAPE_API const FGuid& GetEditingLayer() const;
	LANDSCAPE_API bool IsMaxLayersReached() const;
	LANDSCAPE_API void ShowOnlySelectedLayer(int32 InLayerIndex);
	LANDSCAPE_API void ShowAllLayers();
	UE_DEPRECATED(5.7, "Use UpdateLandscapeSplines without bInUpdateOnlySelected. Edit layer landscapes update all splines regardless of selection state")
	LANDSCAPE_API void UpdateLandscapeSplines(const FGuid& InLayerGuid = FGuid(), bool bInUpdateOnlySelected = false, bool bInForceUpdateAllCompoments = false);
	LANDSCAPE_API void UpdateAllLandscapeSplines(const FGuid& InLayerGuid = FGuid(), bool bInForceUpdateAllCompoments = false);
	LANDSCAPE_API void SetSelectedEditLayerIndex(const int32 InEditLayerIndex);
	LANDSCAPE_API const int32 GetSelectedEditLayerIndex() const;

	LANDSCAPE_API void ReplaceLayerSubstractiveBlendStatus(ULandscapeLayerInfoObject* InFromLayerInfo, ULandscapeLayerInfoObject* InToLayerInfo, bool bInShouldDirtyPackage);

	LANDSCAPE_API int32 GetBrushLayer(const ALandscapeBlueprintBrushBase* InBrush) const;
	LANDSCAPE_API void AddBrushToLayer(int32 InLayerIndex, class ALandscapeBlueprintBrushBase* InBrush);
	LANDSCAPE_API void RemoveBrush(class ALandscapeBlueprintBrushBase* InBrush);
	LANDSCAPE_API void RemoveBrushFromLayer(int32 InLayerIndex, class ALandscapeBlueprintBrushBase* InBrush);
	LANDSCAPE_API void RemoveBrushFromLayer(int32 InLayerIndex, int32 InBrushIndex);
	LANDSCAPE_API int32 GetBrushIndexForLayer(int32 InLayerIndex, class ALandscapeBlueprintBrushBase* InBrush);
	LANDSCAPE_API bool ReorderLayerBrush(int32 InLayerIndex, int32 InStartingLayerBrushIndex, int32 InDestinationLayerBrushIndex);
	LANDSCAPE_API class ALandscapeBlueprintBrushBase* GetBrushForLayer(int32 InLayerIndex, int32 BrushIndex) const;
	LANDSCAPE_API TArray<class ALandscapeBlueprintBrushBase*> GetBrushesForLayer(int32 InLayerIndex) const;
	LANDSCAPE_API void OnBlueprintBrushChanged();
	LANDSCAPE_API void OnLayerInfoSplineFalloffModulationChanged(const ULandscapeLayerInfoObject* InLayerInfo);
	LANDSCAPE_API void OnPreSave();

	void ClearDirtyData(ULandscapeComponent* InLandscapeComponent);
	
	LANDSCAPE_API void ConvertNonEditLayerLandscape();
	UE_DEPRECATED(5.7, "Non-edit layer landscapes are deprecated. Use ConvertNonEditLayerLandscape to convert to non-edit layer landscapes to edit layer based landscapes.")
	LANDSCAPE_API void ToggleCanHaveLayersContent();
	LANDSCAPE_API void ForceUpdateLayersContent();
	UE_DEPRECATED(5.7, "Intermediate renders are a thing of the past. You should now use SelectiveRenderEditLayersHeightmaps/SelectiveRenderEditLayersWeightmaps to render the landscape edit layers in an \"intermediate\" state")
	LANDSCAPE_API void ForceUpdateLayersContent(bool bIntermediateRender);

	LANDSCAPE_API bool SelectiveRenderEditLayersHeightmaps(const FLandscapeEditLayerRenderHeightParams& InSelectiveRenderParams);
	LANDSCAPE_API bool SelectiveRenderEditLayersWeightmaps(const FLandscapeEditLayerRenderWeightParams& InSelectiveRenderParams);

	void FlushLayerContentThisFrame();

	UFUNCTION(BlueprintCallable, Category = "Landscape")
	LANDSCAPE_API void ForceLayersFullUpdate();

	LANDSCAPE_API void InitializeLandscapeLayersWeightmapUsage();

	LANDSCAPE_API bool ComputeLandscapeLayerBrushInfo(FTransform& OutLandscapeTransform, FIntPoint& OutLandscapeSize, FIntPoint& OutLandscapeRenderTargetSize);
	void UpdateProxyLayersWeightmapUsage();
	void ValidateProxyLayersWeightmapUsage() const;

	LANDSCAPE_API void SetUseGeneratedLandscapeSplineMeshesActors(bool bInEnabled);
	LANDSCAPE_API bool GetUseGeneratedLandscapeSplineMeshesActors() const;
	LANDSCAPE_API bool PrepareTextureResources(bool bInWaitForStreaming);

	enum class EPrepareTextureResourcesLimitedMode : uint8
	{
		Default = 0,				// Default operation
		WaitForStreaming = 1 << 0,	// Block until streaming completes
		BypassRateLimit = 1 << 1	// Bypass rate limit and always call PrepareTextureResources
	};
	FRIEND_ENUM_CLASS_FLAGS(EPrepareTextureResourcesLimitedMode);
	bool PrepareTextureResourcesLimited(EPrepareTextureResourcesLimitedMode Mode = EPrepareTextureResourcesLimitedMode::Default);

	bool GetVisibilityLayerAllocationIndex() const { return 0; }
	
	LANDSCAPE_API virtual void DeleteUnusedLayers() override;

	LANDSCAPE_API void EnableNaniteSkirts(bool bInEnable, float InSkirtDepth, bool bInShouldDirtyPackage);

	/** Set the target precision on nanite vertex position.  Precision is set to approximately (2^-InPrecision) in world units. */
	LANDSCAPE_API void SetNanitePositionPrecision(int32 InPrecision,  bool bInShouldDirtyPackage);

	LANDSCAPE_API void SetDisableRuntimeGrassMapGeneration(bool bInDisableRuntimeGrassMapGeneration);

	LANDSCAPE_API FName GenerateUniqueLayerName(FName InName = NAME_None) const;

private:
	FLandscapeLayer* GetLayerInternal(int32 InLayerIndex);
	void OnLayerCreatedInternal(ULandscapeEditLayerBase* EditLayer);
	ULandscapeEditLayerBase* GetEditLayerInternal(int32 InLayerIndex);

	void UpdateLayersContent(bool bInWaitForStreaming, bool bInSkipMonitorLandscapeEdModeChanges, bool bFlushRender);
	bool CanUpdateLayersContent() const;
	void MonitorLandscapeEdModeChanges();
	
	//~ Begin UE::Landscape::EditLayers::IEditLayerRendererProvider implementation
	virtual TArray<UE::Landscape::EditLayers::FEditLayerRendererState> GetEditLayerRendererStates(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) override;
	//~ End UE::Landscape::EditLayers::IEditLayerRendererProvider implementation
	TArray<UE::Landscape::EditLayers::FEditLayerRendererState> GetEditLayerRendererStatesEnableOverride(const UE::Landscape::EditLayers::FMergeContext* InMergeContext, const TBitArray<>& InLayerEnableOverride);

	UE::Landscape::EditLayers::FMergeRenderContext PrepareEditLayersMergeRenderContext(const UE::Landscape::EditLayers::FMergeContext& InMergeContext, const UE::Landscape::EditLayers::FMergeRenderParams& InParams);

	int32 RegenerateLayersHeightmaps(const FUpdateLayersContentContext& InUpdateLayersContentContext);
	int32 PerformLayersHeightmapsBatchedMerge(const FUpdateLayersContentContext& InUpdateLayersContentContext, const FEditLayersHeightmapMergeParams& InMergeParams);
	void ResolveLayersHeightmapTexture(const FTextureToComponentHelper& MapHelper, const TSet<UTexture2D*>& HeightmapsToResolve, TArray<FLandscapeEditLayerComponentReadbackResult>& InOutComponentReadbackResults);

	bool PerformSelectiveRenderEditLayersHeightmapsCPU(const FUpdateLayersContentContext& InUpdateLayersContentContext, const FLandscapeEditLayerRenderHeightParams& InSelectiveRenderParams);
	bool PerformSelectiveRenderEditLayersHeightmapsRT(const FUpdateLayersContentContext& InUpdateLayersContentContext, const FLandscapeEditLayerRenderHeightParams& InSelectiveRenderParams);

	int32 RegenerateLayersWeightmaps(FUpdateLayersContentContext& InUpdateLayersContentContext);
	int32 PerformLayersWeightmapsBatchedMerge(FUpdateLayersContentContext& InUpdateLayersContentContext, const FEditLayersWeightmapMergeParams& InMergeParams);
	bool PerformSelectiveRenderEditLayersWeightmapsCPU(FUpdateLayersContentContext& InUpdateLayersContentContext, const FLandscapeEditLayerRenderWeightParams& InSelectiveRenderParams);
	void ResolveLayersWeightmapTexture(const FTextureToComponentHelper& MapHelper, const TSet<UTexture2D*>& WeightmapsToResolve, TArray<FLandscapeEditLayerComponentReadbackResult>& InOutComponentReadbackResults);

	bool ResolveLayersTexture(FTextureToComponentHelper const& MapHelper, FLandscapeEditLayerReadback* InCPUReadBack, UTexture2D* InOutputTexture,
		TArray<FLandscapeEditLayerComponentReadbackResult>& InOutComponentReadbackResults, bool bIsWeightmap);

	static bool IsUpdateFlagEnabledForModes(ELandscapeComponentUpdateFlag InFlag, uint32 InUpdateModes);
	void UpdateForChangedHeightmaps(const TArrayView<FLandscapeEditLayerComponentReadbackResult>& InComponentReadbackResults);
	void UpdateForChangedWeightmaps(const TArrayView<FLandscapeEditLayerComponentReadbackResult>& InComponentReadbackResults);
	uint32 UpdateCollisionAndClients(const TArrayView<FLandscapeEditLayerComponentReadbackResult>& Components);
	uint32 UpdateAfterReadbackResolves(const TArrayView<FLandscapeEditLayerComponentReadbackResult>& Components);

	bool PrepareLayersTextureResources(bool bInWaitForStreaming);
	bool PrepareLayersTextureResources(const TArray<FLandscapeLayer>& InLayers, bool bInWaitForStreaming);
	bool PrepareLayersResources(EShaderPlatform InShaderPlatform, bool bInWaitForStreaming);
	void InvalidateRVTForTextures(const TSet<UTexture2D*>& InTextures);

	void UpdateLayersMaterialInstances(const TArray<ULandscapeComponent*>& InLandscapeComponents);

	void ReallocateLayersWeightmaps(FUpdateLayersContentContext& InUpdateLayersContentContext, 
		const TMap<ULandscapeComponent*, TArray<ULandscapeLayerInfoObject*>>& InPerComponentAllocations, 
		const TSet<ULandscapeComponent*>& InRestrictTextureSharingToComponents);
	void InitializeLayers();
	
	void UpdateWeightDirtyData(ULandscapeComponent* InLandscapeComponent, UTexture2D const* InWeightmap, FColor const* InOldData, FColor const* InNewData, uint8 InChannel);
	void OnDirtyWeightmap(FTextureToComponentHelper const& MapHelper, UTexture2D const* InWeightmap, FColor const* InOldData, FColor const* InNewData, int32 InMipLevel, uint8 ChangedChannelsMask);
	void UpdateHeightDirtyData(ULandscapeComponent* InLandscapeComponent, UTexture2D const* InHeightmap, FColor const* InOldData, FColor const* InNewData);
	void OnDirtyHeightmap(FTextureToComponentHelper const& MapHelper, UTexture2D const* InWeightmap, FColor const* InOldData, FColor const* InNewData, int32 InMipLevel);

	static bool IsMaterialResourceCompiled(FMaterialResource* InMaterialResource, bool bInWaitForCompilation);

	void OnEditLayerDataChanged(const FOnLandscapeEditLayerDataChangedParams& InParams);
#endif // WITH_EDITOR

private:
	void MarkAllLandscapeRenderStateDirty();
	bool RenderMergedTextureInternal(const FTransform& InRenderAreaWorldTransform, const FBox2D& InRenderAreaExtents, const TArray<FName>& InWeightmapLayerNames, UTextureRenderTarget* OutRenderTarget);

	UPROPERTY(EditAnywhere, Category="Grass")
	bool bGrassEnabled = true;

public:

#if WITH_EDITORONLY_DATA
	/** Landscape actor has authority on default streaming behavior for new actors : LandscapeStreamingProxies & LandscapeSplineActors */
	UPROPERTY(EditAnywhere, Category = WorldPartition)
	bool bAreNewLandscapeActorsSpatiallyLoaded = true;

	/** If true, LandscapeStreamingProxy actors have the grid size included in their name, for backward compatibility we also check the AWorldSettings::bIncludeGridSizeInNameForPartitionedActors */
	UPROPERTY()
	bool bIncludeGridSizeInNameForLandscapeActors = false;

	UE_DEPRECATED(5.7, "Non-edit layer landscapes are deprecated, all landscapes use the edit layer system now.")
	UPROPERTY()
	bool bCanHaveLayersContent_DEPRECATED = false;

	/*
	 * If true, WorldPartitionLandscapeSplineMeshesBuilder is responsible of generating partitioned actors of type ALandscapeSplineMeshesActor that will contain all landscape spline/controlpoints static meshes. 
	 * Source components will be editor only and hidden in game for PIE.
	 */
	UPROPERTY()
	bool bUseGeneratedLandscapeSplineMeshesActors = false;

	DECLARE_EVENT(ALandscape, FLandscapeBlueprintBrushChangedDelegate);
	FLandscapeBlueprintBrushChangedDelegate& OnBlueprintBrushChangedDelegate() { return LandscapeBlueprintBrushChangedDelegate; }

	/** Delegate that will be called whenever an edit layers merge is done */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnEditLayersMergedDelegate, const FOnLandscapeEditLayersMergedParams& /*InParams*/);
	FOnEditLayersMergedDelegate::RegistrationType& OnEditLayersMerged() const { return OnEditLayersMergedDelegate; }

	/** Target Landscape Layer for Landscape Splines */
	UE_DEPRECATED(all, "This has been refactored into the generic ULandscapeEditLayerBase system. Please check for the presence of a ULandscapeEditLayerSplines layer instead")
	UPROPERTY()
	FGuid LandscapeSplinesTargetLayerGuid_DEPRECATED;
	
	/** Current Editing Landscape Layer*/
	// TODO this is used as shared global state for the landscape editor mode. FLandscapeToolStrokeBase::SetEditLayer should manage the shared editor state instead
	FGuid EditingLayer;

	/** Current Selected Edit Layer of this landscape. Used by landscape editor mode to track the current selection */
	UPROPERTY(Transient, DuplicateTransient)
	int32 SelectedEditLayerIndex = 0;

	/** Used to temporarily disable Grass Update in Editor */
	bool bGrassUpdateEnabled;

	UPROPERTY(Transient)
	bool bEnableEditorLayersTick = true;

	UE_DEPRECATED(all, "This property has moved to private. Use the public accessors instead")
	UPROPERTY()
	TArray<FLandscapeLayer> LandscapeLayers_DEPRECATED;

	UE_DEPRECATED(5.7, "Edit layers are now fully processed via BatchedMerge")
	UPROPERTY(Transient)
	TArray<TObjectPtr<UTextureRenderTarget2D>> HeightmapRTList_DEPRECATED;

	UE_DEPRECATED(5.7, "Edit layers are now fully processed via BatchedMerge")
	UPROPERTY(Transient)
	TArray<TObjectPtr<UTextureRenderTarget2D>> WeightmapRTList_DEPRECATED;

	/** List of textures that are not fully streamed in yet (updated every frame to track textures that have finished streaming in) */
	UPROPERTY(Transient, DuplicateTransient, TextExportTransient)	
	TArray<TWeakObjectPtr<UTexture2D>> TrackedStreamingInTextures;

	/** Display Order of the targets */
	UPROPERTY(NonTransactional)
	TArray<FName> TargetDisplayOrderList;

	/** Display Order mode for the targets */
	UPROPERTY(NonTransactional)
	ELandscapeLayerDisplayMode TargetDisplayOrder = ELandscapeLayerDisplayMode::Default;

private:
	UPROPERTY()
	TArray<FLandscapeLayer> LandscapeEditLayers;

	FLandscapeBlueprintBrushChangedDelegate LandscapeBlueprintBrushChangedDelegate;
	mutable FOnEditLayersMergedDelegate OnEditLayersMergedDelegate;

	/** Components affected by landscape splines (used to partially clear Layer Reserved for Splines) */
	UPROPERTY(Transient)
	TSet<TObjectPtr<ULandscapeComponent>> LandscapeSplinesAffectedComponents;

	/** Provides information from LandscapeEdMode */
	ILandscapeEdModeInterface* LandscapeEdMode;

	/** Information provided by LandscapeEdMode */
	struct FLandscapeEdModeInfo
	{
		FLandscapeEdModeInfo();

		int32 ViewMode;
		FGuid SelectedLayer;
		TWeakObjectPtr<ULandscapeLayerInfoObject> SelectedLayerInfoObject;
		ELandscapeToolTargetType ToolTarget;
	};

	FLandscapeEdModeInfo LandscapeEdModeInfo;

	// TODO [jonathan.bard] : remove : this isn't really used anymore
	UPROPERTY(Transient)
	bool bLandscapeLayersAreInitialized; 

	UE_DEPRECATED(5.8, "WasCompilingShaders property is no longer used.")
	UPROPERTY(Transient)
	bool WasCompilingShaders_DEPRECATED = false;

	UPROPERTY(Transient)
	uint32 LayerContentUpdateModes;
		
	UPROPERTY(Transient)
	bool bSplineLayerUpdateRequested;

	bool bWarnedGlobalMergeDimensionsExceeded = false;
	bool bWarnedNumComponentsPerBatchExceeded = false;
	bool bWarnedLayerMergeResolution = false;

	struct FWaitingForResourcesNotificationHelper
	{
		void Notify(ALandscape* InLandscape, class FLandscapeNotificationManager* InNotificationManager, ELandscapeNotificationType InNotificationType, const FText& InNotificationText);
		void Reset();

		/** Time since waiting for resources to be ready */
		double WaitingForResourcesStartTime = -1.0;

		/** Non-stackable user notification for landscape editor */
		TSharedPtr<FLandscapeNotification> Notification;
	};

	/** Non-stackable user notifications for landscape editor */
	FWaitingForResourcesNotificationHelper WaitingForTexturesNotificationHelper;
	FWaitingForResourcesNotificationHelper WaitingForEditLayerResourcesNotificationHelper;

	uint32 LastFlushedLayerUpdateFrame = 0;
	uint32 LastPrepareTextureResourcesCalled = 0;
	bool bLastPrepareTextureResourcesResult = false;
	bool bHasWarnedAboutInvalidShadingModel = false;

	// Counter to detect re-entrance
	uint32 LayerUpdateCount = 0;
#endif
};

#if WITH_EDITOR
class FScopedSetLandscapeEditingLayer
{
public:
	LANDSCAPE_API FScopedSetLandscapeEditingLayer(ALandscape* InLandscape, const FGuid& InLayerGUID, TFunction<void()> InCompletionCallback = TFunction<void()>());
	LANDSCAPE_API ~FScopedSetLandscapeEditingLayer();

private:
	TWeakObjectPtr<ALandscape> Landscape;
	FGuid PreviousLayerGUID;
	TFunction<void()> CompletionCallback;
};
#endif
