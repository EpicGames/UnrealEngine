// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StaticArray.h"
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Engine/TextureStreamingTypes.h"
#include "Components/PrimitiveComponent.h"
#include "UObject/PerPlatformProperties.h"
#include "LandscapeGrassType.h"
#include "LandscapeInfo.h"
#include "LandscapePhysicalMaterial.h"
#include "LandscapeWeightmapUsage.h"
#include "Containers/ArrayView.h"
#include "Engine/StreamableRenderAsset.h"
#include "Engine/Texture2DArray.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Serialization/BulkData.h"
#endif

#include "LandscapeComponent.generated.h"

class ALandscape;
class ALandscapeProxy;
class FLightingBuildOptions;
class FMaterialUpdateContext;
class FMeshMapBuildData;
class FPrimitiveSceneProxy;
class ITargetPlatform;
class ULandscapeComponent;
class ULandscapeHeightfieldCollisionComponent;
class ULandscapeHeightmapTextureEdgeFixup;
class ULandscapeInfo;
class ULandscapeLayerInfoObject;
class ULightComponent;
class UMaterialInstanceConstant;
class UMaterialInterface;
class UTexture2D;
struct FConvexVolume;
struct FEngineShowFlags;
struct FLandscapeEditDataInterface;
struct FLandscapeGroup;
struct FLandscapeMobileRenderData;
struct FLandscapeTextureDataInfo;
struct FStaticLightingPrimitiveInfo;
enum class ELandscapeTextureUsage : uint8;
enum class ELandscapeTextureType : uint8;

//
// FLandscapeEditToolRenderData
//
USTRUCT()
struct FLandscapeEditToolRenderData
{
public:
	GENERATED_USTRUCT_BODY()

	enum SelectionType
	{
		ST_NONE = 0,
		ST_COMPONENT = 1,
		ST_REGION = 2,
		// = 4...
	};

	FLandscapeEditToolRenderData()
		: ToolMaterial(NULL),
		GizmoMaterial(NULL),
		SelectedType(ST_NONE),
		DebugChannelR(INDEX_NONE),
		DebugChannelG(INDEX_NONE),
		DebugChannelB(INDEX_NONE),
		DataTexture(NULL),
		LayerContributionTexture(NULL),
		DirtyTexture(NULL)
	{}

	// Material used to render the tool.
	UPROPERTY(NonTransactional)
	TObjectPtr<UMaterialInterface> ToolMaterial;

	// Material used to render the gizmo selection region...
	UPROPERTY(NonTransactional)
	TObjectPtr<UMaterialInterface> GizmoMaterial;

	// Component is selected
	UPROPERTY(NonTransactional)
	int32 SelectedType;

	UPROPERTY(NonTransactional)
	int32 DebugChannelR;

	UPROPERTY(NonTransactional)
	int32 DebugChannelG;

	UPROPERTY(NonTransactional)
	int32 DebugChannelB;

	UPROPERTY(NonTransactional)
	TObjectPtr<UTexture2D> DataTexture; // Data texture other than height/weight

	UPROPERTY(NonTransactional)
	TObjectPtr<UTexture2D> LayerContributionTexture; // Data texture used to represent layer contribution

	UPROPERTY(NonTransactional)
	TObjectPtr<UTexture2D> DirtyTexture; // Data texture used to represent layer blend dirtied area

#if WITH_EDITOR
	void UpdateDebugColorMaterial(const ULandscapeComponent* const Component);
	void UpdateSelectionMaterial(int32 InSelectedType, const ULandscapeComponent* const Component);
#endif
};

/* Used to uniquely reference a landscape vertex in a component. */
struct FLandscapeVertexRef
{
	FLandscapeVertexRef(int16 InX, int16 InY, int8 InSubX, int8 InSubY)
		: X(InX)
		, Y(InY)
		, SubX(InSubX)
		, SubY(InSubY)
	{}

	uint32 X : 8;
	uint32 Y : 8;
	uint32 SubX : 8;
	uint32 SubY : 8;

	/** Helper to provide a standard ordering for vertex arrays. */
	static int32 GetVertexIndex(FLandscapeVertexRef Vert, int32 SubsectionCount, int32 SubsectionVerts)
	{
		return (Vert.SubY * SubsectionVerts + Vert.Y) * SubsectionVerts * SubsectionCount + Vert.SubX * SubsectionVerts + Vert.X;
	}
};

/** Stores information about which weightmap texture and channel each layer is stored */
USTRUCT()
struct FWeightmapLayerAllocationInfo
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TObjectPtr<ULandscapeLayerInfoObject> LayerInfo;

	UPROPERTY()
	uint8 WeightmapTextureIndex;

	UPROPERTY()
	uint8 WeightmapTextureChannel;

	FWeightmapLayerAllocationInfo()
		: LayerInfo(nullptr)
		, WeightmapTextureIndex(0)
		, WeightmapTextureChannel(0)
	{
	}


	FWeightmapLayerAllocationInfo(ULandscapeLayerInfoObject* InLayerInfo)
		:	LayerInfo(InLayerInfo)
		,	WeightmapTextureIndex(255)	// Indicates an invalid allocation
		,	WeightmapTextureChannel(255)
	{
	}
	
	bool operator == (const FWeightmapLayerAllocationInfo& RHS) const
	{
		return (LayerInfo == RHS.LayerInfo)
			&& (WeightmapTextureIndex == RHS.WeightmapTextureIndex)
			&& (WeightmapTextureChannel == RHS.WeightmapTextureChannel); 
	}

	FName GetLayerName() const;

	uint32 GetHash() const;

	void Free()
	{
		WeightmapTextureChannel = 255;
		WeightmapTextureIndex = 255;
	}

	bool IsAllocated() const { return (WeightmapTextureChannel != 255 && WeightmapTextureIndex != 255); }
};

inline uint32 GetTypeHash(const FWeightmapLayerAllocationInfo& InAllocInfo)
{
	return InAllocInfo.GetHash();
}

template<typename T>
struct IBuffer2DView
{
	// copy up to Count elements to Dest, in X then Y order (standard image order)
	virtual void CopyTo(T* Dest, int32 Count) const = 0;

	// copy up to Count elements to Dest, in X then Y order (standard image order)
	virtual bool CopyToAndCalcIsAllZero(T* Dest, int32 Count) const = 0;

	// return the total number of elements
	virtual int32 Num() const = 0;
};

struct FLandscapeComponentGrassData
{
#if WITH_EDITORONLY_DATA
	// Variables used to detect when grass data needs to be regenerated:

	// Guid per material instance in the hierarchy between the assigned landscape material (instance) and the root UMaterial
	// used to detect changes to material instance parameters or the root material that could affect the grass maps
	UE_DEPRECATED(all, "GenerationHash is now used") 
	TArray<FGuid, TInlineAllocator<2>> MaterialStateIds_DEPRECATED;
	// cached component rotation when material world-position-offset is used,
	// as this will affect the direction of world-position-offset deformation (included in the HeightData below)
	UE_DEPRECATED(all, "GenerationHash is now used") 
	FQuat RotationForWPO_DEPRECATED;

	// Variable used to detect when grass data needs to be regenerated:
	uint32 GenerationHash = 0;
#endif

#if WITH_EDITORONLY_DATA
	// Height data for LODs 1+, keyed on LOD index
	TMap<int32, TArray<uint16>> HeightMipData;
#endif // WITH_EDITORONLY_DATA
	
	static constexpr int32 UnknownNumElements = -1;
	// Elements per contiguous array: for validation and also to indicate whether the grass data is valid (NumElements >= 0, meaning 0 elements is valid but the grass data is all zero and 
	//  therefore empty) or not known yet (== UnknownNumElements)
	int32 NumElements = UnknownNumElements;
	// Serialized in one block to prevent Slack waste
	TMap<FName, int32> WeightOffsets;
	TArray<uint8> HeightWeightData;

	// Note: We need to explicitly disable warnings on these constructors/operators for clang to be happy with deprecated variables
    PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FLandscapeComponentGrassData() = default;
	FLandscapeComponentGrassData(ULandscapeComponent* Component);
    ~FLandscapeComponentGrassData() = default;
    FLandscapeComponentGrassData(const FLandscapeComponentGrassData&) = default;
    FLandscapeComponentGrassData(FLandscapeComponentGrassData&&) = default;
    FLandscapeComponentGrassData& operator=(const FLandscapeComponentGrassData&) = default;
    FLandscapeComponentGrassData& operator=(FLandscapeComponentGrassData&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Returns whether grass data has been computed (or serialized) yet. Returns true even if the data is completely empty (e.g. all-zero weightmap data)
	bool HasValidData() const;

	// Returns whether the data is completely empty (e.g. all-zero weightmap data). Returns false if the data just wasn't computed yet :
	bool HasData() const;

#if WITH_EDITOR
	// Initialize function meant for deserializing from old data : the new format doesn't take ULandscapeGrassType* but FName :
	void InitializeFrom(const TArray<uint16>& HeightData, const TMap<ULandscapeGrassType*, TArray<uint8>>& WeightData);
	void FixupFallbackGrassNames(const TMap<FName, TObjectPtr<ULandscapeGrassType>>& InNamedGrassTypes);
#endif // WITH_EDITOR
	void InitializeFrom(IBuffer2DView<uint16>* HeightData, TMap<FName, IBuffer2DView<uint8>*>& WeightData, bool bStripEmptyWeights);

	bool HasWeightData() const;
	TArrayView<uint8> GetWeightData(const FName& GrassName);
	bool Contains(const FName& GrassName) const;
	TArrayView<uint16> GetHeightData();

	SIZE_T GetAllocatedSize() const;

	// Check whether we can discard any data not needed with current scalability settings
	void ConditionalDiscardDataOnLoad(const TMap<FName, TObjectPtr<ULandscapeGrassType>>& InNamedGrassTypes);

	friend FArchive& operator<<(FArchive& Ar, FLandscapeComponentGrassData& Data);
};

USTRUCT(NotBlueprintable, meta = (Deprecated = "5.1"))
struct UE_DEPRECATED(all, "FLandscapeComponentMaterialOverride is deprecated; please use FLandscapePerLODMaterialOverride instead") FLandscapeComponentMaterialOverride
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = LandscapeComponent, meta=(UIMin=0, UIMax=8, ClampMin=0, ClampMax=8))
	FPerPlatformInt LODIndex;

	UPROPERTY(EditAnywhere, Category = LandscapeComponent)
	TObjectPtr<UMaterialInterface> Material = nullptr;
};

USTRUCT(NotBlueprintable)
struct FLandscapePerLODMaterialOverride
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = Material, meta = (UIMin = 0, UIMax = 8, ClampMin = 0, ClampMax = 8))
	int32 LODIndex = 0;

	UPROPERTY(EditAnywhere, Category = Material)
	TObjectPtr<UMaterialInterface> Material = nullptr;

	bool operator == (const FLandscapePerLODMaterialOverride & InOther) const
	{
		return (LODIndex == InOther.LODIndex)
			&& (Material == InOther.Material);
	}
};

USTRUCT(NotBlueprintable)
struct FWeightmapData
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TArray<TObjectPtr<UTexture2D>> Textures;
	
	UPROPERTY()
	TArray<FWeightmapLayerAllocationInfo> LayerAllocations;

	UPROPERTY(Transient, NonTransactional)
	TArray<TObjectPtr<ULandscapeWeightmapUsage>> TextureUsages;
};

USTRUCT(NotBlueprintable)
struct FHeightmapData
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TObjectPtr<UTexture2D> Texture = nullptr;
};

USTRUCT(NotBlueprintable)
struct FLandscapeLayerComponentData
{
	GENERATED_USTRUCT_BODY()

	FLandscapeLayerComponentData() = default;

#if WITH_EDITOR
	FLandscapeLayerComponentData(const FName& InDebugName)
		: DebugName(InDebugName)
	{}

#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	// Edit layers are referenced by Guid, this name is just there to provide some insights as to what edit layer name this layer data corresponded to in case of a missing edit layer guid
	UPROPERTY()
	FName DebugName; 
#endif // WITH_EDITORONLY_DATA

	UPROPERTY()
	FHeightmapData HeightmapData;

	UPROPERTY()
	FWeightmapData WeightmapData;

	bool IsInitialized() const { return HeightmapData.Texture != nullptr || WeightmapData.Textures.Num() > 0;  }
};

#if WITH_EDITOR
enum ELandscapeComponentUpdateFlag : uint32
{
	// Will call UpdateCollisionHeightData, UpdateCacheBounds, UpdateComponentToWorld on Component
	Component_Update_Heightmap_Collision = 1 << 0,
	// Will call UdateCollisionLayerData on Component
	Component_Update_Weightmap_Collision = 1 << 1,
	// Will call RecreateCollision on Component
	Component_Update_Recreate_Collision = 1 << 2,
	// Will update Component clients: Navigation data, Foliage, Grass, etc.
	Component_Update_Client = 1 << 3,
	// Will update Component clients while editing
	Component_Update_Client_Editing = 1 << 4,
	// Will compute component approximated bounds
	Component_Update_Approximated_Bounds = 1 << 5
};

/**
	ELandscapeLayerUpdateMode flags relate to types of updates to landscape layer data.  Stored on the ALandscape and on
	ULandscapeComponent to indicate pending changes.  Stored on FLandscapeEditLayerComponentReadbackResult to indicate
	what type of change triggered the creation of the new data.
 
	Update_All_Components on ALandscape or passed to RequestLayersContentUpdate will cause all components to be enqueued
	for layer merging, rather than only those components carrying relevant flags of their own.

 	See IsUpdateFlagEnabledForModes, where these flags are translated to ELandscapeComponentUpdateFlag, that indicate
	specific tasks.
**/
enum ELandscapeLayerUpdateMode : uint32
{
	// No Update
	Update_None = 0,
	// Update types
	Update_Heightmap_All_Modes = 1 << 0,
	Update_Heightmap_Editing = 1 << 1,
	Update_Heightmap_Editing_NoCollision = 1 << 2,
	Update_Weightmap_All_Modes = 1 << 3,
	Update_Weightmap_Editing = 1 << 4,
	Update_Weightmap_Editing_NoCollision = 1 << 5,
	Update_All_Components = 1 << 6,
	// Combinations
	Update_Heightmap_All = Update_Heightmap_All_Modes | Update_All_Components,
	Update_Weightmap_All = Update_Weightmap_All_Modes | Update_All_Components,
	Update_All = Update_Weightmap_All | Update_Heightmap_All,
	Update_All_Editing = Update_Weightmap_Editing | Update_Heightmap_Editing,
	Update_All_Editing_NoCollision = Update_Weightmap_Editing_NoCollision | Update_Heightmap_Editing_NoCollision,
	Update_Heightmap_Types = Update_Heightmap_All_Modes | Update_Heightmap_Editing | Update_Heightmap_Editing_NoCollision,
	Update_Weightmap_Types = Update_Weightmap_All_Modes | Update_Weightmap_Editing | Update_Weightmap_Editing_NoCollision,
	// In cases where we couldn't update the clients right away this flag will be set in RegenerateLayersContent
	Update_Client_Deferred = 1 << 7,
	// Update landscape component clients while editing
	Update_Client_Editing = 1 << 8
};

static const uint32 DefaultSplineHash = 0xFFFFFFFF;

#endif

UENUM()
enum UE_DEPRECATED(5.7, "Use ELandscapeToolTargetTypeFlags instead.") ELandscapeClearMode : int
{
	Clear_Weightmap = 1 << 0 UMETA(DisplayName = "Paint"),
	Clear_Heightmap = 1 << 1 UMETA(DisplayName = "Sculpt"),
	Clear_All = Clear_Weightmap | Clear_Heightmap UMETA(DisplayName = "All")
};

UCLASS(MinimalAPI)
class ULandscapeLODStreamingProxy_DEPRECATED : public UStreamableRenderAsset
{
	GENERATED_UCLASS_BODY()
};

UCLASS(hidecategories=(Display, Attachment, Physics, Debug, Collision, Movement, Rendering, PrimitiveComponent, Object, Transform, Mobility, VirtualTexture), showcategories=("Rendering|Material"), MinimalAPI, Within=LandscapeProxy)
class ULandscapeComponent : public UPrimitiveComponent
{
	GENERATED_UCLASS_BODY()
	
	/** X offset from global components grid origin (in quads) */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category=LandscapeComponent)
	int32 SectionBaseX;

	/** Y offset from global components grid origin (in quads) */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category=LandscapeComponent)
	int32 SectionBaseY;

	/** Total number of quads for this component, has to be >0 */
	UPROPERTY()
	int32 ComponentSizeQuads;

	/** Number of quads for a subsection of the component. SubsectionSizeQuads+1 must be a power of two. */
	UPROPERTY()
	int32 SubsectionSizeQuads;

	/** Number of subsections in X or Y axis */
	UPROPERTY()
	int32 NumSubsections;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=LandscapeComponent)
	TObjectPtr<UMaterialInterface> OverrideMaterial;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=LandscapeComponent, AdvancedDisplay)
	TObjectPtr<UMaterialInterface> OverrideHoleMaterial;

#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(all, "OverrideMaterials has been deprecated, use PerLODOverrideMaterials instead.")
	UPROPERTY()
	TArray<FLandscapeComponentMaterialOverride> OverrideMaterials_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(all, "MaterialInstance has been deprecated, use MaterialInstances instead.")
	UPROPERTY()
	TObjectPtr<UMaterialInstanceConstant> MaterialInstance_DEPRECATED;
#endif // WITH_EDITORONLY_DATA

	UPROPERTY(TextExportTransient)
	TArray<TObjectPtr<UMaterialInstanceConstant>> MaterialInstances;

	UPROPERTY(Transient, TextExportTransient)
	TArray<TObjectPtr<UMaterialInstanceDynamic>> MaterialInstancesDynamic;

	/** Mapping between LOD and Material Index*/
	UPROPERTY(TextExportTransient)
	TArray<int8> LODIndexToMaterialIndex;

	UE_DEPRECATED(5.7, "RetopologizeTool/XYOffset deprecated with the removal of non-edit layer landscapes")
	UPROPERTY()
	TObjectPtr<UTexture2D> XYOffsetmapTexture_DEPRECATED;

	/** UV offset to component's weightmap data from component local coordinates*/
	UPROPERTY()
	FVector4 WeightmapScaleBias;

	/** U or V offset into the weightmap for the first subsection, in texture UV space */
	UPROPERTY()
	float WeightmapSubsectionOffset;

	/** UV offset to Heightmap data from component local coordinates */
	UPROPERTY()
	FVector4 HeightmapScaleBias;

	/** Cached local-space bounding box, created at heightmap update time */
	UPROPERTY()
	FBox CachedLocalBox;

	/** Maximum deltas between vertices and their counterparts from other mips. This mip-to-mip data is laid out in a contiguous array following the following pattern : 
	*  Say, we have 5 "relevant" mips and [N -> M] is the delta from mip N to M (where M > N and M < (NumRelevantMips - 1)) then the array will contain : 
	*  [0 -> 1], [0 -> 2], [0 -> 3], [1 -> 2], [1 -> 3], [2 -> 3]
	*  i.e. for mip 0 : (NumRelevantMips - 1) deltas, then for mip 1 : (NumRelevantMips - 2) deltas, until mip == (NumRelevantMips - 2) : 1 delta
	*  Note: a "relevant" mip is one with more than 1 vertex. i.e.:
	*   - In the case of a 1x1 subsection, the last mip index (NumMips - 1) has a single pixel and is therefore not relevant (we cannot draw a landscape component with a single vertex!), hence the last relevant mip index will be NumMips - 2
	*   - In the case of 2x2 subsections, the penultimate mip index (NumMips - 2) has 4 pixels, which means 4 subsections, each with a single pixel, and is therefore not relevant either, hence the last relevant mip index will be NumMips - 3
	*/
	UPROPERTY()
	TArray<double> MipToMipMaxDeltas;

#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(all, "CollisionComponent has been deprecated")
	UPROPERTY()
	TLazyObjectPtr<ULandscapeHeightfieldCollisionComponent> CollisionComponent_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // !WITH_EDITORONLY_DATA

private:
	/** Reference to associated collision component */
	UPROPERTY()
	TObjectPtr<ULandscapeHeightfieldCollisionComponent> CollisionComponentRef;

	/** Store  */ 
	UPROPERTY(Transient)
	bool bUserTriggeredChangeRequested = false;
	
	UPROPERTY(Transient)
	bool bNaniteActive;

#if WITH_EDITORONLY_DATA
	/** Unique ID for this component, used for caching during distributed lighting */
	UPROPERTY()
	FGuid LightingGuid;

	/** Edit Layers that have data for this component store it here */
	UPROPERTY()
	TMap<FGuid, FLandscapeLayerComponentData> LayersData;

	/** Obsolete edit layers (Edit layers not tracked by ALandscape). During registration, moves obsolete layer data out of active LayersData into
	* this transient storage. When obsolete layers exists, creates an additional duplicate of the final merged data on initial registration mapped to FGuid() */
	UPROPERTY(Transient)
	TMap<FGuid, FLandscapeLayerComponentData> ObsoleteEditLayerData;

	// Final layer data
	UPROPERTY(Transient)
	TArray<TObjectPtr<ULandscapeWeightmapUsage>> WeightmapTexturesUsage;

	UPROPERTY(Transient)
	uint32 LayerUpdateFlagPerMode;

	UPROPERTY(Transient)
	bool bPendingCollisionDataUpdate;

	UPROPERTY(Transient)
	bool bPendingLayerCollisionDataUpdate;

	bool bUndoChangedWeightmapAllocs = false;

	// Physical material has been invalidated and no new physical material task has been started since then
	bool bPendingPhysicalMaterialInvalidation = false;

	/** Dirtied collision height region when painting (only used by Landscape Layer System) */
	FIntRect LayerDirtyCollisionHeightData;
#endif // WITH_EDITORONLY_DATA

	/** Heightmap texture reference */
	UPROPERTY()
	TObjectPtr<UTexture2D> HeightmapTexture;

	/** List of layers, and the weightmap and channel they are stored */
	UPROPERTY()
	TArray<FWeightmapLayerAllocationInfo> WeightmapLayerAllocations;

	/** Weightmap texture reference */
	UPROPERTY()
	TArray<TObjectPtr<UTexture2D>> WeightmapTextures;

	UPROPERTY(EditAnywhere, Category = LandscapeComponent)
	TArray<FLandscapePerLODMaterialOverride> PerLODOverrideMaterials;

#if WITH_EDITORONLY_DATA
	/** A CRC of everything that can lead NamedGrassTypes to be changed (e.g. the material changing, or grass type overrides changing) */
	uint32 GrassTypesCRC = 0;

	UPROPERTY()
	TArray<TObjectPtr<ULandscapeGrassType>> GrassTypes_DEPRECATED;
#endif // WITH_EDITORONLY_DATA

	/** Cached list of grass types supported by the component's material.
	* This is needed in a cooked build, as the grass types list is not available
	* on the cooked material.
	* Call UpdateGrassTypes() to ensure this array is up to date */
	UPROPERTY()
	TMap<FName, TObjectPtr<ULandscapeGrassType>> NamedGrassTypes;

public:
	// Non-serialized runtime cache of values derived from the assigned grass types.
	// Call ALandscapeProxy::UpdateGrassTypeSummary() to update.
	struct FGrassTypeSummary
	{
		bool bInvalid = true;
		bool bHasAnyGrass = true;
		double MaxInstanceDiscardDistance = DBL_MAX;
	};
	FGrassTypeSummary GrassTypeSummary;
	inline bool IsGrassTypeSummaryValid() { return !GrassTypeSummary.bInvalid; }

	/** Invalidate the grass type summary.  Call whenever grass types are changed to indicate that the summary values are out of date. */
	LANDSCAPE_API void InvalidateGrassTypeSummary();

	/** Uniquely identifies this component's built map data. */
	UPROPERTY()
	FGuid MapBuildDataId;

	/** Heightfield mipmap used to generate collision */
	UPROPERTY(EditAnywhere, Category=LandscapeComponent)
	int32 CollisionMipLevel;

	/** Heightfield mipmap used to generate simple collision */
	UPROPERTY(EditAnywhere, Category=LandscapeComponent)
	int32 SimpleCollisionMipLevel;

	/** Allows overriding the landscape bounds. This is useful if you distort the landscape with world-position-offset, for example
	 *  Extension value in the negative Z axis, positive value increases bound size */
	UPROPERTY(EditAnywhere, Category=LandscapeComponent)
	float NegativeZBoundsExtension;

	/** Allows overriding the landscape bounds. This is useful if you distort the landscape with world-position-offset, for example
	 *  Extension value in the positive Z axis, positive value increases bound size */
	UPROPERTY(EditAnywhere, Category=LandscapeComponent)
	float PositiveZBoundsExtension;

	/** StaticLightingResolution overriding per component, default value 0 means no overriding */
	UPROPERTY(EditAnywhere, Category=LandscapeComponent, meta=(ClampMax = 4096))
	float StaticLightingResolution;

	/** Forced LOD level to use when rendering */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=LandscapeComponent)
	int32 ForcedLOD;

	/** LOD level Bias to use when rendering */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=LandscapeComponent)
	int32 LODBias;

	/** When the component has registered a heightmap texture edge fixup,
	  * a reference to the edge fixup and the group are stored here for easy access
	  *  (the fixup is also referenced via the heightmap AssetUserDatas,
	  *  and the group can be accessed via the ULandcapeSubsystem)
	  */
	UPROPERTY(Transient, DuplicateTransient, NonTransactional)
	TObjectPtr<ULandscapeHeightmapTextureEdgeFixup> RegisteredEdgeFixup;
	FLandscapeGroup* RegisteredLandscapeGroup;

#if WITH_EDITORONLY_DATA
	/**	Legacy irrelevant lights */
	UE_DEPRECATED(all, "IrrelevantLights is officially deprecated now")
	UPROPERTY()
	TArray<FGuid> IrrelevantLights_DEPRECATED;

	/** LOD level Bias to use when lighting building via lightmass, -1 Means automatic LOD calculation based on ForcedLOD + LODBias */
	UPROPERTY(EditAnywhere, Category=LandscapeComponent)
	int32 LightingLODBias;

	// List of layers allowed to be painted on this component
	UPROPERTY(EditAnywhere, Category=LandscapeComponent)
	TArray<TObjectPtr<ULandscapeLayerInfoObject>> LayerAllowList;

	/** Pointer to data shared with the render thread, used by the editor tools */
	UPROPERTY(Transient, DuplicateTransient, NonTransactional)
	FLandscapeEditToolRenderData EditToolRenderData;

	/** Hash of source for mobile generated data. Used determine if we need to re-generate mobile pixel data. */
	UPROPERTY(NonPIEDuplicateTransient)
	FGuid MobileDataSourceHash;

	/** Represent the chosen material for each LOD */
	UPROPERTY(NonPIEDuplicateTransient)
	TMap<TObjectPtr<UMaterialInterface>, int8> MaterialPerLOD;

	/** Represents hash of last weightmap usage update */
	uint32 WeightmapsHash;

	UPROPERTY()
	uint32 SplineHash;

	/** Represents hash for last PhysicalMaterialTask */
	UPROPERTY()
	uint32 PhysicalMaterialHash;

	/** Represents last saved hash for PhysicalMaterialTask */
	UPROPERTY(Transient)
	uint32 LastSavedPhysicalMaterialHash;

	UE_DEPRECATED(all, "MobileMaterialInterface has been deprecated and will be removed in a future version")
	UPROPERTY(NonPIEDuplicateTransient)
	TObjectPtr<UMaterialInterface> MobileMaterialInterface_DEPRECATED;

    UE_DEPRECATED(all, "MobileCombinationMaterialInstance has been deprecated and will be removed in a future version")
	UPROPERTY(NonPIEDuplicateTransient)
	TObjectPtr<UMaterialInstanceConstant> MobileCombinationMaterialInstance_DEPRECATED;
#endif // WITH_EDITORONLY_DATA

	/** Material interfaces used for mobile */
	UPROPERTY(NonPIEDuplicateTransient)
	TArray<TObjectPtr<UMaterialInterface>> MobileMaterialInterfaces;

	/** Generated weightmap textures used for mobile. The first entry is also used for the normal map. 
	  * Serialized only when cooking or loading cooked builds. */
	UPROPERTY(NonPIEDuplicateTransient)
	TArray<TObjectPtr<UTexture2D>> MobileWeightmapTextures;

	UPROPERTY(NonPIEDuplicateTransient)
	TObjectPtr<UTexture2DArray> MobileWeightmapTextureArray;
	
	/** Layer allocations used by mobile.*/
	UPROPERTY()
	TArray<FWeightmapLayerAllocationInfo> MobileWeightmapLayerAllocations;

#if WITH_EDITORONLY_DATA
	/** The editor needs to save out the combination MIC we'll use for mobile, 
	  because we cannot generate it at runtime for standalone PIE games */
	UPROPERTY(NonPIEDuplicateTransient)
	TArray<TObjectPtr<UMaterialInstanceConstant>> MobileCombinationMaterialInstances;
#endif // WITH_EDITORONLY_DATA

public:
	/** Grass data for generation **/
	TSharedRef<FLandscapeComponentGrassData, ESPMode::ThreadSafe> GrassData;
	
	// This wrapper is needed to filter out exclude boxes that are completely inside of another exclude box
	struct FExcludeBox
	{
		FBox Box;

		FExcludeBox() = default;
		FExcludeBox(const FBox& InBox) : Box(InBox) {}

		bool operator==(const FExcludeBox& Other) const
		{
			return Box.IsInsideOrOn(Other.Box);
		}
	};
	TArray<FExcludeBox> ActiveExcludedBoxes;
	uint32 ChangeTag;

#if WITH_EDITOR
	/** Physical material update task */
	FLandscapePhysicalMaterialRenderTask PhysicalMaterialTask;
	uint32 CalculatePhysicalMaterialTaskHash() const;

	/**
	 * Get the physical materials that are configured by the landscape component graphical material.
	 * Returns false if there are no non-null physical materials. (We probably don't want to use if no physical material connections are bound.)
	 */
	bool GetRenderPhysicalMaterials(TArray<UPhysicalMaterial*>& OutPhysicalMaterials) const;
#endif // WITH_EDITOR

	//~ Begin UObject Interface.	
	virtual void PostInitProperties() override;	
	virtual void Serialize(FArchive& Ar) override;
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	virtual void BeginDestroy() override;
	virtual void PostLoad() override;
#if WITH_EDITORONLY_DATA
	static void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
#endif

#if WITH_EDITOR
	virtual void BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform) override;
	virtual void PreEditUndo() override;
	virtual void PostEditUndo() override;
	virtual void PreEditChange(FProperty* PropertyThatWillChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface

	LANDSCAPE_API void UpdateEditToolRenderData();

	/** Fix up component layers, weightmaps */
	LANDSCAPE_API void FixupWeightmaps();
	LANDSCAPE_API void FixupWeightmaps(const FGuid& InEditLayerGuid);

	/** Repair invalid texture data that might have been introduced by a faulty version. Returns the list of repaired textures  */
	TArray<UTexture*> RepairInvalidTextures();

	// Update layer allow list to include the currently painted layers
	LANDSCAPE_API void UpdateLayerAllowListFromPaintedLayers();
	
	//~ Begin UPrimitiveComponent Interface.
	virtual bool GetLightMapResolution( int32& Width, int32& Height ) const override;
	virtual int32 GetStaticLightMapResolution() const override;
	virtual void GetLightAndShadowMapMemoryUsage( int32& LightMapMemoryUsage, int32& ShadowMapMemoryUsage ) const override;
	virtual void GetStaticLightingInfo(FStaticLightingPrimitiveInfo& OutPrimitiveInfo,const TArray<ULightComponent*>& InRelevantLights,const FLightingBuildOptions& Options) override;
	virtual void AddMapBuildDataGUIDs(TSet<FGuid>& InGUIDs) const override;
#endif
	virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const override;
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	virtual ELightMapInteractionType GetStaticLightingType() const override;
	virtual void GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const override;
	virtual bool IsPrecomputedLightingValid() const override;
	virtual void OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType TeleportType) override;

	virtual TArray<URuntimeVirtualTexture*> const& GetRuntimeVirtualTextures() const override;
	virtual ERuntimeVirtualTextureMainPassType GetVirtualTextureRenderPassType() const override;

	// Returns the heightmap for this component. If InReturnEditingHeightmap is passed, returns the currently active edit layer's heightmap : 
	LANDSCAPE_API UTexture2D* GetHeightmap(bool InReturnEditingHeightmap = false) const;
	// Returns the heightmap for this component and the edit layer specified by InLayerGuid. If InLayerGuid is invalid, returns the final (base) heightmap : 
	LANDSCAPE_API UTexture2D* GetHeightmap(const FGuid& InLayerGuid) const;
	LANDSCAPE_API TArray<TObjectPtr<UTexture2D>>& GetWeightmapTextures(bool InReturnEditingWeightmap = false);
	LANDSCAPE_API const TArray<UTexture2D*>& GetWeightmapTextures(bool InReturnEditingWeightmap = false) const;
	LANDSCAPE_API TArray<TObjectPtr<UTexture2D>>& GetWeightmapTextures(const FGuid& InLayerGuid);
	LANDSCAPE_API const TArray<UTexture2D*>& GetWeightmapTextures(const FGuid& InLayerGuid) const;
	LANDSCAPE_API const TArray<UTexture2D*>& GetRenderedWeightmapTexturesForFeatureLevel(ERHIFeatureLevel::Type FeatureLevel) const;

	LANDSCAPE_API TArray<FWeightmapLayerAllocationInfo>& GetWeightmapLayerAllocations(bool InReturnEditingWeightmap = false);
	LANDSCAPE_API const TArray<FWeightmapLayerAllocationInfo>& GetWeightmapLayerAllocations(bool InReturnEditingWeightmap = false) const;
	LANDSCAPE_API TArray<FWeightmapLayerAllocationInfo>& GetWeightmapLayerAllocations(const FGuid& InLayerGuid);
	LANDSCAPE_API const TArray<FWeightmapLayerAllocationInfo>& GetWeightmapLayerAllocations(const FGuid& InLayerGuid) const;

	LANDSCAPE_API const TArray<UTexture2D*>& GetCurrentRuntimeWeightmapTextures() const;
	LANDSCAPE_API TArray<FWeightmapLayerAllocationInfo>& GetCurrentRuntimeWeightmapLayerAllocations();
	LANDSCAPE_API const TArray<FWeightmapLayerAllocationInfo>& GetCurrentRuntimeWeightmapLayerAllocations() const;

	const TArray<FLandscapePerLODMaterialOverride>& GetPerLODOverrideMaterials() const { return PerLODOverrideMaterials; }
	void SetPerLODOverrideMaterials(const TArray<FLandscapePerLODMaterialOverride>& InValue) { PerLODOverrideMaterials = InValue; }

	LANDSCAPE_API void SetHeightmap(UTexture2D* NewHeightmap);
	LANDSCAPE_API void SetWeightmapTextures(const TArray<UTexture2D*>& InNewWeightmapTextures, bool InApplyToEditingWeightmap = false);
	void SetWeightmapTexturesInternal(const TArray<UTexture2D*>& InNewWeightmapTextures, const FGuid& InEditLayerGuid);

#if WITH_EDITOR
	LANDSCAPE_API void SetWeightmapLayerAllocations(const TArray<FWeightmapLayerAllocationInfo>& InNewWeightmapLayerAllocations);
	LANDSCAPE_API uint32 ComputeLayerHash(bool InReturnEditingHash = true) const;

	LANDSCAPE_API void SetWeightmapTexturesUsage(const TArray<ULandscapeWeightmapUsage*>& InNewWeightmapTexturesUsage, bool InApplyToEditingWeightmap = false);
	void SetWeightmapTexturesUsageInternal(const TArray<ULandscapeWeightmapUsage*>& InNewWeightmapTexturesUsage, const FGuid& InEditLayerGuid);

	LANDSCAPE_API TArray<TObjectPtr<ULandscapeWeightmapUsage>>& GetWeightmapTexturesUsage(bool InReturnEditingWeightmap = false);
	LANDSCAPE_API const TArray<ULandscapeWeightmapUsage*>& GetWeightmapTexturesUsage(bool InReturnEditingWeightmap = false) const;
	LANDSCAPE_API TArray<TObjectPtr<ULandscapeWeightmapUsage>>& GetWeightmapTexturesUsage(const FGuid& InLayerGuid);
	LANDSCAPE_API const TArray<ULandscapeWeightmapUsage*>& GetWeightmapTexturesUsage(const FGuid& InLayerGuid) const;
	LANDSCAPE_API void InitializeLayersWeightmapUsage(const FGuid& InLayerGuid);

	LANDSCAPE_API bool HasLayersData() const;
	LANDSCAPE_API const FLandscapeLayerComponentData* GetLayerData(const FGuid& InLayerGuid) const;
	LANDSCAPE_API FLandscapeLayerComponentData* GetLayerData(const FGuid& InLayerGuid);

	void CopyComponentDataToEditLayer(const FGuid& InSourceEditLayerGuid, const FGuid& InDestEditLayerGuid, bool bInUseObsoleteLayerData, TSet<UTexture2D*>& OutProcessedHeightmaps, TSet<UTexture2D*>& OutProcessedWeightmaps);

	// Obsolete edit layer helpers
	const TMap<FGuid, FLandscapeLayerComponentData>& GetObsoleteEditLayersData() const;
	const FLandscapeLayerComponentData* GetObsoleteEditLayerData(const FGuid& InLayerGuid) const; 
	void RemoveObsoleteEditLayerData(const FGuid& InLayerGuid);
	void InitializeObsoleteEditLayerData(const TSet<FGuid>& InParentEditLayerGuids);

	LANDSCAPE_API void AddLayerData(const FGuid& InLayerGuid, const FLandscapeLayerComponentData& InData);
	LANDSCAPE_API void AddDefaultLayerData(const FGuid& InLayerGuid, const TArray<ULandscapeComponent*>& InComponentsUsingHeightmap, TMap<UTexture2D*, UTexture2D*>& InOutCreatedHeightmapTextures);
	LANDSCAPE_API void RemoveLayerData(const FGuid& InLayerGuid);
	LANDSCAPE_API void ForEachLayer(TFunctionRef<void(const FGuid&, struct FLandscapeLayerComponentData&)> Fn);

	/** Get the Landscape Actor's editing layer data */
	FLandscapeLayerComponentData* GetEditingLayer();
	const FLandscapeLayerComponentData* GetEditingLayer() const;

	/** Get the Landscape Actor's editing layer GUID */
	FGuid GetEditingLayerGUID() const;

	void SetPendingCollisionDataUpdate(bool bInPendingCollisionDataUpdate) { bPendingCollisionDataUpdate = bInPendingCollisionDataUpdate; }
	bool GetPendingCollisionDataUpdate() const { return bPendingCollisionDataUpdate; }
	void SetPendingLayerCollisionDataUpdate(bool bInPendingLayerCollisionDataUpdate) { bPendingLayerCollisionDataUpdate = bInPendingLayerCollisionDataUpdate; }
	bool GetPendingLayerCollisionDataUpdate() const { return bPendingLayerCollisionDataUpdate; }

	void SetUndoChangedWeightmapAllocs(bool bInVal) { bUndoChangedWeightmapAllocs = bInVal; }
	bool GetUndoChangedWeightmapAllocs() const { return bUndoChangedWeightmapAllocs; }
#endif // WITH_EDITOR

	virtual bool IsShown(const FEngineShowFlags& ShowFlags) const override;

#if WITH_EDITOR
	virtual int32 GetNumMaterials() const override;
	virtual UMaterialInterface* GetMaterial(int32 ElementIndex) const override;
	virtual void SetMaterial(int32 ElementIndex, UMaterialInterface* Material) override;
	virtual void PreFeatureLevelChange(ERHIFeatureLevel::Type PendingFeatureLevel) override;
#endif
	
	/** Collect all the PSO precache data used by the landscape component */
	LANDSCAPE_API virtual void CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams) override;
	//~ End UPrimitiveComponent Interface.

	//~ Begin USceneComponent Interface.
#if WITH_EDITOR
	virtual bool GetMaterialPropertyPath(int32 ElementIndex, UObject*& OutOwner, FString& OutPropertyPath, FProperty*& OutProperty) override;
#endif // WITH_EDITOR
	virtual void DestroyComponent(bool bPromoteChildren = false) override;
	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ End USceneComponent Interface.

	//~ Begin UActorComponent Interface.
	virtual void OnRegister() override;
	virtual void OnUnregister() override;
#if WITH_EDITOR
	virtual void InvalidateLightingCacheDetailed(bool bInvalidateBuildEnqueuedLighting, bool bTranslationOnly) override;
#endif
	virtual void PropagateLightingScenarioChange() override;
	virtual bool IsHLODRelevant() const override;
	//~ End UActorComponent Interface.

	/** Gets the landscape info object for this landscape */
	LANDSCAPE_API ULandscapeInfo* GetLandscapeInfo() const;

	UFUNCTION(BlueprintCallable, Category = "Landscape|Grass", meta=(DisplayName = "GetGrassTypes", Tooltip = "Returns the grass types used by the landscape material on this component"))
	TArray<ULandscapeGrassType*> GetGrassTypesBP() const;

	/** Returns the array of grass types used by the landscape material. Call UpdateGrassTypes first to ensure this array is up to date. */
	UE_DEPRECATED(5.8, "Use GetNamedGrassTypes")
	TArray<TObjectPtr<ULandscapeGrassType>> GetGrassTypes() const;

	const TMap<FName, TObjectPtr<ULandscapeGrassType>>& GetNamedGrassTypes() const { return NamedGrassTypes; }

	/** Temporarily sets the grass type for this component. Any call to UpdateGrassTypes may override what has been set using this method. */
	void SetNamedGrassTypes(const TMap<FName, TObjectPtr<ULandscapeGrassType>>& InNamedGrassTypes)
	{
		NamedGrassTypes = InNamedGrassTypes;
		InvalidateGrassTypeSummary();
	}
	
	bool MaterialHasGrass() const { return !GetNamedGrassTypes().IsEmpty(); }

	double GetGrassTypesMaxDiscardDistance() const { return GrassTypeSummary.MaxInstanceDiscardDistance; }
	void SetGrassTypesMaxDiscardDistance(const float InGrassTypesMaxDiscardDistance) { GrassTypeSummary.MaxInstanceDiscardDistance = InGrassTypesMaxDiscardDistance; GrassTypeSummary.bInvalid = false; }

	/** If the LandscapeMaterial has changed, updates the GrassTypes array. Returns true if the GrassTypes array was updated. */
	LANDSCAPE_API bool UpdateGrassTypes(bool bForceUpdate = false);

	/** Installs new or updates existing TextureUserDatas for this component: up to one ULandscapeHeightmapTextureEdgeFixup and
	  * up to one UTextureMipDataProviderFactory (either ULandscapeTextureStorageProviderFactory or ULandscapeTextureMipEdgeOverrideFactory).
	  * 
	  * The edge fixup class ULandscapeHeightmapTextureEdgeFixup is used to store edge snapshots and manage edge blending with neighboring heightmap textures.
	  * 
	  * ULandscapeTextureStorageProviderFactory is used to stream custom-compressed heightmap texture mip data,
	  *   and it supports also applying edge blending from the edge fixup class while streaming its mips.
	  * 
	  * If bUseEdgeFixup is enabled, but bUseCompression is not, the ULandscapeTextureMipEdgeOverrideFactory is used to apply the edge blending
	  *   on top of the standard built-in texture mip streaming.
	  * 
	  * This function will ensure an edge fixup class exists if bUseEdgeFixup is true, or it will remove any existing ones otherwise.
	  * 
	  * [EDITOR ONLY] It will capture a new snapshot for the edge fixup, when bUpdateSnapshotNow is true.  Ignored in non-editor builds.
	  * 
	  * [EDITOR ONLY] It will compress heightmaps to a ULandscapeTextureStorageProviderFactory when bUseCompression is true.  Ignored in non-editor builds.
	  *    NOTE: It will NOT remove an existing one if false -- applying compression is a one-way street, and should only be used in cooked content.
	  */
	ULandscapeHeightmapTextureEdgeFixup* InstallOrUpdateTextureUserDatas(bool bUseEdgeFixup, bool bUseCompression = false, bool bUpdateSnapshotNow = true, int32 HeightmapCompressionMipThreshold = 32);

#if WITH_EDITOR
	/** Deletes a layer from this component if it does not contain data, calling DeleteLayerAllocation. */
	bool DeleteLayerIfAllZero(const FGuid& InEditLayerGuid, const uint8* const TexDataPtr, int32 TexSize, int32 LayerIdx, bool bShouldDirtyPackage);

	/** Deletes a material layer from the current edit layer on this component, removing all its data, adjusting other layer's weightmaps if necessary, etc. */
	LANDSCAPE_API void DeleteLayer(ULandscapeLayerInfoObject* LayerInfo, FLandscapeEditDataInterface& LandscapeEdit);
	
	/** Deletes a material layer from the specified edit layer on this component, removing all its data, adjusting other layer's weightmaps if necessary, etc. */
	void DeleteLayerInternal(ULandscapeLayerInfoObject* LayerInfo, FLandscapeEditDataInterface& LandscapeEdit, const FGuid& InEditLayerGuid);

	/** Deletes a layer from this component, but doesn't do anything else (assumes the user knows what he's doing, use DeleteLayer otherwise) */
	void DeleteLayerAllocation(const FGuid& InEditLayerGuid, int32 InLayerAllocationIdx, bool bInShouldDirtyPackage);

	/** Fills a layer to 100% on this component, adding it if needed and removing other layers that get painted away.  Uses the edit layer specified by LandscapeEdit. */
	LANDSCAPE_API void FillLayer(ULandscapeLayerInfoObject* LayerInfo, FLandscapeEditDataInterface& LandscapeEdit);

	/** Replaces one layerinfo on this component with another */
	LANDSCAPE_API void ReplaceLayer(ULandscapeLayerInfoObject* FromLayerInfo, ULandscapeLayerInfoObject* ToLayerInfo, FLandscapeEditDataInterface& LandscapeEdit);
	void ReplaceLayerInternal(ULandscapeLayerInfoObject* FromLayerInfo, ULandscapeLayerInfoObject* ToLayerInfo, FLandscapeEditDataInterface& LandscapeEdit, const FGuid& InEditLayerGUID);

#endif // WITH_EDITOR

	/** Destroys grass map data stored on the component */
	void RemoveGrassMap();

	/* Could a grassmap currently be generated, disregarding whether our textures are streamed in? */
	LANDSCAPE_API bool CanRenderGrassMap() const;

#if WITH_EDITOR
	/** Computes a hash representing the state of the material and grasstypes used by this component. */
	LANDSCAPE_API uint32 ComputeGrassMapGenerationHash() const;

	/* Returns true if the component HAS grass data, but it is not up to date */
	bool IsGrassMapOutdated() const;

	/** Renders the heightmap of this component (including material world-position-offset) at the specified LOD */
	TArray<uint16> RenderWPOHeightmap(int32 LOD);

	/* Serialize all hashes/guids that record the current state of this component */
	void SerializeStateHashes(FArchive& Ar);

	// Generates mobile platform data for this component
	void GenerateMobileWeightmapLayerAllocations();
	void GenerateMobilePlatformPixelData(bool bIsCooking, const ITargetPlatform* TargetPlatform);

	/** Generate mobile data if it's missing or outdated */
	void CheckGenerateMobilePlatformData(bool bIsCooking, const ITargetPlatform* TargetPlatform);

	virtual TSubclassOf<class UHLODBuilder> GetCustomHLODBuilderClass() const override;
#endif

	int32 GetCurrentRuntimeMaterialInstanceCount() const;
	class UMaterialInterface* GetCurrentRuntimeMaterialInterface(int32 InIndex) const;

	LANDSCAPE_API int32 GetMaterialInstanceCount(bool InDynamic = true) const;
	LANDSCAPE_API class UMaterialInstance* GetMaterialInstance(int32 InIndex, bool InDynamic = true) const;

	/** Gets the landscape material instance dynamic for this component */
	UFUNCTION(BlueprintCallable, Category = "Landscape|Runtime|Material")
	class UMaterialInstanceDynamic* GetMaterialInstanceDynamic(int32 InIndex) const;

	/** Gets the landscape paint layer weight value at the given position using LandscapeLayerInfo . Returns 0 in case it fails. */
	UFUNCTION(BlueprintCallable, Category = "Landscape|Editor")
	LANDSCAPE_API float EditorGetPaintLayerWeightAtLocation(const FVector& InLocation, ULandscapeLayerInfoObject* PaintLayer);

	/** Gets the landscape paint layer weight value at the given position using layer name. Returns 0 in case it fails. */
	UFUNCTION(BlueprintCallable, Category = "Landscape|Editor")
	LANDSCAPE_API float EditorGetPaintLayerWeightByNameAtLocation(const FVector& InLocation, const FName InPaintLayerName);
		
	/** Get the landscape actor associated with this component. */
	LANDSCAPE_API ALandscape* GetLandscapeActor() const;

	/** Get the level in which the owning actor resides */
	ULevel* GetLevel() const;

#if WITH_EDITOR
	/** Returns all generated textures and material instances used by this component. */
	LANDSCAPE_API void GetGeneratedTexturesAndMaterialInstances(TArray<UObject*>& OutTexturesAndMaterials) const;
	LANDSCAPE_API TArray<UTexture*> GetGeneratedTextures() const;
	LANDSCAPE_API TArray<UMaterialInstance*> GetGeneratedMaterialInstances() const;
#endif // WITH_EDITOR

	/** Gets the landscape proxy actor which owns this component */
	LANDSCAPE_API ALandscapeProxy* GetLandscapeProxy() const;

	/** @return Component section base as FIntPoint */
	FIntPoint GetSectionBase() const
	{
		return FIntPoint(SectionBaseX, SectionBaseY);
	}

	/** @return the component's coordinates (aka index, aka component key) in the entire landscape */
	FIntPoint GetComponentKey() const
	{
		return GetSectionBase() / ComponentSizeQuads;
	}

	/** @return a user-friendly name to easily identify this component when dumping debug info, for example */
	LANDSCAPE_API FString GetDebugName() const;

	/** @param InSectionBase new section base for a component */
	void SetSectionBase(FIntPoint InSectionBase)
	{
		SectionBaseX = InSectionBase.X;
		SectionBaseY = InSectionBase.Y;
	}

	/** 
	* Computes the number of mips that are actually usable, i.e.:
	*  - For 1x1 subsection, the last mip is not usable (it has a single vertex)
	*  - For 2x2 subsections, the last 2 mips are not usable (a single vertex per subsection)
	*/
	int32 GetNumRelevantMips() const;

	/** @todo document */
	const FGuid& GetLightingGuid() const
	{
#if WITH_EDITORONLY_DATA
		return LightingGuid;
#else
		static const FGuid NullGuid( 0, 0, 0, 0 );
		return NullGuid;
#endif // WITH_EDITORONLY_DATA
	}

	/** @todo document */
	void SetLightingGuid()
	{
#if WITH_EDITORONLY_DATA
		LightingGuid = FGuid::NewGuid();
#endif // WITH_EDITORONLY_DATA
	}

	FGuid GetMapBuildDataId() const
	{
		return MapBuildDataId;
	}

	LANDSCAPE_API const FMeshMapBuildData* GetMeshMapBuildData() const;

	/** Initialize the landscape component */
	LANDSCAPE_API void Init(int32 InBaseX, int32 InBaseY, int32 InComponentSizeQuads, int32 InNumSubsections, int32 InSubsectionSizeQuads);

	/** Returns the component's LandscapeMaterial, or the Component's OverrideLandscapeMaterial if set */
	LANDSCAPE_API UMaterialInterface* GetLandscapeMaterial(int8 InLODIndex = INDEX_NONE) const;

	/** Returns the components's LandscapeHoleMaterial, or the Component's OverrideLandscapeHoleMaterial if set */
	LANDSCAPE_API UMaterialInterface* GetLandscapeHoleMaterial() const;

	/**
	 * Retrieve the list of all materials that might be used by this landscape component (LandscapeMaterial, LandscapeHoleMaterial, etc.)
	 * @param OutMaterials list of unique materials retrieved. The materials found will be appended to the list
	 */
	LANDSCAPE_API void RetrieveAllLandscapeMaterials(TSet<UMaterialInterface*>& OutMaterials) const;

	FBox ComputeApproximateBounds();

#if WITH_EDITOR
	/**
	 * Recalculate cached bounds using height values.  Returns true when the bounds were changed.
	 */
	LANDSCAPE_API bool UpdateCachedBounds(bool bInApproximateBounds = false);

	/**
	 * Update the MaterialInstance parameters to match the layer and weightmaps for this component
	 * Creates the MaterialInstance if it doesn't exist.
	 */
	LANDSCAPE_API void UpdateMaterialInstances();
	LANDSCAPE_API void UpdateMaterialInstances(FMaterialUpdateContext& InOutMaterialContext, TArray<class FComponentRecreateRenderStateContext>& InOutRecreateRenderStateContext);

	// Internal implementation of UpdateMaterialInstances, not safe to call directly
	void UpdateMaterialInstances_Internal(FMaterialUpdateContext& Context);

	/** Helper function for UpdateMaterialInstance to get Material without set parameters */
	UMaterialInstanceConstant* GetCombinationMaterial(FMaterialUpdateContext* InMaterialUpdateContext, const TArray<FWeightmapLayerAllocationInfo>& Allocations, int8 InLODIndex, bool bMobile = false) const;
	/**
	 * Generate mipmaps for height and tangent data.
	 * @param HeightmapTextureMipData - array of pointers to the locked mip data.
	 *           This should only include the mips that are generated directly from this component's data
	 *           ie where each subsection has at least 2 vertices.
	* @param ComponentX1 - region of texture to update in component space, MAX_int32 meant end of X component in ALandscape::Import()
	* @param ComponentY1 - region of texture to update in component space, MAX_int32 meant end of Y component in ALandscape::Import()
	* @param ComponentX2 (optional) - region of texture to update in component space
	* @param ComponentY2 (optional) - region of texture to update in component space
	* @param TextureDataInfo - FLandscapeTextureDataInfo pointer, to notify of the mip data region updated.
	 */
	void GenerateHeightmapMips(TArray<FColor*>& HeightmapTextureMipData, int32 ComponentX1=0, int32 ComponentY1=0, int32 ComponentX2=MAX_int32, int32 ComponentY2=MAX_int32, FLandscapeTextureDataInfo* TextureDataInfo=nullptr);

	/**
	 * if bClear is true, clears all mips on the specified texture to zero
	 * if bClear is false, clears all mips EXCEPT MIP 0 to zero
	 * Updates the texture hash appropriately for the given usage and texture type
	 */
	LANDSCAPE_API static void CreateEmptyTextureMips(UTexture2D* Texture, ELandscapeTextureUsage TextureUsage, ELandscapeTextureType TextureType, bool bClear = false);

	/**
	 * Generate mipmaps for weightmap
	 * Assumes all weightmaps are unique to this component.
	 * @param WeightmapTextureBaseMipData: array of pointers to each of the weightmaps' base locked mip data.
	 */
	template<typename DataType>

	/** @todo document */
	static void GenerateMipsTempl(int32 InNumSubsections, int32 InSubsectionSizeQuads, UTexture2D* WeightmapTexture, DataType* BaseMipData);

	/** @todo document */
	static void GenerateWeightmapMips(int32 InNumSubsections, int32 InSubsectionSizeQuads, UTexture2D* WeightmapTexture, FColor* BaseMipData);

	/**
	 * Update mipmaps for existing weightmap texture
	 */
	template<typename DataType>

	/** @todo document */
	static void UpdateMipsTempl(int32 InNumSubsections, int32 InSubsectionSizeQuads, UTexture2D* WeightmapTexture, TArray<DataType*>& WeightmapTextureMipData, int32 ComponentX1=0, int32 ComponentY1=0, int32 ComponentX2=MAX_int32, int32 ComponentY2=MAX_int32, FLandscapeTextureDataInfo* TextureDataInfo=nullptr);

	/** @todo document */
	LANDSCAPE_API static void UpdateWeightmapMips(int32 InNumSubsections, int32 InSubsectionSizeQuads, UTexture2D* WeightmapTexture, TArray<FColor*>& WeightmapTextureMipData, int32 ComponentX1=0, int32 ComponentY1=0, int32 ComponentX2=MAX_int32, int32 ComponentY2=MAX_int32, FLandscapeTextureDataInfo* TextureDataInfo=nullptr);

	/** @todo document */
	static void UpdateDataMips(int32 InNumSubsections, int32 InSubsectionSizeQuads, UTexture2D* Texture, TArray<uint8*>& TextureMipData, int32 ComponentX1=0, int32 ComponentY1=0, int32 ComponentX2=MAX_int32, int32 ComponentY2=MAX_int32, FLandscapeTextureDataInfo* TextureDataInfo=nullptr);

	/**
	 * Create or updates collision component height data
	 * @param HeightmapTextureMipData: heightmap data
	 * @param ComponentX1, ComponentY1, ComponentX2, ComponentY2: region to update
	 * @param bUpdateBounds: Whether to update bounds from render component.
	 */
	void UpdateCollisionHeightData(const FColor* HeightmapTextureMipData, const FColor* SimpleCollisionHeightmapTextureData, int32 ComponentX1=0, int32 ComponentY1=0, int32 ComponentX2=MAX_int32, int32 ComponentY2=MAX_int32, bool bUpdateBounds=false, bool bInUpdateHeightfieldRegion=true);

	/**
	 * Deletes Collision Component
	 */
	void DestroyCollisionData();

	/** Updates collision component height data for the entire component, locking and unlocking heightmap textures
	 */
	LANDSCAPE_API void UpdateCollisionData(bool bInUpdateHeightfieldRegion = true);

	/** Cumulates component's dirtied collision region that will need to be updated (used by Layer System)*/
	void UpdateDirtyCollisionHeightData(FIntRect Region);

	/** Clears component's dirtied collision region (used by Layer System)*/
	void ClearDirtyCollisionHeightData();

	/**
	 * Update collision component dominant layer data
	 * @param WeightmapTextureMipData: weightmap data
	 * @param ComponentX1, ComponentY1, ComponentX2, ComponentY2: region to update
	 * @param Whether to update bounds from render component.
	 */
	void UpdateCollisionLayerData(const FColor* const* WeightmapTextureMipData, const FColor* const* const SimpleCollisionWeightmapTextureMipData, int32 ComponentX1=0, int32 ComponentY1=0, int32 ComponentX2=MAX_int32, int32 ComponentY2=MAX_int32);

	/**
	 * Update collision component dominant layer data for the whole component, locking and unlocking the weightmap textures.
	 */
	LANDSCAPE_API void UpdateCollisionLayerData();

	/** Invalidate current or pending physical material data */
	void InvalidatePhysicalMaterial();
	/** Returns true if we can currently update physical materials. */
	bool CanUpdatePhysicalMaterial();
	/** Update physical material render tasks. */
	void UpdatePhysicalMaterialTasks();
	/** Write the physical materials into the LandscapeComponent from the Render & Immediately Rebuild physics if requested */
	void FinalizePhysicalMaterial(bool bInImmediatePhysicsRebuild);
	/** Update collision component physical materials from render task results. */
	void UpdateCollisionPhysicalMaterialData(TArray<UPhysicalMaterial*> const& InPhysicalMaterials, TArray<uint8> const& InMaterialIds);

	/**
	 * Create weightmaps for this component for the layers specified in the WeightmapLayerAllocations array, works in the specified edit layer
	 */
	LANDSCAPE_API TArray<UTexture*> ReallocateWeightmaps(FLandscapeEditDataInterface* DataInterface, const FGuid& InEditLayerGuid, bool bInSaveToTransactionBuffer, bool bInForceReallocate, ALandscapeProxy* InTargetProxy, const TSet<ULandscapeComponent*>* InRestrictSharingToComponents);

	/** Returns true if the component has a valid LandscapeHoleMaterial */
	LANDSCAPE_API bool IsLandscapeHoleMaterialValid() const;

	/** Returns true if this component has visibility painted */
	LANDSCAPE_API bool ComponentHasVisibilityPainted() const;

	LANDSCAPE_API ULandscapeLayerInfoObject* GetVisibilityLayer() const;

	/**
	 * Generate a key for a component's layer allocations to use with MaterialInstanceConstantMap.
	 */
	static FString GetLayerAllocationKey(const TArray<FWeightmapLayerAllocationInfo>& Allocations, UMaterialInterface* LandscapeMaterial, bool bMobile = false);

	bool ValidateCombinationMaterial(UMaterialInstanceConstant* InCombinationMaterial) const;

	/** @todo document */
	void GetLayerDebugColorKey(int32& R, int32& G, int32& B) const;

	/** @todo document */
	void RemoveInvalidWeightmaps();
	void RemoveInvalidWeightmaps(const FGuid& InEditLayerGuid);

	/** @todo document */
	LANDSCAPE_API void InitHeightmapData(TArray<FColor>& Heights, bool bUpdateCollision);

	/** @todo document */
	LANDSCAPE_API void InitWeightmapData(TArray<ULandscapeLayerInfoObject*>& LayerInfos, TArray<TArray<uint8> >& Weights);

	/** @todo document */
	LANDSCAPE_API float GetLayerWeightAtLocation( const FVector& InLocation, ULandscapeLayerInfoObject* LayerInfo, TArray<uint8>* LayerCache = NULL, bool bUseEditingWeightmap = false);

	/** Extends passed region with this component's 2D bounds (values are in landscape quads) */
	LANDSCAPE_API void GetComponentExtent(int32& MinX, int32& MinY, int32& MaxX, int32& MaxY) const;

	/** returns the 2D bounds of this component (in landscape quads) */
	LANDSCAPE_API FIntRect GetComponentExtent() const;

	LANDSCAPE_API void ClearUpdateFlagsForModes(uint32 InModeMask);
	LANDSCAPE_API void RequestWeightmapUpdate(bool bUpdateAll = false, bool bUpdateCollision = true, bool bInUserTriggered = false);
	LANDSCAPE_API void RequestHeightmapUpdate(bool bUpdateAll = false, bool bUpdateCollision = true, bool bInUserTriggered = false);
	LANDSCAPE_API void RequestEditingClientUpdate(bool bInUserTriggered = false);
	LANDSCAPE_API void RequestDeferredClientUpdate();
	uint32 GetLayerUpdateFlagPerMode() const { return LayerUpdateFlagPerMode; }
	LANDSCAPE_API uint32 ComputeWeightmapsHash();

	void GetUsedPaintLayers(const FGuid& InLayerGuid, TArray<ULandscapeLayerInfoObject*>& OutUsedLayerInfos) const;

	void GetLandscapeComponentNeighborsToRender(TSet<ULandscapeComponent*>& NeighborComponents) const;
	void GetLandscapeComponentWeightmapsToRender(TSet<ULandscapeComponent*>& WeightmapComponents) const;
	void GetLandscapeComponentNeighbors3x3(TStaticArray<ULandscapeComponent*, 9>& OutNeighborComponents) const;
#endif

	/** Updates navigation properties to match landscape actor's */
	void UpdateNavigationRelevance();

	/** Updates the reject navmesh underneath flag in the collision component */
	void UpdateRejectNavmeshUnderneath();

	/** Updates the values of component-level properties exposed by the Landscape Actor */
	LANDSCAPE_API void UpdatedSharedPropertiesFromActor();

	friend class FLandscapeComponentSceneProxy;
	friend struct FLandscapeComponentDataInterface;

	LANDSCAPE_API void SetLOD(bool bForced, int32 InLODValue);

	UFUNCTION(BlueprintCallable, Category = "LandscapeComponent")
	LANDSCAPE_API void SetForcedLOD(int32 InForcedLOD);

	UFUNCTION(BlueprintCallable, Category = "LandscapeComponent")
	LANDSCAPE_API void SetLODBias(int32 InLODBias);

	void SetNaniteActive(bool bValue);

	inline bool IsNaniteActive() const
	{
		return bNaniteActive;
	}

	ULandscapeHeightfieldCollisionComponent* GetCollisionComponent() const { return CollisionComponentRef.Get(); }
	void SetCollisionComponent(ULandscapeHeightfieldCollisionComponent* InCollisionComponent) { CollisionComponentRef = InCollisionComponent; }

	void SetUserTriggeredChangeRequested(bool bInUserTriggeredChangeRequested)
	{
		bUserTriggeredChangeRequested = bInUserTriggeredChangeRequested;
	}

	bool GetUserTriggeredChangeRequested() const
	{
		return bUserTriggeredChangeRequested;
	}

protected:

#if WITH_EDITOR
	void RecreateCollisionComponent(bool bUseSimpleCollision);
	void UpdateCollisionHeightBuffer(int32 InComponentX1, int32 InComponentY1, int32 InComponentX2, int32 InComponentY2, int32 InCollisionMipLevel, int32 InHeightmapSizeU, int32 InHeightmapSizeV,
		const FColor* const InHeightmapTextureMipData, uint16* CollisionHeightData, uint16* GrassHeightData);
	void UpdateDominantLayerBuffer(int32 InComponentX1, int32 InComponentY1, int32 InComponentX2, int32 InComponentY2, int32 InCollisionMipLevel, int32 InWeightmapSizeU, int32 InVisibilityLayerIdx, const TArray<uint8*>& InCollisionDataPtrs, const TArray<ULandscapeLayerInfoObject*>& InLayerInfos, uint8* DominantLayerData);
#endif

	/** Whether the component type supports static lighting. */
	virtual bool SupportsStaticLighting() const override
	{
		return true;
	}

#if WITH_EDITOR
public:
	/** Records the ULandscapeComponents that are modified in any undo/redo operation that is being applied currently */
	static uint32 UndoRedoModifiedComponentCount;
	static TMap<ULandscapeComponent*, uint32> UndoRedoModifiedComponents;
#endif // WITH_EDITOR
};
