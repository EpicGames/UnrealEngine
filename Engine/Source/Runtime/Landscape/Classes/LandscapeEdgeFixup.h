// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/StaticArray.h"
#include "Misc/EnumRange.h"
#include "Engine/AssetUserData.h"
#include "Streaming/TextureMipDataProvider.h"
#include "LandscapeEdgeFixup.generated.h"

struct FLandscapeGroup;
class ULandscapeComponent;
class ULandscapeInfo;
class UTexture2D;

namespace UE::Landscape
{
	// Enumerates directions, for the edges or neighbors of a tile in the the landscape group grid.
	// When specifically referencing an edge or a neighbor, use EEdgeIndex or ENeighborIndex
	enum class EDirectionIndex : uint8
	{
		Bottom = 0,
		BottomRight = 1,
		Right = 2,
		TopRight = 3,
		Top = 4,
		TopLeft = 5,
		Left = 6,
		BottomLeft = 7,

		First = 0,
		Last = 7,
		Count = 8,

		FirstEdge = 0,
		LastEdge = 6,
		EdgeCount = 4,

		FirstCorner = 1,
		LastCorner = 7,
		CornerCount = 4,
	};

	// Specifies a set of edges or neighbors.
	enum class EDirectionFlags : uint8
	{
		Bottom = 0x00000001,
		BottomRight = 0x00000002,
		Right = 0x00000004,
		TopRight = 0x00000008,
		Top = 0x00000010,
		TopLeft = 0x00000020,
		Left = 0x00000040,
		BottomLeft = 0x00000080,

		None = 0,
		AllCorners = BottomRight | TopRight | TopLeft | BottomLeft,
		AllEdges = Bottom | Right | Top | Left,
		All = AllEdges | AllCorners
	};

	typedef EDirectionIndex EEdgeIndex;		// EEdgeIndex specifies an edge or corner on the local landscape component.
	typedef EDirectionFlags EEdgeFlags;		// EEdgeFlags specifies a set of edges and corners on the local landscape component.
	typedef EDirectionIndex ENeighborIndex;	// ENeighborIndex specifies a neighbor landscape component, relative to a local landscape component.
	typedef EDirectionFlags ENeighborFlags;	// ENeighborFlags specifies a set of neighboring landscape components.

	struct FHeightmapTexel
	{
		static_assert(PLATFORM_LITTLE_ENDIAN, "This code needs to be adapted to big endian");

		union
		{
			uint32 Data32;
			uint8 Data[4];
			struct 
			{
				uint8 NormalX;
				uint8 HeightL;	// low 8 bits
				uint8 HeightH;	// high 8 bits
				uint8 NormalY;
			};
		};

		uint32 GetHeight16() const
		{
			return (((uint32)HeightH) << 8) + HeightL;
		}

		void SetHeight16(uint16 NewHeight)
		{
			HeightL = NewHeight & 0xff;
			HeightH = (NewHeight >> 8) & 0xff;
		}

		void SetNormal(const FVector& NewNormal)
		{
			const FVector Normal = NewNormal.GetSafeNormal();
			// these seem to produce the nearest result to the GPU normal calculation
			// (matches 'straight up' of 0x7f coming from the GPU)
			NormalX = static_cast<uint8>(FMath::RoundToInt32(127.49999f + Normal.X * 127.5f));
			NormalY = static_cast<uint8>(FMath::RoundToInt32(127.49999f + Normal.Y * 127.5f));
		}

		bool IsSameHeight(const FHeightmapTexel& Other) const
		{
			uint32 HeightBitsXOR = (Data32 ^ Other.Data32) & 0x00FFFF00;
			return HeightBitsXOR == 0;
		}
	};

	// states required to perform edge patching
	struct FNeighborSnapshots
	{
		ENeighborFlags ExistingNeighbors;
		EEdgeFlags EdgesWithAnyModifiedNeighbor;
		FHeightmapTextureEdgeSnapshot* NeighborSnapshots[8];
		FHeightmapTextureEdgeSnapshot* LocalSnapshot;
		TStaticArray<uint32, 8> GPUEdgeHashes;
	};
}
ENUM_RANGE_BY_FIRST_AND_LAST(UE::Landscape::EDirectionIndex, UE::Landscape::EDirectionIndex::First, UE::Landscape::EDirectionIndex::Last);
ENUM_CLASS_FLAGS(UE::Landscape::EDirectionFlags);

namespace UE::Landscape
{
	// Returns the relative offset (in group tile coords) of a neighbor
 	FIntPoint GetNeighborRelativePosition(ENeighborIndex NeighborIndex);

	// Returns the set of neighbors that blend with any local edge in EdgeFlags
	ENeighborFlags EdgesToAffectedNeighbors(EEdgeFlags LocalEdgeFlags);

	// Converts a neighbor index to a neighbor flag (or an edge index to an edge flag)
	EDirectionFlags ToFlag(EDirectionIndex Index);

	// Returns a debug string describing the neighbor (or edge) direction
	const FString& GetDirectionString(EDirectionIndex Index);
}


/**
  * The snapshots contain a copy of the heightmap edge texels (both height and normal info).
  * It is filled out in editor or at cook time, to make available at runtime for dynamic edge fixup.
  */
USTRUCT()
struct FHeightmapTextureEdgeSnapshot
{
	GENERATED_USTRUCT_BODY()
	friend class ULandscapeHeightmapTextureEdgeFixup;

private:
	int32 EdgeLength = 0;						// edge length for recorded edge data here - when up to date, should match texture resolution (width AND height)
	TArray<uint32> EdgeData;					// height and normal data for each edge & mip (in heightmap texture format 32bpp). Use GetEdgeData() to access specific edges and mips.
	TStaticArray<uint32, 8> SnapshotEdgeHashes;	// hash of each edge / corner (at full resolution) in the EdgeData
	TStaticArray<uint32, 8> InitialEdgeHashes;	// hash of each edge / corner (at full resolution) in the GPU Texture Resource (at initial unpatched state)

#if WITH_EDITOR
	FGuid TextureSourceID;						// used to detect when this is out of date with texture sourcesource
#endif // WITH_EDITOR

public:
	/** Return edge snapshot data for this component, for the specified neighbor direction and mip.
	  * Horizontal edges are stored left to right, and vertical edges bottom to top.
	  */
	TArrayView<UE::Landscape::FHeightmapTexel> GetEdgeData(UE::Landscape::EEdgeIndex InEdgeIndex, int32 InMipIndex);
	UE::Landscape::FHeightmapTexel GetCornerData(UE::Landscape::EEdgeIndex InEdgeIndex);

#if WITH_EDITOR
	// Create a snapshot from the heightmap source
	static TSharedPtr<FHeightmapTextureEdgeSnapshot> CreateEdgeSnapshotFromHeightmapSource(UTexture2D* InHeightmap, const FVector& LandscapeGridScale);
#endif // WITH_EDITOR

	// Return the set of edges that are different (according to edge hashes) -- i.e. changes that could cause neighbors to patch
	UE::Landscape::EEdgeFlags CompareEdges(const FHeightmapTextureEdgeSnapshot& OldSnapshot) const;

	friend FArchive& operator<<(FArchive& Ar, FHeightmapTextureEdgeSnapshot& Data);

	FString GetTextureSourceIDAsString()
	{
#if WITH_EDITOR
		return TextureSourceID.ToString();
#else
		return FString("<NONE>");
#endif // WITH_EDITOR
	}

private:
	void ResizeForEdgeLength(const int32 InEdgeLength);

	// This function is not thread safe to call on existing snapshots
	// it should only be used on a newly allocated FHeightmapTextureEdgeSnapshot
	void CaptureEdgeDataFromTextureData_Internal(const TArrayView64<UE::Landscape::FHeightmapTexel>& InHeightmapTextureData, int32 InEdgeLength, const FVector& LandscapeGridScale, const FGuid& InTextureSourceID);
	
	// must call ResizeForEdgeLength to set the texture size before calling CopyTextureEdgeDataAndRecomputeNormals
	void CaptureSingleEdgeDataAndComputeNormalsAndHashes(const UE::Landscape::FHeightmapTexel* TextureData, const UE::Landscape::EEdgeIndex EdgeOrCorner, const FVector& LandscapeGridScale);
};

// This UAssetUserData is attached to landscape heightmap UTexture2D's and tracks the heightmap texture's edge fixup state
// This is used by mip providers to apply edge fixup on mip streaming/creation, and
// also used by runtime dynamic fixup when neighboring landscape components are pulled in
UCLASS(NotBlueprintable, MinimalAPI, Within = Texture2D)
class ULandscapeHeightmapTextureEdgeFixup : public UAssetUserData
{
	GENERATED_UCLASS_BODY()
	friend struct FLandscapeGroup;
	friend class FLandscapeTextureMipEdgeOverrideProvider;
	friend class FLandscapeTextureStorageMipProvider;

private:
	// SERIALIZED snapshot of the heightmap edge data
	// COPY-ON-WRITE so we can use it safely from other threads. do not modify an existing snapshot, create a new snapshot and replace this reference.
	TSharedRef<FHeightmapTextureEdgeSnapshot, ESPMode::ThreadSafe> EdgeSnapshot;

	// transient runtime tracking data
	TObjectPtr<UTexture2D> HeightmapTexture;			// heightmap texture (set to our parent heightmap, on first registration)

	TObjectPtr<ULandscapeComponent> ActiveComponent;	// the active component, that is patching HeightmapTexture
	FLandscapeGroup* ActiveGroup = nullptr;				// the active group, that is patching HeightmapTexture

	TStaticArray<uint32, 8> GPUEdgeHashes;				// hash for the current GPU edge state, initialized on first registration
	UE::Landscape::EEdgeFlags GPUEdgeModifiedFlags =	// edges that have been patched / modified from the initial state
		UE::Landscape::EEdgeFlags::None;

#if WITH_EDITOR
	bool bDoNotPatchUntilGPUEdgeHashesUpdated = false;
	bool bUpdateGPUEdgeHashes = false;
#endif // WITH_EDITOR

	// per-group settings (apply to the active group/component)
	bool bMapped = false;
	bool bForceUpdateSnapshot = true;					// set to true initially so that we do a force update on the very first request
	FIntPoint GroupCoord = FIntPoint::ZeroValue;		// coordinate of this heightmap in the active group (when bMapped)

	// components that also want to use this heightmap & edge fixup, but were disabled as we can only support one active component at a time.
	// these must be weak object pointers, as they can be unregistered while still in this list, and can be garbage collected out from under us.
	// Note that this array is generally empty except in scenarios where there are multiple active worlds sharing the same landscape textures (PIE)
	// so the expense of accessing the TWeakObjectPtr is a PIE-only cost.
	TArray<TWeakObjectPtr<ULandscapeComponent>> DisabledComponents;

public:
	inline bool IsActive() { return ActiveGroup != nullptr; }
	inline UTexture2D* GetHeightmapTexture() { return HeightmapTexture; }
	inline bool IsComponentActive(ULandscapeComponent* Component) { return ActiveComponent == Component; }
	inline const FIntPoint& GetGroupCoord() { return GroupCoord; }

#if WITH_EDITOR
	inline bool IsTextureEdgePatchingPaused() { return bDoNotPatchUntilGPUEdgeHashesUpdated; }
	inline void PauseTextureEdgePatchingUntilGPUEdgeHashesUpdated() { bDoNotPatchUntilGPUEdgeHashesUpdated = true; }
#endif // WITH_EDITOR

	virtual void Serialize(FArchive& Ar) override;
	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	// [main thread] -- Find or Create an edge fixup class for the given heightmap texture.  Will only create in editor (the data used to create is only available in editor)
	static ULandscapeHeightmapTextureEdgeFixup* FindOrCreateFor(UTexture2D* TargetTexture);

	// set the heightmap texture (can only be called once)
	void SetHeightmapTexture(UTexture2D* InHeightmapTexture);

	// set the active landscape component, handling unmapping the old, and mapping the new.
	// if bDisableCurrentActive, it will move the current active component, if any, to the disabled list.
	void SetActiveComponent(ULandscapeComponent* InComponent, FLandscapeGroup* InGroup, const bool bDisableCurrentActive = true);

	// Request edge texture patching on a set of neighbors.
	void RequestEdgeTexturePatchingForNeighbors(UE::Landscape::ENeighborFlags NeighborsNeedingPatching);

#if WITH_EDITOR
	// Request an update to the edge snapshot (capturing from the heightmap source).
	// When bUpdateGPUEdgeHashes is true, it will also update the GPU Edge Hashes to match.
	// This is set when we know the heightmap source exactly reflects the GPU texture state
	// i.e. after reading back from GPU to CPU after a layer merge, or after UpdateResource().
	void RequestEdgeSnapshotUpdateFromHeightmapSource(bool bUpdateGPUEdgeHashes = false);

	// Update the edge snapshot from heightmap source.  Returns the set of edges that changed since the previous snapshot.
	UE::Landscape::EEdgeFlags UpdateEdgeSnapshotFromHeightmapSource(const FVector& LandscapeGridScale, bool bForceUpdate = false, bool* bOutSuccess = nullptr);
#endif // WITH_EDITOR

	// Patch the GPU texture edges if needed, using the current snapshot and corresponding neighbor snapshots as source data.
	// Returns the number of edges patched.
	int32 CheckAndPatchTextureEdgesFromEdgeSnapshots();

	// Get the set of neighbor snapshots (nullptr if they don't exist), and gather existence and modified flags
	// this is the data necessary to perform patching on a component
	void GetNeighborSnapshots(UE::Landscape::FNeighborSnapshots& OutSnapshots);

	// Patch all of the edges for a single texture mip.
	// Called during streaming operations to patch a newly streamed mip in flight.
	// Returns the number of edges and corners patched
	static int32 PatchTextureEdgesForSingleMip(int32 MipIndex, FTextureMipInfo& DestMipInfo, const UE::Landscape::FNeighborSnapshots& NeighborSnapshots);
	static int32 PatchTextureEdgesForStreamingMips(int32 FirstMipIndexInclusive, int32 LastMipIndexExclusive, FTextureMipInfoArray& DestMipInfos, const UE::Landscape::FNeighborSnapshots& NeighborSnapshots);

private:
	void PatchTextureEdge_Internal(UE::Landscape::EEdgeIndex EdgeIndex);
	void PatchTextureCorner_Internal(UE::Landscape::EEdgeIndex CornerIndex, UE::Landscape::FHeightmapTexel Texel);

	// Helper functions that generate blended edge or corner data from snapshots, to use in texture patching
	static void BlendEdgeData(FHeightmapTextureEdgeSnapshot& EdgeSnapshot, UE::Landscape::EEdgeIndex EdgeIndex, int32 MipIndex, FHeightmapTextureEdgeSnapshot& NeighborEdgeSnapshot, TStridedView<UE::Landscape::FHeightmapTexel>& OutDestView);
	static void BlendCornerData(UE::Landscape::FHeightmapTexel& OutTexel, UE::Landscape::EEdgeIndex CornerIndex, const UE::Landscape::FNeighborSnapshots& NeighborSnapshots);
};

