// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Landscape.h"
#include "LandscapeEditLayerRenderer.h"
#include "LandscapeEditTypes.h"
#include "Math/Interval.h"

#include "LandscapeEditLayer.generated.h"

enum class ELandscapeToolTargetType : uint8;

struct FOnLandscapeEditLayerDataChangedParams
{
	FOnLandscapeEditLayerDataChangedParams(const FPropertyChangedEvent& InPropertyChangedEvent = FPropertyChangedEvent(/*InProperty = */nullptr))
		: PropertyChangedEvent(InPropertyChangedEvent)
	{}

	/** Provides some additional context about how data has changed (property, type of change...) */
	FPropertyChangedEvent PropertyChangedEvent;

	/** Indicates a user-initiated property change */
	bool bUserTriggered = false;

	/** Indicates the change requires a full landscape update (e.g. parameter affecting heightmap or weightmap...) */
	bool bRequiresLandscapeUpdate = true;

	/** The delegate is triggered each time a data change is requested, even when the data didn't actually change. This indicates that the 
	 * was actually modified. This can occur for example when several EPropertyChangeType::Interactive changes are triggered because of the user
	 * manipulating a slider : this will be followed by a final EPropertyChangeType::ValueSet but when this occurs, the data usually is not actually 
	 * modified so, to be consistent, we'll still trigger the delegate but indicate that the value didn't actually change, to let the user react appropriately
	 */
	bool bHasValueChanged = true;
};

/** 
* Base class for all landscape edit layers. By implementing the various virtual functions, we are able to customize the behavior of the edit layer
*  wrt the landscape tools in a generic way (e.g. does it support sculpting tools? painting tools? can it be collapsed?, etc.)
*/
UCLASS(MinimalAPI, Abstract, BlueprintType)
class ULandscapeEditLayerBase : public UObject
#if CPP && WITH_EDITOR
	, public UE::Landscape::EditLayers::IEditLayerRendererProvider
#endif // CPP && WITH_EDITOR
{
	GENERATED_BODY()

public:
	class UE_DEPRECATED(5.7, "Expose custom actions to the editor with IEditLayerCustomization and RegisterCustomEditLayerClassLayout") FEditLayerAction
	{
		FEditLayerAction() = delete;

	public:
		class UE_DEPRECATED(5.7, "Expose custom actions to the editor with IEditLayerCustomization and RegisterCustomEditLayerClassLayout") FExecuteParams
		{
			FExecuteParams() = delete;

		public:
			FExecuteParams(const ULandscapeEditLayerBase* InEditLayer, ALandscape* InLandscape)
				: EditLayer(InEditLayer)
				, Landscape(InLandscape)
			{
				check((InEditLayer != nullptr) && (InLandscape != nullptr));
			}

			inline const ULandscapeEditLayerBase* GetEditLayer() const { return EditLayer; }
			inline ALandscape* GetLandscape() const { return Landscape; }

		private:
			const ULandscapeEditLayerBase* EditLayer = nullptr;
			const FLandscapeLayer* Layer = nullptr;
			ALandscape* Landscape = nullptr;
		};

		struct UE_DEPRECATED(5.7, "Expose custom actions to the editor with IEditLayerCustomization and RegisterCustomEditLayerClassLayout") FExecuteResult
		{
			FExecuteResult() = default;
			FExecuteResult(bool bInSuccess, const FText& InReason = FText())
				: bSuccess(bInSuccess)
				, Reason(InReason)
			{}

			bool bSuccess = true;
			FText Reason;
		};

		DECLARE_DELEGATE_RetVal_OneParam(FExecuteResult, FExecuteDelegate, const FExecuteParams& /*InParams*/);
		DECLARE_DELEGATE_RetVal_TwoParams(bool, FCanExecuteDelegate, const FExecuteParams& /*InParams*/, FText& /*OutReason*/);

		FEditLayerAction(const FText& InLabel, const FExecuteDelegate& InExecuteDelegate, const FCanExecuteDelegate& InCanExecuteDelegate)
			: Label(InLabel)
			, ExecuteDelegate(InExecuteDelegate)
			, CanExecuteDelegate(InCanExecuteDelegate)
		{}

		inline const FText& GetLabel() const { return Label; }
		inline const FExecuteDelegate& GetExecuteDelegate() const { return ExecuteDelegate; }
		inline const FCanExecuteDelegate& GetCanExecuteDelegate() const { return CanExecuteDelegate; }

	private:
		FText Label;
		FExecuteDelegate ExecuteDelegate;
		FCanExecuteDelegate CanExecuteDelegate;
	};

public:
	/**
	 * @param InType : tool target type (Heightmap, Weightmap, Visibility) 
	 * @return true if the this edit layer has support for the target type (heightmap, weightmap, visibility)
	 */
	virtual bool SupportsTargetType(ELandscapeToolTargetType InType) const
		PURE_VIRTUAL(ULandscapeEditLayerBase::SupportsTargetType, return true; );

	/**
	 * @return true if the edit layer can store heightmaps/weightmaps in the ALandscapeProxy (e.g. should return false for purely procedural layers, to avoid allocating textures)
	 */
	virtual bool NeedsPersistentTextures() const
		PURE_VIRTUAL(ULandscapeEditLayerBase::NeedsPersistentTextures, return false; );

	/**
	* @return true if the edit layer can be manually edited via the landscape editing tools :
	*/
	virtual bool SupportsEditingTools() const
		PURE_VIRTUAL(ULandscapeEditLayerBase::SupportsEditingTools, return true; );

	/**
	 * @return true if it's allowed to have more than one edit layer of this type at a time
	 */
	virtual bool SupportsMultiple() const
		PURE_VIRTUAL(ULandscapeEditLayerBase::SupportsMultiple, return true; );

	/**
	 * @return true if the layer supports a layer above being collapsed onto it
	 */
	virtual bool SupportsBeingCollapsedAway() const
		PURE_VIRTUAL(ULandscapeEditLayerBase::SupportsBeingCollapsedAway, return true; );

	/**
	 * @return true if the layer supports being collapsed onto a layer underneath
	 */
	virtual bool SupportsCollapsingTo() const
		PURE_VIRTUAL(ULandscapeEditLayerBase::SupportsCollapsingTo, return true; );

	/**
	 * @return true if the layer supports blueprint brushes
	 */
	virtual bool SupportsBlueprintBrushes() const
		PURE_VIRTUAL(ULandscapeEditLayerBase::SupportsBlueprintBrushes(), return false; );

	/**
	* @return the default name to use when creating a new layer of this type
	*/
	virtual FString GetDefaultName() const
		PURE_VIRTUAL(ULandscapeEditLayerBase::GetDefaultName, return FString(); );

#if WITH_EDITOR
	/**
	 * @param InType : tool target type (Heightmap, Weightmap, Visibility) 
	 * @return true if the layer supports alpha for a given target type
	 */
	LANDSCAPE_API virtual bool SupportsAlphaForTargetType(ELandscapeToolTargetType InType) const;

	/**
	 * Sets the alpha value for a given target type
	 * @param InType : the type of target (Heightmap, Weightmap) on which to set the alpha
	 * @param InNewValue : the new alpha value
	 * @param bInModify : true when Modify() needs to be called
	 * @param InChangeType : type of change (normal, interactive, ...)
	 */
	LANDSCAPE_API virtual void SetAlphaForTargetType(ELandscapeToolTargetType InType, float InNewValue, bool bInModify, EPropertyChangeType::Type InChangeType);

	/**
	 * Gets the alpha value for a given target type
	 * @param InType : tool target type (Heightmap, Weightmap, Visibility) 
	 * @return the alpha value for a given target type
	 */
	LANDSCAPE_API virtual float GetAlphaForTargetType(ELandscapeToolTargetType InType) const;

	/**
	* @return the valid alpha value interval for a given target type
	*/
	LANDSCAPE_API virtual FFloatInterval GetAlphaRangeForTargetType(ELandscapeToolTargetType InType) const;

	/**
	 * Sets the layer's Guid value
	 * @param InGuid : the new Guid value
	 * @param bInModify : true when Modify() needs to be called
	 */
	LANDSCAPE_API virtual void SetGuid(const FGuid& InGuid, bool bInModify);

	LANDSCAPE_API virtual const FGuid& GetGuid() const;

	/**
	 * Sets the layer's Name value
	 * @param InName : the new Name value
	 * @param bInModify : true when Modify() needs to be called
	 */
	LANDSCAPE_API virtual void SetName(FName InName, bool bInModify);

	LANDSCAPE_API virtual FName GetName() const;

	/**
	 * Sets the layer's visibility value
	 * @param bInVisible : true to set visible, false to set invisible
	 * @param bInModify : true when Modify() needs to be called
	 */
	LANDSCAPE_API virtual void SetVisible(bool bInVisible, bool bInModify);
	
	LANDSCAPE_API virtual bool IsVisible() const;

	/**
	 * Sets the layer's locked value 
	 * @param bInLocked : true to lock layer, false to unlock
	 * @param bInModify : true when Modify() needs to be called
	 */
	LANDSCAPE_API virtual void SetLocked(bool bInLocked, bool bInModify);

	LANDSCAPE_API virtual bool IsLocked() const;

	/**
	 * @return the layer's BlendMode - LSBM_AdditiveBlend by default
	 * Marked as UE_INTERNAL to prevent external usage before blend refactor with the introduction of blend groups and premultiplied alpha blending
	 */
	UE_INTERNAL LANDSCAPE_API virtual ELandscapeBlendMode GetBlendMode() const;

	/**
	 * Remove and Copy a weightmap layer allocation for a given LayerInfoObj
	 * @param InKey : the ULandscapeLayerInfoObj key to be removed
	 * @param bOutValue: the layer allocation value stored in the map for InKey
	 * @param bInModify : true when Modify() needs to be called
	 * Marked as UE_INTERNAL to prevent external usage before blend refactor with the introduction of blend groups and premultiplied alpha blending
	 */
	UE_INTERNAL LANDSCAPE_API virtual bool RemoveAndCopyWeightmapAllocationLayerBlend(TObjectPtr<ULandscapeLayerInfoObject> InKey, bool& bOutValue, bool bInModify);

	/**
	 * Updates the value of an existing layer allocation or adds a weightmap layer allocation if no entry is found
	 * @param InKey : the ULandscapeLayerInfoObj map key
	 * @param bInValue: the new allocation value
	 * @param bInModify : true when Modify() needs to be called
	 * Marked as UE_INTERNAL to prevent external usage before blend refactor with the introduction of blend groups and premultiplied alpha blending
	 */
	UE_INTERNAL LANDSCAPE_API virtual void AddOrUpdateWeightmapAllocationLayerBlend(TObjectPtr<ULandscapeLayerInfoObject> InKey, bool InValue, bool bInModify);

	/**
	 * @return the layer's weightmap layer allocation blend map. Layers do not have an entry in the map until a user sets the blend mode (selects Subtractive)
	 * Marked as UE_INTERNAL to prevent external usage before blend refactor with the introduction of blend groups and premultiplied alpha blending
	 */
	UE_INTERNAL LANDSCAPE_API virtual const TMap<TObjectPtr<ULandscapeLayerInfoObject>, bool>& GetWeightmapLayerAllocationBlend() const;

	/**
	 * Sets the layers WeightmapLayerAllocation map
	 * @param InWeightmapLayerAllocationBlend : the new WeightmapLayerAllocation map
	 * @param bInModify : true when Modify() needs to be called
	 * Marked as UE_INTERNAL to prevent external usage before blend refactor with the introduction of blend groups and premultiplied alpha blending
	 */
	UE_INTERNAL LANDSCAPE_API virtual void SetWeightmapLayerAllocationBlend(const TMap<TObjectPtr<ULandscapeLayerInfoObject>, bool>& InWeightmapLayerAllocationBlend, bool bInModify);

#endif // WITH_EDITOR

	// BP accessors 

	/**
	* @return the name of the edit layer (editor-only : returns an invalid name at runtime)
	*/
	UFUNCTION(BlueprintCallable, DisplayName = GetName, Category = "Landscape|Editor", meta = (DevelopmentOnly))
	FName GetNameBP() const;

	UE_DEPRECATED(5.7, "Register custom context menu actions using RegisterCustomEditLayerClassLayout from ILandscapeEditorModule")
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual TArray<FEditLayerAction> GetActions() const 
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{ 
		return {}; 
	}

	/**
	* @return a list of UObjects that this layers needs in order to render properly. This is necessary to avoid trying to render a layer while some of its
	*  resources are not fully ready. 
	*  These can be UTexture (requires all mips to be fully loaded) or UMaterialInterface (requires shader maps to be fully compiled)
	*/
	virtual void GetRenderDependencies(TSet<UObject*>& OutDependencies) const 
	{}

	// Called by landscape after removing this layer from its list so that the layer can do
	// any cleanup that it might need to do.
	// TODO: Should this be protected and then we friend ALandscape?
	virtual void OnLayerRemoved() 
	{}

#if WITH_EDITOR
	//~ Begin IEditLayerRendererProvider implementation
	// By default this does nothing in a landscape edit layer, but subclasses can override it if 
	//  they would like to provide additional renderers.
	virtual TArray<UE::Landscape::EditLayers::FEditLayerRendererState> GetEditLayerRendererStates(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) {  return {};  };
	//~ End IEditLayerRendererProvider implementation


	/** Delegate triggered whenever a change occurred on the edit layer's data */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnLayerDataChanged, const FOnLandscapeEditLayerDataChangedParams& /*InParams*/);
	FOnLayerDataChanged::RegistrationType& OnLayerDataChanged() const
	{
		return OnLayerDataChangedDelegate;
	}

	/**
	* @return the mask of the target types currently enabled on this edit layer 
	*/
	LANDSCAPE_API virtual ELandscapeToolTargetTypeFlags GetEnabledTargetTypeMask() const;
#endif // WITH_EDITOR

protected:
#if WITH_EDITOR
	// Begin UObject implementation
	LANDSCAPE_API virtual void PostLoad() override;
	LANDSCAPE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	LANDSCAPE_API virtual void PostEditUndo() override;
	LANDSCAPE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
	// End UObject implementation
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	// Setters for UPROPERTY (should be used for blueprint setters eventually) :  
	void SetGuidInternal(const FGuid& InGuid);
	void SetNameInternal(FName InName);
	void SetVisibleInternal(bool bInVisible);
	void SetLockedInternal(bool bInLocked);
	void SetHeightmapAlphaInternal(float InNewValue);
	void SetWeightmapAlphaInternal(float InNewValue);
	// Marked as UE_INTERNAL to prevent external usage before blend refactor with the introduction of blend groups and premultiplied alpha blending
	UE_INTERNAL void SetWeightmapLayerAllocationBlendInternal(const TMap<TObjectPtr<ULandscapeLayerInfoObject>, bool>& InWeightmapLayerAllocationBlend);
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	/** Internal function that returns a reference to the alpha value corresponding to a given target type (HeightmapAlpha, WeightmapAlpha, ...) */
	float& GetAlphaForTargetTypeRef(ELandscapeToolTargetType InType);
	/** Internal function that returns the alpha FProperty corresponding to a given target type (HeightmapAlpha, WeightmapAlpha, ...)  */
	FProperty* GetAlphaPropertyForTargetType(ELandscapeToolTargetType InType) const;
	/** Internal function meant to be called whenever the edit layer data changes, broadcast the OnLayerDataChanged event */
	void BroadcastOnLayerDataChanged(const FPropertyChangedEvent& InPropertyChangeEvent, bool bInUserTriggered, bool bRequiresLandscapeUpdate, bool bInHasValueChanged);
#endif // WITH_EDITOR
protected:
#if WITH_EDITORONLY_DATA
	UPROPERTY(Setter = "SetGuidInternal", meta = (EditCondition = "!bLocked"))
	FGuid Guid = FGuid::NewGuid();

	UPROPERTY(Setter = "SetNameInternal", meta = (EditCondition = "!bLocked"))
	FName LayerName = NAME_None;

	UPROPERTY(Category = "Edit Layer", EditAnywhere, Setter = "SetVisibleInternal", meta = (EditCondition = "!bLocked"))
	bool bVisible = true;

	UPROPERTY(Category = "Edit Layer", EditAnywhere, Setter = "SetLockedInternal")
	bool bLocked = false;

	UPROPERTY(Category = "Edit Layer", EditAnywhere, Setter = "SetHeightmapAlphaInternal", meta = (UIMin = "-1.0", UIMax = "1.0", ClampMin = "-1.0", ClampMax = "1.0", EditCondition = "!bLocked"))
	float HeightmapAlpha = 1.0f;

	UPROPERTY(Category = "Edit Layer", EditAnywhere, Setter = "SetWeightmapAlphaInternal", meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", EditCondition = "!bLocked"))
	float WeightmapAlpha = 1.0f;

	// TODO: This might be removed once more things are moved from FLandscapeLayer to ULandscapeLayer
	UPROPERTY()
	TWeakObjectPtr<ALandscape> OwningLandscape;

private:
	// TODO: This might be removed once the guid is stored here and subclasses have a way to request landscape updates.
	friend class ALandscape;
	void SetBackPointer(ALandscape* Landscape);

	mutable FOnLayerDataChanged OnLayerDataChangedDelegate;

	UPROPERTY(Setter = "SetWeightmapLayerAllocationBlendInternal", meta = (EditCondition = "!bLocked"))
	TMap<TObjectPtr<ULandscapeLayerInfoObject>, bool> WeightmapLayerAllocationBlend; // True -> Substractive, False -> Add
#endif //WITH_EDITORONLY_DATA
};

/** 
* Base class for persistent layers, i.e. layers that have a set of backing textures (heightmaps, weightmaps) and can therefore be rendered in a similar fashion
*/
UCLASS(MinimalAPI, Abstract)
class ULandscapeEditLayerPersistent : public ULandscapeEditLayerBase
	, public ILandscapeEditLayerRenderer
{
	GENERATED_BODY()

public:
	// Begin ULandscapeEditLayerBase implementation
	virtual bool NeedsPersistentTextures() const override { return true; };
	virtual bool SupportsCollapsingTo() const override { return true; } // If the layer has persistent textures, it can be collapsed to another layer (one that supports being collapsed away, that is)
	virtual bool SupportsBlueprintBrushes() const override { return false; }
	// End ULandscapeEditLayerBase implementation

#if WITH_EDITOR
	//~ Begin ILandscapeEditLayerRenderer implementation
	LANDSCAPE_API virtual void GetRendererStateInfo(const UE::Landscape::EditLayers::FMergeContext* InMergeContext,
		UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutSupportedTargetTypeState, UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutEnabledTargetTypeState, TArray<UE::Landscape::EditLayers::FTargetLayerGroup>& OutTargetLayerGroups) const override;
	virtual UE::Landscape::EditLayers::ERenderFlags GetRenderFlags(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const override;
	LANDSCAPE_API virtual TArray<UE::Landscape::EditLayers::FEditLayerRenderItem> GetRenderItems(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const override;
	LANDSCAPE_API virtual bool RenderLayer(UE::Landscape::EditLayers::FRenderParams& RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder) override;
	LANDSCAPE_API virtual void BlendLayer(UE::Landscape::EditLayers::FRenderParams& RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder) override;
	LANDSCAPE_API virtual FString GetEditLayerRendererDebugName() const override;
	//~ End ILandscapeEditLayerRenderer implementation
#endif // WITH_EDITOR
};

/** 
* This is the standard type of edit layer. It can be manually authored (sculpted, painted, etc.) in the landscape editor 
*/
UCLASS(MinimalAPI, meta = (ShortToolTip = "Standard edit layer"))
class ULandscapeEditLayer : public ULandscapeEditLayerPersistent
{
	GENERATED_BODY()

public:
	// Begin ULandscapeEditLayerBase implementation
	virtual bool SupportsTargetType(ELandscapeToolTargetType InType) const override;
	virtual bool SupportsEditingTools() const override { return true; }
	virtual bool SupportsMultiple() const override { return true; }
	virtual bool SupportsBeingCollapsedAway() const override { return true; } 
	virtual bool SupportsBlueprintBrushes() const override { return true; }
	virtual FString GetDefaultName() const { return TEXT("Layer"); }
	// End ULandscapeEditLayerBase implementation

protected:

};

/** 
* Base class for procedural layers. Procedural layers cannot be edited through standard editing tools
*/
UCLASS(MinimalAPI, Abstract)
class ULandscapeEditLayerProcedural : public ULandscapeEditLayerBase
{
	GENERATED_BODY()

public:
	// Begin ULandscapeEditLayerBase implementation
	virtual bool SupportsEditingTools() const override { return false; } // procedural layers cannot be edited through standard editing tools
	virtual bool SupportsCollapsingTo() const override { return false; } // for now, don't support collapsing to a layer underneath for a procedural layer (this may become unneeded if we make the collapse happen on the GPU)
	virtual bool SupportsBeingCollapsedAway() const override { return false; } // this is a procedural and therefore cannot be collapsed 
	virtual bool SupportsBlueprintBrushes() const override { return false; }
	// End ULandscapeEditLayerBase implementation
};

/** 
* Procedural edit layer that lets the user manipulate its content using landscape splines (Splines tool in the Manage panel) 
*/
UCLASS(MinimalAPI, meta = (ShortToolTip = "Special edit layer for landscape splines"))
class ULandscapeEditLayerSplines : public ULandscapeEditLayerPersistent
{
	GENERATED_BODY()

public:
	// Begin ULandscapeEditLayerBase implementation
	virtual bool SupportsEditingTools() const override { return false; } // procedural layers cannot be edited through standard editing tools
	virtual bool SupportsTargetType(ELandscapeToolTargetType InType) const override;
	virtual bool NeedsPersistentTextures() const override { return true; }; // it's a layer computed on the CPU and outputting to persistent textures
	virtual bool SupportsMultiple() const override { return false; } // only one layer of this type is allowed
	virtual bool SupportsBeingCollapsedAway() const override { return false; } // this is a procedural and therefore cannot be collapsed 
	virtual FString GetDefaultName() const override { return TEXT("Splines"); }
#if WITH_EDITOR
	virtual ELandscapeBlendMode GetBlendMode() const override { return ELandscapeBlendMode::LSBM_AlphaBlend; };
	bool SupportsAlphaForTargetType(ELandscapeToolTargetType InType) const override { return false; };
	virtual float GetAlphaForTargetType(ELandscapeToolTargetType InType) const override { return 1.0f; };
	virtual void SetAlphaForTargetType(ELandscapeToolTargetType InType, float InNewValue, bool bInModify, EPropertyChangeType::Type InChangeType) override { /* do nothing */ };
	// End ULandscapeEditLayerBase implementation
#endif // WITH_EDITOR
protected:

};