// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "LandscapeGizmoActor.h"

#include "LandscapeGizmoActiveActor.generated.h"

class ULandscapeInfo;
class ULandscapeLayerInfoObject;
class UMaterial;
class UMaterialInstance;
class UTexture2D;

UENUM()
enum ELandscapeGizmoType : int
{
	LGT_None,
	LGT_Height,
	LGT_Weight,
	LGT_MAX,
};

UENUM()
enum class ELandscapeGizmoSnapType
{
	/** No snapping of the gizmo */
	None,

	/** Snap the gizmo to the closest landscape component bounds */
	Component,

	/** Snap the gizmo to the closest landscape texel (vertex) */
	Texel
};

USTRUCT()
struct FGizmoSelectData
{
	GENERATED_USTRUCT_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	float Ratio;

	UPROPERTY()
	float HeightData;
#endif // WITH_EDITORONLY_DATA

	TMap<ULandscapeLayerInfoObject*, float>	WeightDataMap;

	FGizmoSelectData()
#if WITH_EDITORONLY_DATA
		: Ratio(0.0f)
		, HeightData(0.0f)
		, WeightDataMap()
#endif
	{
	}
	
};

UCLASS(notplaceable, MinimalAPI)
class ALandscapeGizmoActiveActor : public ALandscapeGizmoActor
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	TEnumAsByte<enum ELandscapeGizmoType> DataType;

	UPROPERTY(Transient)
	TObjectPtr<UTexture2D> GizmoTexture;

	UPROPERTY()
	FVector2D TextureScale;

	UPROPERTY()
	TArray<FVector> SampledHeight;

	UPROPERTY()
	TArray<FVector> SampledNormal;

	UPROPERTY()
	int32 SampleSizeX;

	UPROPERTY()
	int32 SampleSizeY;

	UPROPERTY()
	float CachedWidth;

	UPROPERTY()
	float CachedHeight;

	UPROPERTY()
	float CachedScaleXY;

	UPROPERTY(Transient)
	FVector FrustumVerts[8];

	UPROPERTY()
	TObjectPtr<UMaterial> GizmoMaterial;

	UPROPERTY()
	TObjectPtr<UMaterialInstance> GizmoDataMaterial;

	UPROPERTY()
	TObjectPtr<UMaterial> GizmoMeshMaterial;

	UPROPERTY(Category=LandscapeGizmoActiveActor, VisibleAnywhere)
	TArray<TObjectPtr<ULandscapeLayerInfoObject>> LayerInfos;

	UPROPERTY(transient)
	ELandscapeGizmoSnapType SnapType = ELandscapeGizmoSnapType::None;

	UPROPERTY(transient)
	bool bFollowTerrainHeight = true;

	UPROPERTY(transient)
	FRotator UnsnappedRotation;
#endif // WITH_EDITORONLY_DATA

public:
	TMap<FIntPoint, FGizmoSelectData> SelectedData;

#if WITH_EDITOR
	//~ Begin UObject Interface.
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditMove(bool bFinished) override;
	//~ End UObject Interface.

	virtual FVector SnapToLandscapeGrid(const FVector& GizmoLocation) const;
	virtual FRotator SnapToLandscapeGrid(const FRotator& GizmoRotation) const;

	virtual void EditorApplyTranslation(const FVector& DeltaTranslation, bool bAltDown, bool bShiftDown, bool bCtrlDown) override;
	virtual void EditorApplyRotation(const FRotator& DeltaRotation, bool bAltDown, bool bShiftDown, bool bCtrlDown) override;
	virtual void EditorApplyScale(const FVector& DeltaScale, const FVector* PivotLocation, bool bAltDown, bool bShiftDown, bool bCtrlDown) override;

	/**
	 * Whenever the decal actor has moved:
	 *  - Copy the actor rot/pos info over to the decal component
	 *  - Trigger updates on the decal component to recompute its matrices and generate new geometry.
	 */
	LANDSCAPE_API ALandscapeGizmoActor* SpawnGizmoActor();

	// @todo document
	LANDSCAPE_API void ClearGizmoData();

	// @todo document
	LANDSCAPE_API void FitToSelection();

	// @todo document
	LANDSCAPE_API void FitMinMaxHeight();

	// @todo document
	LANDSCAPE_API void SetTargetLandscape(ULandscapeInfo* LandscapeInfo);

	// @todo document
	void CalcNormal();

	// @todo document
	LANDSCAPE_API void SampleData(int32 SizeX, int32 SizeY);

	// @todo document
	LANDSCAPE_API void ExportToClipboard();

	// @todo document
	LANDSCAPE_API void ImportFromClipboard();

	// @todo document
	LANDSCAPE_API void Import(int32 VertsX, int32 VertsY, uint16* HeightData, TArray<ULandscapeLayerInfoObject*> ImportLayerInfos, uint8* LayerDataPointers[] );

	// @todo document
	LANDSCAPE_API void Export(int32 Index, TArray<FString>& Filenames);

	// @todo document
	LANDSCAPE_API float GetNormalizedHeight(uint16 LandscapeHeight) const;

	// @todo document
	LANDSCAPE_API float GetLandscapeHeight(float NormalizedHeight) const;

	// @todo document
	float GetWidth() const
	{
		return static_cast<float>(Width * GetRootComponent()->GetRelativeScale3D().X);
	}

	// @todo document
	float GetHeight() const
	{
		return static_cast<float>(Height * GetRootComponent()->GetRelativeScale3D().Y);
	}

	// @todo document
	float GetLength() const
	{
		return static_cast<float>(LengthZ * GetRootComponent()->GetRelativeScale3D().Z);
	}

	// @todo document
	void SetLength(float WorldLength)
	{
		LengthZ = static_cast<float>(WorldLength / GetRootComponent()->GetRelativeScale3D().Z);
	}

	static const int32 DataTexSize = 128;

private:
	// @todo document
	FORCEINLINE float GetWorldHeight(float NormalizedHeight) const;
#endif
};
