// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"

#include "LandscapeGizmoActor.generated.h"

class UBillboardComponent;
class ULandscapeInfo;

UCLASS(notplaceable, MinimalAPI, NotBlueprintable)
class ALandscapeGizmoActor : public AActor
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category=Gizmo)
	float Width;

	UPROPERTY(EditAnywhere, Category=Gizmo)
	float Height;

	UPROPERTY(EditAnywhere, Category=Gizmo)
	float LengthZ;

	UPROPERTY(EditAnywhere, Category=Gizmo)
	float MarginZ;

	UPROPERTY(EditAnywhere, Category=Gizmo)
	float MinRelativeZ;

	UPROPERTY(EditAnywhere, Category=Gizmo)
	float RelativeScaleZ;

	UPROPERTY(EditAnywhere, transient, Category=Gizmo)
	TWeakObjectPtr<ULandscapeInfo> TargetLandscapeInfo;

private:
	UPROPERTY()
	TObjectPtr<UBillboardComponent> SpriteComponent;
#endif // WITH_EDITORONLY_DATA
public:


#if WITH_EDITOR
	virtual bool ShouldExport() override { return false; } // Prevent copy-pasting
	virtual void Duplicate(ALandscapeGizmoActor* Gizmo); 

	bool EditorCanAttachTo(const AActor* InParent, FText& OutReason) const override	{ return false;	}
#endif

	/** 
	 * Indicates whether this actor should participate in level bounds calculations
	 */
	virtual bool IsLevelBoundsRelevant() const override { return false; }

public:
#if WITH_EDITORONLY_DATA
	/** Returns SpriteComponent subobject **/
	LANDSCAPE_API UBillboardComponent* GetSpriteComponent() const;
#endif
};



