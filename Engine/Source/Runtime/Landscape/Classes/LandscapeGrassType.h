// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Engine/EngineTypes.h"
#include "UObject/PerPlatformProperties.h"
#include "PerQualityLevelProperties.h"
#include "SceneTypes.h"
#include "LandscapeGrassType.generated.h"

#define UE_API LANDSCAPE_API

class UStaticMesh;
struct FPropertyChangedEvent;

UENUM()
enum class EGrassScaling : uint8
{
	/** Grass instances will have uniform X, Y and Z scales. */
	Uniform,
	/** Grass instances will have random X, Y and Z scales. */
	Free,
	/** X and Y will be the same random scale, Z will be another */
	LockXY,
};

USTRUCT(BlueprintType)
struct FGrassVariety
{
	GENERATED_USTRUCT_BODY()

	UE_API FGrassVariety();

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=Grass)
	TObjectPtr<UStaticMesh> GrassMesh;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=Grass, meta = (ToolTip = "Material Overrides."))
	TArray<TObjectPtr<class UMaterialInterface>> OverrideMaterials;

	/* Instances per 10 square meters. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=Grass, meta = (UIMin = 0, ClampMin = 0, UIMax = 1000, ClampMax = 1000))
	FPerPlatformFloat GrassDensity;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=Grass, meta = (UIMin = 0, ClampMin = 0, UIMax = 1000, ClampMax = 1000))
	FPerQualityLevelFloat GrassDensityQuality;

	/* If true, use a jittered grid sequence for placement, otherwise use a halton sequence. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=Grass)
	bool bUseGrid;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=Grass, meta = (EditCondition = "bUseGrid", UIMin = 0, ClampMin = 0, UIMax = 1, ClampMax = 1))
	float PlacementJitter;

	/* The distance where instances will begin to fade out if using a PerInstanceFadeAmount material node. 0 disables. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=Grass, meta = (UIMin = 0, ClampMin = 0, UIMax = 1000000, ClampMax = 1000000))
	FPerPlatformInt StartCullDistance;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Grass, meta = (UIMin = 0, ClampMin = 0, UIMax = 1000000, ClampMax = 1000000))
	FPerQualityLevelInt StartCullDistanceQuality;

	/**
	 * The distance where instances will have completely faded out when using a PerInstanceFadeAmount material node. 0 disables. 
	 * When the entire cluster is beyond this distance, the cluster is completely culled and not rendered at all.
	 */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Grass, meta = (UIMin = 0, ClampMin = 0, UIMax = 1000000, ClampMax = 1000000))
	FPerPlatformInt EndCullDistance;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Grass, meta = (UIMin = 0, ClampMin = 0, UIMax = 1000000, ClampMax = 1000000))
	FPerQualityLevelInt EndCullDistanceQuality;

	/** 
	 * Specifies the smallest LOD that will be used for this component.
	 * If -1 (default), the MinLOD of the static mesh asset will be used instead.
	 */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Grass, meta = (UIMin = -1, ClampMin = -1, UIMax = 8, ClampMax = 8))
	int32 MinLOD;

	/** Specifies the density range where the grass variety is allowed to be spawned ([0,1] represents the entire range). */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=Grass, meta = (UIMin = 0.0, ClampMin = 0.0, UIMax = 1.0, ClampMax = 1.0))
	FFloatInterval AllowedDensityRange;

	/** Specifies grass instance scaling type */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=Grass)
	EGrassScaling Scaling;

	/** Specifies the range of scale, from minimum to maximum, to apply to a grass instance's X Scale property */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=Grass)
	FFloatInterval ScaleX;

	/** Specifies the range of scale, from minimum to maximum, to apply to a grass instance's Y Scale property */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=Grass, meta = (EditCondition = "Scaling == EGrassScaling::Free"))
	FFloatInterval ScaleY;

	/** Specifies the range of scale, from minimum to maximum, to apply to a grass instance's Z Scale property */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=Grass, meta = (EditCondition = "Scaling == EGrassScaling::Free || Scaling == EGrassScaling::LockXY"))
	FFloatInterval ScaleZ;

	/** If enabled the the scale of instances is reduced as the weight (density) decreases */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, AdvancedDisplay, Category = Grass)
	bool bWeightAttenuatesMaxScale = false;

	/** Modulate the scale of the instances based on weight (normalized density). The weight range (ScaleWeightAttenuation, 1.0) maps to (scaleMin, scaleMax), weight values less than ScaleWeightAttenuation are set to minScale */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, AdvancedDisplay, Category = Grass, meta = (UIMin = 0, ClampMin = 0, UIMax = 1, ClampMax = 1, EditCondition = "bWeightAttenuatesMaxScale == true"))
	float MaxScaleWeightAttenuation = 0.5f;

	/** Whether the grass instances should be placed at random rotation (true) or all at the same rotation (false) */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Grass)
	bool RandomRotation;

	/** Whether the grass instances should be tilted to the normal of the landscape (true), or always vertical (false) */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Grass)
	bool AlignToSurface;

	/** When aligning to the surface when using grid mode, whether we should align to triangle normal (true) or a smooth grid-based normal (false).  Halton always uses triangle normal. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Grass, meta = (EditCondition = "AlignToSurface && bUseGrid"))
	bool bAlignToTriangleNormals;

	/* Whether to use the landscape's lightmap when rendering the grass. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Grass)
	bool bUseLandscapeLightmap;

	/**
	 * Lighting channels that the grass will be assigned. Lights with matching channels will affect the grass.
	 * These channels only apply to opaque materials, direct lighting, and dynamic lighting and shadowing.
	 */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, AdvancedDisplay, Category = Grass)
	FLightingChannels LightingChannels;

	/** Whether the grass instances should receive decals. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Grass)
	bool bReceivesDecals;

	/** Controls whether the primitive should affect dynamic distance field lighting methods. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Grass)
	bool bAffectDistanceFieldLighting;

	/** Whether the grass should cast shadows when using non-precomputed shadowing. **/
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Grass)
	bool bCastDynamicShadow;

	/** Whether the grass should cast contact shadows. **/
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Grass)
	bool bCastContactShadow;

	/** Whether we should keep a cpu copy of the instance buffer. This should be set to true if you plan on using GetOverlappingXXXXCount functions of the component otherwise it won't return any data.**/
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Grass)
	bool bKeepInstanceBufferCPUCopy;

	/** Distance at which to grass instances should disable WPO for performance reasons */
	UPROPERTY(EditAnywhere, Category = Grass)
	uint32 InstanceWorldPositionOffsetDisableDistance;

	/** Control shadow invalidation behavior, in particular with respect to Virtual Shadow Maps and material effects like World Position Offset. */
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category=Grass, AdvancedDisplay)
	EShadowCacheInvalidationBehavior ShadowCacheInvalidationBehavior;

	UE_API bool IsGrassQualityLevelEnable() const;

	UE_API int32 GetStartCullDistance() const;

	UE_API int32 GetEndCullDistance() const;

	UE_API float GetDensity() const;

};

UCLASS(BlueprintType, MinimalAPI)
class ULandscapeGrassType : public UObject
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Grass)
	TArray<FGrassVariety> GrassVarieties;

	/**
	* Whether this grass type should be affected by the Engine Scalability system's Foliage grass.DensityScale setting. 
	* This is enabled by default but can be disabled should this grass type be important for gameplay reasons.
	*/
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = Scalability)
	uint32 bEnableDensityScaling : 1;

	UPROPERTY()
	uint32 StateHash = 0;

	UPROPERTY()
	TObjectPtr<UStaticMesh> GrassMesh_DEPRECATED;
	UPROPERTY()
	float GrassDensity_DEPRECATED;
	UPROPERTY()
	float PlacementJitter_DEPRECATED;
	UPROPERTY()
	int32 StartCullDistance_DEPRECATED;
	UPROPERTY()
	int32 EndCullDistance_DEPRECATED;
	UPROPERTY()
	bool RandomRotation_DEPRECATED;
	UPROPERTY()
	bool AlignToSurface_DEPRECATED;
	
	//~ Begin UObject Interface
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject Interface

	uint32 ComputeStateHash();

	/** 
	 * Return a FName that corresponds to this grass type. It used to be that grass types were unique for spawning grass on a given component. Nowadays, 
	 *  we use named "slots" for each grass type (the GrassName being the slot's name in the landscape material). This function here returns a fallback
	 *  name for when serializing old data that was grass type-based only. 
	*/
	inline FName GetGrassNameFallback() const
	{
		// Simply use the UObject's name as the fallback since it's unique on a given landscape component (because grass types were unique on a given component already)
		return GetFName();
	}
};

#undef UE_API
