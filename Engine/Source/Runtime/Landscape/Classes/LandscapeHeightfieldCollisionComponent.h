// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Templates/RefCounting.h"
#include "EngineDefines.h"
#include "AI/Navigation/NavigationTypes.h"
#include "Components/PrimitiveComponent.h"
#include "Serialization/BulkData.h"
#include "HAL/CriticalSection.h"

#include "LandscapeHeightfieldCollisionComponent.generated.h"

class ALandscapeProxy;
class FAsyncPreRegisterDDCRequest;
class ULandscapeComponent;
class ULandscapeInfo;
class ULandscapeLayerInfoObject;
class UPhysicalMaterial;
struct FConvexVolume;
struct FEngineShowFlags;
struct FNavigableGeometryExport;

namespace Chaos
{
	class FHeightField;
}

enum class EHeightfieldSource
{
	None,
	Simple,
	Complex,
	Editor
};

UCLASS(MinimalAPI, Within=LandscapeProxy)
class ULandscapeHeightfieldCollisionComponent : public UPrimitiveComponent
{
	friend class FLandscapeHeightfieldCollisionComponentSceneProxy;

	GENERATED_UCLASS_BODY()

	ULandscapeHeightfieldCollisionComponent(FVTableHelper& Helper);
	virtual ~ULandscapeHeightfieldCollisionComponent();

	/** List of layers painted on this component. Matches the WeightmapLayerAllocations array in the LandscapeComponent. */
	UPROPERTY()
	TArray<TObjectPtr<ULandscapeLayerInfoObject>> ComponentLayerInfos;

	/** Offset of component in landscape quads */
	UPROPERTY()
	int32 SectionBaseX;

	UPROPERTY()
	int32 SectionBaseY;

	/** Size of component in collision quads */
	UPROPERTY()
	int32 CollisionSizeQuads;

	/** Collision scale: (ComponentSizeQuads) / (CollisionSizeQuads) */
	UPROPERTY()
	float CollisionScale;

	/** Size of component's "simple collision" in collision quads */
	UPROPERTY()
	int32 SimpleCollisionSizeQuads;

	bool bCookedCollisionDataWasDeleted = false;

	/** The flags for each collision quad. See ECollisionQuadFlags. */
	UPROPERTY()
	TArray<uint8> CollisionQuadFlags;

	/** Guid used to share Physics heightfield objects in the editor */
	UPROPERTY()
	FGuid HeightfieldGuid;

	/** Cached local-space bounding box, created at heightmap update time */
	UPROPERTY()
	FBox CachedLocalBox;

#if WITH_EDITORONLY_DATA
	/** Reference to render component */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY()
	TLazyObjectPtr<ULandscapeComponent> RenderComponent_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // !WITH_EDITORONLY_DATA

private:
	/** Reference to render component */
	UPROPERTY()
	TObjectPtr<ULandscapeComponent> RenderComponentRef;

	UPROPERTY(Transient, DuplicateTransient, TextExportTransient, NonTransactional)
	TArray<FBox> DeferredSnapFoliageInstanceBoxes;

public:
	/** Returns associated landscape component */
	UFUNCTION(BlueprintCallable, Category = "Landscape")
	LANDSCAPE_API ULandscapeComponent* GetRenderComponent() const;

	struct FHeightfieldGeometryRef : public UE::Private::FQueryableRefCountedObject
	{
		FGuid Guid;

		TArray<Chaos::FMaterialHandle> UsedChaosMaterials;
		Chaos::FHeightFieldPtr HeightfieldGeometry;
	    Chaos::FHeightFieldPtr HeightfieldSimpleGeometry;

#if WITH_EDITORONLY_DATA
		Chaos::FHeightFieldPtr EditorHeightfieldGeometry;
#endif // WITH_EDITORONLY_DATA

		FHeightfieldGeometryRef(FGuid& InGuid);
		virtual ~FHeightfieldGeometryRef();
		
		void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize);
	};
	
#if WITH_EDITORONLY_DATA
	friend struct FEnableCollisionHashOptimScope;
	
	UPROPERTY()
	uint32										CollisionHash = 0;

	/** The collision height values. Stripped from cooked content */
	FWordBulkData								CollisionHeightData;

	/** Indices into the ComponentLayers array for the per-vertex dominant layer. Stripped from cooked content */
	FByteBulkData								DominantLayerData;

	/** Indices for physical materials generated by the render material. Stripped from cooked content */
	FByteBulkData								PhysicalMaterialRenderData;

	/** Physical materials objects referenced by the indices in PhysicalMaterialRenderData. Stripped from cooked content */
	UPROPERTY()
	TArray<TObjectPtr<UPhysicalMaterial>>					PhysicalMaterialRenderObjects;

	/*  Cooked editor specific heightfield data, never serialized  */
	TArray<uint8>								CookedCollisionDataEd;

	/** 
	 *	Flag to indicate that the next time we cook data, we should save it to the DDC.
	 *	Used to ensure DDC is populated when loading content for the first time. 
	 *  For editor and full version of collision objects
	 */
	mutable bool								bShouldSaveCookedDataToDDC[2];

	/**
	  * Async DCC load for cooked collision representation. We speculatively
	  * load this to remove hitch when streaming 
	  */
	mutable TSharedPtr<FAsyncPreRegisterDDCRequest>	SpeculativeDDCRequest;

#endif //WITH_EDITORONLY_DATA

	/** 
	 *	Cooked HeightField data. Serialized only with cooked content 
	 *	Stored as array instead of BulkData to take advantage of precaching during async loading
	 */
	TArray<uint8>								CookedCollisionData;
	
	/** This is a list of physical materials that is actually used by a cooked HeightField */
	UPROPERTY()
	TArray<TObjectPtr<UPhysicalMaterial>>					CookedPhysicalMaterials;
	
	/** Physics engine version of heightfield data. */
	TRefCountPtr<FHeightfieldGeometryRef>	HeightfieldRef;
	
	// local non-serialized ref counted pointers to keep the chaos heightfields alive between Unregister() and actual destruction of the component.
	// this allows us to re-use them if the component gets a call to Register() again
	Chaos::FHeightFieldPtr LocalHeightfieldGeometryRef;
	Chaos::FHeightFieldPtr LocalHeightfieldSimpleGeometryRef;

	/** Cached PxHeightFieldSamples values for navmesh generation. Note that it's being used only if navigation octree is set up for lazy geometry exporting */
	int32 HeightfieldRowsCount;
	int32 HeightfieldColumnsCount;

	mutable FRWLock CachedHeightFieldSamplesLock;
	FNavHeightfieldSamples CachedHeightFieldSamples;

	enum ECollisionQuadFlags : uint8
	{
		QF_PhysicalMaterialMask = 63,	// Mask value for the physical material index, stored in the lower 6 bits.
		QF_EdgeTurned = 64,				// This quad's diagonal has been turned.
		QF_NoCollision = 128,			// This quad has no collision.
	};

#if WITH_EDITORONLY_DATA
private:
	bool bEnableCollisionHashOptim = false;
#endif //WITH_EDITORONLY_DATA
	
	//~ Begin UActorComponent Interface.
protected:
	virtual void OnCreatePhysicsState() override;
	virtual void OnDestroyPhysicsState() override;
public:

	virtual void ApplyWorldOffset(const FVector& InOffset, bool bWorldShift) override;
	//~ End UActorComponent Interface.

	//~ Begin USceneComponent Interface.
	virtual void DestroyComponent(bool bPromoteChildren = false) override;
	virtual FBoxSphereBounds CalcBounds(const FTransform &BoundTransform) const override;

	virtual ECollisionEnabled::Type GetCollisionEnabled() const override;
	virtual ECollisionResponse GetCollisionResponseToChannel(ECollisionChannel Channel) const override;
	virtual ECollisionChannel GetCollisionObjectType() const override;
	virtual const FCollisionResponseContainer& GetCollisionResponseToChannels() const override;
	virtual void OnRegister() override;
	virtual void OnUnregister() override;
	virtual bool AllowsAsyncPhysicsStateCreation() const override;
	//~ End USceneComponent Interface.

	//~ Begin UPrimitiveComponent Interface
	virtual bool DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const override;
	virtual bool IsShown(const FEngineShowFlags& ShowFlags) const override;
	//End UPrimitiveComponent interface

	//~ Begin INavRelevantInterface Interface
	virtual bool SupportsGatheringGeometrySlices() const override { return true; }
	virtual void GatherGeometrySlice(FNavigableGeometryExport& GeomExport, const FBox& SliceBox) const override;
	virtual ENavDataGatheringMode GetGeometryGatheringMode() const override;
	virtual void PrepareGeometryExportSync() override;
	//~ End INavRelevantInterface Interface

	//~ Begin UObject Interface.
	virtual void Serialize(FArchive& Ar) override;
	virtual void BeginDestroy() override;
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	virtual void PostLoad() override;
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
#if WITH_EDITOR
	virtual bool NeedsLoadForClient() const override;
	virtual bool NeedsLoadForServer() const override;
	virtual void ExportCustomProperties(FOutputDevice& Out, uint32 Indent) override;
	virtual void ImportCustomProperties(const TCHAR* SourceText, FFeedbackContext* Warn) override;
	//~ End UObject Interface.
		
	/**  We speculatively async load collision object from DDC to remove hitch when streaming */
	void SpeculativelyLoadAsyncDDCCollsionData();

	/** 
	 * Cooks raw height data into collision object binary stream
	 */
	bool GenerateCollisionObjects(const FName& Format, bool bUseDefaultMaterialOnly, Chaos::FHeightFieldPtr& OutHeightField, bool bGenerateSimpleCollision, Chaos::FHeightFieldPtr& OutSimpleHeightField, TArray<UPhysicalMaterial*>& InOutMaterials) const;
	virtual bool CookCollisionData(const FName& Format, bool bUseDefaultMaterialOnly, bool bCheckDDC, TArray<uint8>& OutCookedData, TArray<UPhysicalMaterial*>& InOutMaterials) const;

	/** Modify a sub-region of the physics heightfield. Note that this does not update the physical material */
	void UpdateHeightfieldRegion(int32 ComponentX1, int32 ComponentY1, int32 ComponentX2, int32 ComponentY2);

	/** Computes a hash of all the data that will impact final collision */
	virtual uint32 ComputeCollisionHash() const;

	LANDSCAPE_API void MarkAreaForSnapFoliageInstances(const FBox& InBox);
#endif

	struct FCollisionSampleInfo
	{
		int32 CollisionSizeVerts = 0;
		int32 SimpleCollisionSizeVerts = 0;
		int32 NumSamples = 0;
		int32 NumSimpleSamples = 0;
	};
	FCollisionSampleInfo GetCollisionSampleInfo() const;

	struct FWriteRuntimeDataParams
	{
		bool bUseDefaultMaterialOnly = false;
		bool bProcessRenderIndices = true;
		bool bProcessVisibilityLayer = true;
		TArrayView<const uint16> Heights;
		TArrayView<const uint16> SimpleHeights;
		TArrayView<const uint8> DominantLayers;
		TArrayView<const uint8> SimpleDominantLayers;
		TArrayView<const uint8> RenderPhysicalMaterialIds;
		TArrayView<const uint8> SimpleRenderPhysicalMaterialIds;
		TArrayView<const TObjectPtr<UPhysicalMaterial>> PhysicalMaterialRenderObjects;
		TArrayView<const TObjectPtr<ULandscapeLayerInfoObject>> ComponentLayerInfos;
		int32 VisibilityLayerIndex = INDEX_NONE;
	};

#if WITH_EDITOR
	// Set up params to call WriteRuntimeData using editor data members.
	FWriteRuntimeDataParams MakeWriteRuntimeDataParams(bool bUseDefaultMaterialOnly) const;
#endif

	// Writes to a cooked data buffer using raw heightfield data
	LANDSCAPE_API bool WriteRuntimeData(const FWriteRuntimeDataParams& Params, TArray<uint8>& OutHeightfieldData, TArray<UPhysicalMaterial*>& InOutMaterials) const;

	bool GenerateCollisionData(const FWriteRuntimeDataParams& Params, Chaos::FHeightFieldPtr& OutHeightField, bool bGenerateSimpleCollision, Chaos::FHeightFieldPtr& OutSimpleHeightField, TArray<UPhysicalMaterial*>& InOutMaterials) const;

	/** Gets the landscape info object for this landscape */
	ULandscapeInfo* GetLandscapeInfo() const;

#if UE_ENABLE_DEBUG_DRAWING
	// The scene proxy is only for debug purposes :
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
#endif // UE_ENABLE_DEBUG_DRAWING

	/** Creates collision object from a cooked collision data */
	virtual void CreateCollisionObject();

	/** Creates collision object from raw runtime data.  Data is assumed at this point to contain valid physical material indices with visibility layer set to 0xFF*/
	LANDSCAPE_API void CreateCollisionObject(
		bool bUseDefaultMaterialOnly,
		TArrayView<const uint16> Heights, TArrayView<const uint16> SimpleHeights,
		TArrayView<const uint8> PhysicalMaterialIds, TArrayView<const uint8> SimplePhysicalMaterialIds,
		TArrayView<const TObjectPtr<UPhysicalMaterial>> PhysicalMaterialObjects);

	/** Return the landscape actor associated with this component. */
	LANDSCAPE_API ALandscapeProxy* GetLandscapeProxy() const;

	/** @return Component section base as FIntPoint */
	LANDSCAPE_API FIntPoint GetSectionBase() const; 

	/** @param InSectionBase new section base for a component */
	LANDSCAPE_API void SetSectionBase(FIntPoint InSectionBase);

	/** Recreate heightfield and restart physics */
	LANDSCAPE_API virtual bool RecreateCollision();

private:
	// @todo(chaos): remove when implicit objects are ref counted
	void DeferredDestroyCollision(const TRefCountPtr<FHeightfieldGeometryRef>& HeightfieldRefLifetimeExtender);
public:

#if WITH_EDITORONLY_DATA
	// Foliage snapping functions : called from editor code to manage foliage instances on landscape.
	UE_DEPRECATED(5.8, "Use the method taking a ESnapFoliageInstancesFlags parameter")
	LANDSCAPE_API void SnapFoliageInstances();

	enum class ESnapFoliageInstancesFlags
	{
		ForceAll = 1 << 0, // Force-snap foliage instances across the entire component
		MarkedAreasOnly = 1 << 1, // Only update the areas that have already been marked for snapping via MarkAreaForSnapFoliageInstances
	};
	FRIEND_ENUM_CLASS_FLAGS(ESnapFoliageInstancesFlags);
	/** Snap foliage instances overlapping with the component */
	LANDSCAPE_API void SnapFoliageInstances(ESnapFoliageInstancesFlags InFlags);
	/** Snap foliage instances overlapping with the specified area only */
	LANDSCAPE_API void SnapFoliageInstances(const FBox& InInstanceBox);

#endif

	void SetRenderComponent(ULandscapeComponent* InRenderComponent) { RenderComponentRef = InRenderComponent; }

public:
	LANDSCAPE_API TOptional<float> GetHeight(float X, float Y, EHeightfieldSource HeightFieldSource);
	LANDSCAPE_API UPhysicalMaterial* GetPhysicalMaterial(float X, float Y, EHeightfieldSource HeightFieldSource);

	/**
	 * Populates a supplied array with the heights from the heightfield.  Samples are placed
	 * in a tile defined by the starting point (Offset) and the stride/row
	 */
	LANDSCAPE_API bool FillHeightTile(TArrayView<float> Heights, int32 Offset, int32 Stride) const;
	LANDSCAPE_API bool FillMaterialIndexTile(TArrayView<uint8> Materials, int32 Offset, int32 Stride) const;
};



