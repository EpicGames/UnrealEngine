// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "UObject/LazyObjectPtr.h"
#include "UObject/ScriptInterface.h"
#include "ILandscapeSplineInterface.h"
#include "LandscapeEditTypes.h"

#if WITH_EDITOR
#include "WorldPartition/WorldPartitionHandle.h"
#endif

#include "LandscapeInfo.generated.h"

class ALandscape;
class ALandscapeProxy;
class ALandscapeStreamingProxy;
class ALandscapeSplineActor;
class ULandscapeComponent;
class ULandscapeLayerInfoObject;
class ULevel;
class UMaterialInstanceConstant;
class ULandscapeSplinesComponent;
class ULandscapeSplineControlPoint;
class ULandscapeSplineSegment;
class ULandscapeHeightfieldCollisionComponent;
class ULandscapeInfo;
class FModulateAlpha;
struct FLandscapeEditorLayerSettings;
struct FLandscapeTargetLayerSettings;

/** Structure storing Collision for LandscapeComponent Add */
#if WITH_EDITORONLY_DATA
struct FLandscapeAddCollision
{
	FVector Corners[4];

	FLandscapeAddCollision()
	{
		Corners[0] = Corners[1] = Corners[2] = Corners[3] = FVector::ZeroVector;
	}
};
#endif // WITH_EDITORONLY_DATA


#if WITH_EDITOR
struct FLandscapeDirtyOnlyInModeScope
{
	FLandscapeDirtyOnlyInModeScope() = delete;
	FLandscapeDirtyOnlyInModeScope(ULandscapeInfo* InLandscapeInfo);
	FLandscapeDirtyOnlyInModeScope(ULandscapeInfo* InLandscapeInfo, bool bInOverrideDirtyMode); /* Override the dirtying in Landscape mode behaviour*/
	~FLandscapeDirtyOnlyInModeScope();

private:
	ULandscapeInfo* LandscapeInfo;
	bool bDirtyOnlyInModePrevious;
};
#endif

USTRUCT()
struct FLandscapeInfoLayerSettings
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TObjectPtr<ULandscapeLayerInfoObject> LayerInfoObj;

	UPROPERTY()
	FName LayerName;

#if WITH_EDITORONLY_DATA
	UPROPERTY(transient)
	TObjectPtr<UMaterialInstanceConstant> ThumbnailMIC;

	UPROPERTY()
	TWeakObjectPtr<ALandscapeProxy> Owner;

	UPROPERTY(transient)
	int32 DebugColorChannel;
#endif // WITH_EDITORONLY_DATA

	FLandscapeInfoLayerSettings()
		: LayerInfoObj(nullptr)
		, LayerName(NAME_None)
#if WITH_EDITORONLY_DATA
		, ThumbnailMIC(nullptr)
		, Owner(nullptr)
		, DebugColorChannel(0)
#endif // WITH_EDITORONLY_DATA
	{
	}

	LANDSCAPE_API FLandscapeInfoLayerSettings(ULandscapeLayerInfoObject* InLayerInfo, ALandscapeProxy* InProxy);
	LANDSCAPE_API FLandscapeInfoLayerSettings(FName InPlaceholderLayerName, ALandscapeProxy* InProxy);

	LANDSCAPE_API FName GetLayerName() const;

#if WITH_EDITORONLY_DATA
	LANDSCAPE_API const FLandscapeTargetLayerSettings& GetTargetLayerSettings() const;
#endif // WITH_EDITORONLY_DATA
};

USTRUCT()
struct FTargetLayerBlendGroup
{
	GENERATED_USTRUCT_BODY()

public:
	friend class ULandscapeInfo;

	FTargetLayerBlendGroup() = default;
	FTargetLayerBlendGroup(const FName& InName, ELandscapeTargetLayerBlendMethod InBlendMethod, const TSet<ULandscapeLayerInfoObject*>& InTargetLayers)
		: Name(InName)
		, BlendMethod(InBlendMethod)
		, TargetLayers(ObjectPtrWrap(InTargetLayers))
	{
	}

	const FName& GetName() const
	{
		return Name;
	}

	ELandscapeTargetLayerBlendMethod GetBlendMethod() const
	{
		return BlendMethod;
	}

	const TSet<TObjectPtr<ULandscapeLayerInfoObject>>& GetTargetLayers() const
	{
		return TargetLayers;
	}

private:
	UPROPERTY()
	FName Name;

	UPROPERTY()
	ELandscapeTargetLayerBlendMethod BlendMethod = ELandscapeTargetLayerBlendMethod::Count;

	UPROPERTY()
	TSet<TObjectPtr<ULandscapeLayerInfoObject>> TargetLayers;
};

UCLASS(Transient)
class ULandscapeInfo : public UObject
{
	GENERATED_UCLASS_BODY()

	friend class ALandscapeProxy;

	UPROPERTY()
	TWeakObjectPtr<ALandscape> LandscapeActor;

	UPROPERTY()
	FGuid LandscapeGuid;

	UPROPERTY()
	int32 ComponentSizeQuads;
	
	UPROPERTY()
	int32 SubsectionSizeQuads;

	UPROPERTY()
	int32 ComponentNumSubsections;
	
#if WITH_EDITORONLY_DATA
	/** True when the draw scale has been set by the landscape Actor (false if default or set by a streaming proxy) */
	UPROPERTY()
	bool bDrawScaleSetByActor = false;

	UPROPERTY()
	FVector DrawScale = FVector(100.0);

	UPROPERTY()
	TArray<FLandscapeInfoLayerSettings> Layers;

	UPROPERTY()
	int32 RegionSizeInComponents;
#endif // WITH_EDITORONLY_DATA

public:
	/** Map of the offsets (in component space) to the component. Valid in editor only. */
	TMap<FIntPoint, ULandscapeComponent*> XYtoComponentMap;
    /** Map of the offsets (in component space) to the collision components. Should always be valid. */
	TMap<FIntPoint, ULandscapeHeightfieldCollisionComponent*> XYtoCollisionComponentMap;

#if WITH_EDITORONLY_DATA
	/** Lookup map used by the "add component" tool. Only available near valid LandscapeComponents.
	    only for use by the "add component" tool. Todo - move into the tool? */
	TMap<FIntPoint, FLandscapeAddCollision> XYtoAddCollisionMap;
#endif // WITH_EDITORONLY_DATA

	UE_DEPRECATED(5.7, "StreamingProxies rely on a sorted state and should not be modified outside of ULandscapeInfo. Use GetSortedStreamingProxies instead.")
	UPROPERTY()
	TArray<TWeakObjectPtr<ALandscapeStreamingProxy>> StreamingProxies_DEPRECATED;

private:
#if WITH_EDITORONLY_DATA
	// SORTED list of all actors implementing the spline interface that are registered with this landscape info
	UPROPERTY()
	TArray<TScriptInterface<ILandscapeSplineInterface>> SplineActors;

	// Not properties since these shouldn't be modified through transactions (no undo/redo)
	TSet<TWeakObjectPtr<UPackage>, TWeakObjectPtrSetKeyFuncs<TWeakObjectPtr<UPackage>>> ModifiedPackages;
	TSet<TWeakObjectPtr<AActor>, TWeakObjectPtrSetKeyFuncs<TWeakObjectPtr<AActor>>> ActorsToDelete;

	bool bDirtyOnlyInMode;

	friend struct FLandscapeDirtyOnlyInModeScope;
#endif // WITH_EDITORONLY_DATA

	TSet<ULandscapeComponent*> SelectedComponents;

	TSet<ULandscapeComponent*> SelectedRegionComponents;

	FIntRect XYComponentBounds;

public:
	TMap<FIntPoint,float> SelectedRegion;

	//~ Begin UObject Interface.
	virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface

	LANDSCAPE_API FBox GetLoadedBounds() const;

#if WITH_EDITOR
	LANDSCAPE_API FBox GetCompleteBounds() const;
#endif

#if WITH_EDITOR
	// @todo document 
	// all below.
	LANDSCAPE_API bool SupportsLandscapeEditing() const;
	// False if landscape editing not supported, in PIE world, or in a Commandlet
	LANDSCAPE_API bool IsLandscapeEditableWorld() const;

	LANDSCAPE_API bool AreAllComponentsRegistered() const;
	LANDSCAPE_API void GetComponentsInRegion(int32 X1, int32 Y1, int32 X2, int32 Y2, TSet<ULandscapeComponent*>& OutComponents, bool bOverlap = true) const;
	LANDSCAPE_API bool HasUnloadedComponentsInRegion(int32 X1, int32 Y1, int32 X2, int32 Y2) const;
	LANDSCAPE_API bool GetLandscapeExtent(ALandscapeProxy* Proxy, FIntRect& ProxyExtent) const;
	LANDSCAPE_API bool GetLandscapeExtent(FIntRect& LandscapeExtent) const;
	LANDSCAPE_API bool GetLandscapeExtent(int32& MinX, int32& MinY, int32& MaxX, int32& MaxY) const;
	LANDSCAPE_API FIntRect GetCompleteLandscapeExtent() const;
	LANDSCAPE_API bool GetLandscapeXYComponentBounds(FIntRect& OutXYComponentBounds) const;
	LANDSCAPE_API void ForAllLandscapeComponents(TFunctionRef<void(ULandscapeComponent*)> Fn) const;
	LANDSCAPE_API void ExportHeightmap(const FString& Filename);
	LANDSCAPE_API void ExportHeightmap(const FString& Filename, const FIntRect& ExportRegion);
	LANDSCAPE_API void ExportLayer(ULandscapeLayerInfoObject* LayerInfo, const FString& Filename);
	LANDSCAPE_API void ExportLayer(ULandscapeLayerInfoObject* LayerInfo, const FString& Filename, const FIntRect& ExportRegion);
	UE_DEPRECATED(5.7, "Use ApplySplines without bOnlySelected. Edit layer landscapes update all splines regardless of selection state")
	LANDSCAPE_API bool ApplySplines(bool bOnlySelected, TSet<TObjectPtr<ULandscapeComponent>>* OutModifiedComponents = nullptr, bool bMarkPackageDirty = true);
	LANDSCAPE_API bool ApplySplines(TSet<TObjectPtr<ULandscapeComponent>>* OutModifiedComponents = nullptr, bool bMarkPackageDirty = true);

	LANDSCAPE_API bool GetSelectedExtent(int32& MinX, int32& MinY, int32& MaxX, int32& MaxY) const;
	FVector GetLandscapeCenterPos(float& LengthZ, int32 MinX = MAX_int32, int32 MinY = MAX_int32, int32 MaxX = MIN_int32, int32 MaxY = MIN_int32);
	LANDSCAPE_API bool IsValidPosition(int32 X, int32 Y);
	LANDSCAPE_API void DeleteLayer(ULandscapeLayerInfoObject* LayerInfo, const FName& LayerName);
	LANDSCAPE_API void ReplaceLayer(ULandscapeLayerInfoObject* FromLayerInfo, ULandscapeLayerInfoObject* ToLayerInfo);
	LANDSCAPE_API void GetUsedPaintLayers(const FGuid& InLayerGuid, TArray<ULandscapeLayerInfoObject*>& OutUsedLayerInfos) const;

	LANDSCAPE_API void UpdateDebugColorMaterial();

	LANDSCAPE_API TSet<ULandscapeComponent*> GetSelectedComponents() const;
	LANDSCAPE_API TSet<ULandscapeComponent*> GetSelectedRegionComponents() const;
	LANDSCAPE_API void UpdateSelectedComponents(TSet<ULandscapeComponent*>& NewComponents, bool bIsComponentwise = true);
	LANDSCAPE_API void ClearSelectedRegion(bool bIsComponentwise = true);

	// only for use by the "add component" tool. Todo - move into the tool?
	LANDSCAPE_API void UpdateAllAddCollisions();
	LANDSCAPE_API void UpdateAddCollision(FIntPoint LandscapeKey);

	LANDSCAPE_API const FLandscapeTargetLayerSettings& GetTargetLayerSettings(ULandscapeLayerInfoObject* LayerInfo) const;

	LANDSCAPE_API void CreateTargetLayerSettingsFor(ULandscapeLayerInfoObject* LayerInfo);

	LANDSCAPE_API ULandscapeLayerInfoObject* GetLayerInfoByName(FName LayerName, ALandscapeProxy* Owner = nullptr) const;
	LANDSCAPE_API int32 GetLayerInfoIndex(FName LayerName, ALandscapeProxy* Owner = nullptr) const;
	LANDSCAPE_API int32 GetLayerInfoIndex(ULandscapeLayerInfoObject* LayerInfo, ALandscapeProxy* Owner = nullptr) const;
	LANDSCAPE_API bool UpdateLayerInfoMap(ALandscapeProxy* Proxy = nullptr, bool bInvalidate = false);

	LANDSCAPE_API bool CanDeleteLandscape(FText& OutReason) const;

	/**
	 *  Returns the landscape proxy of this landscape info in the given level (if it exists)
	 *  @param  Level  Level to look in
	 *	@return        Landscape or landscape proxy found in the given level, or null if none
	 */
	LANDSCAPE_API ALandscapeProxy* GetLandscapeProxyForLevel(ULevel* Level) const;

	LANDSCAPE_API bool GetDirtyOnlyInMode() const;

	LANDSCAPE_API void OnModifiedPackageSaved(UPackage* InPackage);
	LANDSCAPE_API int32 GetModifiedPackageCount() const;
	LANDSCAPE_API TArray<UPackage*> GetModifiedPackages() const;
	LANDSCAPE_API bool IsPackageModified(UPackage* InPackage) const;
	/**
	 * Goes through the list of modified packages (as in IsPackageModified) and marks them as dirty
	 * 
	 * @return the number of packages effectively dirtied
	 */
	LANDSCAPE_API int32 MarkModifiedPackagesAsDirty();

	/** Landscapes are a bit special in that they contain derived data within their source data, which can be updated on-the-fly depending on many external factors (e.g. a procedural edit layer can invalidate some 
		data on load). Depending on project preference, the modifications can be either automatically taken into account (leading to landscape actors becoming dirty on load) or manually by the user (i.e. invalidated
		landscape actors are not made dirty on load but the user is required to save the modified actors through a Build menu action). 
		Therefore we use these 2 functions as a replacement to Modify or MarkPackageDirty in cases where we don't know if the modification was initiated by the user : */

	/**
	 * See UObject::Modify
	 * @param InObject Object being modified. Must be either a ALandscapeProxy or one of its inner objects
	 * @param bAlwaysMarkDirty if true, marks the package dirty even if we aren't currently recording an active undo/redo transaction
	 * @return true if the object could not be marked dirty but has been added to the list of ModifiedPackages
	 */
	LANDSCAPE_API bool ModifyObject(UObject* InObject, bool bAlwaysMarkDirty = true);
	/**
	 * See UObjectBaseUtility::MarkPackageDirty
	 * @param InObject Object to mark dirty.
	 * @param bInForceResave passed to true in order to bypass the bDirtyOnlyInMode and force the object into the list of modified packages. This ensures the object is marked as needing resaving in case we want to 
	 *  help the user re-save its landscape actors (e.g. following a data fixup)
	 * @param InLandscapeOverride if set, MarkObjectDirty will use this ALandscape instead of the local one in the method.
	 * @return true if the object could not be marked dirty but has been added to the list of ModifiedPackages
	 */
	LANDSCAPE_API bool MarkObjectDirty(UObject* InObject, bool bInForceResave = false, const ALandscape* InLandscapeOverride = nullptr);

	/**
	 * Enqueue an actor to be deleted when deferred dirty status from MarkObjectDirty is finally applied.
	 * @param InActor Actor to delete.
	 * @param bInAllowUI allows the standard delete actors UX to be displayed, allowing the user to remove lingering reference to these actors, etc.
	 */
	LANDSCAPE_API void DeleteActorWhenApplyingModifiedStatus(AActor* InActor, bool bInAllowUI);

	/** Helper function to invalidate the runtime virtual texture data associated with the specified landscape region. 
	 *   This forces the dirtied RVT pages to be updated in priority
	 */
	LANDSCAPE_API void DirtyRuntimeVirtualTextureForLandscapeArea(int32 X1, int32 Y1, int32 X2, int32 Y2) const;
#endif //WITH_EDITOR

	/**
	 *  Returns landscape which is spawned in the current level that was previously added to this landscape info object
	 *  @param	bRegistered		Whether to consider only registered(visible) landscapes
	 *	@return					Landscape or landscape proxy found in the current level 
	 */
	LANDSCAPE_API ALandscapeProxy* GetCurrentLevelLandscapeProxy(bool bRegistered) const;
	
	/** 
	 *	returns shared landscape or landscape proxy, mostly for transformations
	 *	@todo: should be removed
	 */
	LANDSCAPE_API ALandscapeProxy* GetLandscapeProxy() const;

#if WITH_EDITOR

	/** Resets all actors, proxies, components registrations */
	LANDSCAPE_API void Reset();

	/** Recreate all LandscapeInfo objects in given world
	 *  @param  bMapCheck	Whether to warn about landscape errors
	 */
	LANDSCAPE_API static void RecreateLandscapeInfo(UWorld* InWorld, bool bMapCheck, bool bKeepRegistrationStatus = false);

	/** 
	 *  Fixes up proxies relative position to landscape actor
	 *  basically makes sure that each LandscapeProxy RootComponent transform reflects LandscapeSectionOffset value
	 *  requires LandscapeActor to be loaded
	 *  Does not work in World composition mode!
	 */
	LANDSCAPE_API void FixupProxiesTransform(bool bDirty = false);
	
	// Update per-component layer allow list to include the currently painted layers
	LANDSCAPE_API void UpdateComponentLayerAllowList();

	LANDSCAPE_API void RecreateCollisionComponents();

	LANDSCAPE_API void RemoveXYOffsets();

	/** Will tell if the landscape actor can have some content related to the layer system */
	UE_DEPRECATED(5.7, "Non-edit layer landscapes are deprecated, all landscapes use the edit layer system now.")
	LANDSCAPE_API bool CanHaveLayersContent() const;

	/** Will clear all component dirty data */
	LANDSCAPE_API void ClearDirtyData();

	/** Moves Components to target level. Creates ALandscapeProxy if needed. */
	LANDSCAPE_API ALandscapeProxy* MoveComponentsToLevel(const TArray<ULandscapeComponent*>& InComponents, ULevel* TargetLevel, FName NewProxyName = NAME_None);

	/** Moves Components to target proxy. */
	LANDSCAPE_API ALandscapeProxy* MoveComponentsToProxy(const TArray<ULandscapeComponent*>& InComponents, ALandscapeProxy* LandscapeProxy, bool bSetPositionAndOffset = false, ULevel* TargetLevel = nullptr);

	/** Moves Splines connected to this control point to target level. Creates ULandscapeSplinesComponent if needed. */
	LANDSCAPE_API void MoveSplineToLevel(ULandscapeSplineControlPoint* InControlPoint, ULevel* TargetLevel);

	/** Moves all Splines to target level. Creates ULandscapeSplinesComponent if needed. */
	LANDSCAPE_API void MoveSplinesToLevel(ULandscapeSplinesComponent* InSplineComponent, ULevel* TargetLevel);

	/** Moves Splines connected to this control point to target Proxy. Creates ULandscapeSplinesComponent if needed. */
	LANDSCAPE_API void MoveSplineToProxy(ULandscapeSplineControlPoint* InControlPoint, ALandscapeProxy* InLandscapeProxy);

	/** Moves all Splines to target Proxy. Creates ULandscapeSplineComponent if needed */
	LANDSCAPE_API void MoveSplinesToProxy(ULandscapeSplinesComponent* InSplineComponent, ALandscapeProxy* InLandscapeProxy);

	/** Moves Splines connected to this control point to target Proxy. Creates ULandscapeSplinesComponent if needed. */
	LANDSCAPE_API void MoveSpline(ULandscapeSplineControlPoint* InControlPoint, TScriptInterface<ILandscapeSplineInterface> InNewOwner);

	/** Moves all Splines to target Spline owner. Creates ULandscapeSplinesComponent if needed. */
	LANDSCAPE_API void MoveSplines(ULandscapeSplinesComponent* InSplineComponent, TScriptInterface<ILandscapeSplineInterface> InNewOwner);

	/** Will call UpdateAllComponentMaterialInstances on all LandscapeProxies */
	LANDSCAPE_API void UpdateAllComponentMaterialInstances(bool bInInvalidateCombinationMaterials = false);

	/** Returns LandscapeStreamingProxy Cell Size in WorldPartition */
	LANDSCAPE_API uint32 GetGridSize(uint32 InGridSizeInComponents) const;

	/** Returns true if new Landscape actors should be spatially loaded in WorldPartition (LandscapeStreamingProxy & LandscapeSplineActor) */
	LANDSCAPE_API bool AreNewLandscapeActorsSpatiallyLoaded() const;
#endif
	LANDSCAPE_API static ULandscapeInfo* Find(UWorld* InWorld, const FGuid& LandscapeGuid);
	LANDSCAPE_API static ULandscapeInfo* FindOrCreate(UWorld* InWorld, const FGuid& LandscapeGuid);
	LANDSCAPE_API static int32 RemoveLandscapeInfo(UWorld* InWorld, const FGuid& LandscapeGuid);

	/** Called after creating object so that it can initialize its state */
	void Initialize(UWorld* InWorld, const FGuid& InLandscapeGuid);

	/** Returns a const reference to the sorted registered streaming proxies */
	LANDSCAPE_API const TArray<TWeakObjectPtr<ALandscapeStreamingProxy>>& GetSortedStreamingProxies() const;

	/**
	 * Runs the given function on the root landscape actor and all streaming proxies, with the possibility of early exit
	 * Most easily used with a lambda as follows:
	 * ForEachLandscapeProxy([](ALandscapeProxy* Proxy) -> bool
	 * {
	 *     return continueLoop ? true : false;
	 * });
	 */
	LANDSCAPE_API void ForEachLandscapeProxy(TFunctionRef<bool(ALandscapeProxy*)> Fn) const;

	void UpdateNanite(const ITargetPlatform* InTargetPlatform);

	/** Associates passed actor with this info object
	 *  @param	Proxy		Landscape actor to register
	 *  @param  bMapCheck	Whether to warn about landscape errors
	 */
	LANDSCAPE_API void RegisterActor(ALandscapeProxy* Proxy, bool bMapCheck = false, bool bUpdateAllAddCollisions = true);

	/** Deassociates passed actor with this info object*/
	LANDSCAPE_API void UnregisterActor(ALandscapeProxy* Proxy);

	/** Returns true if the Proxy is currently registered with this ULandscapeInfo */
	bool IsRegistered(const ALandscapeProxy* Proxy) const;

	/** Associates passed landscape component with this info object
	 *  @param	Component	Landscape component to register
	 *  @param  bMapCheck	Whether to warn about landscape errors
	 */
	LANDSCAPE_API void RegisterActorComponent(ULandscapeComponent* Component, bool bMapCheck = false);

	/** Deassociates passed landscape component with this info object*/
	LANDSCAPE_API void UnregisterActorComponent(ULandscapeComponent* Component);

	/** Server doesn't have ULandscapeComponent use CollisionComponents instead to get height on landscape */
	LANDSCAPE_API void RegisterCollisionComponent(ULandscapeHeightfieldCollisionComponent* Component);
	LANDSCAPE_API void UnregisterCollisionComponent(ULandscapeHeightfieldCollisionComponent* Component);

	/**
	 * Retrieve the components currently loaded that overlap with a given "window" area
	 * @param InAreaWorldTransform : transform of the requested area (at the center)
	 * @param InAreaExtents : extents of requested area (i.e. around the center)
	 * @param OutOverlappedComponents : loaded components that overlap this area (key = xy index of the component, value = component)
	 * @param OutComponentIndicesBoundingRect : bounding rectangle of the overlapped components, in component index space, with
	 *  the max being exclusive. For instance a rectangle with min=(0,0) and max=(2,1) includes components (0,0) and (1,0).
	 * 
	 * @return true if at least one overlapped component
	 */
	// TODO [jonathan.bard] : There should be multiple versions of this. 
	//  1. This currently performs AABB to AABB checks and is therefore not optimal (but faster than OOBB to OOBB could ever be, which is why we might need 2 versions) :
	//  2. Returning a TMap<> is a bit overkill, since the component key can be retrieved trivially from the component
	LANDSCAPE_API bool GetOverlappedComponents(const FTransform& InAreaWorldTransform, const FBox2D& InAreaExtents, TMap<FIntPoint, ULandscapeComponent*>& OutOverlappedComponents, FIntRect& OutComponentIndicesBoundingRect);

#if WITH_EDITOR
	LANDSCAPE_API TArray<FTargetLayerBlendGroup> GetTargetLayerBlendGroups() const;
	LANDSCAPE_API TOptional<FTargetLayerBlendGroup> GetBlendGroupForTargetLayer(ULandscapeLayerInfoObject* InLayerInfo) const;

	LANDSCAPE_API ALandscapeSplineActor* CreateSplineActor(const FVector& Location);

	LANDSCAPE_API void ForAllSplineActors(TFunctionRef<void(TScriptInterface<ILandscapeSplineInterface>)> Fn) const;
	LANDSCAPE_API TArray<TScriptInterface<ILandscapeSplineInterface>> GetSplineActors() const;

	LANDSCAPE_API void RegisterSplineActor(TScriptInterface<ILandscapeSplineInterface> SplineActor);
	LANDSCAPE_API void UnregisterSplineActor(TScriptInterface<ILandscapeSplineInterface> SplineActor);

	/** Updates the spline registrations (with ALL landscape infos) when a spline has changed the landscape it is targeting
	 */
	LANDSCAPE_API static void UpdateRegistrationForSplineActor(UWorld* InWorld, TScriptInterface<ILandscapeSplineInterface> InSplineActor);

	LANDSCAPE_API void RequestSplineLayerUpdate();
	LANDSCAPE_API void ForceLayersFullUpdate();

	static void SetForceNonSpatiallyLoadedByDefault(bool bInForceNonSpatiallyLoadedByDefault) { bForceNonSpatiallyLoadedByDefault = bInForceNonSpatiallyLoadedByDefault; }

private:
	inline static bool bForceNonSpatiallyLoadedByDefault = false;

	bool ApplySplinesInternal(TScriptInterface<ILandscapeSplineInterface> SplineOwner, TSet<TObjectPtr<ULandscapeComponent>>* OutModifiedComponents, bool bMarkPackageDirty, int32 LandscapeMinX, int32 LandscapeMinY, int32 LandscapeMaxX, int32 LandscapeMaxY, TFunctionRef<TSharedPtr<FModulateAlpha>(ULandscapeLayerInfoObject*)> GetOrCreateModulate);
	void MoveSegment(ULandscapeSplineSegment* InSegment, TScriptInterface<ILandscapeSplineInterface> From, TScriptInterface<ILandscapeSplineInterface> To);
	void MoveControlPoint(ULandscapeSplineControlPoint* InControlPoint, TScriptInterface<ILandscapeSplineInterface> From, TScriptInterface<ILandscapeSplineInterface> To);
	bool UpdateLayerInfoMapInternal(ALandscapeProxy* Proxy);
	bool TryAddToModifiedPackages(UPackage* InPackage, const ALandscape* InLandscapeOverride = nullptr);
	void ProcessDeferredDeletions();

	void OnMarkPackageDirty(UPackage* InPackage, bool bIsDirty);
#endif

private:
	void RegisterLandscapeActorWithProxyInternal(ALandscapeProxy* Proxy, bool bMapCheck);

	/** Sorts registered streaming proxies by their section base offset */
	void SortStreamingProxies();

	UPROPERTY()
	TArray<TWeakObjectPtr<ALandscapeStreamingProxy>> SortedStreamingProxies;
};
