// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "Misc/Guid.h"

#include "LandscapeInfoMap.generated.h"

class ULandscapeInfo;

UCLASS()
class ULandscapeInfoMap : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	void BeginDestroy() override;
	void PostDuplicate(bool bDuplicateForPIE) override;
	void Serialize(FArchive& Ar) override;

	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	TMap<FGuid, TObjectPtr<ULandscapeInfo>> Map;
	TWeakObjectPtr<UWorld> World;

	/**
	* Gets landscape-specific data for given world.
	*
	* @param World A pointer to world that return data should be associated with.
	*
	* @returns Landscape-specific data associated with given world.
	*/
	LANDSCAPE_API static ULandscapeInfoMap& GetLandscapeInfoMap(const UWorld* World);
	LANDSCAPE_API static ULandscapeInfoMap* FindLandscapeInfoMap(const UWorld* World);
};
