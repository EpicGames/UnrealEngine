// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LandscapeEditTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "LandscapeLayerInfoObject.generated.h"

class UPhysicalMaterial;
class UTexture2D;
struct FPropertyChangedEvent;


struct FOnLandscapeLayerInfoDataChangedParams
{
	FOnLandscapeLayerInfoDataChangedParams(const ULandscapeLayerInfoObject& InLayerInfoObject, const FPropertyChangedEvent& InPropertyChangedEvent = FPropertyChangedEvent(/*InProperty = */nullptr))
		: PropertyChangedEvent(InPropertyChangedEvent)
		, LayerInfoObject(&InLayerInfoObject)
	{
	}

	/** Provides some additional context about how data has changed (property, type of change...) */
	FPropertyChangedEvent PropertyChangedEvent;

	/** Indicates a user-initiated property change */
	bool bUserTriggered = false;

	/** Indicates the change requires a full landscape update (e.g. parameter affecting heightmap or weightmap...) */
	bool bRequiresLandscapeUpdate = true;

	/** The delegate is triggered each time a data change is requested, even when the data didn't actually change. This indicates that the
	 * was actually modified. This can occur for example when several EPropertyChangeType::Interactive changes are triggered because of the user
	 * manipulating a slider : this will be followed by a final EPropertyChangeType::ValueSet but when this occurs, the data usually is not actually
	 * modified so, to be consistent, we'll still trigger the delegate but indicate that the value didn't actually change, to let the user react appropriately
	 */
	bool bHasValueChanged = true;

	/** Pointer to the ULandscapeLayerInfoObject that broadcast the data change event */
	const ULandscapeLayerInfoObject* LayerInfoObject = nullptr;
};

// ----------------------------------------------------------------------------------

UENUM()
enum class ESplineModulationColorMask : uint8
{
	Red,
	Green,
	Blue,
	Alpha
};


// ----------------------------------------------------------------------------------

UCLASS(MinimalAPI, BlueprintType)
class ULandscapeLayerInfoObject : public UObject
{
	GENERATED_UCLASS_BODY()

	UE_DEPRECATED(5.7, "Property will be made private. Use public Getters/Setter instead.")
	UPROPERTY(VisibleAnywhere, Category = LandscapeLayerInfoObject, AssetRegistrySearchable)
	FName LayerName;

	UE_DEPRECATED(5.7, "Property will be made private. Use public Getters/Setter instead.")
	UPROPERTY(EditAnywhere, Category = LandscapeLayerInfoObject, Meta = (DisplayName = "Physical Material", Tooltip = "Physical material to use when this layer is the predominant one at a given location. Note: this is ignored if the Landscape Physical Material node is used in the landscape material. "))
	TObjectPtr<UPhysicalMaterial> PhysMaterial;

	UE_DEPRECATED(5.7, "Property will be made private. Use public Getters/Setter instead.")
	UPROPERTY(EditAnywhere, Category = LandscapeLayerInfoObject, Meta = (ClampMin = "0", ClampMax = "1", Tooltip = "Defines how much 'resistance' areas painted with this layer will offer to the Erosion tool. A hardness of 0 means the layer is fully affected by erosion, while 1 means fully unaffected."))
	float Hardness = 0.0f;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.7, "Property will be made private. Use public Getters/Setter instead.")
	UPROPERTY(EditAnywhere, Category = LandscapeLayerInfoObject, Meta = (ClampMin = "0", ClampMax = "1", Tooltip = "The minimum weight that needs to be painted for that layer to be picked up as the dominant physical layer."))
	float MinimumCollisionRelevanceWeight = 0.0f;

	UE_DEPRECATED(5.7, "bNoWeightBlend has been replaced by BlendMethod (false is ELandscapeTargetLayerBlendMethod::FinalWeightBlending, true is ELandscapeTargetLayerBlendMethod::None)")
	UPROPERTY()
	uint32 bNoWeightBlend_DEPRECATED:1;

	UE_DEPRECATED(5.7, "Property will be made private. Use public Getters/Setter instead.")
	UPROPERTY(EditAnywhere, Category = SplineFalloffModulation, Meta = (DisplayName="Texture", Tooltip = "Texture to modulate the Splines Falloff Layer Alpha"))
	TObjectPtr<UTexture2D> SplineFalloffModulationTexture;

	UE_DEPRECATED(5.7, "Property will be made private. Use public Getters/Setter instead.")
	UPROPERTY(EditAnywhere, Category = SplineFalloffModulation, Meta = (DisplayName = "Color Mask", Tooltip = "Defines which channel of the Spline Falloff Modulation Texture to use."))
	ESplineModulationColorMask SplineFalloffModulationColorMask = ESplineModulationColorMask::Red;

	UE_DEPRECATED(5.7, "Property will be made private. Use public Getters/Setter instead.")
	UPROPERTY(EditAnywhere, Category = SplineFalloffModulation, Meta = (DisplayName = "Tiling", ClampMin = "0.01", Tooltip = "Defines the tiling to use when sampling the Spline Falloff Modulation Texture."))
	float SplineFalloffModulationTiling = 1.0f;

	UE_DEPRECATED(5.7, "Property will be made private. Use public Getters/Setter instead.")
	UPROPERTY(EditAnywhere, Category = SplineFalloffModulation, Meta = (DisplayName = "Bias", ClampMin = "0", Tooltip = "Defines the offset to use when sampling the Spline Falloff Modulation Texture."))
	float SplineFalloffModulationBias = 0.5f;

	UE_DEPRECATED(5.7, "Property will be made private. Use public Getters/Setter instead.")
	UPROPERTY(EditAnywhere, Category = SplineFalloffModulation, Meta = (DisplayName = "Scale", ClampMin = "0", Tooltip = "Allows to scale the value sampled from the Spline Falloff Modulation Texture."))
	float SplineFalloffModulationScale = 1.0f;

	// TODO [jared.ritchie] Property is used to hide the unused target layers in the UX when ShowUnusedLayers is false. This data should be per-landscape, so it's not valid to store it in the layer object 
	// Two landscapes can loaded and reference the same layer info object with one of them saying "there is data associated with this layer" and the other, the contrary
	UE_DEPRECATED(5.7, "This property will be removed in a future release.")
	UPROPERTY(NonTransactional, Transient)
	bool IsReferencedFromLoadedData = false;
#endif // WITH_EDITORONLY_DATA

	UE_DEPRECATED(5.7, "Property will be made private. Use public Getters/Setter instead.")
	UPROPERTY(EditAnywhere, Category = LandscapeLayerInfoObject, Meta = (Tooltip = "The color to use for layer usage debug"))
	FLinearColor LayerUsageDebugColor;

public:
	inline ELandscapeTargetLayerBlendMethod GetBlendMethod() const 
	{
		return BlendMethod;
	}
	static FName GetBlendMethodMemberName() 
	{
		return GET_MEMBER_NAME_CHECKED(ULandscapeLayerInfoObject, BlendMethod); 
	};
	LANDSCAPE_API void SetBlendMethod(ELandscapeTargetLayerBlendMethod InBlendMethod, bool bInModify);

	LANDSCAPE_API FName GetBlendGroup() const; 
	static FName GetBlendGroupMemberName()
	{
		return GET_MEMBER_NAME_CHECKED(ULandscapeLayerInfoObject, BlendGroup);
	};
	LANDSCAPE_API void SetBlendGroup(const FName& InBlendGroup, bool bInModify);

	// TODO [jared.ritchie] in 5.9 when public properties are made private, re-enable deprecation warnings
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	inline const FName& GetLayerName() const
	{
		return LayerName;
	}
	static FName GetLayerNameMemberName()
	{
		return GET_MEMBER_NAME_CHECKED(ULandscapeLayerInfoObject, LayerName);
	};
	LANDSCAPE_API void SetLayerName(const FName& InLayerName, bool bInModify);

	inline TObjectPtr<UPhysicalMaterial> GetPhysicalMaterial() const
	{
		return PhysMaterial;
	}
	static FName GetPhysicalMaterialMemberName()
	{
		return GET_MEMBER_NAME_CHECKED(ULandscapeLayerInfoObject, PhysMaterial);
	};
	LANDSCAPE_API void SetPhysicalMaterial(TObjectPtr<UPhysicalMaterial> InPhysicalMaterial, bool bInModify);

	inline float GetHardness() const
	{
		return Hardness;
	}
	static FName GetHardnessMemberName()
	{
		return GET_MEMBER_NAME_CHECKED(ULandscapeLayerInfoObject, Hardness);
	};
	LANDSCAPE_API void SetHardness(float InHardness, bool bInModify, EPropertyChangeType::Type InChangeType);

	inline const FLinearColor& GetLayerUsageDebugColor() const
	{
		return LayerUsageDebugColor;
	}
	static FName GetLayerUsageDebugColorMemberName()
	{
		return GET_MEMBER_NAME_CHECKED(ULandscapeLayerInfoObject, LayerUsageDebugColor);
	};
	LANDSCAPE_API void SetLayerUsageDebugColor(const FLinearColor& InLayerUsageDebugColor, bool bInModify, EPropertyChangeType::Type InChangeType);

#if WITH_EDITORONLY_DATA
	inline float GetMinimumCollisionRelevanceWeight() const
	{
		return MinimumCollisionRelevanceWeight;
	}
	static FName GetMinimumCollisionRelevanceWeightMemberName()
	{
		return GET_MEMBER_NAME_CHECKED(ULandscapeLayerInfoObject, MinimumCollisionRelevanceWeight);
	};
	LANDSCAPE_API void SetMinimumCollisionRelevanceWeight(float InMinimumCollisionRelevanceWeight, bool bInModify, EPropertyChangeType::Type InChangeType);

	inline TObjectPtr<UTexture2D> GetSplineFalloffModulationTexture() const
	{
		return SplineFalloffModulationTexture;
	}
	static FName GetSplineFalloffModulationTextureMemberName()
	{
		return GET_MEMBER_NAME_CHECKED(ULandscapeLayerInfoObject, SplineFalloffModulationTexture);
	};
	LANDSCAPE_API void SetSplineFalloffModulationTexture(TObjectPtr<UTexture2D> InSplineFalloffModulationTexture, bool bInModify);

	inline ESplineModulationColorMask GetSplineFalloffModulationColorMask() const
	{
		return SplineFalloffModulationColorMask;
	}
	static FName GetSplineFalloffModulationColorMaskMemberName()
	{
		return GET_MEMBER_NAME_CHECKED(ULandscapeLayerInfoObject, SplineFalloffModulationColorMask);
	};
	LANDSCAPE_API void SetSplineFalloffModulationColorMask(ESplineModulationColorMask InSplineFalloffModulationColorMask, bool bInModify);

	inline float GetSplineFalloffModulationTiling() const
	{
		return SplineFalloffModulationTiling;
	}
	static FName GetSplineFalloffModulationTilingMemberName()
	{
		return GET_MEMBER_NAME_CHECKED(ULandscapeLayerInfoObject, SplineFalloffModulationTiling);
	};
	LANDSCAPE_API void SetSplineFalloffModulationTiling(float InSplineFalloffModulationTiling, bool bInModify, EPropertyChangeType::Type InChangeType);

	inline float GetSplineFalloffModulationBias() const
	{
		return SplineFalloffModulationBias;
	}
	static FName GetSplineFalloffModulationBiasMemberName()
	{
		return GET_MEMBER_NAME_CHECKED(ULandscapeLayerInfoObject, SplineFalloffModulationBias);
	};
	LANDSCAPE_API void SetSplineFalloffModulationBias(float InSplineFalloffModulationBias, bool bInModify, EPropertyChangeType::Type InChangeType);

	inline float GetSplineFalloffModulationScale() const
	{
		return SplineFalloffModulationScale;
	}
	static FName GetSplineFalloffModulationScaleMemberName()
	{
		return GET_MEMBER_NAME_CHECKED(ULandscapeLayerInfoObject, SplineFalloffModulationScale);
	};
	LANDSCAPE_API void SetSplineFalloffModulationScale(float InSplineFalloffModulationScale, bool bInModify, EPropertyChangeType::Type InChangeType);
#endif // WITH_EDITORONLY_DATA
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual void Serialize(FArchive& Ar) override;

#if WITH_EDITOR
	//~ Begin UObject Interface
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostLoad() override;
	//~ End UObject Interface

	/** Delegate triggered whenever a change occurrs in the layer info object data */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnLayerInfoDataChanged, const FOnLandscapeLayerInfoDataChangedParams& /*InParams*/);
	FOnLayerInfoDataChanged::RegistrationType& OnLayerInfoChanged() const
	{
		return OnLayerInfoObjectChangedDelegate;
	}

	void BroadcastOnLayerInfoObjectDataChanged(FName InPropertyName, bool bInUserTriggered, bool bInRequiresLandscapeUpdate, bool bInHasValueChanged, EPropertyChangeType::Type InChangeType);
#endif

	LANDSCAPE_API FLinearColor GenerateLayerUsageDebugColor() const;

private:
	UPROPERTY(EditAnywhere, Category = LandscapeLayerInfoObject, Meta = (Tooltip = "Allows this layer's final weight to be adjusted against others."))
	ELandscapeTargetLayerBlendMethod BlendMethod = ELandscapeTargetLayerBlendMethod::None;

	UPROPERTY(EditAnywhere, Category = LandscapeLayerInfoObject, Meta = (EditCondition = "BlendMethod == ELandscapeTargetLayerBlendMethod::PremultipliedAlphaBlending", Tooltip = "Only available for Advanced Weight Blending. Allows target layers from the same group only to have their weight adjusted against one another."))
	FName BlendGroup;

#if WITH_EDITOR
	mutable FOnLayerInfoDataChanged OnLayerInfoObjectChangedDelegate;
#endif // WITH_EDITOR
};
