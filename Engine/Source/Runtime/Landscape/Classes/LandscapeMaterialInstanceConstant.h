// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialInstanceConstant.h"
#include "LandscapeMaterialInstanceConstant.generated.h"

USTRUCT()
struct FLandscapeMaterialTextureStreamingInfo
{
	GENERATED_USTRUCT_BODY()

	FLandscapeMaterialTextureStreamingInfo()
		: TexelFactor(0.0f)
	{}

	UPROPERTY()
	FName TextureName;

	UPROPERTY()
	float TexelFactor;
};

UCLASS(MinimalAPI)
class ULandscapeMaterialInstanceConstant : public UMaterialInstanceConstant
{
	GENERATED_UCLASS_BODY()

	UPROPERTY()
	TArray<FLandscapeMaterialTextureStreamingInfo> TextureStreamingInfo;

	UPROPERTY()
	uint32 bIsLayerThumbnail:1;

	UPROPERTY()
	uint32 bDisableTessellation_DEPRECATED:1;

	UPROPERTY()
	uint32 bMobile:1;

	UPROPERTY()
	uint32 bEditorToolUsage:1;

	virtual bool WritesToRuntimeVirtualTexture() const override;

	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	virtual FMaterialResource* AllocatePermutationResource() override;
	virtual bool HasOverridenBaseProperties() const override;

#if WITH_EDITOR
	FLandscapeMaterialTextureStreamingInfo& AcquireTextureStreamingInfo(const FName& TextureName);
	void UpdateCachedTextureStreaming();
#endif

public:
	float GetLandscapeTexelFactor(const FName& TextureName) const;
};

