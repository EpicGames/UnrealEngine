// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Templates/RefCounting.h"
#include "EngineDefines.h"
#include "Serialization/BulkData.h"
#include "LandscapeHeightfieldCollisionComponent.h"

#include "Chaos/PhysicalMaterials.h"

#include "LandscapeMeshCollisionComponent.generated.h"

namespace Chaos
{
	class FTriangleMeshImplicitObject;
}

UCLASS()
class UE_DEPRECATED(5.7, "RetopologizeTool/XYOffset deprecated with the removal of non-edit layer landscapes") ULandscapeMeshCollisionComponent_DEPRECATED : public ULandscapeHeightfieldCollisionComponent
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FGuid MeshGuid;

	struct UE_DEPRECATED(5.7, "RetopologizeTool/XYOffset deprecated with the removal of non-edit layer landscapes") FTriMeshGeometryRef : public FRefCountedObject
	{
		FGuid Guid;

		TArray<Chaos::FMaterialHandle> UsedChaosMaterials;
		Chaos::FTriangleMeshImplicitObjectPtr TrimeshGeometry;
		
#if WITH_EDITORONLY_DATA
		Chaos::FTriangleMeshImplicitObjectPtr EditorTrimeshGeometry;
#endif // WITH_EDITORONLY_DATA

		FTriMeshGeometryRef();
		FTriMeshGeometryRef(FGuid& InGuid);
		virtual ~FTriMeshGeometryRef();

		void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize);
	};

#if WITH_EDITORONLY_DATA
	/** The collision mesh values. */
	FWordBulkData CollisionXYOffsetData; //  X, Y Offset in raw format...
#endif //WITH_EDITORONLY_DATA

	/** Physics engine version of heightfield data. */
	TRefCountPtr<FTriMeshGeometryRef> MeshRef;
};
