// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"

#include "LandscapeMeshProxyActor.generated.h"

UCLASS(MinimalAPI)
class ALandscapeMeshProxyActor : public AActor
{
	GENERATED_UCLASS_BODY()

private:
	UPROPERTY(Category = LandscapeMeshProxyActor, VisibleAnywhere, BlueprintReadOnly, meta = (ExposeFunctionCategories = "Mesh,Rendering,Physics,Components|StaticMesh", AllowPrivateAccess = "true"))
	TObjectPtr<class ULandscapeMeshProxyComponent> LandscapeMeshProxyComponent;

public:
	/** Returns StaticMeshComponent subobject **/
	class ULandscapeMeshProxyComponent* GetLandscapeMeshProxyComponent() const { return LandscapeMeshProxyComponent; }
};

