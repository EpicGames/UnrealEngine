// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Components/StaticMeshComponent.h"

#include "LandscapeMeshProxyComponent.generated.h"

class ALandscapeProxy;
class FPrimitiveSceneProxy;

UCLASS(MinimalAPI)
class ULandscapeMeshProxyComponent : public UStaticMeshComponent
{
	GENERATED_BODY()

public:
	ULandscapeMeshProxyComponent(const FObjectInitializer& ObjectInitializer);

private:
	/* The landscape this proxy was generated for */
	UPROPERTY()
	FGuid LandscapeGuid;

	/* The section coordinates of the landscape components that this proxy was generated for.  Used to register with LandscapeRender when LODGroupKey == 0 */
	UPROPERTY()
	TArray<FIntPoint> ProxyComponentBases;

	/* The center of the landscape components that this proxy was generated for, in local component space.  Used to register with LandscapeRender when LODGroupKey != 0 */
	UPROPERTY()
	TArray<FVector> ProxyComponentCentersObjectSpace;

	UPROPERTY()
	FVector ComponentXVectorObjectSpace;

	UPROPERTY()
	FVector ComponentYVectorObjectSpace;

	UPROPERTY()
	int32 ComponentResolution;

	/* LOD level this proxy was generated for */
	UPROPERTY()
	int8 ProxyLOD;

	UPROPERTY()
	uint32 LODGroupKey;

public:
	LANDSCAPE_API void InitializeForLandscape(ALandscapeProxy* Landscape, int8 InProxyLOD);
	
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	virtual void PostLoad() override;
};

