// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if WITH_EDITOR

/** 
 * A single task for calculating the landscape physical material.
 * The work uses the GPU so this object handles reading back the data without stalling.
 * We need to call Tick() until IsComplete() returns true.
 * We need to call Release() after use to prevent leaking of internal objects.
 * All calls are expected to be made on the game thread only.
 */
class FLandscapePhysicalMaterialRenderTask
{
public:
	FLandscapePhysicalMaterialRenderTask()
	{}

	~FLandscapePhysicalMaterialRenderTask()
	{
		Release();
	}

	/** Initialize the task for a component. A task in progress can safely call Init() again to rekick the task. Returns false if no physical material needs to be rendered. */
	bool Init(class ULandscapeComponent const* LandscapeComponent, uint32 InHash);
	/** Release the task. After calling this IsValid() will return false until we call Init() again. */
	void Release();

	/** Returns true if Init() has been called without a Release(). */
	bool IsValid() const;
	/** Returns true if the task is complete and the result is available. */
	bool IsComplete() const;
	/** Is this Task in progress (valid and incomplete) */
	bool IsInProgress() const;


	/** Update a task. Does nothing if the task is complete. */
	void Tick();
	/** Flush a task to completion. This may stall waiting for completion. */
	void Flush();
	
	/** Get the result data. Assumes that IsComplete() is true. This doesn't Release() the data. */
	TArray<uint8> const& GetResultIds() const;
	/** Get the result data. Assumes that IsComplete() is true. This doesn't Release() the data. */
	TArray <class UPhysicalMaterial* > const& GetResultMaterials() const;

	uint32 GetHash() const { return Hash; }
private:
	void UpdateInternal(bool bInFlush);

private:
	int32 PoolHandle = -1;
	
	friend class FLandscapePhysicalMaterialRenderTaskPool;
	FLandscapePhysicalMaterialRenderTask(FLandscapePhysicalMaterialRenderTask const&);

	uint32 Hash = 0;
};

namespace LandscapePhysicalMaterial
{
	/** Optional call to allow garbage collection of any FLandscapePhysicalMaterialRenderTask resources and check for leaks. */
	void GarbageCollectTasks();
}

#endif // WITH_EDITOR
