// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Actor.h"
#include "PhysicsEngine/BodyInstance.h"
#include "Async/AsyncWork.h"
#include "Engine/Texture.h"
#include "UObject/PerPlatformProperties.h"
#include "LandscapeNaniteComponent.h"
#include "LandscapeWeightmapUsage.h"
#include "LandscapeHeightfieldCollisionComponent.h"
#include "VT/RuntimeVirtualTextureEnum.h"
#include "ActorPartition/PartitionActor.h"
#include "ILandscapeSplineInterface.h"
#include "Engine/Texture2DArray.h"

#if WITH_EDITOR
#include "WorldPartition/WorldPartitionHandle.h"
#endif

#include "LandscapeProxy.generated.h"

class ALandscape;
class ALandscapeProxy;
class UHierarchicalInstancedStaticMeshComponent;
class ULandscapeNaniteComponent;
class ULandscapeComponent;
class ULandscapeGrassType;
class ULandscapeHeightfieldCollisionComponent;
class ULandscapeInfo;
class ULandscapeLayerInfoObject;
class ULandscapeMaterialInstanceConstant;
class ULandscapeSplinesComponent;
class UMaterialInstanceConstant;
class UMaterialInterface;
class UPhysicalMaterial;
class USplineComponent;
class UTexture2D;
class FLandscapeEditLayerReadback;
class FMaterialUpdateContext;
struct FAsyncGrassBuilder;
struct FLandscapeInfoLayerSettings;
struct FLandscapePerLODMaterialOverride;
struct FOnLandscapeLayerInfoDataChangedParams;
class FLandscapeProxyComponentDataChangedParams;
struct FMeshDescription;

enum class ENavDataGatheringMode : uint8;
namespace UE::Landscape
{
	enum class EOutdatedDataFlags : uint8;
	enum class EBuildFlags : uint8;

	namespace Nanite
	{
		struct FAsyncBuildData;
	} // end of namespace UE::Landscape::Nanite
} // end of namespace UE::Landscape

#if WITH_EDITOR
LANDSCAPE_API extern bool GLandscapeEditModeActive;
extern int32 GGrassMapUseRuntimeGeneration;

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnLandscapeProxyComponentDataChanged, ALandscapeProxy*, const FLandscapeProxyComponentDataChangedParams&);

struct FOnLandscapeProxyMaterialChangedParams
{
};
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnLandscapeProxyMaterialChanged, ALandscapeProxy*, const FOnLandscapeProxyMaterialChangedParams&);

struct FOnLandscapeProxyFixupSharedDataParams
{
	/** Parent landscape actor for the landscape proxy on which OnLandscapeProxyFixupSharedData is triggered */
	ALandscape* Landscape = nullptr;
	/** Indicates whether UpgradeSharedProperties has been called on this proxy already (before this call) */
	bool bUpgradeSharedPropertiesPerformed = false;
};
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnLandscapeProxyFixupSharedDataDelegate, ALandscapeProxy* /*InProxy*/, const FOnLandscapeProxyFixupSharedDataParams& /*InParams*/);
#endif // WITH_EDITOR


// ----------------------------------------------------------------------------------

// This struct is a FName wrapper.It's just a little trick to be able to customize the display of the grass name in such a way that it can work as a TMap's key.
USTRUCT()
struct FLandscapeGrassName
{
	GENERATED_USTRUCT_BODY()

	FLandscapeGrassName() = default;
	FLandscapeGrassName(const FName& InName)
		: Name(InName)
	{}

	UPROPERTY(EditAnywhere, Category=Grass)
	FName Name;

	/** Used so we can have a TMap of this struct */
	inline bool operator==(FLandscapeGrassName const& Other) const
	{
		return Name == Other.Name;
	}

	/** Used so we can have a TMap of this struct */
	inline friend uint32 GetTypeHash(const FLandscapeGrassName& InGrassName)
	{
		return GetTypeHash(InGrassName.Name);
	}
};

// ----------------------------------------------------------------------------------

USTRUCT()
struct FLandscapeTargetLayerSettings
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(Category = "General", EditAnywhere)
	TObjectPtr<ULandscapeLayerInfoObject> LayerInfoObj;
	
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FString ReimportLayerFilePath;

	FLandscapeTargetLayerSettings() = default;

	explicit FLandscapeTargetLayerSettings(ULandscapeLayerInfoObject* InLayerInfo, const FString& InFilePath = FString())
		: LayerInfoObj(InLayerInfo)
		, ReimportLayerFilePath(InFilePath)
	{
	}

	bool operator==(const FLandscapeTargetLayerSettings& Other) const
    {
    	return LayerInfoObj == Other.LayerInfoObj;
    }
#endif // WITH_EDITORONLY_DATA
};

USTRUCT()
struct UE_DEPRECATED(all, "FLandscapeEditorLayerSettings is deprecated; please use FLandscapeTargetLayerSettings instead")  FLandscapeEditorLayerSettings
{
	GENERATED_USTRUCT_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TObjectPtr<ULandscapeLayerInfoObject> LayerInfoObj;

	UPROPERTY()
	FString ReimportLayerFilePath;

	FLandscapeEditorLayerSettings()
		: LayerInfoObj(nullptr)
		, ReimportLayerFilePath()
	{
	}

	explicit FLandscapeEditorLayerSettings(ULandscapeLayerInfoObject* InLayerInfo, const FString& InFilePath = FString())
		: LayerInfoObj(InLayerInfo)
		, ReimportLayerFilePath(InFilePath)
	{
	}

	// To allow FindByKey etc
	bool operator==(const ULandscapeLayerInfoObject* LayerInfo) const
	{
		return LayerInfoObj == LayerInfo;
	}
#endif // WITH_EDITORONLY_DATA
};

UENUM()
enum class ELandscapeImportAlphamapType : uint8
{
	// Three layers blended 50/30/20 represented as 0.5, 0.3, and 0.2 in the alpha maps
	// All alpha maps for blended layers total to 1.0
	// This is the style used by UE internally for blended layers
	Additive,

	// Three layers blended 50/30/20 represented as 0.5, 0.6, and 1.0 in the alpha maps
	// Each alpha map only specifies the remainder from previous layers, so the last layer used will always be 1.0
	// Some other tools use this format
	Layered,
};

/** Structure storing Layer Data for import */
USTRUCT()
struct FLandscapeImportLayerInfo
{
	GENERATED_USTRUCT_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY(Category="Import", VisibleAnywhere)
	FName LayerName;

	UPROPERTY(Category="Import", EditAnywhere, meta=(NoCreate))
	TObjectPtr<ULandscapeLayerInfoObject> LayerInfo;

	UPROPERTY(Category="Import", EditAnywhere, meta=(DisplayName="Layer File"))
	FString SourceFilePath; // Optional
	
	// Raw weightmap data
	TArray<uint8> LayerData;
#endif

#if WITH_EDITOR
	FLandscapeImportLayerInfo(FName InLayerName = NAME_None)
	:	LayerName(InLayerName)
	,	LayerInfo(nullptr)
	,	SourceFilePath("")
	{
	}

	LANDSCAPE_API FLandscapeImportLayerInfo(const FLandscapeInfoLayerSettings& InLayerSettings);
#endif
};

// this is only here because putting it in LandscapeEditorObject.h (where it belongs)
// results in Engine being dependent on LandscapeEditor, as the actual landscape editing
// code (e.g. LandscapeEdit.h) is in /Engine/ for some reason...
UENUM()
enum class ELandscapeLayerPaintingRestriction : uint8
{
	/** No restriction, can paint anywhere (default). */
	None         UMETA(DisplayName="None"),

	/** Uses the MaxPaintedLayersPerComponent setting from the LandscapeProxy. */
	UseMaxLayers UMETA(DisplayName="Limit Layer Count"),

	/** Restricts painting to only components that already have this layer. */
	ExistingOnly UMETA(DisplayName="Existing Layers Only"),

	/** Restricts painting to only components that have this layer in their allow list. */
	UseComponentAllowList UMETA(DisplayName="Component Allow List"),
};

UENUM()
enum class ELandscapeLayerDisplayMode : uint8
{
	/** Material sorting display mode */
	Default,

	/** Alphabetical sorting display mode */
	Alphabetical,

	/** User specific sorting display mode */
	UserSpecific,

	/** By blend type sorting display mode */
	ByBlendMethod,
};

UENUM()
enum class ELandscapeHLODTextureSizePolicy : uint8
{
	/** Automatic texture size, based on the expected HLOD draw distance and the landscape size. */
	AutomaticSize,

	/** User specified texture size. */
	SpecificSize
};

UENUM()
enum class ELandscapeHLODMeshSourceLODPolicy : uint8
{
	/** Automatic LOD selection, based on the expected HLOD draw distance and the landscape LOD Distribution settings. */
	AutomaticLOD,

	/** User specified landscape LOD. */
	SpecificLOD,

	/** Use the lowest detailed LOD of this landscape. */
	LowestDetailLOD
};

UENUM()
namespace ELandscapeLODFalloff
{
	enum Type : int
	{
		/** Default mode. */
		Linear			UMETA(DisplayName = "Linear"),
		/** Square Root give more natural transition, and also keep the same LOD. */
		SquareRoot		UMETA(DisplayName = "Square Root"),
	};
}

struct FCachedLandscapeFoliage
{
	struct FGrassCompKey
	{
		TWeakObjectPtr<ULandscapeComponent> BasedOn;
		// We store both the GrassName and GrassType in the key, since they're not entirely synonymous anymore : 
		//  A given grass type can be used by 2 different grass names now. They will be treated as 2 separate entries in the foliage cache
		TWeakObjectPtr<ULandscapeGrassType> GrassType;
		FName GrassName;
		int32 SqrtSubsections;
		int32 CachedMaxInstancesPerComponent;
		int32 SubsectionX;
		int32 SubsectionY;
		int32 NumVarieties;
		int32 VarietyIndex;

		FGrassCompKey()
			: SqrtSubsections(0)
			, CachedMaxInstancesPerComponent(0)
			, SubsectionX(0)
			, SubsectionY(0)
			, NumVarieties(0)
			, VarietyIndex(-1)
		{
		}
		inline bool operator==(const FGrassCompKey& Other) const
		{
			return 
				SqrtSubsections == Other.SqrtSubsections &&
				CachedMaxInstancesPerComponent == Other.CachedMaxInstancesPerComponent &&
				SubsectionX == Other.SubsectionX &&
				SubsectionY == Other.SubsectionY &&
				BasedOn == Other.BasedOn &&
				GrassType == Other.GrassType &&
				GrassName == Other.GrassName &&
				NumVarieties == Other.NumVarieties &&
				VarietyIndex == Other.VarietyIndex;
		}

		friend uint32 GetTypeHash(const FGrassCompKey& Key)
		{
			return GetTypeHash(Key.BasedOn) ^ GetTypeHash(Key.GrassType)  ^ GetTypeHash(Key.GrassName) ^ Key.SqrtSubsections ^ Key.CachedMaxInstancesPerComponent ^ (Key.SubsectionX << 16) ^ (Key.SubsectionY << 24) ^ (Key.NumVarieties << 3) ^ (Key.VarietyIndex << 13);
		}

	};

	struct FGrassComp
	{
		FGrassCompKey Key;
		TWeakObjectPtr<UHierarchicalInstancedStaticMeshComponent> Foliage;
		TWeakObjectPtr<UHierarchicalInstancedStaticMeshComponent> PreviousFoliage;
		TArray<FBox> ExcludedBoxes;
		uint32 LastUsedFrameNumber;
		uint32 ExclusionChangeTag;
		double LastUsedTime;
		bool Pending;
		bool PendingRemovalRebuild;

		FGrassComp()
			: ExclusionChangeTag(0)
			, Pending(true)
			, PendingRemovalRebuild(false)
		{
			Touch();
		}
		void Touch()
		{
			LastUsedFrameNumber = GFrameNumber;
			LastUsedTime = FPlatformTime::Seconds();
		}
	};

	struct FGrassCompKeyFuncs : BaseKeyFuncs<FGrassComp,FGrassCompKey>
	{
		static KeyInitType GetSetKey(const FGrassComp& Element)
		{
			return Element.Key;
		}

		static bool Matches(KeyInitType A, KeyInitType B)
		{
			return A == B;
		}

		static uint32 GetKeyHash(KeyInitType Key)
		{
			return GetTypeHash(Key);
		}
	};

	typedef TSet<FGrassComp, FGrassCompKeyFuncs> TGrassSet;
	TSet<FGrassComp, FGrassCompKeyFuncs> CachedGrassComps;

	void ClearCache()
	{
		CachedGrassComps.Empty();
	}
};

class FAsyncGrassTask : public FNonAbandonableTask
{
public:
	FAsyncGrassBuilder* Builder;
	FCachedLandscapeFoliage::FGrassCompKey Key;
	TWeakObjectPtr<UHierarchicalInstancedStaticMeshComponent> Foliage;

	FAsyncGrassTask(FAsyncGrassBuilder* InBuilder, const FCachedLandscapeFoliage::FGrassCompKey& InKey, UHierarchicalInstancedStaticMeshComponent* InFoliage);
	void DoWork();

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FAsyncGrassTask, STATGROUP_ThreadPoolAsyncTasks);
	}

	~FAsyncGrassTask();
};

USTRUCT(meta = (Deprecated = "5.1"))
struct UE_DEPRECATED(all, "FLandscapeProxyMaterialOverride is deprecated; please use FLandscapePerLODMaterialOverride instead") FLandscapeProxyMaterialOverride
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FPerPlatformInt LODIndex;

	UPROPERTY()
	TObjectPtr<UMaterialInterface> Material = nullptr;
};


#if WITH_EDITOR
// Tracks delayed updates triggered by landscape updates.
// It's responsible for preventing expensive async operations while the user is still editing the landscape
struct FAsyncWorkMonitor 
{
public:
	enum class EAsyncWorkType : uint8
	{
		BuildNaniteMeshes,
		Max
	};

	bool CheckIfUpdateTriggeredAndClear(EAsyncWorkType WorkType);
	void SetDelayedUpdateTimer(EAsyncWorkType WorkType, float InSecondsUntilDelayedUpdateTrigger);
	void Tick(float Detaltime);

private:
	struct FAsyncWorkTypeInfo
	{
		bool bUpdateTriggered = false;
		float SecondsUntilDelayedUpdateTrigger = 0.0f;
	};

	TStaticArray<FAsyncWorkTypeInfo, static_cast<uint32>(EAsyncWorkType::Max)> WorkTypeInfos; // [static_cast<uint32>(EAsyncWorkType::Max)] ;
};
#endif 
UCLASS(Abstract, MinimalAPI, NotBlueprintable, NotPlaceable, hidecategories=(Display, Attachment, Physics, Debug, Lighting), showcategories=(Lighting, Rendering, Transformation), hidecategories=(Mobility))
class ALandscapeProxy : public APartitionActor, public ILandscapeSplineInterface
{
	GENERATED_BODY()

public:
	ALandscapeProxy(const FObjectInitializer& ObjectInitializer);

	virtual ~ALandscapeProxy();

protected:

#if WITH_EDITORONLY_DATA
	/** 
	* Hard refs to actors that need to be loaded when this proxy is loaded.
	* It is currently used for 2 cases : 
	* 1- ALandscapeStreamingProxy forces the loading of its intersecting ALandscapeSplineActor.
	* 2- ALandscape forces the loading of its child landscape proxies when the Landscape has Layer brushes.
	*    This is a temporary solution until landscape layer brushes support partial landscape loading.
	*/
	TSet<FWorldPartitionReference> ActorDescReferences;

	friend class FLandscapeActorDesc;
#endif // WITH_EDITORONLY_DATA

	UPROPERTY()
	TObjectPtr<ULandscapeSplinesComponent> SplineComponent;

	/** Guid for LandscapeEditorInfo, all proxies that belong to the same landscape should have the same LandscapeGuid, even if split across world partitions 
	  * Note that this value may change when the landscape is instanced (or in PIE) in order to allow multiple instances of the same landscape to exist.
	  **/
	UPROPERTY(meta = (LandscapeInherited))
	FGuid LandscapeGuid;

	/** 
	  * The original unmutated LandscapeGuid on the source asset, before instancing modifications.
	  **/
	UPROPERTY(Transient, DuplicateTransient, meta = (LandscapeInherited))
	FGuid OriginalLandscapeGuid;

	/** Use Nanite to render landscape as a mesh on supported platforms. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Nanite, meta = (LandscapeInherited))
	bool bEnableNanite = false;

	/** Specify an alternate material to use for a given LOD level (for example, to use a simpler material at a distance) This is not effective on Nanite landscape. */
	UPROPERTY(EditAnywhere, Category = Landscape, meta = (LandscapeOverridable))
	TArray<FLandscapePerLODMaterialOverride> PerLODOverrideMaterials;

#if WITH_EDITORONLY_DATA
	/** LOD level of the landscape when generating the Nanite mesh. Mostly there for debug reasons, since Nanite is meant to allow high density meshes, we want to use 0 most of the times. */
	UPROPERTY(EditAnywhere, Category = Nanite, AdvancedDisplay, meta = (EditCondition = "bEnableNanite", LandscapeInherited))
	int32 NaniteLODIndex = 0;

	UPROPERTY(EditAnywhere, Category = Nanite, AdvancedDisplay, meta = (EditCondition = "bEnableNanite", LandscapeInherited))
	bool bNaniteSkirtEnabled = false;

	UPROPERTY(EditAnywhere, Category = Nanite, AdvancedDisplay, meta = (EditCondition = "bEnableNanite", LandscapeInherited))
	float NaniteSkirtDepth = 0.1f;
	
	UPROPERTY(EditAnywhere, Category = Nanite, AdvancedDisplay, meta = (EditCondition = "bEnableNanite", LandscapeInherited))
	int32 NanitePositionPrecision = 0;

	UPROPERTY(EditAnywhere, Category = Nanite, AdvancedDisplay, meta = (EditCondition = "bEnableNanite", LandscapeInherited))
	float NaniteMaxEdgeLengthFactor = 16.0f;

	LANDSCAPE_API static FOnLandscapeProxyFixupSharedDataDelegate OnLandscapeProxyFixupSharedDataDelegate;
#endif // WITH_EDITORONLY_DATA

	/** Disable runtime grass data generation.  If disabled, the grass maps will be serialized at cook time. Do not set directly, use ALandscape::SetDisableRuntimeGrassMapGeneration to ensure it is set on all loaded proxies. */
	UPROPERTY(meta = (LandscapeInherited))
	bool bDisableRuntimeGrassMapGeneration = false;

public:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TMap<FName, TObjectPtr<ULandscapeLayerInfoObject>> TargetLayersForFixup;

	UE_DEPRECATED(5.7, "Use private SectionBase with GetSectionBase instead.")
	FIntPoint LandscapeSectionOffset;
#endif //WITH_EDITORONLY_DATA

	/** Max LOD level to use when rendering, -1 means the max available */
	UPROPERTY(EditAnywhere, Category=LOD, meta = (LandscapeInherited))
	int32 MaxLODLevel;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(all, "Unused property, replaced by LOD0DistributionSetting and LODDistributionSetting")
	UPROPERTY()
	float LODDistanceFactor_DEPRECATED;

	UE_DEPRECATED(all, "Unused property, replaced by LOD0DistributionSetting and LODDistributionSetting")
	UPROPERTY()
	TEnumAsByte<ELandscapeLODFalloff::Type> LODFalloff_DEPRECATED;
#endif // WITH_EDITORONLY_DATA

	/** This is the starting screen size used to calculate the distribution. You can increase the value if you want less LOD0 component, and you use very large landscape component. */
	UPROPERTY(EditAnywhere, Category = "LOD Distribution", meta = (EditCondition = "!bUseScalableLODSettings", DisplayName = "LOD 0 Screen Size", ClampMin = "0.1", ClampMax = "10.0", UIMin = "0.1", UIMax = "10.0", LandscapeInherited))
	float LOD0ScreenSize = 0.5f;
	
	/** Specifies the LOD Group (Zero is No Group). All landscapes in the same group calculate their LOD together, allowing matching border LODs to fix geometry seams. */
	UPROPERTY(EditAnywhere, Category = LOD, AdvancedDisplay, meta = (LandscapeInherited))
	uint32 LODGroupKey = 0;

	/** The distribution setting used to change the LOD 0 generation, 1.25 is the normal distribution, numbers influence directly the LOD0 proportion on screen. */
	UPROPERTY(EditAnywhere, Category = "LOD Distribution", meta = (EditCondition = "!bUseScalableLODSettings", DisplayName = "LOD 0", ClampMin = "1.0", ClampMax = "10.0", UIMin = "1.0", UIMax = "10.0", LandscapeInherited))
	float LOD0DistributionSetting = 1.25f;

	/** The distribution setting used to change the LOD generation, 3 is the normal distribution, small number mean you want your last LODs to take more screen space and big number mean you want your first LODs to take more screen space. */
	UPROPERTY(EditAnywhere, Category = "LOD Distribution", meta = (EditCondition = "!bUseScalableLODSettings", DisplayName = "Other LODs", ClampMin = "1.0", ClampMax = "10.0", UIMin = "1.0", UIMax = "10.0", LandscapeInherited))
	float LODDistributionSetting = 3.0f;

	/** Scalable (per-quality) version of 'LOD 0 Screen Size'. */
	UPROPERTY(EditAnywhere, Category = "LOD Distribution", meta = (EditCondition = "bUseScalableLODSettings", DisplayName = "Scalable LOD 0 Screen Size", ClampMin = "0.1", ClampMax = "10.0", UIMin = "0.1", UIMax = "10.0", LandscapeInherited))
	FPerQualityLevelFloat ScalableLOD0ScreenSize = 0.5f;
	
	/** Scalable (per-quality) version of 'LOD 0'. */
	UPROPERTY(EditAnywhere, Category = "LOD Distribution", meta = (EditCondition = "bUseScalableLODSettings", DisplayName = "Scalable LOD 0", ClampMin = "1.0", ClampMax = "10.0", UIMin = "1.0", UIMax = "10.0", LandscapeInherited))
	FPerQualityLevelFloat ScalableLOD0DistributionSetting = 1.25f;
	
	/** Scalable (per-quality) version of 'Other LODs'. */
	UPROPERTY(EditAnywhere, Category = "LOD Distribution", meta = (EditCondition = "bUseScalableLODSettings", DisplayName = "Scalable Other LODs", ClampMin = "1.0", ClampMax = "10.0", UIMin = "1.0", UIMax = "10.0", LandscapeInherited))
	FPerQualityLevelFloat ScalableLODDistributionSetting = 3.0f;

	/** Allows to specify LOD distribution settings per quality level. Using this will ignore the r.LandscapeLOD0DistributionScale CVar. */
	UPROPERTY(EditAnywhere, Category = "LOD Distribution", meta = (LandscapeInherited))
	bool bUseScalableLODSettings = false;

	/** This controls the area that blends LOD between neighboring sections. At 1.0 it blends across the entire section, and lower numbers reduce the blend region to be closer to the boundary. */
	UPROPERTY(EditAnywhere, Category = "LOD Distribution", meta = (DisplayName = "Blend Range", ClampMin = "0.01", ClampMax = "1.0", UIMin = "0.01", UIMax = "1.0", LandscapeInherited))
	float LODBlendRange = 0.1f;

#if WITH_EDITORONLY_DATA
	/** LOD level to use when exporting the landscape to obj or FBX */
	UPROPERTY(EditAnywhere, Category=LOD, AdvancedDisplay, meta = (LandscapeOverridable))
	int32 ExportLOD;
#endif

	/** LOD level to use when running lightmass (increase to 1 or 2 for large landscapes to stop lightmass crashing) */
	UPROPERTY(EditAnywhere, Category=Lighting, meta = (LandscapeOverridable))
	int32 StaticLightingLOD; 

	/** Default physical material, used when no per-layer values physical materials */
	UPROPERTY(EditAnywhere, Category=Landscape, meta = (LandscapeOverridable))
	TObjectPtr<UPhysicalMaterial> DefaultPhysMaterial;

	/**
	 * Allows artists to adjust the distance where textures using UV 0 are streamed in/out.
	 * 1.0 is the default, whereas a higher value increases the streamed-in resolution.
	 * Value can be < 0 (from legcay content, or code changes)
	 */
	UPROPERTY(EditAnywhere, Category=Landscape, meta = (LandscapeOverridable))
	float StreamingDistanceMultiplier;

	/** Combined material used to render the landscape */
	UPROPERTY(EditAnywhere, BlueprintSetter=EditorSetLandscapeMaterial, Category=Landscape, meta=(LandscapeOverridable))
	TObjectPtr<UMaterialInterface> LandscapeMaterial;

	/** Material used to render landscape components with holes. If not set, LandscapeMaterial will be used (blend mode will be overridden to Masked if it is set to Opaque) */
	UPROPERTY(EditAnywhere, Category=Landscape, AdvancedDisplay, meta = (LandscapeOverridable))
	TObjectPtr<UMaterialInterface> LandscapeHoleMaterial;


#if WITH_EDITORONLY_DATA

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(all, "LandscapeComponentMaterialOverride has been deprecated, use PerLODOverrideMaterials instead.")
	UPROPERTY()
	TArray<FLandscapeProxyMaterialOverride> LandscapeMaterialsOverride_DEPRECATED;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#endif // WITH_EDITORONLY_DATA

	/**
	 * Array of runtime virtual textures into which we draw this landscape.
	 * The material also needs to be set up to output to a virtual texture.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = VirtualTexture, meta = (DisplayName = "Draw in Virtual Textures", LandscapeOverridable))
	TArray<TObjectPtr<URuntimeVirtualTexture>> RuntimeVirtualTextures;

	/** Placeholder for details customization button. */
	UPROPERTY(VisibleAnywhere, Transient, Category = VirtualTexture)
	bool bSetCreateRuntimeVirtualTextureVolumes;

	/** 
	 * Use a single quad to render this landscape to runtime virtual texture pages. 
	 * This is the fastest path but it only gives correct results if the runtime virtual texture orientation is aligned with the landscape.
	 * If the two are unaligned we need to render to the virtual texture using LODs with sufficient density.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = VirtualTexture, meta = (LandscapeOverridable))
	bool bVirtualTextureRenderWithQuad = false;

	/** 
	 * Use highest quality heightmap interpolation when using a single quad to render this landscape to runtime virtual texture pages.
	 * This also requires the project setting: r.VT.RVT.HighQualityPerPixelHeight.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = VirtualTexture, meta = (DisplayName = "High Quality PerPixel Height", EditCondition = "bVirtualTextureRenderWithQuad", LandscapeOverridable))
	bool bVirtualTextureRenderWithQuadHQ = true;

	/** 
	 * Number of mesh levels to use when rendering landscape into runtime virtual texture.
	 * Lower values reduce vertex count when rendering to the runtime virtual texture but decrease accuracy when using values that require vertex interpolation.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = VirtualTexture, meta = (DisplayName = "Virtual Texture Num LODs", EditCondition = "!bVirtualTextureRenderWithQuad", UIMin = "0", UIMax = "7", LandscapeOverridable))
	int32 VirtualTextureNumLods = 6;

	/** 
	 * Bias to the LOD selected for rendering to runtime virtual textures.
	 * Higher values reduce vertex count when rendering to the runtime virtual texture.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = VirtualTexture, meta = (DisplayName = "Virtual Texture LOD Bias", EditCondition = "!bVirtualTextureRenderWithQuad", UIMin = "0", UIMax = "7", LandscapeOverridable))
	int32 VirtualTextureLodBias = 0;

	/** Controls if this component draws in the main pass as well as in the virtual texture. */
	UPROPERTY(EditAnywhere, BlueprintSetter = SetVirtualTextureRenderPassType, Category = VirtualTexture, meta = (DisplayName = "Draw in Main Pass", LandscapeOverridable))
	ERuntimeVirtualTextureMainPassType VirtualTextureRenderPassType = ERuntimeVirtualTextureMainPassType::Always;

	UFUNCTION(BlueprintSetter)
	void SetVirtualTextureRenderPassType(ERuntimeVirtualTextureMainPassType InType);

	/** Allows overriding the landscape bounds. This is useful if you distort the landscape with world-position-offset, for example
	 *  Extension value in the negative Z axis, positive value increases bound size
	 *  Note that this can also be overridden per-component when the component is selected with the component select tool */
	UPROPERTY(EditAnywhere, Category=Landscape, meta = (LandscapeOverridable))
	float NegativeZBoundsExtension;

	/** Allows overriding the landscape bounds. This is useful if you distort the landscape with world-position-offset, for example
	 *  Extension value in the positive Z axis, positive value increases bound size
	 *  Note that this can also be overridden per-component when the component is selected with the component select tool */
	UPROPERTY(EditAnywhere, Category=Landscape, meta = (LandscapeOverridable))
	float PositiveZBoundsExtension;

	/** The array of LandscapeComponent that are used by the landscape */
	UPROPERTY()
	TArray<TObjectPtr<ULandscapeComponent>> LandscapeComponents;

	/** Array of LandscapeHeightfieldCollisionComponent */
	UPROPERTY()
	TArray<TObjectPtr<ULandscapeHeightfieldCollisionComponent>> CollisionComponents;

	UPROPERTY(transient, duplicatetransient)
	TArray<TObjectPtr<UHierarchicalInstancedStaticMeshComponent>> FoliageComponents;

	UE_DEPRECATED(all, "NaniteComponent has been deprecated, use NaniteComponents instead.")
	UPROPERTY()
	TObjectPtr<ULandscapeNaniteComponent> NaniteComponent_DEPRECATED;

	UPROPERTY(NonTransactional, TextExportTransient, NonPIEDuplicateTransient)
	TArray<TObjectPtr<ULandscapeNaniteComponent>> NaniteComponents;

	/** A transient data structure for tracking the grass */
	FCachedLandscapeFoliage FoliageCache;
	/** A transient data structure for tracking the grass tasks*/
	TArray<FAsyncTask<FAsyncGrassTask>* > AsyncFoliageTasks;
	/** Frame offset for tick interval*/
	uint32 FrameOffsetForTickInterval;

	// Only used outside of the editor (e.g. in cooked builds) - this value is no longer authoritative TODO [chris.tchou] remove
	// Cached grass max discard distance for all grass types in all landscape components with landscape grass configured
	UPROPERTY()
	float GrassTypesMaxDiscardDistance = 0.0f;

	// Non-serialized runtime cache of values derived from the assigned grass types.
	// Call ALandscapeProxy::UpdateGrassTypeSummary() to update.
	struct FGrassTypeSummary
	{
		// Used to track validity of these values, as it automatically invalidates if you add or remove a component.
		// If you do both add AND remove, then the add should also trigger UpdateGrassTypes() which will invalidate this cache.
		// Negative is the invalid state.
		int32 LandscapeComponentCount = -1;

		bool bHasAnyGrass = true;
		double MaxInstanceDiscardDistance = DBL_MAX;
	};
	FGrassTypeSummary GrassTypeSummary;
	inline bool IsGrassTypeSummaryValid() { return LandscapeComponents.Num() == GrassTypeSummary.LandscapeComponentCount; }
	inline void InvalidateGrassTypeSummary() { GrassTypeSummary.LandscapeComponentCount = -1; }
	void UpdateGrassTypeSummary();

	inline bool GetDisableRuntimeGrassMapGeneration() { return bDisableRuntimeGrassMapGeneration; }
	inline void SetDisableRuntimeGrassMapGenerationProxyOnly(bool bInDisableRuntimeGrassMapGeneration) { bDisableRuntimeGrassMapGeneration = bInDisableRuntimeGrassMapGeneration; }

	/** Whether grass primitives should be generated for this landscape. */
	LANDSCAPE_API bool ShouldGenerateGrass() const;

	/**
	 *	The resolution to cache lighting at, in texels/quad in one axis
	 *  Total resolution would be changed by StaticLightingResolution*StaticLightingResolution
	 *	Automatically calculate proper value for removing seams
	 */
	UPROPERTY(EditAnywhere, Category=Lighting, meta = (LandscapeOverridable))
	float StaticLightingResolution;

	/** Controls whether the primitive component should cast a shadow or not. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Lighting, meta = (LandscapeOverridable))
	uint8 CastShadow : 1;

	/** Controls whether the primitive should cast shadows in the case of non precomputed shadowing.  This flag is only used if CastShadow is true. **/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Lighting, AdvancedDisplay, meta = (EditCondition = "CastShadow", DisplayName = "Dynamic Shadow", LandscapeOverridable))
	uint8 bCastDynamicShadow : 1;

	/** Whether the object should cast a static shadow from shadow casting lights.  This flag is only used if CastShadow is true. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Lighting, AdvancedDisplay, meta = (EditCondition = "CastShadow", DisplayName = "Static Shadow", LandscapeOverridable))
	uint8 bCastStaticShadow : 1;

	/** Control shadow invalidation behavior, in particular with respect to Virtual Shadow Maps and material effects like World Position Offset. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Lighting, AdvancedDisplay, meta=(EditCondition="CastShadow", LandscapeOverridable))
	EShadowCacheInvalidationBehavior ShadowCacheInvalidationBehavior;

	/** Whether the object should cast contact shadows. This flag is only used if CastShadow is true. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Lighting, AdvancedDisplay, meta = (EditCondition = "CastShadow", DisplayName = "Contact Shadow", LandscapeOverridable))
	uint8 bCastContactShadow : 1;

	/** When enabled, the component will be rendering into the far shadow cascades(only for directional lights).  This flag is only used if CastShadow is true. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Lighting, meta = (EditCondition = "CastShadow", DisplayName = "Far Shadow", LandscapeOverridable))
	uint32 bCastFarShadow : 1;

	/** If true, the primitive will cast shadows even if bHidden is true.  Controls whether the primitive should cast shadows when hidden.  This flag is only used if CastShadow is true. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Lighting, meta = (EditCondition = "CastShadow", DisplayName = "Hidden Shadow", LandscapeOverridable))
	uint8 bCastHiddenShadow : 1;

	/** Whether this primitive should cast dynamic shadows as if it were a two sided material.  This flag is only used if CastShadow is true. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Lighting, meta = (EditCondition = "CastShadow", DisplayName = "Shadow Two Sided", LandscapeOverridable))
	uint32 bCastShadowAsTwoSided : 1;

	/** Controls whether the primitive should affect dynamic distance field lighting methods.  This flag is only used if CastShadow is true. **/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Lighting, AdvancedDisplay, meta = (EditCondition = "CastShadow", LandscapeOverridable))
	uint8 bAffectDistanceFieldLighting:1;

	/** Controls whether the primitive should influence indirect lighting. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Lighting, AdvancedDisplay, meta=(LandscapeOverridable), Interp)
	uint8 bAffectDynamicIndirectLighting:1;

	/** Controls whether the primitive should affect indirect lighting when hidden. This flag is only used if bAffectDynamicIndirectLighting is true. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Lighting, meta=(EditCondition="bAffectDynamicIndirectLighting", DisplayName = "Affect Indirect Lighting While Hidden", LandscapeOverridable), Interp)
	uint8 bAffectIndirectLightingWhileHidden:1;

	/**
	* Channels that this Landscape should be in.  Lights with matching channels will affect the Landscape.
	* These channels only apply to opaque materials, direct lighting, and dynamic lighting and shadowing.
	*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Lighting, meta = (LandscapeOverridable))
	FLightingChannels LightingChannels;

	/** Whether to use the landscape material's vertical world position offset when calculating static lighting. Z (vertical) offset is supported */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Lighting, meta = (LandscapeInherited))
	uint32 bUseMaterialPositionOffsetInStaticLighting:1;

	/** Constant bias to handle the worst artifacts of the continuous LOD morphing when rendering to VSM.  
	* Only applies when using non-Nanite landscape and VSM. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Lighting, meta = (LandscapeOverridable))
	float NonNaniteVirtualShadowMapConstantDepthBias = 150.0f;

	/** For non-Nanite landscape, cached VSM pages need to be invalidated when continuous LOD morphing introduces a height difference that is too large between the current landscape component's profile and the one that was used when the shadow was shadow was last cached.
	* This height threshold (in Unreal units) controls this invalidation rate (a smaller threshold will reduce the likeliness of shadow artifacts, but will make the invalidations occur more frequently, which is not desirable in terms of performance.
	* Disabled if 0.0.
	* Only applies when using non-Nanite landscape and VSM. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Lighting, meta = (LandscapeOverridable, UIMin = 0.0, ClampMin = 0.0))
	float NonNaniteVirtualShadowMapInvalidationHeightErrorThreshold = 250.0f;

	/** Screen size under which VSM invalidation stops occurring.
	* As the height difference between 2 mip levels increases when the LOD level increases (because of undersampling), VSM pages tend to be invalidated more frequently even though it's getting less and less relevant to do so, since this will mean that the screen size of the landscape section decreases, thus the artifacts actually become less noticeable.
	* We therefore artificially attenuate the VSM invalidation rate as the screen size decreases, to avoid invalidating VSM pages too often, as it becomes less and less impactful. 
	* A higher value will accentuate this attenuation (better performance but more artifacts) and vice versa.
	* If 0.0, the attenuation of the VSM invalidation rate will be disabled entirely.
	* Only applies when using non-Nanite landscape and VSM. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Lighting, meta = (LandscapeOverridable, ClampMin = "0.0", ClampMax = "10.0", UIMin = "0.0", UIMax = "10.0"))
	float NonNaniteVirtualShadowMapInvalidationScreenSizeLimit = 0.2f;

	/** If this is True, this primitive will render black with an alpha of 0, but all secondary effects (shadows, reflections, indirect lighting) remain. This feature requires the project settings "Alpha Output" and "Support Primitive Alpha Holdout". */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Rendering, meta=(LandscapeOverridable), Interp)
	uint8 bHoldout : 1;

	/** If true, the Landscape will be rendered in the CustomDepth pass (usually used for outlines) */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Rendering, meta=(DisplayName = "Render CustomDepth Pass", LandscapeOverridable))
	uint32 bRenderCustomDepth:1;

	/** Mask used for stencil buffer writes. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = Rendering, meta = (editcondition = "bRenderCustomDepth", LandscapeOverridable))
	ERendererStencilMask CustomDepthStencilWriteMask;

	/** Optionally write this 0-255 value to the stencil buffer in CustomDepth pass (Requires project setting or r.CustomDepth == 3) */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Rendering,  meta=(UIMin = "0", UIMax = "255", editcondition = "bRenderCustomDepth", DisplayName = "CustomDepth Stencil Value", LandscapeOverridable))
	int32 CustomDepthStencilValue;

	/**  Max draw distance exposed to LDs. The real max draw distance is the min (disregarding 0) of this and volumes affecting this object. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category=Rendering, meta = (DisplayName = "Desired Max Draw Distance", LandscapeOverridable))
	float LDMaxDrawDistance;

	/** The Lightmass settings for this object. */
	UPROPERTY(EditAnywhere, Category=Lightmass, meta = (LandscapeInherited))
	FLightmassPrimitiveSettings LightmassSettings;

	/** Specify alternate landscape grass types for some specific grass map names. This allows to reuse a given material's grass outputs, but change the grass meshes it spawns in the context of a specific landscape actor. */
	UPROPERTY(EditAnywhere, Category=Grass, meta = (LandscapeOverridable, ForceInlineRow))
	TMap<FLandscapeGrassName, TObjectPtr<ULandscapeGrassType>> GrassTypesOverrides;

	// Landscape LOD to use for collision tests. Higher numbers use less memory and process faster, but are much less accurate
	UPROPERTY(EditAnywhere, Category=Collision, meta = (LandscapeOverridable))
	int32 CollisionMipLevel;

	// If set higher than the "Collision Mip Level", this specifies the Landscape LOD to use for "simple collision" tests, otherwise the "Collision Mip Level" is used for both simple and complex collision.
	UPROPERTY(EditAnywhere, Category=Collision, meta = (LandscapeOverridable))
	int32 SimpleCollisionMipLevel;

	/** Collision profile settings for this landscape */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Collision, meta = (LandscapeOverridable))
	FBodyInstance BodyInstance;

	/**
	 * If true, Landscape will generate overlap events when other components are overlapping it (eg Begin Overlap).
	 * Both the Landscape and the other component must have this flag enabled for overlap events to occur.
	 *
	 * @see [Overlap Events](https://docs.unrealengine.com/InteractiveExperiences/Physics/Collision/Overview#overlapandgenerateoverlapevents)
	 * @see UpdateOverlaps(), BeginComponentOverlap(), EndComponentOverlap()
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Collision, meta = (LandscapeOverridable))
	uint32 bGenerateOverlapEvents : 1;

	/** Whether to bake the landscape material's vertical world position offset into the collision heightfield. Only z (vertical) offset is supported. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category=Collision, meta = (LandscapeInherited))
	uint32 bBakeMaterialPositionOffsetIntoCollision:1;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(all, "TargetLayers will be used instead")
	UPROPERTY()
	TArray<TObjectPtr<ULandscapeLayerInfoObject>> EditorCachedLayerInfos_DEPRECATED;

	UPROPERTY()
	FString ReimportHeightmapFilePath;

	/** Height and weightmap import destination layer guid */
	UPROPERTY()
	FGuid ReimportDestinationLayerGuid;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(all, "TargetLayers will be used instead")
	UPROPERTY()
	TArray<FLandscapeEditorLayerSettings> EditorLayerSettings_DEPRECATED;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	TMap<UTexture2D*, FLandscapeEditLayerReadback*> HeightmapsCPUReadback;
	TMap<UTexture2D*, FLandscapeEditLayerReadback*> WeightmapsCPUReadback;

	/** Map of weightmap usage */
	UPROPERTY(Transient, NonTransactional)
	TMap<TObjectPtr<UTexture2D>, TObjectPtr<ULandscapeWeightmapUsage>> WeightmapUsageMap;

	/** CurrentVersion is bumped whenever a landscape component has an undo/redo operation applied. This lets us detect when the weightmap fixup needs to be run. */
	uint32 CurrentVersion = 1;
	uint32 WeightmapFixupVersion = 0;
	/** Tracks the number of times this Proxy has been registered since load */
	uint32 RegistrationCount = 0;

	/** True when this Proxy is registered with the LandscapeInfo */
	bool bIsRegisteredWithLandscapeInfo : 1 = false;

	/** Set to true when on undo, when it's necessary to completely regenerate weightmap usages (since some weightmap allocations are transactional and others not, e.g. splines edit layer) */
	bool bNeedsWeightmapUsagesUpdate : 1 = false;

	/** Set to true when we know that weightmap usages are being reconstructed and might be temporarily invalid as a result (ValidateProxyLayersWeightmapUsage should be called after setting this back to false) */
	bool bTemporarilyDisableWeightmapUsagesValidation : 1 = false;
#endif // WITH_EDITORONLY_DATA

	/** True when this Proxy's LandscapeActor has registered with the LandscapeInfo, (and our shared properties are fully updated) */
	bool bIsLandscapeActorRegisteredWithLandscapeInfo : 1 = false;

	/** True when this Proxy is registered with the ULandscapeSubsystem */
	bool bIsRegisteredWithSubsystem : 1 = false;

	/** Used to detect incorrect registration/unregistration behavior */
	UPROPERTY(Transient, DuplicateTransient, NonTransactional, SkipSerialization)
	TObjectPtr<ULandscapeSubsystem> RegisteredToSubsystem;

	/** Data set at creation time */
	UPROPERTY(meta = (LandscapeInherited))
	int32 ComponentSizeQuads;    // Total number of quads in each component

	UPROPERTY(meta = (LandscapeInherited))
	int32 SubsectionSizeQuads;    // Number of quads for a subsection of a component. SubsectionSizeQuads+1 must be a power of two.

	UPROPERTY(meta = (LandscapeInherited))
	int32 NumSubsections;    // Number of subsections in X and Y axis

	/** Hints navigation system whether this landscape will ever be navigated on. true by default, but make sure to set it to false for faraway, background landscapes */
	UPROPERTY(EditAnywhere, Category=Navigation, meta = (LandscapeOverridable))
	uint32 bUsedForNavigation:1;

	/** Set to true to prevent navmesh generation under the terrain geometry */
	UPROPERTY(EditAnywhere, Category = Navigation, meta = (LandscapeOverridable))
	uint32 bFillCollisionUnderLandscapeForNavmesh:1;

	UPROPERTY(EditAnywhere, Category = Navigation, AdvancedDisplay, meta = (LandscapeOverridable))
	ENavDataGatheringMode NavigationGeometryGatheringMode;

	/** When set to true it will generate MaterialInstanceDynamic for each components, so material can be changed at runtime */
	UPROPERTY(EditAnywhere, Category = Landscape, meta = (LandscapeOverridable))
	bool bUseDynamicMaterialInstance;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category=Landscape, meta = (LandscapeOverridable))
	int32 MaxPaintedLayersPerComponent; // 0 = disabled
#endif // WITH_EDITORONLY_DATA

	/** Flag whether or not this Landscape's surface can be used for culling hidden triangles **/
	UPROPERTY(EditAnywhere, Category = HLOD, meta = (LandscapeOverridable))
	bool bUseLandscapeForCullingInvisibleHLODVertices;

#if WITH_EDITORONLY_DATA
	/** Specify how to choose the texture size of the resulting HLOD mesh. Specifying an HLOD Material Override will disable this option as no texture will be baked. */
	UPROPERTY(EditAnywhere, Category = HLOD, meta = (DisplayName = "HLOD Texture Size Policy", LandscapeOverridable, EditCondition = "HLODMaterialOverride == nullptr"))
	ELandscapeHLODTextureSizePolicy HLODTextureSizePolicy;

	/** Specify the texture size to use for the HLOD mesh if HLODTextureSizePolicy is set to SpecificSize. Specifying an HLOD Material Override will disable this option as no texture will be baked. */
	UPROPERTY(EditAnywhere, Category = HLOD, meta = (DisplayName = "HLOD Texture Size", LandscapeOverridable, EditCondition = "HLODMaterialOverride == nullptr && HLODTextureSizePolicy == ELandscapeHLODTextureSizePolicy::SpecificSize", ClampMin = "16", ClampMax = "8192"))
	int32 HLODTextureSize;

	/** Specify a custom HLOD material to apply to the HLOD mesh. Specifying an HLOD Material Override will result in no texture being baked for the HLOD mesh. */
	UPROPERTY(EditAnywhere, Category = HLOD, meta = (DisplayName = "HLOD Material Override", LandscapeOverridable))
	TObjectPtr<UMaterialInterface> HLODMaterialOverride;

	/** Specify how to choose the LOD used as input for the HLOD mesh. */
	UPROPERTY(EditAnywhere, Category = HLOD, meta = (DisplayName = "HLOD Mesh Source LOD Policy", LandscapeOverridable))
	ELandscapeHLODMeshSourceLODPolicy HLODMeshSourceLODPolicy;

	/** Specify which LOD to use for the HLOD mesh if HLODMeshSourceLODPolicy is set to SpecificLOD. */
	UPROPERTY(EditAnywhere, Category = HLOD, meta = (DisplayName = "HLOD Mesh Source LOD", LandscapeOverridable, EditCondition = "HLODMeshSourceLODPolicy == ELandscapeHLODMeshSourceLODPolicy::SpecificLOD", ClampMin = "0"))
	int32 HLODMeshSourceLOD;
#endif

	UE_DEPRECATED(5.7, "Non-edit layer landscapes are deprecated, all landscapes use the edit layer system now.")
	UPROPERTY()
	bool bHasLayersContent_DEPRECATED = false;

	UE_DEPRECATED(5.7, "Non-edit layer landscapes are deprecated, all landscapes use the edit layer system now.")
	TOptional<bool> HadLayersContentAtPostLoadTime;

	/** Strip Physics/collision components when cooked for client */
	UPROPERTY(EditAnywhere, Category = Landscape, AdvancedDisplay, meta = (LandscapeOverridable))
	bool bStripPhysicsWhenCookedClient = false;

	/** Strip Physics/collision components when cooked for server */
	UPROPERTY(EditAnywhere, Category = Landscape, AdvancedDisplay, meta = (LandscapeOverridable))
	bool bStripPhysicsWhenCookedServer = false;

	/** Strip Grass data when cooked for client */
	UPROPERTY(EditAnywhere, Category = Landscape, AdvancedDisplay, meta = (LandscapeOverridable))
	bool bStripGrassWhenCookedClient = false;

	/** Strip Grass data when cooked for server */
	UPROPERTY(EditAnywhere, Category = Landscape, AdvancedDisplay, meta = (LandscapeOverridable))
	bool bStripGrassWhenCookedServer = false;

#if WITH_EDITOR
	LANDSCAPE_API static ULandscapeLayerInfoObject* VisibilityLayer;
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	/** Map of material instance constants used to for the components. Key is generated with ULandscapeComponent::GetLayerAllocationKey() */
	TMap<FString, TObjectPtr<UMaterialInstanceConstant>> MaterialInstanceConstantMap;
#endif // WITH_EDITORONLY_DATA

	// Blueprint functions

	/** Change the Level of Detail distance factor */
	UFUNCTION(BlueprintCallable, Category = "Rendering", meta=(DeprecatedFunction, DeprecationMessage = "This value can't be changed anymore, you should edit the property LODDistributionSetting of the Landscape"))
	virtual void ChangeLODDistanceFactor(float InLODDistanceFactor);

	UFUNCTION(BlueprintCallable, Category = "Rendering", meta = (DeprecatedFunction, DeprecationMessage = "This value can't be changed anymore and has been ineffective for several versions now. Please stop using it"))
	virtual void ChangeComponentScreenSizeToUseSubSections(float InComponentScreenSizeToUseSubSections);

	/* Setter for LandscapeMaterial. Has no effect outside the editor. */
	UFUNCTION(BlueprintSetter)
	void EditorSetLandscapeMaterial(UMaterialInterface* NewLandscapeMaterial);

	// Editor-time blueprint functions

	/** Deform landscape using a given spline
	 * @param InSplineComponent - The component containing the spline data
	 * @param StartWidth - Width of the spline at the start node, in Spline Component local space
	 * @param EndWidth   - Width of the spline at the end node, in Spline Component local space
	 * @param StartSideFalloff - Width of the falloff at either side of the spline at the start node, in Spline Component local space
	 * @param EndSideFalloff - Width of the falloff at either side of the spline at the end node, in Spline Component local space
	 * @param StartRoll - Roll applied to the spline at the start node, in degrees. 0 is flat
	 * @param EndRoll - Roll applied to the spline at the end node, in degrees. 0 is flat
	 * @param NumSubdivisions - Number of triangles to place along the spline when applying it to the landscape. Higher numbers give better results, but setting it too high will be slow and may cause artifacts
	 * @param bRaiseHeights - Allow the landscape to be raised up to the level of the spline. If both bRaiseHeights and bLowerHeights are false, no height modification of the landscape will be performed
	 * @param bLowerHeights - Allow the landscape to be lowered down to the level of the spline. If both bRaiseHeights and bLowerHeights are false, no height modification of the landscape will be performed
	 * @param PaintLayer - LayerInfo to paint, or none to skip painting. The landscape must be configured with the same layer info in one of its layers or this will do nothing!
	 * @param EditLayerName - Name of the landscape edit layer to affect (in Edit Layers mode)
	 */
	UFUNCTION(BlueprintCallable, Category = "Landscape|Editor")
	LANDSCAPE_API void EditorApplySpline(USplineComponent* InSplineComponent, float StartWidth = 200, float EndWidth = 200, float StartSideFalloff = 200, float EndSideFalloff = 200, float StartRoll = 0, float EndRoll = 0, int32 NumSubdivisions = 20, bool bRaiseHeights = true, bool bLowerHeights = true, ULandscapeLayerInfoObject* PaintLayer = nullptr, FName EditLayerName = TEXT(""));

	/** Set an MID texture parameter value for all landscape components. */
	UFUNCTION(BlueprintCallable, Category = "Landscape|Runtime|Material")
	LANDSCAPE_API void SetLandscapeMaterialTextureParameterValue(FName ParameterName, class UTexture* Value);

	/** Set an MID vector parameter value for all landscape components. */
	UFUNCTION(BlueprintCallable, meta = (Keywords = "SetColorParameterValue"), Category = "Landscape|Runtime|Material")
	LANDSCAPE_API void SetLandscapeMaterialVectorParameterValue(FName ParameterName, FLinearColor Value);

	/** Set a MID scalar (float) parameter value for all landscape components. */
	UFUNCTION(BlueprintCallable, meta = (Keywords = "SetFloatParameterValue"), Category = "Landscape|Runtime|Material")
	LANDSCAPE_API void SetLandscapeMaterialScalarParameterValue(FName ParameterName, float Value);

	// End blueprint functions

	//~ Begin AActor Interface
	virtual void PostRegisterAllComponents() override;
	virtual void UnregisterAllComponents(bool bForReregister = false) override;
	virtual bool IsLevelBoundsRelevant() const override { return true; }

	virtual void PostInitProperties() override;
	virtual void FinishDestroy() override;

#if WITH_EDITOR
	virtual void RerunConstructionScripts() override {}
	virtual void Destroyed() override;
	virtual void EditorApplyScale(const FVector& DeltaScale, const FVector* PivotLocation, bool bAltDown, bool bShiftDown, bool bCtrlDown) override;
	virtual void EditorApplyMirror(const FVector& MirrorScale, const FVector& PivotLocation) override;
	virtual void PostEditMove(bool bFinished) override;
	virtual void OnLoadedActorRemovedFromLevel() override;
	//~ End AActor Interface

	virtual uint32 GetDefaultGridSize(UWorld* InWorld) const override { return 1; }
	virtual FGuid GetGridGuid() const override { return LandscapeGuid; }
	virtual TUniquePtr<class FWorldPartitionActorDesc> CreateClassActorDesc() const override;
	virtual bool EditorCanAttachTo(const AActor* InParent, FText& OutReason) const override { return false; }
	virtual bool GetReferencedContentObjects(TArray<UObject*>& Objects) const override;
	virtual bool IsNaniteEnabled() const { return bEnableNanite; }
	virtual int32 GetNaniteLODIndex() const { return NaniteLODIndex; }
	virtual bool IsNaniteSkirtEnabled() const { return bNaniteSkirtEnabled; }
	virtual float GetNaniteSkirtDepth() const { return NaniteSkirtDepth;  }
	virtual int32 GetNanitePositionPrecision() const { return NanitePositionPrecision; }
	virtual float GetNaniteMaxEdgeLengthFactor() const { return NaniteMaxEdgeLengthFactor; }
	virtual UE::Landscape::EOutdatedDataFlags GetOutdatedDataFlags() const;

	void UpdateNaniteSharedPropertiesFromActor();
	void RemoveNaniteComponents();
	void ClearNaniteTransactional();
	static constexpr int32 NaniteComponentMaxSide = 8;
	static constexpr int32 NaniteMaxComponents = NaniteComponentMaxSide * NaniteComponentMaxSide;
	int32 NumNaniteRequiredComponents() const { return FMath::DivideAndRoundUp(LandscapeComponents.Num(), NaniteMaxComponents); }
#endif	//WITH_EDITOR

	bool AreNaniteComponentsValid(const FGuid& InProxyContentId) const;
	bool HasNaniteComponents() const { return !NaniteComponents.IsEmpty(); }
	LANDSCAPE_API TSet<FPrimitiveComponentId> GetNanitePrimitiveComponentIds() const;

	bool AuditNaniteMaterials() const;
	void EnableNaniteComponents(bool bInNaniteActive);

	LANDSCAPE_API TOptional<float> GetHeightAtLocation(FVector Location, EHeightfieldSource HeightFieldSource = EHeightfieldSource::Complex) const;
	LANDSCAPE_API UPhysicalMaterial* GetPhysicalMaterialAtLocation(FVector Location, EHeightfieldSource HeightFieldSOurce = EHeightfieldSource::Complex) const;

	/** Fills an array with height values **/
	LANDSCAPE_API void GetHeightValues(int32& SizeX, int32& SizeY, TArray<float>& ArrayValue) const;

	/* Return the landscape guid, used to identify landscape proxies (and splines) that belong to the same landscape, even across world partitions.
	 * Also used as the world partition grid guid.  This value may be modified when the landscape is instanced, to allow multiple instances of the same
	 * landscape to exist simultaneously.  If you want the original (uninstanced) value, use GetOriginalLandscapeGuid().
	 */
	virtual FGuid GetLandscapeGuid() const override { return LandscapeGuid; }
	void SetLandscapeGuid(const FGuid& Guid, bool bValidateGuid = true)
	{
		// we probably shouldn't be setting the landscape guid on instanced landscapes
		check(!bValidateGuid || (OriginalLandscapeGuid == LandscapeGuid) || !OriginalLandscapeGuid.IsValid());
		LandscapeGuid = Guid;
		OriginalLandscapeGuid = Guid;
	}

	/** Computes the rendering key for this landscape proxy */
	LANDSCAPE_API uint32 ComputeLandscapeKey() const;
	LANDSCAPE_API static uint32 ComputeLandscapeKey(const UWorld* World, uint32 InLODGroupKey, FGuid InLandscapeGuid);

	/* Return the original landscape guid, before it was modified by instancing.
	 * When not instanced, this value is equal to LandscapeGuid.
	 */
	const FGuid& GetOriginalLandscapeGuid() const { return OriginalLandscapeGuid; }

	void SetLODGroupKeyInternal(uint32 InLODGroupKey);

	UFUNCTION(BlueprintCallable, Category = "Landscape|Runtime")
	virtual ALandscape* GetLandscapeActor() PURE_VIRTUAL(GetLandscapeActor, return nullptr;)
	virtual const ALandscape* GetLandscapeActor() const PURE_VIRTUAL(GetLandscapeActor, return nullptr;)

	const TArray<FLandscapePerLODMaterialOverride>& GetPerLODOverrideMaterials() const { return PerLODOverrideMaterials; }
	LANDSCAPE_API void SetPerLODOverrideMaterials(const TArray<FLandscapePerLODMaterialOverride>& InValue);

	static void SetGrassUpdateInterval(int32 Interval) { GrassUpdateInterval = Interval; }

	/* Determine whether we should update dynamic grass instances this update tick */
	FORCEINLINE bool ShouldTickGrass() const
	{
		// don't tick grass if there's no grass to tick
		if (!GrassTypeSummary.bHasAnyGrass)
		{
			return false;
		}

		const int32 UpdateInterval = GetGrassUpdateInterval();
		if (UpdateInterval > 1)
		{
			if ((GFrameNumber + FrameOffsetForTickInterval) % uint32(UpdateInterval))
			{
				return false;
			}
		}

		return true;
	}
	void ProcessAsyncGrassInstanceTasks(bool bWaitAsyncTasks, bool bForceSync, const TSet<UHierarchicalInstancedStaticMeshComponent*>& StillUsed);

	/** Flush the grass cache, removing grass instances on the given components (or all proxy components if the component set is not specified).
	*     bFlushGrassMaps will delete the grass data / density maps on the components as well, but only in editor mode, and only if the grass maps are renderable (i.e. they can be regenerated).
	*/
	LANDSCAPE_API void FlushGrassComponents(const TSet<ULandscapeComponent*>* OnlyForComponents = nullptr, bool bFlushGrassMaps = true);

	/**
		Update Grass -- Builds grass instances given the camera locations
		* @param Cameras to use for culling, if empty, then NO culling
		* @param InOutNumComponentsCreated, value can increase if components were created, it is also used internally to limit the number of creations
		* @param bForceSync if true, block and finish all work so that grass is fully populated for the given cameras
	*/
	LANDSCAPE_API void UpdateGrass(const TArray<FVector>& Cameras, int32& InOutNumComponentsCreated, bool bForceSync = false);
	LANDSCAPE_API void UpdateGrass(const TArray<FVector>& Cameras, bool bForceSync = false);


	/**
	 * Registers an axis-aligned bounding box that will act as an exclusion volume for all landscape grass overlapping it
	 * @param Owning UObject of the box. Acts as an identifier of the exclusion volume. Also, when the UObject becomes stale, the box will be automatically unregistered from landscapes
	 * @param BoxToRemove AABBox (excluded volume)
	 */
	// TODO [jonathan.bard] : Rename to "AddGrassExclusionBox" + no reason for any of this to be static
	LANDSCAPE_API static void AddExclusionBox(FWeakObjectPtr Owner, const FBox& BoxToRemove);
	/**
	 * Unregisters a previously-registered exclusion box. Landscape grass will be able to be spawned again in this area after the operation
	 * @param Owner Identifier of the box to remove (see AddExclusionBox)
	 */
	LANDSCAPE_API static void RemoveExclusionBox(FWeakObjectPtr Owner);

	/**
	 * Unregisters all existing exclusion boxes.
	 */
	LANDSCAPE_API static void RemoveAllExclusionBoxes();

	static void RemoveInvalidExclusionBoxes();

	static void DebugDrawExclusionBoxes(const UWorld* World);

	/* Invalidate the precomputed grass and baked texture data for the specified components */
	LANDSCAPE_API static void InvalidateGeneratedComponentData(const TSet<ULandscapeComponent*>& Components, bool bInvalidateLightingCache = false);
	LANDSCAPE_API static void InvalidateGeneratedComponentData(const TArray<ULandscapeComponent*>& Components, bool bInvalidateLightingCache = false);

	/* Invalidate the precomputed grass and baked texture data on all components */
	LANDSCAPE_API void InvalidateGeneratedComponentData(bool bInvalidateLightingCache = false);

	LANDSCAPE_API void UpdateRenderingMethod();

#if WITH_EDITOR
	/**
	 * Update the landscape physical material render tasks
	 * @param bInShouldMarkDirty indicates whether the proxy should be marked as dirty if anything was built
	 * 
	 * @return true if anything was built on this proxy
	 */
	bool UpdatePhysicalMaterialTasks(bool bInShouldMarkDirty = false);
	void UpdatePhysicalMaterialTasksStatus(TSet<ULandscapeComponent*>* OutdatedComponents, int32* OutdatedComponentsCount) const;

	/** Editor notification when changing feature level */
	void OnFeatureLevelChanged(ERHIFeatureLevel::Type NewFeatureLevel);

	/** Handle so we can unregister the delegate */
	FDelegateHandle FeatureLevelChangedDelegateHandle;

	FGuid GetNaniteContentId() const;

	void ValidateGrassTypeOverrides();
#endif // WITH_EDITOR

	//~ Begin UObject Interface.
#if WITH_EDITOR
	virtual void CheckForErrors() override;
	virtual void BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform) override;
	virtual void OnCookEvent(UE::Cook::ECookEvent CookEvent, UE::Cook::FCookEventContext& CookContext) override;
#endif // WITH_EDITOR

	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	virtual void Serialize(FArchive& Ar) override;
	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	virtual void PostLoad() override;

	/** Creates the LandscapeInfo if necessary, then registers this proxy with it */
	LANDSCAPE_API ULandscapeInfo* CreateLandscapeInfo(bool bMapCheck = false, bool bUpdateAllAddCollisions = true);
	virtual LANDSCAPE_API ULandscapeInfo* GetLandscapeInfo() const override;

	/** Get the LandcapeActor-to-world transform with respect to landscape section offset*/
	virtual LANDSCAPE_API FTransform LandscapeActorToWorld() const override;

	/** Differs from GetActorBounds as it only considers LandscapeComponents/CollisionComponents bounds */
	LANDSCAPE_API FBox GetProxyBounds() const;

	/**
	* Output a landscape heightmap to a render target
	* @param InRenderTarget - Valid render target with a format of RTF_RGBA16f, RTF_RGBA32f or RTF_RGBA8
	* @param InExportHeightIntoRGChannel - Tell us if we should export the height that is internally stored as R & G (for 16 bits) to a single R channel of the render target (the format need to be RTF_RGBA16f or RTF_RGBA32f)
	*									   Note that using RTF_RGBA16f with InExportHeightIntoRGChannel == false, could have precision loss.
	* @param InExportLandscapeProxies - Option to also export components of all proxies of Landscape actor (if LandscapeProxy is the Landscape Actor)
	*/
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Landscape Export Heightmap to RenderTarget", Keywords = "Push Landscape Heightmap to RenderTarget", UnsafeDuringActorConstruction = "true"), Category = Rendering)
	bool LandscapeExportHeightmapToRenderTarget(UTextureRenderTarget2D* InRenderTarget, bool InExportHeightIntoRGChannel = false, bool InExportLandscapeProxies = true);

	UE_DEPRECATED(5.7, "Use GetSectionBase instead.")
	LANDSCAPE_API FIntPoint GetSectionBaseOffset() const;

	/** Get landscape position in section space */
	LANDSCAPE_API FIntPoint GetSectionBase() const;

	// ILandscapeSplineInterface
	virtual ULandscapeSplinesComponent* GetSplinesComponent() const override { return SplineComponent; }
	LANDSCAPE_API virtual void UpdateSharedProperties(ULandscapeInfo* InLandscapeInfo) override;
	virtual bool IsSplineOwnerValid() const override { return true; }

	// Retrieve the screen size at which each LOD should be rendered
	LANDSCAPE_API TArray<float> GetLODScreenSizeArray() const;

#if WITH_EDITOR
	// Copy properties from parent Landscape actor
	LANDSCAPE_API void CopySharedProperties(ALandscapeProxy* InLandscape, bool bInExecutePostEditChangeProperty = false);

	/**
	* Enforce property sharing and requirements related to the parent Landscape actor
	* @param InLandscape the parent landscape to use as a template.
	* @param bInExecutePostEditChangeProperty When true, executes PostEditChangeProperty for each property synchronized
	* @return An array containing all synchronized properties names (properties that were different from the provided landscape).
	*/
	LANDSCAPE_API TArray<FName> SynchronizeSharedProperties(ALandscapeProxy* InLandscape, bool bInExecutePostEditChangeProperty = false);

	// Returns true if the property is shared.
	LANDSCAPE_API bool IsSharedProperty(const FName& InPropertyName) const;

	// Returns true if the property is shared.
	LANDSCAPE_API bool IsSharedProperty(const FProperty* InProperty) const;

	// Returns true if the property is inherited.
	LANDSCAPE_API bool IsPropertyInherited(const FProperty* InProperty) const;
	
	// Returns true if the property is overridable
	LANDSCAPE_API bool IsPropertyOverridable(const FProperty* InProperty) const;
	
	// Returns true if the shared property is overridden by the object.
	virtual bool IsSharedPropertyOverridden(const FName& InPropertyName) const { return false; }

	// Modifies the override state of the property given as argument.
	virtual void SetSharedPropertyOverride(const FName& InPropertyName, const bool bIsOverriden, const bool bInExecutePostEditChangeProperty = true) { }

	/** Delegate that will be called whenever FixupSharedData is called, to inject some custom logic */
	static FOnLandscapeProxyFixupSharedDataDelegate::RegistrationType& OnLandscapeProxyFixupSharedData() { return OnLandscapeProxyFixupSharedDataDelegate; }
#endif // WITH_EDITOR

	// Get Landscape Material assigned to this Landscape
	virtual UMaterialInterface* GetLandscapeMaterial(int8 InLODIndex = INDEX_NONE) const;

	// Get Hole Landscape Material assigned to this Landscape
	virtual UMaterialInterface* GetLandscapeHoleMaterial() const;

	/** 
	 * Retrieve the list of all materials that might be used by this landscape proxy or its components (LandscapeMaterial, LandscapeHoleMaterial, component overrides, etc.) 
	 * @param OutMaterials list of unique materials retrieved. The materials found will be appended to the list
	 */
	LANDSCAPE_API void RetrieveAllLandscapeMaterials(TSet<UMaterialInterface*>& OutMaterials) const;

	inline TMap<FLandscapeGrassName, TObjectPtr<ULandscapeGrassType>> GetGrassTypesOverrides() const
	{
		return GrassTypesOverrides;
	}

#if WITH_EDITOR
	/* Serialize all hashes/guids that record the current state of this proxy */
	void SerializeStateHashes(FArchive& Ar);

	void SetSplinesComponent(ULandscapeSplinesComponent* InSplineComponent) { check(!SplineComponent || (SplineComponent == InSplineComponent)); SplineComponent = InSplineComponent; }

	virtual bool SupportsForeignSplineMesh() const override { return true; }

	LANDSCAPE_API int32 GetOutdatedGrassMapCount() const;
	LANDSCAPE_API void BuildGrassMaps();
	/**
	 * Builds the physical materials on the components of this proxy
	 * 
	 * @return true if anything has been built
	 */
	LANDSCAPE_API bool BuildPhysicalMaterial();
	LANDSCAPE_API void InvalidatePhysicalMaterial();
	LANDSCAPE_API int32 GetOudatedPhysicalMaterialComponentsCount() const;
	LANDSCAPE_API virtual void CreateSplineComponent() override;
	LANDSCAPE_API virtual void CreateSplineComponent(const FVector& Scale3D) override;

	virtual bool CanEditChange(const FProperty* InProperty) const override;
	virtual void PostTransacted(const FTransactionObjectEvent& InTransactionEvent) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditImport() override;
	virtual void PostEditUndo() override;
	//~ End UObject Interface

	LANDSCAPE_API void InitializeProxyLayersWeightmapUsage();
	void ValidateProxyLayersWeightmapUsage() const;
	void UpdateProxyLayersWeightmapUsage();
	void RequestProxyLayersWeightmapUsageUpdate();

	UE_DEPRECATED(5.8, "Use UE::Landscape::RetrieveTargetLayerNamesFromMaterial")
	LANDSCAPE_API static const TArray<FName> GetLayersFromMaterial(UMaterialInterface* Material);
	UE_DEPRECATED(5.8, "Use RetrieveTargetLayerNamesFromMaterials")
	LANDSCAPE_API const TArray<FName> GetLayersFromMaterial() const;

	// Get all Layer names used by the materials assigned the proxy and components (Proxy Material, Component, Component LOD Material and Hole Material)
	LANDSCAPE_API TArray<FName> RetrieveTargetLayerNamesFromMaterials(bool bIncludeVisibilityLayer = true) const;
	
	// Query all the LandscapeLayerInfo Objects from the weight layers allocated on this proxy.
	TMap<FName, ULandscapeLayerInfoObject*> RetrieveTargetLayerInfosFromAllocations() const;

	/** Fix up component layers, weightmaps */
	void FixupWeightmaps();

	/** Repair invalid texture data that might have been introduced by a faulty version :  */
	void RepairInvalidTextures();

	// Remove Invalid weightmaps
	LANDSCAPE_API void RemoveInvalidWeightmaps();

	// Changed Physical Material
	LANDSCAPE_API void ChangedPhysMaterial();

	// Converts shared properties overridden before explicit override, asking the user how to proceed.
	LANDSCAPE_API void UpgradeSharedProperties(ALandscape* InParentLandscape, const bool bInExecutePostEditChangeProperty = true);

	// Displays the obsolete edit layers within the LandscapeComponents in a actionable MapCheck log
	// Asks users to copy obsolete layer data or delete for good.
 	void DisplayObsoleteEditLayerMapCheck(ALandscape* InParentLandscape);

	// Assign only mismatching data and mark proxy package dirty
	LANDSCAPE_API void FixupSharedData(ALandscape* Landscape, bool bMapCheck = false, const bool bInExecutePostEditChangeProperty = true);

	UE_DEPRECATED(5.7, "Use SetSectionBase instead.")
	LANDSCAPE_API void SetAbsoluteSectionBase(FIntPoint SectionOffset);

	/** Set landscape location in section space */
	LANDSCAPE_API void SetSectionBase(FIntPoint SectionOffset);

	/** Recreate all components rendering and collision states */
	LANDSCAPE_API void RecreateComponentsState();

	/** Recreate all component rendering states after applying a given function to each*/
	LANDSCAPE_API void RecreateComponentsRenderState(TFunctionRef<void(ULandscapeComponent*)> Fn);

	/** Recreate all collision components based on render component */
	LANDSCAPE_API void RecreateCollisionComponents();

	/** Remove all XYOffset values */
	LANDSCAPE_API void RemoveXYOffsets();

	/** Ensure the Update the Materials on the Nanite Static Meshes from the source ULandscapeComponent */
	void UpdateNaniteMaterials();
	
	/** Update the material instances for all the landscape components */
	LANDSCAPE_API void UpdateAllComponentMaterialInstances(bool bInInvalidateCombinationMaterials = false);
	LANDSCAPE_API void UpdateAllComponentMaterialInstances(FMaterialUpdateContext& InOutMaterialContext, TArray<class FComponentRecreateRenderStateContext>& InOutRecreateRenderStateContext, bool bInInvalidateCombinationMaterials = false);

	/** Import the given Height/Weight data into this landscape */
	LANDSCAPE_API void Import(const FGuid& InGuid, int32 InMinX, int32 InMinY, int32 InMaxX, int32 InMaxY, int32 InNumSubsections, int32 InSubsectionSizeQuads, const TMap<FGuid, TArray<uint16>>& InImportHeightData,
		const TCHAR* const InHeightmapFileName, const TMap<FGuid, TArray<FLandscapeImportLayerInfo>>& InImportMaterialLayerInfos, ELandscapeImportAlphamapType InImportMaterialLayerType,
		const TArrayView<const struct FLandscapeLayer>& InImportLayers);

	struct FRawMeshExportParams
	{
		static constexpr int32 MaxUVCount = 6;

		/** Describes what information the export will write in a given UV channel */
		enum class EUVMappingType : uint8
		{
			None, /** Don't export UV */
			//RelativeToBoundsUV, /** Only valid when ExportBounds is set : normalized UVs spanning the export bounds, i.e. (0,0) at the bottom left corner of ExportBounds -> (1,1) at the top right corner of ExportBounds. */
			// TODO [jonathan.bard] : RelativeToComponentsBoundsUV, /** Only valid when ComponentsToExport is set : normalized UVs spanning the ComponentToExport's bounds, i.e. (0,0) at the bottom left corner of those components' lower left component -> (1,1) at the top right corner of those components' upper right component */
			RelativeToProxyBoundsUV, /** Normalized UVs spanning the landscape proxy's bounds, i.e. (0,0) at the bottom left corner of the proxy's lower left landscape component -> (1,1) at the top right corner of the proxy's upper right component */
			// TODO[jonathan.bard] : RelativeToLandscapeBoundsUV, /** Normalized UVs spanning the entire landscape bounds, i.e. (0,0) at the bottom left corner of the landscape's lower left landscape component -> (1,1) at the top right corner of the landscape's upper right component */
			HeightmapUV,			/** Export the heightmaps' UV mapping */
			WeightmapUV,			/** Export the weightmaps' UV mapping */
			TerrainCoordMapping_XY, /** Similar to ETerrainCoordMappingType::TCMT_XY */
			TerrainCoordMapping_XZ, /** Similar to ETerrainCoordMappingType::TCMT_XZ */
			TerrainCoordMapping_YZ, /** Similar to ETerrainCoordMappingType::TCMT_YZ */
			LightmapUV,				/** 0-1, with a bit of padding, across the mesh */

			Num
		};

		/** Describes what to export on each UV channel */
		struct FUVConfiguration
		{
			LANDSCAPE_API FUVConfiguration();
			LANDSCAPE_API int32 GetNumUVChannelsNeeded() const;

		public:
			// Index 0 = UVChannel 0, Index 1 = UVChannel 1... 
			TArray<EUVMappingType> ExportUVMappingTypes; 
		};

		enum class EExportCoordinatesType : uint8
		{
			Absolute,
			RelativeToProxy,
			// TODO [jonathan.bard] : RelativeToComponentsBounds,
			// TODO [jonathan.bard] : RelativeToProxyBounds,
			// TODO [jonathan.bard] : RelativeToLandscapeBounds,
		};

	public:
		FRawMeshExportParams() = default;
		LANDSCAPE_API const FUVConfiguration& GetUVConfiguration(int32 InComponentIndex) const;
		LANDSCAPE_API const FName& GetMaterialSlotName(int32 InComponentIndex) const;
		LANDSCAPE_API int32 GetNumUVChannelsNeeded() const;

	public:
		/** LOD level to export. If none specified, LOD 0 will be used */
		int32 ExportLOD = INDEX_NONE;

		/** Describes what each UV channel should contain. */
		FUVConfiguration UVConfiguration;

		/** Referential for the vertex coordinates. */
		EExportCoordinatesType ExportCoordinatesType = EExportCoordinatesType::Absolute;

		/** Name of the default polygon group's material slot in the mesh. */
		FName MaterialSlotName = "LandscapeMat";

		/** Box/Sphere bounds which limits the geometry exported out into OutRawMesh (optional: if none specified, the entire mesh is exported) */
		TOptional<FBoxSphereBounds> ExportBounds;

		/** Depth of a one quad skirt to generate around the Proxy*/
		TOptional<float> SkirtDepth;

		/** List of components from the proxy to actually export (optional : if none specified, all landscape components will be exported) */
		TOptional<TArrayView<ULandscapeComponent*>> ComponentsToExport;

		/** Per-component UV configuration, in case ComponentsToExport is specified (optional: if none specified, all UV channels will contain the same information, as specified by UVConfiguration + there must be as many elements as there are in ComponentsToExport*/
		TOptional<TArrayView<FUVConfiguration>> ComponentsUVConfiguration;

		/** Per-component material slot name (optional : if specified, one polygon group per component will be assigned the corresponding material slot's name, otherwise, MaterialSlotName will be used. */
		TOptional<TArrayView<FName>> ComponentsMaterialSlotName;
	};

	/**
	* Exports landscape geometry into a raw mesh according to the export params
	*
	* @param InExportParams - Details about what should be exported and how
	* @param OutRawMesh - Resulting raw mesh
	* @return true if successful
	*/
	LANDSCAPE_API bool ExportToRawMesh(const FRawMeshExportParams& InExportParams, FMeshDescription& OutRawMesh) const;

	LANDSCAPE_API TSharedRef<UE::Landscape::Nanite::FAsyncBuildData> MakeAsyncNaniteBuildData(int32 InLODToExport, const TArray<ULandscapeComponent*>& ComponentsToExport) const;

	bool ExportToRawMeshDataCopy(const FRawMeshExportParams& InExportParams, FMeshDescription& OutRawMesh, const UE::Landscape::Nanite::FAsyncBuildData& AsyncData) const;

	LANDSCAPE_API void CheckGenerateMobilePlatformData(bool bIsCooking, const ITargetPlatform* TargetPlatform);

	/** Returns true if the Nanite representation is missing or outdated */
	LANDSCAPE_API bool IsNaniteMeshUpToDate() const;

	/** Update Nanite representation if it's missing or outdated. 
	  *  Important note : despite what the name might imply when compared with UpdateNaniteRepresentationAsync, this is not necessarily a synchronous call. If waiting for completion
	  *  is absolutely needed, it should be performed by ULandscapeSubsystem::FinishAllNaniteBuildsInFlightNow.
	  */
	LANDSCAPE_API void UpdateNaniteRepresentation(const ITargetPlatform* InTargetPlatform);

	/** Update Nanite representation if it's missing or outdated Async */
	FGraphEventRef UpdateNaniteRepresentationAsync(const ITargetPlatform* InTargetPlatform);

	/** 
	* Invalidate and disable Nanite representation until a subsequent rebuild occurs
	*
	* @param bInCheckContentId - If true, only invalidate when the content Id of the proxy mismatches with the Nanite representation
	*/
	LANDSCAPE_API void InvalidateNaniteRepresentation(bool bInCheckContentId);
	
	/**
	* Invalidate Nanite representation or rebuild it in case live update is active :
	*
	* @param bInCheckContentId - If true, only invalidate when the content Id of the proxy mismatches with the Nanite representation
	*/
	LANDSCAPE_API void InvalidateOrUpdateNaniteRepresentation(bool bInCheckContentId, const ITargetPlatform* InTargetPlatform);

	/** @return Current size of bounding rectangle in quads space */
	LANDSCAPE_API FIntRect GetBoundingRect() const;

	/** Creates a Texture2D for use by this landscape proxy or one of its components. If OptionalOverrideOuter is not specified, the proxy is used. */
	LANDSCAPE_API UTexture2D* CreateLandscapeTexture(int32 InSizeX, int32 InSizeY, TextureGroup InLODGroup, ETextureSourceFormat InFormat, UObject* OptionalOverrideOuter = nullptr, bool bCompress = false, bool bMipChain = true) const;

	/** Creates a Texture2DArray for use by this landscape proxy or one of its components. If OptionalOverrideOuter is not specified, the proxy is used. */
	LANDSCAPE_API UTexture2DArray* CreateLandscapeTextureArray(int32 InSizeX, int32 InSizeY, int32 Slices, TextureGroup InLODGroup, ETextureSourceFormat InFormat, UObject* OptionalOverrideOuter = nullptr);
	
	/** Creates a Texture2D for use by this landscape proxy or one of its components for tools .*/ 
	LANDSCAPE_API UTexture2D* CreateLandscapeToolTexture(int32 InSizeX, int32 InSizeY, TextureGroup InLODGroup, ETextureSourceFormat InFormat) const;

	/** Creates a LandscapeWeightMapUsage object outered to this proxy. */
	LANDSCAPE_API ULandscapeWeightmapUsage* CreateWeightmapUsage();

	/** remove an overlapping component. Called from MapCheck. */
	LANDSCAPE_API void RemoveOverlappingComponent(ULandscapeComponent* Component);

	/**
	* Samples an array of values from a Texture Render Target 2D. 
	* Only works in the editor
	*/
	LANDSCAPE_API static TArray<FLinearColor> SampleRTData(UTextureRenderTarget2D* InRenderTarget, FLinearColor InRect);

	/**
	* Overwrites a landscape heightmap with render target data
	* @param InRenderTarget - Valid render target with a format of RTF_RGBA16f, RTF_RGBA32f or RTF_RGBA8
	* @param InImportHeightFromRGChannel - Only relevant when using format RTF_RGBA16f or RTF_RGBA32f, and will tell us if we should import the height data from the R channel only of the Render target or from R & G. 
	*									   Note that using RTF_RGBA16f with InImportHeightFromRGChannel == false, could have precision loss
	* Only works in the editor
	*/
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Landscape Import Heightmap from RenderTarget", Keywords = "Push RenderTarget to Landscape Heightmap", UnsafeDuringActorConstruction = "true"), Category = Rendering)
	bool LandscapeImportHeightmapFromRenderTarget(UTextureRenderTarget2D* InRenderTarget, bool InImportHeightFromRGChannel = false, int32 InEditLayerIndex = 0);

	/**
	* Overwrites a landscape weightmap with render target data
	* Only works in the editor
	*/
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Landscape Import Weightmap from RenderTarget", Keywords = "Push RenderTarget to Landscape Weightmap", UnsafeDuringActorConstruction = "true"), Category = Rendering)
	bool LandscapeImportWeightmapFromRenderTarget(UTextureRenderTarget2D* InRenderTarget, FName InLayerName, int32 InEditLayerIndex = 0);

	UE_DEPRECATED(5.7, "Deprecating placeholder function. Not needed after non-edit layer landscape deprecation. ")
	UFUNCTION()
	bool LandscapeExportWeightmapToRenderTarget(UTextureRenderTarget2D* InRenderTarget, FName InLayerName);

	UE_DEPRECATED(5.7, "Non-edit layer landscapes are deprecated, all landscapes use the edit layer system now.")
	LANDSCAPE_API virtual bool HasLayersContent() const;

	UE_DEPRECATED(5.7, "Non-edit layer landscapes are deprecated, all landscapes use the edit layer system now.")
	LANDSCAPE_API bool CanHaveLayersContent() const;

	UE_DEPRECATED(5.7, "Non-edit layer landscapes are deprecated, all landscapes use the edit layer system now.")
	LANDSCAPE_API virtual void UpdateCachedHasLayersContent(bool InCheckComponentDataIntegrity = false);

	FAsyncWorkMonitor& GetAsyncWorkMonitor() { return AsyncWorkMonitor; }

	/**
	 * Delete all unused layers in components. Warning: any update of the component could re-introduce them.
	 */
	UFUNCTION(BlueprintCallable, Category = "Landscape")
	LANDSCAPE_API virtual void DeleteUnusedLayers();
	
	// TargetLayer API Start
	// ---------------------
	
	// Add a TargetLayer with a unique name and no layer object info
	LANDSCAPE_API FLandscapeTargetLayerSettings& AddTargetLayer();

	// Add a TargetLayer with the given name or a generated unique name if the given name is invalid
	LANDSCAPE_API FLandscapeTargetLayerSettings& AddTargetLayer(const FName& Name, const FLandscapeTargetLayerSettings& TargetLayerSettings, bool bPostEditChange = true);

	// Generate a unique TargetLayer name
	LANDSCAPE_API const FName GenerateUniqueTargetLayerName() const;

	// Update a existing Target Layer
	LANDSCAPE_API bool UpdateTargetLayer(const FName& Name, const FLandscapeTargetLayerSettings& TargetLayerSettings, bool bPostEditChange = true);

	// Remove Named TargetLayer,
	// Returns true if a layer has been removed.
	LANDSCAPE_API bool RemoveTargetLayer(const FName& Name, bool bPostEditChange = true);

	// Query if a TargetLayer exists by Name or by checking the ULandscapeLayerInfoObject in  TargetLayerSettings
	LANDSCAPE_API bool HasTargetLayer(const FName& Name) const;
	LANDSCAPE_API bool HasTargetLayer(const FLandscapeTargetLayerSettings& TargetLayerSettings) const;
	LANDSCAPE_API bool HasTargetLayer(const ULandscapeLayerInfoObject* LayerInfoObject) const;

	// All the Target layers 
	LANDSCAPE_API const TMap<FName, FLandscapeTargetLayerSettings>& GetTargetLayers() const;
	LANDSCAPE_API TSet<ULandscapeLayerInfoObject*> GetValidTargetLayerObjects() const;

	// TargetLayer API End
	// -------------------
protected:
	friend class ALandscape;

	/** Add Layer if it doesn't exist yet.
	* @return True if layer was added.
	*/
	LANDSCAPE_API bool AddLayer(const FGuid& InLayerGuid);

	/** Delete Layer.
	*/
	LANDSCAPE_API void DeleteLayer(const FGuid& InLayerGuid);

	/** Removes Obsolete Edit layer data from all landscape components
	* @param bShowMapCheckWarning When true, opens MapCheck to display a log for each edit layer removed
	* @return True if some layers were removed.
	*/
	LANDSCAPE_API bool RemoveObsoleteEditLayers(bool bShowMapCheckWarning);

	UE_DEPRECATED(5.7, "Use RemoveObsoleteEditLayers based on ALandscape or use DeleteLayer to directly remove layers")
	LANDSCAPE_API bool RemoveObsoleteLayers(const TSet<FGuid>& InExistingLayers);

	// Returns a map of this Proxy's LandscapeComponents obsolete edit layers Guids to Debug Names
	TMap<FGuid, FName> GetObsoleteEditLayersInComponents();

	/** Initialize Layer with empty content if it hasn't been initialized yet. */
	LANDSCAPE_API void InitializeLayerWithEmptyContent(const FGuid& InLayerGuid);

	/** Fixup any internal representation for shared properties. Will be used when deprecating or renaming a property. */
	virtual void FixupOverriddenSharedProperties() {}

	/** Validates that overridden shared properties are properly setup. Only run on load and MapCheck since it's just to inform the user that something is not properly setup */
	virtual void CheckOverriddenSharedProperties() {}

	/**
	* Synchronizes all shared properties that cannot be marked by regular meta tags.
	* @param The landscape to use as a template.
	* @return An array containing all synchronized properties.
	*/
	LANDSCAPE_API virtual TArray<FName> SynchronizeUnmarkedSharedProperties(ALandscapeProxy* InLandscape);


protected:
	// Used to know if the deprecation of shared properties modified before the enforcement system introduction has been performed on load.
	bool bUpgradeSharedPropertiesPerformed = false;
#endif // WITH_EDITOR
private:
	
	UPROPERTY(Category = "Target Layers", VisibleAnywhere, meta = (NoResetToDefault, LandscapeInherited) )
	TMap<FName, FLandscapeTargetLayerSettings> TargetLayers;
	
	/** Returns Grass Update interval */
	FORCEINLINE int32 GetGrassUpdateInterval() const 
	{
#if WITH_EDITOR
		// When editing landscape, force update interval to be every frame
		if (GLandscapeEditModeActive)
		{
			return 1;
		}
#endif
		return GrassUpdateInterval;
	}
	static int32 GrassUpdateInterval;

	FName GenerateUniqueLandscapeTextureName(UObject* InOuter, TextureGroup InLODGroup) const;

#if WITH_EDITOR
	/** Create Blank Nanite Component */
	void CreateNaniteComponents(int32 NumComponents);

	/**
	* Sets the source landscape components for each nanite component in this proxy
	* @param bSetEmptyComponentsOnly When true, only update nanite components with no source landscape components
	*/
	void SetSourceComponentsForNaniteComponents(bool bSetEmptyComponentsOnly = false) const;

	void OnLayerInfoObjectDataChanged(const FOnLandscapeLayerInfoDataChangedParams& InParams);
#endif

	void PostLoadFixupLandscapeGuidsIfInstanced();

#if WITH_EDITOR
	void InstallOrUpdateTextureUserDatas(const ITargetPlatform* TargetPlatform);
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
public:
	static const TArray<ALandscapeProxy*>& GetLandscapeProxies() { return LandscapeProxies; }

	inline bool HasEditLayersDataOnLoad() const
	{
		return bHasEditLayersOnLoad;
	}

private:
	/** Maintain list of Proxies for faster iteration */
	static TArray<ALandscapeProxy*> LandscapeProxies;
	FAsyncWorkMonitor AsyncWorkMonitor;
#endif // WITH_EDITORONLY_DATA

	/** Offset in quads from global components grid origin (in quads) **/
	UPROPERTY()
	FIntPoint SectionBase = FIntPoint::ZeroValue;

	/** True if this proxy's first component has edit layer data on load. When false, proxy has no components or is part of a legacy non-edit layer landscape and needs to be auto-converted */
	bool bHasEditLayersOnLoad = false;

#if WITH_EDITOR
	static constexpr TCHAR const* LandscapeInheritedTag = TEXT("LandscapeInherited");
	static constexpr TCHAR const* LandscapeOverridableTag = TEXT("LandscapeOverridable");
#endif // WITH_EDITOR
};


#if WITH_EDITOR
/**
 * Helper class used to Build or monitor Landscape Physical Material
 */
class FLandscapePhysicalMaterialBuilder
{
public:
	LANDSCAPE_API FLandscapePhysicalMaterialBuilder(UWorld* InWorld);
	LANDSCAPE_API void Build(UE::Landscape::EBuildFlags InBuildFlags);
	LANDSCAPE_API int32 GetOudatedPhysicalMaterialComponentsCount();
private:
	UWorld* World;
	int32 OudatedPhysicalMaterialComponentsCount;
};

/**
* Helper class to store proxy changes information 
*/
class FLandscapeProxyComponentDataChangedParams
{
public:
	LANDSCAPE_API FLandscapeProxyComponentDataChangedParams(const TSet<ULandscapeComponent*>& InComponents);
	LANDSCAPE_API void ForEachComponent(TFunctionRef<void(const ULandscapeComponent*)> Func) const;
	const TArray<ULandscapeComponent*>& GetComponents() const { return Components; }

private:
	TArray<ULandscapeComponent*> Components;
};

DEFINE_ACTORDESC_TYPE(ALandscapeProxy, FLandscapeActorDesc);
#endif
