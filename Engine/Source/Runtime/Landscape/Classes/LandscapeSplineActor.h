// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "ILandscapeSplineInterface.h"
#include "LandscapeSplineActor.generated.h"

class ALandscape;

UCLASS(MinimalAPI, NotBlueprintable, NotPlaceable, hidecategories = (Display, Attachment, Physics, Debug, Lighting, Input))
class ALandscapeSplineActor : public AActor, public ILandscapeSplineInterface
{
	GENERATED_UCLASS_BODY()

protected:
	/** Guid for LandscapeInfo **/
	UPROPERTY()
	FGuid LandscapeGuid;

#if WITH_EDITORONLY_DATA
	/** Landscape **/
	UPROPERTY(EditAnywhere, Category = "Target Landscape")
	TObjectPtr<ALandscape> LandscapeActor;
#endif
	
public:
	virtual FGuid GetLandscapeGuid() const override { return LandscapeGuid; }
	virtual ULandscapeSplinesComponent* GetSplinesComponent() const override;
	virtual FTransform LandscapeActorToWorld() const override;
	virtual ULandscapeInfo* GetLandscapeInfo() const override;
	virtual void UpdateSharedProperties(ULandscapeInfo* InLandscapeInfo) override;
	virtual bool IsSplineOwnerValid() const override;
		
#if WITH_EDITOR
	LANDSCAPE_API bool ShouldPartitionSpline() const;
	LANDSCAPE_API bool HasGeneratedLandscapeSplineMeshesActors() const;
	void SetLandscapeGuid(const FGuid& InGuid);
	virtual void GetActorDescProperties(FPropertyPairsMap& PropertyPairsMap) const override;

	void GetSharedProperties(ULandscapeInfo* InLandscapeInfo);

	virtual void Destroyed() override;
	virtual void PostRegisterAllComponents() override;
	virtual void UnregisterAllComponents(bool bForReregister = false) override;
	virtual void PostEditMove(bool bFinished) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual bool EditorCanAttachTo(const AActor* InParent, FText& OutReason) const override { return false; }
	virtual AActor* GetSceneOutlinerParent() const;
	virtual bool SupportsForeignSplineMesh() const override { return false; }

	// Interface existes for backward compatibility. Should already be created since its this actor's root component.
	virtual void CreateSplineComponent() override { check(false); }
	virtual void CreateSplineComponent(const FVector& Scale3D) override { check(false); }

private:
	LANDSCAPE_API void SetLandscapeActor(ALandscape* InLandscapeActor);
	friend class UWorldPartitionLandscapeSplineMeshesBuilder;
#endif
};
