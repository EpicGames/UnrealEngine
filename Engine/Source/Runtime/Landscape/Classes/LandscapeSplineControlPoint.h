// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// for FLandscapeSplineInterpPoint

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "LandscapeSplineSegment.h"

#include "LandscapeSplineControlPoint.generated.h"

class UControlPointMeshComponent;
class ULandscapeSplinesComponent;
class UStaticMesh;

USTRUCT()
struct FLandscapeSplineConnection
{
	GENERATED_USTRUCT_BODY()

	FLandscapeSplineConnection() : Segment(nullptr), End(0) {}

	FLandscapeSplineConnection(ULandscapeSplineSegment* InSegment, int32 InEnd)
		: Segment(InSegment)
		, End(InEnd)
	{
	}

	bool operator == (const FLandscapeSplineConnection& rhs) const
	{
		return Segment == rhs.Segment && End == rhs.End;
	}

	// Segment connected to this control point
	UPROPERTY()
	TObjectPtr<ULandscapeSplineSegment> Segment;

	// Which end of the segment is connected to this control point
	UPROPERTY()
	uint32 End:1;

	LANDSCAPE_API FLandscapeSplineSegmentConnection& GetNearConnection() const;
	LANDSCAPE_API FLandscapeSplineSegmentConnection& GetFarConnection() const;
};

UCLASS(Within=LandscapeSplinesComponent,autoExpandCategories=LandscapeSplineControlPoint,MinimalAPI)
class ULandscapeSplineControlPoint : public UObject
{
	GENERATED_UCLASS_BODY()

// Directly editable data:
	/** Location in Landscape-space */
	UPROPERTY(EditAnywhere, Category=LandscapeSpline)
	FVector Location;

	/** Rotation of tangent vector at this point (in landscape-space) */
	UPROPERTY(EditAnywhere, Category=LandscapeSpline)
	FRotator Rotation;

	/** Half-Width of the spline at this point. */
	UPROPERTY(EditAnywhere, Category=LandscapeSpline, meta = (DisplayName = "Half-Width"))
	float Width;

	/** Layer Width ratio of the spline at this point. */
	UPROPERTY(EditAnywhere, Category = LandscapeSpline)
	float LayerWidthRatio = 1.f;

	/** Falloff at the sides of the spline at this point. */
	UPROPERTY(EditAnywhere, Category=LandscapeSpline)
	float SideFalloff;

	UPROPERTY(EditAnywhere, Category = LandscapeSpline, meta=(UIMin = 0, ClampMin = 0, UIMax = 1, ClampMax = 1))
	float LeftSideFalloffFactor = 1.f;

	UPROPERTY(EditAnywhere, Category = LandscapeSpline, meta = (UIMin = 0, ClampMin = 0, UIMax = 1, ClampMax = 1))
	float RightSideFalloffFactor = 1.f;

	UPROPERTY(EditAnywhere, Category = LandscapeSpline, meta = (UIMin = 0, ClampMin = 0, UIMax = 1, ClampMax = 1))
	float LeftSideLayerFalloffFactor = 0.5f;

	UPROPERTY(EditAnywhere, Category = LandscapeSpline, meta = (UIMin = 0, ClampMin = 0, UIMax = 1, ClampMax = 1))
	float RightSideLayerFalloffFactor = 0.5f;

	/** Falloff at the start/end of the spline (if this point is a start or end point, otherwise ignored). */
	UPROPERTY(EditAnywhere, Category=LandscapeSpline)
	float EndFalloff;

#if WITH_EDITORONLY_DATA
	/** Vertical offset of the spline segment mesh. Useful for a river's surface, among other things. */
	UPROPERTY(EditAnywhere, Category=LandscapeSpline, meta=(DisplayName="Mesh Vertical Offset"))
	float SegmentMeshOffset;

	/**
	 * Name of blend layer to paint when applying spline to landscape
	 * If "none", no layer is painted
	 */
	UPROPERTY(EditAnywhere, Category=LandscapeDeformation)
	FName LayerName;

	/** If the spline is above the terrain, whether to raise the terrain up to the level of the spline when applying it to the landscape. */
	UPROPERTY(EditAnywhere, Category=LandscapeDeformation)
	uint32 bRaiseTerrain:1;

	/** If the spline is below the terrain, whether to lower the terrain down to the level of the spline when applying it to the landscape. */
	UPROPERTY(EditAnywhere, Category=LandscapeDeformation)
	uint32 bLowerTerrain:1;

	/** Mesh to use on the control point */
	UPROPERTY(EditAnywhere, Category=Mesh)
	TObjectPtr<UStaticMesh> Mesh;

	/** Overrides mesh's materials */
	UPROPERTY(EditAnywhere, Category=Mesh)
	TArray<TObjectPtr<UMaterialInterface>> MaterialOverrides;

	/** Scale of the control point mesh */
	UPROPERTY(EditAnywhere, Category=Mesh)
	FVector MeshScale;

	UPROPERTY()
	uint32 bEnableCollision_DEPRECATED:1;

	UPROPERTY()
	FName CollisionProfileName_DEPRECATED;

	/** Whether the Control Point Mesh should cast a shadow. */
	UPROPERTY(EditAnywhere, Category=Mesh)
	uint32 bCastShadow:1;

	/** Whether to hide the mesh in game */
	UPROPERTY(EditAnywhere, Category = Mesh, AdvancedDisplay)
	uint8 bHiddenInGame : 1;

	/** Whether control point mesh should be placed in landscape proxy streaming level (true) or the spline's level (false) */
	UPROPERTY(EditAnywhere, Category = Mesh, AdvancedDisplay)
	uint32 bPlaceSplineMeshesInStreamingLevels : 1;

	/**  Max draw distance for the mesh used on this control point */
	UPROPERTY(EditAnywhere, Category=Mesh, AdvancedDisplay, meta=(DisplayName="Max Draw Distance"))
	float LDMaxDrawDistance;

	/**
	 * Translucent objects with a lower sort priority draw behind objects with a higher priority.
	 * Translucent objects with the same priority are rendered from back-to-front based on their bounds origin.
	 * This setting is also used to sort objects being drawn into a runtime virtual texture.
	 *
	 * Ignored if the object is not translucent.  The default priority is zero.
	 * Warning: This should never be set to a non-default value unless you know what you are doing, as it will prevent the renderer from sorting correctly.
	 */
	UPROPERTY(EditAnywhere, Category=Mesh, AdvancedDisplay)
	int32 TranslucencySortPriority;

	/** If true, this component will be rendered in the CustomDepth pass (usually used for outlines) */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Mesh, meta = (DisplayName = "Render CustomDepth Pass"))
	uint8 bRenderCustomDepth : 1;

	/** Mask used for stencil buffer writes. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Mesh, meta = (editcondition = "bRenderCustomDepth"))
	ERendererStencilMask CustomDepthStencilWriteMask;

	/** Optionally write this 0-255 value to the stencil buffer in CustomDepth pass (Requires project setting or r.CustomDepth == 3) */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Mesh, meta = (UIMin = "0", UIMax = "255", editcondition = "bRenderCustomDepth", DisplayName = "CustomDepth Stencil Value"))
	int32 CustomDepthStencilValue;

	/** 
	 * Array of runtime virtual textures into which we draw the spline segment. 
	 * The material also needs to be set up to output to a virtual texture. 
	 */
	UPROPERTY(EditAnywhere, Category = VirtualTexture, meta = (DisplayName = "Draw in Virtual Textures"))
	TArray<TObjectPtr<URuntimeVirtualTexture>> RuntimeVirtualTextures;

	/** Lod bias for rendering to runtime virtual texture. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = VirtualTexture, meta = (DisplayName = "Virtual Texture LOD Bias", UIMin = "-7", UIMax = "8"))
	int32 VirtualTextureLodBias = 0;

	/**
	 * Number of lower mips in the runtime virtual texture to skip for rendering this primitive.
	 * Larger values reduce the effective draw distance in the runtime virtual texture.
	 * This culling method doesn't take into account primitive size or virtual texture size.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay,  Category = VirtualTexture, meta = (DisplayName = "Virtual Texture Skip Mips", UIMin = "0", UIMax = "7"))
	int32 VirtualTextureCullMips = 0;

	/** Desired cull distance in the main pass if we are rendering to both the virtual texture AND the main pass. A value of 0 has no effect. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = VirtualTexture, meta = (DisplayName = "Max Draw Distance in Main Pass"))
	float VirtualTextureMainPassMaxDrawDistance = 0.f;

	/** Controls if this component draws in the main pass as well as in the virtual texture. */
	UPROPERTY(EditAnywhere, Category = VirtualTexture, meta = (DisplayName = "Draw in Main Pass"))
	ERuntimeVirtualTextureMainPassType VirtualTextureRenderPassType = ERuntimeVirtualTextureMainPassType::Exclusive;

	/** Mesh Collision Settings */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Collision, meta = (ShowOnlyInnerProperties))
	FBodyInstance BodyInstance;

	/** Custom HLOD layer to use for this spline control point. Only relevant when using World Partition with spline partitioning. */
	UPROPERTY(EditAnywhere, Category = HLOD, AdvancedDisplay)
	TObjectPtr<UHLODLayer> HLODLayerOverrideForSplinePartitioning;

protected:
	UPROPERTY(Transient)
	uint32 bSelected : 1;

	UPROPERTY(Transient)
	uint32 bNavDirty : 1;
#endif

// Procedural data:
public:
	UPROPERTY(TextExportTransient)
	TArray<FLandscapeSplineConnection> ConnectedSegments;

protected:
	/** Spline points */
	UPROPERTY()
	TArray<FLandscapeSplineInterpPoint> Points;

	/** Bounds of points */
	UPROPERTY()
	FBox Bounds;

	/** Control point mesh */
	UPROPERTY(TextExportTransient)
	TObjectPtr<UControlPointMeshComponent> LocalMeshComponent;

#if WITH_EDITORONLY_DATA
	/** World reference for if mesh component is stored in another streaming level */
	UPROPERTY(TextExportTransient, NonPIEDuplicateTransient)
	TSoftObjectPtr<UWorld> ForeignWorld;

	/** Key for tracking whether this segment has been modified relative to the mesh component stored in another streaming level */
	UPROPERTY(TextExportTransient, NonPIEDuplicateTransient)
	FGuid ModificationKey;
#endif

public:
	const FBox& GetBounds() const { return Bounds; }
	const TArray<FLandscapeSplineInterpPoint>& GetPoints() const { return Points; }

#if WITH_EDITOR
	bool SupportsForeignSplineMesh() const;

	// Get the name of the best connection point (socket) to use for a particular destination
	virtual FName GetBestConnectionTo(FVector Destination) const;

	// Get the location and rotation of a connection point (socket) in spline space
	virtual void GetConnectionLocalLocationAndRotation(FName SocketName, OUT FVector& OutLocation, OUT FRotator& OutRotation) const;

	// Get the location and rotation of a connection point (socket) in spline space
	virtual void GetConnectionLocationAndRotation(FName SocketName, OUT FVector& OutLocation, OUT FRotator& OutRotation) const;

	bool IsSplineSelected() const { return bSelected; }
	virtual void SetSplineSelected(bool bInSelected);

	/** Calculates rotation from connected segments */
	virtual void AutoCalcRotation(bool bAlwaysRotateForward);

	/**  */
	virtual void AutoFlipTangents();

	virtual void AutoSetConnections(bool bIncludingValid);

	LANDSCAPE_API TMap<ULandscapeSplinesComponent*, UControlPointMeshComponent*> GetForeignMeshComponents();

	/** Update spline points */
	virtual void UpdateSplinePoints(bool bUpdateCollision = true, bool bUpdateAttachedSegments = true, bool bUpdateMeshLevel = false);

	/** Update spline mesh selection state */
	virtual void UpdateSplineMeshSelectionState();

	/** Delete spline points */
	virtual void DeleteSplinePoints();

	const TSoftObjectPtr<UWorld>& GetForeignWorld() const { return ForeignWorld; }
	FGuid GetModificationKey() const { return ModificationKey; }

	LANDSCAPE_API FName GetCollisionProfileName() const;
#endif // WITH_EDITOR

	//~ Begin UObject Interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditUndo() override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	virtual void PostEditImport() override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual bool IsCapturingAsRootObjectForTransaction() const override { return true; }
	//~ End UObject Interface
#endif // WITH_EDITOR

	/** Use GetOuterSafe instead of GetOuterULandscapeSplinesComponent if there is any danger that this point has been deleted (via DeleteSplinePoints).
	  * This will return nullptr in that case, because the outer is no longer a ULandscapeSplinesComponent */
	LANDSCAPE_API ULandscapeSplinesComponent* GetOuterSafe() const;
	/** Use GetOuterChecked when the call is always expected to succeed.  Will assert if it fails. */
	LANDSCAPE_API ULandscapeSplinesComponent* GetOuterChecked() const;

	friend class FLandscapeToolSplines;
	friend class ULandscapeInfo;
};
