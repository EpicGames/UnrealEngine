// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorPartition/PartitionActor.h"
#include "LandscapeSplineMeshesActor.generated.h"

//
// This class is only intended to be used by UWorldPartitionLandscapeSplineMeshesBuilder which extracts and clones
// landscape spline and control point static meshes into partitioned actors
//
// This serves as an optimization as these actors will be streamed at runtime
//
UCLASS(NotPlaceable, NotBlueprintable, hideCategories = (Input), showCategories = ("Input|MouseInput", "Input|TouchInput"), ConversionRoot, ComponentWrapperClass, meta = (ChildCanTick), MinimalAPI)
class ALandscapeSplineMeshesActor : public APartitionActor
{
	GENERATED_UCLASS_BODY()

private:
	UPROPERTY(Category = PartitionSplineMeshActor, VisibleAnywhere, BlueprintReadOnly, meta = (ExposeFunctionCategories = "Mesh,Rendering,Physics,Components|StaticMesh,Components|SplineMesh", AllowPrivateAccess = "true"))
	TArray<TObjectPtr<class UStaticMeshComponent>> StaticMeshComponents;

public:

#if WITH_EDITOR
	LANDSCAPE_API class UStaticMeshComponent* CreateStaticMeshComponent(const TSubclassOf<class UStaticMeshComponent>& InComponentClass);
	LANDSCAPE_API void ClearStaticMeshComponents();

	//~ Begin AActor Interface
	LANDSCAPE_API virtual void CheckForErrors() override;
	LANDSCAPE_API virtual bool GetReferencedContentObjects(TArray<UObject*>& Objects) const override;
	virtual bool IsRuntimeOnly() const override { return true; }
	//~ End AActor Interface

	//~ Begin APartitionActor Interface
	LANDSCAPE_API virtual uint32 GetDefaultGridSize(UWorld* InWorld) const override;
	virtual FGuid GetGridGuid() const override { return GridGuid; }
	//~ End APartitionActor Interface

	void SetGridGuid(const FGuid& InGuid) { GridGuid = InGuid; }
#endif // WITH_EDITOR

	LANDSCAPE_API const TArray<class UStaticMeshComponent*>& GetStaticMeshComponents() const;

protected:
	//~ Begin UObject Interface.
	LANDSCAPE_API virtual FString GetDetailedInfoInternal() const override;
	//~ End UObject Interface.

private:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FGuid GridGuid;
#endif
};
