// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "Components/SplineMeshComponent.h"
#include "LandscapeSplinesComponent.h"
#include "VT/RuntimeVirtualTextureEnum.h"
#include "LandscapeSplineSegment.generated.h"

class ULandscapeSplineControlPoint;
class UStaticMesh;

//Forward declarations
class ULandscapeSplineControlPoint;
class ULandscapeSplinesComponent;

USTRUCT()
struct FLandscapeSplineInterpPoint
{
	GENERATED_USTRUCT_BODY()

	/** Center Point */
	UPROPERTY()
	FVector Center;

	/** Left Point */
	UPROPERTY()
	FVector Left;

	/** Right Point */
	UPROPERTY()
	FVector Right;

	/** Left Falloff Point */
	UPROPERTY()
	FVector FalloffLeft;

	/** Right FalloffPoint */
	UPROPERTY()
	FVector FalloffRight;

	/** Layer Left Point */
	UPROPERTY()
	FVector LayerLeft;

	/** Layer Right Point */
	UPROPERTY()
	FVector LayerRight;

	/** Left Layer Falloff Point */
	UPROPERTY()
	FVector LayerFalloffLeft;

	/** Right Layer FalloffPoint */
	UPROPERTY()
	FVector LayerFalloffRight;


	/** Start/End Falloff fraction */
	UPROPERTY()
	float StartEndFalloff;

	FLandscapeSplineInterpPoint()
		: Center(ForceInitToZero)
		, Left(ForceInitToZero)
		, Right(ForceInitToZero)
		, FalloffLeft(ForceInitToZero)
		, FalloffRight(ForceInitToZero)
		, LayerLeft(ForceInitToZero)
		, LayerRight(ForceInitToZero)
		, LayerFalloffLeft(ForceInitToZero)
		, LayerFalloffRight(ForceInitToZero)
		, StartEndFalloff(0.0f)
	{
	}

	FLandscapeSplineInterpPoint(FVector InCenter, FVector InLeft, FVector InRight, FVector InFalloffLeft, FVector InFalloffRight, FVector InLayerLeft, FVector InLayerRight, FVector InLayerFalloffLeft, FVector InLayerFalloffRight, float InStartEndFalloff) :
		Center(InCenter),
		Left(InLeft),
		Right(InRight),
		FalloffLeft(InFalloffLeft),
		FalloffRight(InFalloffRight),
		LayerLeft(InLayerLeft),
		LayerRight(InLayerRight),
		LayerFalloffLeft(InLayerFalloffLeft),
		LayerFalloffRight(InLayerFalloffRight),
		StartEndFalloff(InStartEndFalloff)
	{
	}
};

USTRUCT()
struct FLandscapeSplineSegmentConnection
{
	GENERATED_USTRUCT_BODY()

	// Control point connected to this end of the segment
	UPROPERTY()
	TObjectPtr<ULandscapeSplineControlPoint> ControlPoint;

	// Tangent length of the connection
	UPROPERTY(EditAnywhere, Category=LandscapeSplineSegmentConnection)
	float TangentLen;

	// Socket on the control point that we are connected to
	UPROPERTY(EditAnywhere, Category=LandscapeSplineSegmentConnection)
	FName SocketName;

	FLandscapeSplineSegmentConnection()
		: ControlPoint(nullptr)
		, TangentLen(0.0f)
		, SocketName(NAME_None)
	{
	}
};

// Deprecated
UENUM()
enum LandscapeSplineMeshOrientation : int
{
	LSMO_XUp,
	LSMO_YUp,
	LSMO_MAX,
};

USTRUCT()
struct FLandscapeSplineMeshEntry
{
	GENERATED_USTRUCT_BODY()

	/** Mesh to use on the spline */
	UPROPERTY(EditAnywhere, Category=LandscapeSplineMeshEntry)
	TObjectPtr<UStaticMesh> Mesh;

	/** Overrides mesh's materials */
	UPROPERTY(EditAnywhere, Category=LandscapeSplineMeshEntry, AdvancedDisplay)
	TArray<TObjectPtr<UMaterialInterface>> MaterialOverrides;

	/** Whether to automatically center the mesh horizontally on the spline */
	UPROPERTY(EditAnywhere, Category=LandscapeSplineMeshEntry, meta=(DisplayName="Center Horizontally"))
	uint32 bCenterH:1;

	/** Tweak to center the mesh correctly on the spline */
	UPROPERTY(EditAnywhere, Category=LandscapeSplineMeshEntry, AdvancedDisplay, meta=(DisplayName="Center Adjust"))
	FVector2D CenterAdjust;

	/** Whether to scale the mesh to fit the width of the spline */
	UPROPERTY(EditAnywhere, Category=LandscapeSplineMeshEntry)
	uint32 bScaleToWidth:1;

	/** Disables scale to width on the mesh Z coordinate */
	UPROPERTY(EditAnywhere, Category = LandscapeSplineMeshEntry, meta = (EditCondition = "bScaleToWidth"))
	uint32 bNoZScaling:1;

	/** Scale of the spline mesh, (Z=Forwards) */
	UPROPERTY(EditAnywhere, Category=LandscapeSplineMeshEntry)
	FVector Scale;

	/** Orientation of the spline mesh, X=Up or Y=Up */
	UPROPERTY()
	TEnumAsByte<LandscapeSplineMeshOrientation> Orientation_DEPRECATED;

	/** Chooses the forward axis for the spline mesh orientation */
	UPROPERTY(EditAnywhere, Category=LandscapeSplineMeshEntry)
	TEnumAsByte<ESplineMeshAxis::Type> ForwardAxis;

	/** Chooses the up axis for the spline mesh orientation */
	UPROPERTY(EditAnywhere, Category=LandscapeSplineMeshEntry)
	TEnumAsByte<ESplineMeshAxis::Type> UpAxis;

	FLandscapeSplineMeshEntry() :
		Mesh(nullptr),
		MaterialOverrides(),
		bCenterH(true),
		CenterAdjust(0, 0),
		bScaleToWidth(true),
		bNoZScaling(false),
		Scale(1,1,1),
		Orientation_DEPRECATED(LSMO_YUp),
		ForwardAxis(ESplineMeshAxis::X),
		UpAxis(ESplineMeshAxis::Z)
	{
	}

	bool IsValid() const;
};


UCLASS(Within=LandscapeSplinesComponent,autoExpandCategories=(LandscapeSplineSegment,LandscapeSplineMeshes),MinimalAPI)
class ULandscapeSplineSegment : public UObject
{
	GENERATED_UCLASS_BODY()

// Directly editable data:
	UPROPERTY(EditAnywhere, EditFixedSize, Category=LandscapeSplineSegment)
	FLandscapeSplineSegmentConnection Connections[2];

#if WITH_EDITORONLY_DATA
	/**
	 * Name of blend layer to paint when applying spline to landscape
	 * If "none", no layer is painted
	 */
	UPROPERTY(EditAnywhere, Category=LandscapeDeformation)
	FName LayerName;

	/** If the spline is above the terrain, whether to raise the terrain up to the level of the spline when applying it to the landscape. */
	UPROPERTY(EditAnywhere, Category=LandscapeDeformation)
	uint32 bRaiseTerrain:1;

	/** If the spline is below the terrain, whether to lower the terrain down to the level of the spline when applying it to the landscape. */
	UPROPERTY(EditAnywhere, Category=LandscapeDeformation)
	uint32 bLowerTerrain:1;

	/** Spline meshes from this list are used in random order along the spline. */
	UPROPERTY(EditAnywhere, Category=LandscapeSplineMeshes)
	TArray<FLandscapeSplineMeshEntry> SplineMeshes;

	UPROPERTY()
	uint32 bEnableCollision_DEPRECATED:1;

	UPROPERTY()
	FName CollisionProfileName_DEPRECATED;

	/** Whether the Spline Meshes should cast a shadow. */
	UPROPERTY(EditAnywhere, Category=LandscapeSplineMeshes)
	uint32 bCastShadow:1;

	/** Whether to hide the mesh in game */
	UPROPERTY(EditAnywhere, Category = LandscapeSplineMeshes, AdvancedDisplay)
	uint32 bHiddenInGame : 1;

	/** Whether spline meshes should be placed in landscape proxy streaming levels (true) or the spline's level (false) */
	UPROPERTY(EditAnywhere, Category = LandscapeSplineMeshes, AdvancedDisplay)
	uint32 bPlaceSplineMeshesInStreamingLevels : 1;

	/** Random seed used for choosing which order to use spline meshes. Ignored if only one mesh is set. */
	UPROPERTY(EditAnywhere, Category=LandscapeSplineMeshes, AdvancedDisplay)
	int32 RandomSeed;

	/**  Max draw distance for all the mesh pieces used in this spline */
	UPROPERTY(EditAnywhere, Category=LandscapeSplineMeshes, AdvancedDisplay, meta=(DisplayName="Max Draw Distance"))
	float LDMaxDrawDistance;

	/**
	 * Translucent objects with a lower sort priority draw behind objects with a higher priority.
	 * Translucent objects with the same priority are rendered from back-to-front based on their bounds origin.
	 * This setting is also used to sort objects being drawn into a runtime virtual texture.
	 *
	 * Ignored if the object is not translucent.  The default priority is zero.
	 * Warning: This should never be set to a non-default value unless you know what you are doing, as it will prevent the renderer from sorting correctly.
	 */
	UPROPERTY(EditAnywhere, Category=LandscapeSplineMeshes, AdvancedDisplay)
	int32 TranslucencySortPriority;

	/** If true, this component will be rendered in the CustomDepth pass (usually used for outlines) */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = LandscapeSplineMeshes, meta = (DisplayName = "Render CustomDepth Pass"))
	uint8 bRenderCustomDepth : 1;

	/** Mask used for stencil buffer writes. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = LandscapeSplineMeshes, meta = (editcondition = "bRenderCustomDepth"))
	ERendererStencilMask CustomDepthStencilWriteMask;

	/** Optionally write this 0-255 value to the stencil buffer in CustomDepth pass (Requires project setting or r.CustomDepth == 3) */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = LandscapeSplineMeshes, meta = (UIMin = "0", UIMax = "255", editcondition = "bRenderCustomDepth", DisplayName = "CustomDepth Stencil Value"))
	int32 CustomDepthStencilValue;

	/** 
	 * Array of runtime virtual textures into which we draw the spline segment. 
	 * The material also needs to be set up to output to a virtual texture. 
	 */
	UPROPERTY(EditAnywhere, Category = VirtualTexture, meta = (DisplayName = "Draw in Virtual Textures"))
	TArray<TObjectPtr<URuntimeVirtualTexture>> RuntimeVirtualTextures;

	/** Lod bias for rendering to runtime virtual texture. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = VirtualTexture, meta = (DisplayName = "Virtual Texture LOD Bias", UIMin = "-7", UIMax = "8"))
	int32 VirtualTextureLodBias = 0;

	/**
	 * Number of lower mips in the runtime virtual texture to skip for rendering this primitive.
	 * Larger values reduce the effective draw distance in the runtime virtual texture.
	 * This culling method doesn't take into account primitive size or virtual texture size.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = VirtualTexture, meta = (DisplayName = "Virtual Texture Skip Mips", UIMin = "0", UIMax = "7"))
	int32 VirtualTextureCullMips = 0;

	/** Desired cull distance in the main pass if we are rendering to both the virtual texture AND the main pass. A value of 0 has no effect. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = VirtualTexture, meta = (DisplayName = "Max Draw Distance in Main Pass"))
	float VirtualTextureMainPassMaxDrawDistance = 0.f;

	/** Controls if this component draws in the main pass as well as in the virtual texture. */
	UPROPERTY(EditAnywhere, Category = VirtualTexture, meta = (DisplayName = "Draw in Main Pass"))
	ERuntimeVirtualTextureMainPassType VirtualTextureRenderPassType = ERuntimeVirtualTextureMainPassType::Exclusive;

	/** Mesh Collision Settings */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Collision, meta = (ShowOnlyInnerProperties))
	FBodyInstance BodyInstance;

	/** Custom HLOD layer to use for this spline segment. Only relevant when using World Partition with spline partitioning. */
	UPROPERTY(EditAnywhere, Category = HLOD, AdvancedDisplay)
	TObjectPtr<UHLODLayer> HLODLayerOverrideForSplinePartitioning;

protected:
	UPROPERTY(Transient)
	uint32 bSelected : 1;

	UPROPERTY(Transient)
	uint32 bNavDirty : 1;
#endif

// Procedural data:
protected:
	/** Actual data for spline. */
	UPROPERTY()
	FInterpCurveVector SplineInfo;

	/** Spline points */
	UPROPERTY()
	TArray<FLandscapeSplineInterpPoint> Points;

	/** Bounds of points */
	UPROPERTY()
	FBox Bounds;

	/** Spline meshes */
	UPROPERTY(TextExportTransient)
	TArray<TObjectPtr<USplineMeshComponent>> LocalMeshComponents;

#if WITH_EDITORONLY_DATA
	/** World references for mesh components stored in other streaming levels */
	UPROPERTY(TextExportTransient, NonPIEDuplicateTransient)
	TArray<TSoftObjectPtr<UWorld>> ForeignWorlds;

	/** Key for tracking whether this segment has been modified relative to the mesh components stored in other streaming levels */
	UPROPERTY(TextExportTransient, NonPIEDuplicateTransient)
	FGuid ModificationKey;
#endif

public:
	const FBox& GetBounds() const { return Bounds; }
	const TArray<FLandscapeSplineInterpPoint>& GetPoints() const { return Points; }

#if WITH_EDITOR
	bool SupportsForeignSplineMesh() const;

	bool IsSplineSelected() const { return bSelected; }
	virtual void SetSplineSelected(bool bInSelected);

	virtual void AutoFlipTangents();

	LANDSCAPE_API TMap<ULandscapeSplinesComponent*, TArray<USplineMeshComponent*>> GetForeignMeshComponents();
	TArray<USplineMeshComponent*> GetLocalMeshComponents() const;
	
	virtual void UpdateSplinePoints(bool bUpdateCollision = true, bool bUpdateMeshLevel = false);
	virtual void UpdateSplineMeshSelectionState();
	void UpdateSplineEditorMesh();
	virtual void DeleteSplinePoints();

	LANDSCAPE_API FName GetCollisionProfileName() const;

	const TArray<TSoftObjectPtr<UWorld>>& GetForeignWorlds() const { return ForeignWorlds; }
	FGuid GetModificationKey() const { return ModificationKey; }
#endif

	virtual void FindNearest(const FVector& InLocation, float& t, FVector& OutLocation, FVector& OutTangent);

	/** Return the key interval for the segment curve points */
	LANDSCAPE_API FFloatInterval GetKeyInterval() const;

	/** Evaluate the output for an arbitrary input value. For keys outside the range, the first/last key value is assumed. */
	LANDSCAPE_API FVector Evaluate(const float Key, const FVector& Default = FVector::ZeroVector) const;

	//~ Begin UObject Interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditUndo() override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	virtual void PostEditImport() override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual bool IsCapturingAsRootObjectForTransaction() const override { return true; }
#endif // WITH_EDITOR
protected:
	virtual void PostInitProperties() override;
private:
	void UpdateMeshCollisionProfile(USplineMeshComponent* MeshComponent);
public:
	//~ End UObject Interface

	/** Use GetOuterSafe instead of GetOuterULandscapeSplinesComponent if there is any danger that this segment has been deleted (via DeleteSplinePoints).
	  * This will return nullptr in that case, because the outer is no longer a ULandscapeSplinesComponent */
	LANDSCAPE_API ULandscapeSplinesComponent* GetOuterSafe() const;
	/** Use GetOuterChecked when the call is always expected to succeed.  Will assert if it fails. */
	LANDSCAPE_API ULandscapeSplinesComponent* GetOuterChecked() const;

	friend class FLandscapeToolSplines;
	friend class ULandscapeInfo;
};
