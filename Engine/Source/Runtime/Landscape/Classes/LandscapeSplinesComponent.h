// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "LandscapeInfo.h"
#include "Components/PrimitiveComponent.h"
#include "Components/SplineComponent.h"
#include "LandscapeSplinesComponent.generated.h"

class ALandscapeProxy;
class FPrimitiveSceneProxy;
class UControlPointMeshComponent;
class ULandscapeSplineControlPoint;
class ULandscapeSplineSegment;
class UMeshComponent;
class USplineMeshComponent;
class UStaticMesh;
class UTexture2D;
class ILandscapeSplineInterface;

// structs for ForeignWorldSplineDataMap
// these are editor-only, but we don't have the concept of an editor-only USTRUCT
USTRUCT()
struct FForeignControlPointData
{
	GENERATED_USTRUCT_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FGuid ModificationKey;

	UPROPERTY()
	TObjectPtr<UControlPointMeshComponent> MeshComponent = nullptr;

	UPROPERTY()
	TLazyObjectPtr<ULandscapeSplineControlPoint> Identifier;

	friend bool operator==(const FForeignControlPointData& LHS, const FForeignControlPointData& RHS)
	{
		return LHS.Identifier == RHS.Identifier;
	}
#endif
};

USTRUCT()
struct FForeignSplineSegmentData
{
	GENERATED_USTRUCT_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FGuid ModificationKey;

	UPROPERTY()
	TArray<TObjectPtr<USplineMeshComponent>> MeshComponents;

	UPROPERTY()
	TLazyObjectPtr<ULandscapeSplineSegment> Identifier;

	friend bool operator==(const FForeignSplineSegmentData& LHS, const FForeignSplineSegmentData& RHS)
	{
		return LHS.Identifier == RHS.Identifier;
	}
#endif
};

USTRUCT()
struct FForeignWorldSplineData
{
	GENERATED_USTRUCT_BODY()

#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY()
	TMap<TLazyObjectPtr<ULandscapeSplineControlPoint>, FForeignControlPointData> ForeignControlPointDataMap_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UPROPERTY()
	TArray<FForeignControlPointData> ForeignControlPointData;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY()
	TMap<TLazyObjectPtr<ULandscapeSplineSegment>, FForeignSplineSegmentData> ForeignSplineSegmentDataMap_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UPROPERTY()
	TArray<FForeignSplineSegmentData> ForeignSplineSegmentData;
#endif

#if WITH_EDITOR
	bool IsEmpty();

	FForeignControlPointData* FindControlPoint(ULandscapeSplineControlPoint* InIdentifer);
	FForeignSplineSegmentData* FindSegmentData(ULandscapeSplineSegment* InIdentifer);
#endif
};

//////////////////////////////////////////////////////////////////////////
// ULandscapeSplinesComponent
//////////////////////////////////////////////////////////////////////////

UCLASS(MinimalAPI)
class ULandscapeSplinesComponent : public UPrimitiveComponent
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITORONLY_DATA
	/** Resolution of the spline, in distance per point */
	UPROPERTY()
	float SplineResolution;

	/** Color to use to draw the splines */
	UPROPERTY()
	FColor SplineColor;

	/** Sprite used to draw control points */
	UPROPERTY()
	TObjectPtr<UTexture2D> ControlPointSprite;

	/** Mesh used to draw splines that have no mesh */
	UPROPERTY()
	TObjectPtr<UStaticMesh> SplineEditorMesh;

	/** Whether we are in-editor and showing spline editor meshes */
	UPROPERTY(NonTransactional, Transient)
	uint32 bShowSplineEditorMesh:1;
#endif

protected:
	UPROPERTY(TextExportTransient)
	TArray<TObjectPtr<ULandscapeSplineControlPoint>> ControlPoints;

	UPROPERTY(TextExportTransient)
	TArray<TObjectPtr<ULandscapeSplineSegment>> Segments;

#if WITH_EDITORONLY_DATA
	// Serialized
	UPROPERTY(TextExportTransient)
	TMap<TSoftObjectPtr<UWorld>, FForeignWorldSplineData> ForeignWorldSplineDataMap;

	// Transient - rebuilt on load
	TMap<UMeshComponent*, UObject*> MeshComponentLocalOwnersMap;
	TMap<UMeshComponent*, TLazyObjectPtr<UObject>> MeshComponentForeignOwnersMap;
#endif

	// References to components owned by landscape splines in other levels
	// for cooked build (uncooked keeps references via ForeignWorldSplineDataMap)
	UPROPERTY(TextExportTransient)
	TArray<TObjectPtr<UMeshComponent>> CookedForeignMeshComponents;

public:
	/** Get a list of spline mesh components representing this landscape spline (Editor only) */
	UFUNCTION(BlueprintCallable, Category = LandscapeSplines)
	TArray<USplineMeshComponent*> GetSplineMeshComponents();

	LANDSCAPE_API ILandscapeSplineInterface* GetSplineOwner();

	// Run some ensures on the internal state of the data
	LANDSCAPE_API void CheckSplinesValid();

	bool ModifySplines(bool bAlwaysMarkDirty = true);

	const TArray<TObjectPtr<ULandscapeSplineControlPoint>>& GetControlPoints() const { return ControlPoints; }
	TArray<TObjectPtr<ULandscapeSplineControlPoint>>& GetControlPoints() { return ControlPoints; }

	const TArray<TObjectPtr<ULandscapeSplineSegment>>& GetSegments() const { return Segments; }
	TArray<TObjectPtr<ULandscapeSplineSegment>>& GetSegments() { return Segments; }

#if WITH_EDITOR
	void RequestSplineLayerUpdate();

	bool HasAnyControlPointsOrSegments() const { return ControlPoints.Num() > 0 || Segments.Num() > 0; }

	virtual void ShowSplineEditorMesh(bool bShow);

	// Rebuilds all spline points and meshes for all spline control points and segments in this splines component
	// @param bBuildCollision Building collision data is very slow, so during interactive changes pass false to improve user experience (make sure to call with true when done)
	virtual void RebuildAllSplines(bool bBuildCollision = true);

	// returns a suitable ULandscapeSplinesComponent to place streaming meshes into, given a location
	// falls back to "this" if it can't find another suitable, so never returns nullptr
	// @param bCreate whether to create a component if a suitable actor is found but it has no splines component yet
	LANDSCAPE_API ULandscapeSplinesComponent* GetStreamingSplinesComponentByLocation(const FVector& LocalLocation, bool bCreate = true);

	// returns the matching ULandscapeSplinesComponent for a given level, *can return null*
	// @param bCreate whether to create a component if a suitable actor is found but it has no splines component yet
	ULandscapeSplinesComponent* GetStreamingSplinesComponentForLevel(ULevel* Level, bool bCreate = true);

	// gathers and returns all currently existing 
	TArray<ULandscapeSplinesComponent*> GetAllStreamingSplinesComponents();

	virtual void UpdateModificationKey(ULandscapeSplineSegment* Owner);
	virtual void UpdateModificationKey(ULandscapeSplineControlPoint* Owner);
	virtual void AddForeignMeshComponent(ULandscapeSplineSegment* Owner, USplineMeshComponent* Component);
	virtual void RemoveForeignMeshComponent(ULandscapeSplineSegment* Owner, USplineMeshComponent* Component);
	virtual void RemoveAllForeignMeshComponents(ULandscapeSplineSegment* Owner);
	virtual void AddForeignMeshComponent(ULandscapeSplineControlPoint* Owner, UControlPointMeshComponent* Component);
	virtual void RemoveForeignMeshComponent(ULandscapeSplineControlPoint* Owner, UControlPointMeshComponent* Component);
	virtual void DestroyOrphanedForeignSplineMeshComponents(UWorld* OwnerWorld);
	virtual void DestroyOrphanedForeignControlPointMeshComponents(UWorld* OwnerWorld);

	void DestroyUnreferencedForeignMeshComponents();
	void ForEachUnreferencedForeignMeshComponent(TFunctionRef<bool(ULandscapeSplineSegment*, USplineMeshComponent*, ULandscapeSplineControlPoint*, UControlPointMeshComponent*)> Func);
	
	virtual UControlPointMeshComponent*   GetForeignMeshComponent(ULandscapeSplineControlPoint* Owner);
	virtual TArray<USplineMeshComponent*> GetForeignMeshComponents(ULandscapeSplineSegment* Owner);

	virtual UObject* GetOwnerForMeshComponent(const UMeshComponent* SplineMeshComponent);

	void AutoFixMeshComponentErrors(UWorld* OtherWorld);

	bool IsUsingEditorMesh(const USplineMeshComponent* SplineMeshComponent) const;

	bool IsUsingLayerInfo(const ULandscapeLayerInfoObject* LayerInfo) const;

	// Iterates throug a copy of the ControlPoints list.
	LANDSCAPE_API void ForEachControlPoint(TFunctionRef<void(ULandscapeSplineControlPoint*)> Func);

	/**
	 * Converts points along the landscape spline into points on a spline component
	 * Works best with landscape splines that contain no intersections or splits
	 * @param SplineComponent Spline component to be populated
	**/
	LANDSCAPE_API void CopyToSplineComponent(USplineComponent* SplineComponent);
#endif

	//~ Begin UObject Interface

	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditUndo() override;
#endif // WITH_EDITOR
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	//~ End UObject Interface

	//~ Begin UActorComponent Interface
#if WITH_EDITOR
	virtual void CheckForErrors() override;
#endif
	virtual void OnRegister() override;
	//~ End UActorComponent Interface
	
	//~ Begin UPrimitiveComponent Interface.
#if WITH_EDITOR
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
#endif
	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ End UPrimitiveComponent Interface.

	// many friends
	friend class FLandscapeToolSplines;
	friend class FLandscapeSplinesSceneProxy;
	friend class ULandscapeSplineControlPoint;
	friend class ULandscapeSplineSegment;
	friend class ULandscapeInfo;
};
