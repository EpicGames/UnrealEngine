// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "LandscapeProxy.h"

#include "LandscapeStreamingProxy.generated.h"

class ALandscape;

#if WITH_EDITOR
class UMaterialInterface;
struct FActorPartitionIdentifier;
enum ELandscapeLayerUpdateMode : uint32;
#endif

UCLASS(MinimalAPI, notplaceable)
class ALandscapeStreamingProxy : public ALandscapeProxy
{
	GENERATED_BODY()

public:
	ALandscapeStreamingProxy(const FObjectInitializer& ObjectInitializer);

#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY()
	TLazyObjectPtr<ALandscape> LandscapeActor_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA

private:
	UPROPERTY(EditAnywhere, Category = LandscapeProxy, Meta = (DisplayName = "Landscape Actor"))
	TSoftObjectPtr<ALandscape> LandscapeActorRef;

	UPROPERTY()
	TSet<FName> OverriddenSharedProperties;

#if WITH_EDITORONLY_DATA
	/** Tracks layer update requests made by this proxy's components when the ALandscape is unreachable. Requests are processed on the next registration once the parent ALandscape is valid */
	UPROPERTY(Transient)
	uint32 DeferredLayerContentUpdateModes = 0;
#endif 

public:
	//~ Begin UObject Interface
#if WITH_EDITOR
	virtual bool ShouldExport() override { return false;  }
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual bool CanEditChange(const FProperty* InProperty) const override;
	virtual void PostRegisterAllComponents() override;
	virtual AActor* GetSceneOutlinerParent() const override;
	virtual bool CanDeleteSelectedActor(FText& OutReason) const override;
	virtual bool GetReferencedContentObjects(TArray<UObject*>& Objects) const override;
	virtual bool CanChangeIsSpatiallyLoadedFlag() const override { return AActor::CanChangeIsSpatiallyLoadedFlag(); }
	virtual bool ShouldIncludeGridSizeInName(UWorld* InWorld, const FActorPartitionIdentifier& InIdentifier) const override;
	virtual void GetActorDescProperties(FPropertyPairsMap& PropertyPairsMap) const override;
	virtual void CheckForErrors() override;
#endif
	//~ End UObject Interface

	//~ Begin ALandscapeBase Interface
	virtual ALandscape* GetLandscapeActor() override;
	virtual const ALandscape* GetLandscapeActor() const override;
	void LANDSCAPE_API SetLandscapeActor(ALandscape* InLandscape);
	virtual UMaterialInterface* GetLandscapeMaterial(int8 InLODIndex = INDEX_NONE) const override;
	virtual UMaterialInterface* GetLandscapeHoleMaterial() const override;
	//~ End ALandscapeBase Interface

	// Check input Landscape actor is match for this LandscapeProxy (by GUID)
	bool IsValidLandscapeActor(ALandscape* Landscape);

#if WITH_EDITOR
	//~ Begin ALandscapeProxy Interface
	virtual bool IsSharedPropertyOverridden(const FName& InPropertyName) const override;
	virtual void SetSharedPropertyOverride(const FName& InPropertyName, const bool bIsOverridden, const bool bInExecutePostEditChangeProperty = true) override;
	//~ End ALandscapeProxy Interface

	void RequestDeferredLayerContentUpdateModes(ELandscapeLayerUpdateMode InLayerContentUpdateModes);
	void PerformDeferredLayerContentUpdateModesRequest();
protected:
	virtual void CheckOverriddenSharedProperties() override;
	virtual void FixupOverriddenSharedProperties() override;
#endif // WITH_EDITOR
};
