// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/AssetUserData.h"
#include "LandscapeTextureHash.generated.h"

class UTexture2D;

UENUM()
enum class ELandscapeTextureType : uint8
{
	Unknown,
	Heightmap,
	Weightmap
};

UENUM()
enum class ELandscapeTextureUsage : uint8
{
	Unknown,
	EditLayerData,			// used as data for an edit layer, input to the layer merge operation
	FinalData				// used for runtime/rendering
};

UCLASS(NotBlueprintable, MinimalAPI, Within = Texture2D)
class ULandscapeTextureHash : public UAssetUserData
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid TextureHashGUID;

	UPROPERTY()
	FGuid LastSourceID;

	// heightmap or weightmap.  When unknown, we fallback to using the texture source ID as hash (old behavior)
	UPROPERTY()
	ELandscapeTextureType TextureType = ELandscapeTextureType::Unknown;

	// edit layer data or final data.  When unknown, we fallback to using the texture source ID as hash (old behavior)
	UPROPERTY()
	ELandscapeTextureUsage TextureUsage = ELandscapeTextureUsage::Unknown;

	// cache of recently serialized hash values
	// this ensure that if the texture is brought back to match exactly a recently serialized state, it will have exactly the same hash that it was serialized with.
	// This helps us come back to the serialized (presumably cache-friendly) state after no-ops like:  modify + undo, hide layer + unhide layer, etc.
	TMap<FGuid, FGuid> RecentlySerializedHashes;

public:

#if WITH_EDITORONLY_DATA
	// setup initial state on load, if it doesn't yet exist
	static void SetInitialStateOnPostLoad(UTexture2D* LandscapeTexture, ELandscapeTextureUsage TextureUsage, ELandscapeTextureType TextureType);

	// Update the stored hash based on the source data
	static void UpdateHash(UTexture2D* LandscapeTexture, ELandscapeTextureUsage TextureUsage = ELandscapeTextureUsage::Unknown, ELandscapeTextureType TextureType = ELandscapeTextureType::Unknown, bool bForceUpdate = false);

	// Explicitly set the hash for the specified LandscapeTexture
	static void SetHash64(UTexture2D* LandscapeTexture, uint64 NewHash, ELandscapeTextureUsage TextureUsage, ELandscapeTextureType TextureType);

	// calculate the texture hash directly from the source mip0 contents
	static uint64 CalculateTextureHash64(UTexture2D* LandscapeTexture, ELandscapeTextureType TextureType, bool& bOutHashIsValid);
	static FGuid CalculateTextureHashGUID(UTexture2D* LandscapeTexture, ELandscapeTextureType TextureType, bool& bOutHashIsValid);

	// calculate the texture hash from the Mip0Data buffer
	static uint64 CalculateTextureHash64(const FColor* Mip0Data, int32 PixelCount, ELandscapeTextureType TextureType);

	// check if the Mip0Data is within the change threshold of OldMip0Data
	static bool DoesTextureDataChangeExceedThreshold(const FColor* Mip0Data, const FColor* OldMip0Data, int32 PixelCount, ELandscapeTextureType TextureType, uint64 OldHash, uint64 NewHash, TOptional<uint8>& OutChangedWeightmapChannelsMasks);

	// get the current stored hash for the landscape texture, checking that it is up to date
	static FGuid GetHash(UTexture2D* LandscapeTexture);

	// Check the hash is up to date
	static void CheckHashIsUpToDate(UTexture2D* LandscapeTexture);

	// UObject Interface
	virtual void Serialize(FArchive& Ar) override;
	virtual bool IsEditorOnly() const override { return true; }
	virtual bool NeedsLoadForClient() const override { return false; }
	virtual bool NeedsLoadForServer() const override { return false; }
#endif // WITH_EDITORONLY_DATA
};
