// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/TextureMipDataProviderFactory.h"
#include "Streaming/TextureMipDataProvider.h"
#include "Containers/ContainersFwd.h"
#include "Algo/Accumulate.h"
#include "Serialization/BulkData.h"
#include "Engine/TextureAllMipDataProviderFactory.h"
#include "LandscapeEdgeFixup.h"
#include "LandscapeTextureStorageProvider.generated.h"

class ULandscapeTextureStorageProviderFactory;
class ULandscapeHeightmapTextureEdgeFixup;
class UTexture2D;

USTRUCT()
struct FLandscapeTexture2DMipMap
{
	GENERATED_BODY()

	/** Width of the mip-map. */
	UPROPERTY()
	int32 SizeX = 0;
	/** Height of the mip-map. */
	UPROPERTY()
	int32 SizeY = 0;
	/** Whether the bulk data is compressed or not */
	UPROPERTY()
	bool bCompressed = false;

	FByteBulkData BulkData;

	void Serialize(FArchive& Ar, UObject* Owner, uint32 SaveOverrideFlags);
};

class FLandscapeTextureStorageMipProvider : public FTextureMipDataProvider
{
public:
	FLandscapeTextureStorageMipProvider(ULandscapeTextureStorageProviderFactory* InFactory);
	~FLandscapeTextureStorageMipProvider();

	// ********************************************************
	// ********* FLandscapeTextureStorageMipProvider implementation **********
	// ********************************************************

	void Init(const FTextureUpdateContext& Context, const FTextureUpdateSyncOptions& SyncOptions) final override;
	int32 GetMips(const FTextureUpdateContext& Context, int32 StartingMipIndex, const FTextureMipInfoArray& MipInfos, const FTextureUpdateSyncOptions& SyncOptions) final override;
	bool PollMips(const FTextureUpdateSyncOptions& SyncOptions) final override;
	void AbortPollMips() final override;
	void CleanUp(const FTextureUpdateSyncOptions& SyncOptions) final override;
	void Cancel(const FTextureUpdateSyncOptions& SyncOptions) final override;
	ETickThread GetCancelThread() const final override;

private:
	// the Factory (that actually has the mip data)
	ULandscapeTextureStorageProviderFactory* Factory = nullptr;

	// A structured with information about which file contains which mips.
	struct FIORequest
	{
		FIoFilenameHash FilenameHash = INVALID_IO_FILENAME_HASH;
		TUniquePtr<IBulkDataIORequest> BulkDataIORequest;
		uint8* DestMipData = nullptr;
	};

	// Pending async requests created in GetMips().
	TArray<FIORequest, TInlineAllocator<MAX_TEXTURE_MIP_COUNT>> IORequests;

	// We make a copy of this, as it is provided in GetMips, and we fill it out in PollMips
	FTextureMipInfoArray DestMipInfos;

	int32 FirstRequestedMipIndex = INDEX_NONE;

	// The asset name, used to log IO errors.
	FName TextureName;
	// Whether async read requests must be created with high priority (executes faster). 
	bool bPrioritizedIORequest = false;
	// Whether async read requests where cancelled for any reasons.
	bool bIORequestCancelled = false;
	// Whether async read requests were required to abort through AbortPollMips().
	bool bIORequestAborted = false;

	// A callback to be executed once all IO pending requests are completed.
	FBulkDataIORequestCallBack AsyncFileCallBack;

	// Helper to create the AsyncFileCallBack -- which can handle IO Request completions (and will trigger the next step to be scheduled via SyncOptions.RescheduleCallback())
	void CreateAsyncFileCallback(const FTextureUpdateSyncOptions& SyncOptions);

	void ClearIORequests();
};



class FLandscapeTextureMipEdgeOverrideProvider : public FTextureMipDataProvider
{
	// This class is actually a "Modifier" in that it doesn't provide any mips, it only modifies mips provided by the default providers
	// It relies on some assumptions about the existing provider execution that are currently true, but may not remain true forever:
	// 1) that all providers are executed, even if they are not actually handling any mips
	// 2) that all default providers do NOT modify mip data in PollMips  (this is true for both FTexture2DMipDataProvider_IO and FTexture2DMipDataProvider_DDC currently)
	//    this allows us to modify mip data in PollMips and not worry about being stomped on.
	// IF either of these cease to be true, we should probably write a first class FTextureMipDataModifier path, and convert this to be a modifier instead of a provider.

public:
	FLandscapeTextureMipEdgeOverrideProvider(ULandscapeHeightmapTextureEdgeFixup* InEdgeFixup, UTexture2D* InTexture);
	~FLandscapeTextureMipEdgeOverrideProvider();

	// ****************************************************************************
	// ********* FLandscapeTextureMipEdgeOverrideProvider implementation **********
	// ****************************************************************************

	void Init(const FTextureUpdateContext& Context, const FTextureUpdateSyncOptions& SyncOptions) final override;
	int32 GetMips(const FTextureUpdateContext& Context, int32 StartingMipIndex, const FTextureMipInfoArray& MipInfos, const FTextureUpdateSyncOptions& SyncOptions) final override;
	bool PollMips(const FTextureUpdateSyncOptions& SyncOptions) final override;
	void CleanUp(const FTextureUpdateSyncOptions& SyncOptions) final override;
	void Cancel(const FTextureUpdateSyncOptions& SyncOptions) final override;
	ETickThread GetCancelThread() const final override;

private:
	// the Edge Fixup (that actually has the edge override data)
	ULandscapeHeightmapTextureEdgeFixup* EdgeFixup = nullptr;

	// We make a copy of this, as it is provided in GetMips, and we fill it out in PollMips
	FTextureMipInfoArray DestMipInfos;

	// The asset name, used to log IO errors.
	FName TextureName;
};


// serves as a hook into the texture streaming process, allowing us to inject edge data
UCLASS(NotBlueprintable, MinimalAPI, Within = Texture2D)
class ULandscapeTextureMipEdgeOverrideFactory : public UTextureMipDataProviderFactory
{
	GENERATED_UCLASS_BODY()
public:
	TObjectPtr<UTexture2D> Texture;									// should be the same as GetOuter()
	TObjectPtr<ULandscapeHeightmapTextureEdgeFixup> EdgeFixup;		// assigned when the EdgeFixup is Active

	static ULandscapeTextureMipEdgeOverrideFactory* AddTo(UTexture2D* TargetTexture);

	void SetupEdgeFixup(ULandscapeHeightmapTextureEdgeFixup* InEdgeFixup);

	virtual void Serialize(FArchive& Ar) override;
	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	/**
	 * Create a FTextureMipDataProvider to handle a single StreamIn mip operation.
	 * The object lifetime will be managed by FRenderAssetUpdate.
	 *
	 * @param InAsset - the texture on which the stream in operation will be performed.
	 */
	virtual FTextureMipDataProvider* AllocateMipDataProvider(UTexture* Asset) override
	{
		check(Asset == (UTexture*) Texture.Get());
		return new FLandscapeTextureMipEdgeOverrideProvider(EdgeFixup, Texture);
	}

	/**
	 * Returns true if TextureMipDataProviders allocated by this factory can provide MipData by themselves,
	 * even without loading from disk at all, so streaming can be enabled for their textures.
	 */
	virtual bool WillProvideMipDataWithoutDisk() const
	{
		return true;
	}
};


UCLASS(NotBlueprintable, MinimalAPI, Within = Texture2D)
class ULandscapeTextureStorageProviderFactory : public UTextureAllMipDataProviderFactory
{
	GENERATED_UCLASS_BODY()

public:
	
	int32 NumNonOptionalMips = 0;
	int32 NumNonStreamingMips = 0;
	FVector LandscapeGridScale = FVector(ForceInit);

	// Tracks the offset of the texture boundaries (defined by number of subsection/multi-components in texture). Each texture has (NumSubsections * NumComponents) - 1 boundaries for each axis 
	UPROPERTY()
	int32 BoundaryOffset = 0;

	// Tracks the number of duplicated column boundaries (based on subsection/multi-component texture count)
	UPROPERTY()
	int32 BoundaryCountX = 0;

	// Tracks the number of duplicated row boundaries (based on subsection/multi-component texture count) 
	UPROPERTY()
	int32 BoundaryCountY = 0;

	TArray<FLandscapeTexture2DMipMap> Mips;
	TObjectPtr<UTexture2D> Texture;

	// Reference to the edge fixup on the Texture, if any.  (This can be null, if edge fixup is disabled)
	// It is set when the Texture's landscape component is registered.
	TObjectPtr<ULandscapeHeightmapTextureEdgeFixup> EdgeFixup;

	#if WITH_EDITORONLY_DATA
	static ULandscapeTextureStorageProviderFactory* ApplyTo(UTexture2D* InTargetTexture, const FVector& InLandsapeGridScale, int32 InHeightmapCompressionMipThreshold, int32 InNumSubsections, int32 InSubsectionSizeQuads);
	void UpdateCompressedDataFromSource(UTexture2D* InTargetTexture, const FVector& InLandsapeGridScale, int32 InHeightmapCompressionMipThreshold, int32 InNumSubsections, int32 InSubsectionSizeQuads);
	#endif // WITH_EDITORONLY_DATA

	void SetupEdgeFixup(ULandscapeHeightmapTextureEdgeFixup* EdgeFixup);

	// compress the mip data.
	static void CompressMipToBulkData(int32 MipIndex, int32 MipSizeX, int32 MipSizeY, uint8* SourceData, int32 SourceDataBytes, FByteBulkData& DestBulkData, int32 InMipBoundaryOffset, int32 InBoundaryCountX, int32 InBoundaryCountY);
	static void CopyMipToBulkData(int32 MipIndex, int32 MipSizeX, int32 MipSizeY, uint8* SourceData, int32 SourceDataBytes, FByteBulkData& DestBulkData);

	// decompress the mip data. Currently operates in-place (though this could be changed in the future)
	void DecompressMip(uint8* SourceData, int64 SourceDataBytes, uint8* DestData, int64 DestDataBytes, int32 MipIndex);
	void DecompressMipOriginalUnoptimized(uint8* SourceData, int64 SourceDataBytes, uint8* DestData, int64 DestDataBytes, int32 MipIndex);

	virtual void Serialize(FArchive& Ar) override;
	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	/**
	* Create a FTextureMipDataProvider to handle a single StreamIn mip operation.
	* The object lifetime will be managed by FRenderAssetUpdate.
	*
	* @param InAsset - the texture on which the stream in operation will be performed.
	*/
	virtual FTextureMipDataProvider* AllocateMipDataProvider(UTexture* Asset) override
	{
		return new FLandscapeTextureStorageMipProvider(this);
	}

	/**
	* Returns true if TextureMipDataProviders allocated by this factory can provide MipData by themselves,
	* even without loading from disk at all, so streaming can be enabled for their textures.
	*/
	virtual bool WillProvideMipDataWithoutDisk() const
	{
		return true;
	}

	// load mip data (used to access preloaded inline mips for creating the initial version of the texture)
	virtual bool GetInitialMipData(int32 FirstMipToLoad, TArrayView<void*> OutMipData, TArrayView<int64> OutMipSize, FStringView DebugContext) override;

	// return the initial streaming state of the texture
	virtual FStreamableRenderResourceState GetResourcePostInitState(const UTexture* Owner, bool bAllowStreaming) override;

	bool DoesMipDataExist(int32 MipIndex)
	{
		// we need to check the bulk data exists (might not for optional mips)
		return Mips.IsValidIndex(MipIndex) && Mips[MipIndex].BulkData.DoesExist();
	}

	const FLandscapeTexture2DMipMap* GetMip(int32 MipIndex)
	{
		int32 Index = MipIndex;
		if (Mips.IsValidIndex(Index))
		{
			return &Mips[Index];
		}
		return nullptr;
	}

	int32 GetTotalBytes() const
	{
		return Algo::TransformAccumulate(Mips, [](const FLandscapeTexture2DMipMap& Mip) -> int32 { return Mip.BulkData.GetBulkDataSize(); }, 0);
	}
};
