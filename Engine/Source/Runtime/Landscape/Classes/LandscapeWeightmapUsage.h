// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include "LandscapeWeightmapUsage.generated.h"

class ULandscapeComponent;

UCLASS(MinimalAPI, NotBlueprintable)
class ULandscapeWeightmapUsage : public UObject
{
	GENERATED_UCLASS_BODY()

	enum { NumChannels = 4 };

public:
	UPROPERTY()
	TObjectPtr<ULandscapeComponent> ChannelUsage[NumChannels];

	UPROPERTY()
	FGuid LayerGuid;

	int32 FreeChannelCount() const
	{
		int32 Count = 0;

		for (int8 i = 0; i < NumChannels; ++i)
		{
			Count += (ChannelUsage[i] == nullptr) ? 1 : 0;
		}

		return	Count;
	}

	void ClearUsage()
	{
		for (int8 i = 0; i < NumChannels; ++i)
		{
			ChannelUsage[i] = nullptr;
		}
	}

	void ClearUsage(ULandscapeComponent* Component)
	{
		for (int8 i = 0; i < NumChannels; ++i)
		{
			if (ChannelUsage[i] == Component)
			{
				ChannelUsage[i] = nullptr;
			}
		}
	}

	bool IsEmpty() const
	{
		return FreeChannelCount() == NumChannels;
	}

	TArray<ULandscapeComponent*, TInlineAllocator<4>> GetUniqueValidComponents() const
	{
		TArray<ULandscapeComponent*, TInlineAllocator<4>> UniqueComponents;
		for (const TObjectPtr<ULandscapeComponent>& Component : ChannelUsage)
		{
			if (Component != nullptr)
			{
				UniqueComponents.AddUnique(Component);
			}
		}
		return UniqueComponents;
	}
};
