// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MaterialExpressionIO.h"
#include "MaterialValueType.h"
#include "Materials/MaterialExpressionCustomOutput.h"
#include "MaterialExpressionLandscapeGrassOutput.generated.h"

class ULandscapeGrassType;
struct FPropertyChangedEvent;

USTRUCT()
struct FGrassInput
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = Grass)
	FName Name;

	UPROPERTY(EditAnywhere, Category = Grass)
	TObjectPtr<ULandscapeGrassType> GrassType;

	UPROPERTY(meta = (RequiredInput = "true"))
	FExpressionInput Input;

	FGrassInput()
	: GrassType(nullptr)
	{}
	
	FGrassInput(FName InName)
	: Name(InName)
	, GrassType(nullptr)
	{}
};

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionLandscapeGrassOutput : public UMaterialExpressionCustomOutput
{
	GENERATED_UCLASS_BODY()

	// Maximum number of supported grass types on a given landscape material. Whenever adjusting this, make sure to update LandscapeGrassWeight.usf accordingly:
	static constexpr int32 MaxGrassTypes = 32;

#if WITH_EDITOR
	LANDSCAPE_API virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	LANDSCAPE_API virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	LANDSCAPE_API virtual TArrayView<FExpressionInput*> GetInputsView() override;
	LANDSCAPE_API virtual FExpressionInput* GetInput(int32 InputIndex) override;
	LANDSCAPE_API virtual FName GetInputName(int32 InputIndex) const override;
	LANDSCAPE_API void ValidateInputName(FGrassInput& Input) const;
#endif

	//~ Begin UObject Interface
#if WITH_EDITOR
	LANDSCAPE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface

	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override { return MCT_Float; }
#endif

	virtual int32 GetNumOutputs() const override { return GrassTypes.Num(); }
	virtual int32 GetMaxOutputs() const { return MaxGrassTypes; };
	virtual FString GetFunctionName() const override { return TEXT("GetGrassWeight"); }

	UPROPERTY(EditAnywhere, Category = UMaterialExpressionLandscapeGrassOutput)
	TArray<FGrassInput> GrassTypes;

private:
	static LANDSCAPE_API FName PinDefaultName;
};



