// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "MaterialExpressionIO.h"
#include "MaterialValueType.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionLandscapeLayerBlend.generated.h"

class UTexture;
struct FPropertyChangedEvent;
struct FMaterialParameterInfo;

UENUM()
enum ELandscapeLayerBlendType : int
{
	LB_WeightBlend,
	LB_AlphaBlend,
	LB_HeightBlend,
};

USTRUCT()
struct FLayerBlendInput
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category=LayerBlendInput)
	FName LayerName;

	UPROPERTY(EditAnywhere, Category=LayerBlendInput)
	TEnumAsByte<ELandscapeLayerBlendType> BlendType;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstLayerInput' if not specified"))
	FExpressionInput LayerInput;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstHeightInput' if not specified"))
	FExpressionInput HeightInput;

	UPROPERTY(EditAnywhere, Category=LayerBlendInput)
	float PreviewWeight;

	/** only used if LayerInput is not hooked up */
	UPROPERTY(EditAnywhere, Category = LayerBlendInput)
	FVector ConstLayerInput;

	/** only used if HeightInput is not hooked up */
	UPROPERTY(EditAnywhere, Category=LayerBlendInput)
	float ConstHeightInput;

	FLayerBlendInput()
		: BlendType(0)
		, PreviewWeight(0.0f)
		, ConstLayerInput(0.0f, 0.0f, 0.0f)
		, ConstHeightInput(0.0f)
	{
	}
};

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionLandscapeLayerBlend : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=MaterialExpressionLandscapeLayerBlend)
	TArray<FLayerBlendInput> Layers;

	//~ Begin UObject Interface
	LANDSCAPE_API virtual void Serialize(FStructuredArchive::FRecord Record) override;
#if WITH_EDITOR
	LANDSCAPE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface

	//~ Begin UMaterialExpression Interface
	LANDSCAPE_API virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual EMaterialValueType GetOutputValueType(int32 InputIndex) override { return MCT_Unknown; }
	LANDSCAPE_API virtual bool IsResultMaterialAttributes(int32 OutputIndex) override;
	LANDSCAPE_API virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	LANDSCAPE_API virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	LANDSCAPE_API virtual TArrayView<FExpressionInput*> GetInputsView() override;
	LANDSCAPE_API virtual FExpressionInput* GetInput(int32 InputIndex) override;
	LANDSCAPE_API virtual FName GetInputName(int32 InputIndex) const override;


	/**
	 * Gets the landscape layer names
	 */
	LANDSCAPE_API virtual void GetLandscapeLayerNames(TArray<FName>& OutLayers) const override;
#endif
	LANDSCAPE_API virtual UObject* GetReferencedTexture() const override;
	LANDSCAPE_API virtual ReferencedTextureArray GetReferencedTextures() const override;
	virtual bool CanReferenceTexture() const override { return true; }
	//~ End UMaterialExpression Interface

private:
	LANDSCAPE_API int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex, bool bForPreview);
};



