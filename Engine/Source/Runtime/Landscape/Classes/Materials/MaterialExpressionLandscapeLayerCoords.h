// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionLandscapeLayerCoords.generated.h"

UENUM()
enum ETerrainCoordMappingType : int
{
	TCMT_Auto,
	TCMT_XY,
	TCMT_XZ,
	TCMT_YZ,
	TCMT_MAX,
};

UENUM()
enum ELandscapeCustomizedCoordType : int
{
	/** Don't use customized UV, just use original UV. */
	LCCT_None,
	LCCT_CustomUV0,
	LCCT_CustomUV1,
	LCCT_CustomUV2,
	/** Use original WeightMapUV, which could not be customized. */
	LCCT_WeightMapUV,
	LCCT_MAX,
};

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionLandscapeLayerCoords : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	/** Determines the mapping place to use on the terrain. */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionLandscapeLayerCoords)
	TEnumAsByte<enum ETerrainCoordMappingType> MappingType;

	/** Determines the mapping place to use on the terrain. */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionLandscapeLayerCoords)
	TEnumAsByte<enum ELandscapeCustomizedCoordType> CustomUVType;

	/** Uniform scale to apply to the mapping. */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionLandscapeLayerCoords)
	float MappingScale;

	/** Rotation to apply to the mapping. */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionLandscapeLayerCoords)
	float MappingRotation;

	/** Offset to apply to the mapping along U. */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionLandscapeLayerCoords)
	float MappingPanU;

	/** Offset to apply to the mapping along V. */
	UPROPERTY(EditAnywhere, Category=MaterialExpressionLandscapeLayerCoords)
	float MappingPanV;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	LANDSCAPE_API virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	LANDSCAPE_API virtual void GetCaption(TArray<FString>& OutCaptions) const override;

#endif
	//~ End UMaterialExpression Interface
};



