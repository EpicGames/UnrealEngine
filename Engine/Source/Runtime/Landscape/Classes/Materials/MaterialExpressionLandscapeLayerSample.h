// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Materials/MaterialExpression.h"

#include "MaterialExpressionLandscapeLayerSample.generated.h"

class UTexture;
struct FMaterialParameterInfo;

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionLandscapeLayerSample : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=MaterialExpressionLandscapeLayerWeight)
	FName ParameterName;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionLandscapeLayerWeight)
	float PreviewWeight;

public:

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	virtual bool CanRenameNode() const override { return true; }
	LANDSCAPE_API virtual FString GetEditableName() const override;
	LANDSCAPE_API virtual void SetEditableName(const FString& NewName) override;
	LANDSCAPE_API virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	LANDSCAPE_API virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	LANDSCAPE_API virtual bool MatchesSearchQuery(const TCHAR* SearchQuery) override;


	/**
	 * Gets the landscape layer names
	 */
	LANDSCAPE_API virtual void GetLandscapeLayerNames(TArray<FName>& OutLayers) const override;
#endif
	LANDSCAPE_API virtual UObject* GetReferencedTexture() const override;
	LANDSCAPE_API virtual ReferencedTextureArray GetReferencedTextures() const override;
	virtual bool CanReferenceTexture() const override { return true; }
	//~ End UMaterialExpression Interface
};



