// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "MaterialExpressionIO.h"
#include "MaterialValueType.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionLandscapeLayerWeight.generated.h"

class UTexture;
struct FMaterialParameterInfo;

UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionLandscapeLayerWeight : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstBase' if not specified"))
	FExpressionInput Base;

	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Ignored if not specified"))
	FExpressionInput Layer;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionLandscapeLayerWeight)
	FName ParameterName;

	UPROPERTY(EditAnywhere, Category=MaterialExpressionLandscapeLayerWeight)
	float PreviewWeight;

	/** only used if Base is not hooked up */
	UPROPERTY(EditAnywhere, Category = MaterialExpressionLandscapeLayerWeight, meta = (OverridingInputProperty = "Base"))
	FVector ConstBase;

public:

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	LANDSCAPE_API virtual bool IsResultMaterialAttributes(int32 OutputIndex) override;
	LANDSCAPE_API virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual bool CanRenameNode() const override { return true; }
	LANDSCAPE_API virtual FString GetEditableName() const override;
	LANDSCAPE_API virtual void SetEditableName(const FString& NewName) override;
	LANDSCAPE_API virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	LANDSCAPE_API virtual bool MatchesSearchQuery(const TCHAR* SearchQuery) override;
#endif
	LANDSCAPE_API virtual UObject* GetReferencedTexture() const override;
	LANDSCAPE_API virtual ReferencedTextureArray GetReferencedTextures() const override;
	virtual bool CanReferenceTexture() const override { return true; }
#if WITH_EDITOR
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override {return static_cast<EMaterialValueType>(MCT_Float | MCT_MaterialAttributes);}

	/**
	 * Gets the landscape layer names
	 */
	LANDSCAPE_API virtual void GetLandscapeLayerNames(TArray<FName>& OutLayers) const override;
#endif //WITH_EDITOR
	//~ End UMaterialExpression Interface

	//~ Begin UObject Interface
	/**
	 * Do any object-specific cleanup required immediately after loading an object,
	 * and immediately after any undo/redo.
	 */
	LANDSCAPE_API virtual void PostLoad() override;
	//~ End UObject Interface
};



