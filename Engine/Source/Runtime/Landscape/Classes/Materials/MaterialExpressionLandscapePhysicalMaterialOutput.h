// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Materials/MaterialExpressionCustomOutput.h"
#include "MaterialValueType.h"
#include "MaterialExpressionLandscapePhysicalMaterialOutput.generated.h"

/** Structure linking a material expression input with a physical material. For use by UMaterialExpressionLandscapePhysicalMaterialOutput. */
USTRUCT()
struct FPhysicalMaterialInput
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = PhysicalMaterial)
	TObjectPtr<class UPhysicalMaterial> PhysicalMaterial;

	UPROPERTY(meta = (RequiredInput = "true"))
	FExpressionInput Input;

	FPhysicalMaterialInput()
		: PhysicalMaterial(nullptr)
	{}
};

/** 
 * Custom output node to write out physical material weights.
 * This can be used to generate the dominant physical material for each point on a landscape.
 * Note that the use of a material output node to generate this information is optional and when a node of this type is not present we fall back on a CPU path which analyzes landscape layer data.
 */
UCLASS(collapsecategories, hidecategories=Object, MinimalAPI)
class UMaterialExpressionLandscapePhysicalMaterialOutput : public UMaterialExpressionCustomOutput
{
	GENERATED_UCLASS_BODY()

	// Maximum number of supported grass types on a given landscape material. Whenever adjusting this, make sure to update LandscapeGrassWeight.usf accordingly:
	static constexpr int32 MaxPhysicalMaterials = 16;

	/** Array of physical material inputs. */
	UPROPERTY(EditAnywhere, Category = PhysicalMaterial)
	TArray< FPhysicalMaterialInput > Inputs;

#if WITH_EDITOR
	//~ Begin UObject Interface
	LANDSCAPE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface

	//~ Begin UMaterialExpression Interface
	LANDSCAPE_API virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	LANDSCAPE_API virtual TArrayView<FExpressionInput*> GetInputsView() override;
	LANDSCAPE_API virtual FExpressionInput* GetInput(int32 InputIndex) override;
	LANDSCAPE_API virtual FName GetInputName(int32 InputIndex) const override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override { return MCT_Float; }
	LANDSCAPE_API virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;

	//~ End UMaterialExpression Interface
#endif

	//~ Begin UMaterialExpressionCustomOutput Interface
	virtual int32 GetNumOutputs() const override { return Inputs.Num(); }
	virtual int32 GetMaxOutputs() const { return MaxPhysicalMaterials; };
	virtual FString GetFunctionName() const override { return TEXT("GetPhysicalMaterial"); }
	//~ End UMaterialExpressionCustomOutput Interface
};
