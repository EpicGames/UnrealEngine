// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Materials/MaterialExpression.h"
#include "MaterialExpressionLandscapeVisibilityMask.generated.h"

class UTexture;
struct FMaterialParameterInfo;

UCLASS(collapseCategories, hideCategories=Object, MinimalAPI)
class UMaterialExpressionLandscapeVisibilityMask : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

public:

	static LANDSCAPE_API FName ParameterName;

	//~ Begin UMaterialExpression Interface
#if WITH_EDITOR
	LANDSCAPE_API virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	LANDSCAPE_API virtual void GetCaption(TArray<FString>& OutCaptions) const override;


	/**
	 * Gets the landscape layer names
	 */
	LANDSCAPE_API virtual void GetLandscapeLayerNames(TArray<FName>& OutLayers) const override;
#endif
	LANDSCAPE_API virtual UObject* GetReferencedTexture() const override;
	LANDSCAPE_API virtual ReferencedTextureArray GetReferencedTextures() const override;
	virtual bool CanReferenceTexture() const override { return true; }
	//~ End UMaterialExpression Interface
};



