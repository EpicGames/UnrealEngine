// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "LandscapeComponent.h"
#include "Templates/ValueOrError.h"
#include "MeshMaterialShader.h"
#include "RenderGraphFwd.h"

class ALandscape;
class ALandscapeProxy;
class FLandscapeAsyncTextureReadback;
class FLandscapeComponentSceneProxy;
class FRDGTexture;
class FTextureRenderTarget2DResource;
class UTextureRenderTarget2D;
using FRDGTextureRef = FRDGTexture*;

namespace UE::Landscape
{
	UE_DEPRECATED(5.8, "Use ULandscapeComponent::CanRenderGrassMap instead")
	LANDSCAPE_API bool CanRenderGrassMap(ULandscapeComponent* Component);
	UE_DEPRECATED(5.8, "Renamed to SupportsRuntimeGrassMapGeneration (LandscapeUtils.h)")
	LANDSCAPE_API bool IsRuntimeGrassMapGenerationSupported();
} // namespace UE::Landscape

namespace UE::Landscape::Grass
{
	enum class EGrassWeightExporterFlags
	{
		None = 0,
		NeedsGrassmap = (1 << 0),
		NeedsHeightmap = (1 << 1),
		RenderImmediately = (1 << 2),
		ReadbackToCPU = (1 << 3),
		Default = NeedsGrassmap | NeedsHeightmap | RenderImmediately | ReadbackToCPU,
	};
	ENUM_CLASS_FLAGS(EGrassWeightExporterFlags);
} // UE::Landscape::Grass


// ----------------------------------------------------------------------------------
// 
// Hacky base class to avoid 8 bytes of padding after the vtable
class FLandscapeGrassWeightExporter_RenderThread_FixLayout
{
public:
	virtual ~FLandscapeGrassWeightExporter_RenderThread_FixLayout() = default;
};

// data also accessible by render thread
class FLandscapeGrassWeightExporter_RenderThread : public FLandscapeGrassWeightExporter_RenderThread_FixLayout
{
	friend class FLandscapeGrassWeightExporter;
	friend class FLandscapeGrassMapsBuilder;
	friend class FComponentRenderInfo;

private:
	FLandscapeGrassWeightExporter_RenderThread(int32 InComponentSizeVerts, int32 InSubsectionSizeQuads, int32 InNumSubsections, 
		TConstArrayView<int32> InHeightMips, UE::Landscape::Grass::EGrassWeightExporterFlags InFlags);

public:
	LANDSCAPE_API virtual ~FLandscapeGrassWeightExporter_RenderThread();

	const FIntPoint& GetTargetSize() const { return TargetSize; }

	int32 GetTileSize() const
	{
		return ComponentSizeVerts;
	}

	/** 
	 * Renders the components to the given texture. 
	 */
	LANDSCAPE_API void RenderLandscapeComponentToTexture_RenderThread(FRDGBuilder& GraphBuilder, FRDGTextureRef OutputTexture);

	enum class EHeightmapTextureInfoError
	{
		InvalidComponentKey,
		InvalidFlags,
		InvalidMipIndex,
	};
	/**
	 * Return the location of the height data for the requested component / mip in the output texture
	 * @param InComponentKey Key of the component being requested
	 * @param InMipIndex Requested mip index. If < 0, will return mip 0. Otherwise, the mip needs to have been requested in HeightMips
	 * 
	 * @return If the component / mip data is found, FIntRect contains the pixels containing the requested height data in the generated texture (channels R and G). 
	 *  Returns an error otherwise.
	 */
	LANDSCAPE_API TValueOrError<FIntRect /*OutputRect*/, EHeightmapTextureInfoError> GetTextureInfoForHeight(const FIntPoint& InComponentKey, int32 InMipIndex = INDEX_NONE);

	enum class EGrassmapTextureInfoError
	{
		InvalidComponentKey, 
		InvalidFlags,
		InvalidGrassName,
	};
	/**
	 * Return the location of the grass data for the requested component in the output texture
	 * @param InComponentKey Key of the component being requested
	 * @param InGrassName Name of the grass map being requested 
	 * 
	 * @return If the component / grass data is found, a FIntRect that contains the pixels containing the requested grass data in the generated texture, along with the channel index. 
	 *  Returns an error otherwise.
	 */
	LANDSCAPE_API TValueOrError<TPair<FIntRect /*OutputRect*/, uint8 /*ChannelIndex*/>, EGrassmapTextureInfoError> GetTextureInfoForGrass(const FIntPoint& InComponentKey, const FName& InGrassName);

private:
	/** Creates a texture and renders the component to it, and then and triggers a readback of the texture. */
	void RenderLandscapeComponentToTexture_RenderThread(FRHICommandListImmediate& RHICmdList, FLandscapeAsyncTextureReadback* AsyncReadbackPtr);

private:
	struct FComponentInfo
	{
		FComponentInfo(ULandscapeComponent* InComponent, UE::Landscape::Grass::EGrassWeightExporterFlags InFlags,
			TConstArrayView<FName> InRequestedGrassNames, TConstArrayView<int32> InRequestedHeightMips);

		int32 GetNumPasses() const
		{
			return PerPassPerChannelGrassWeightIndices.Num();
		}

		bool HasMipHeightData() const
		{
			return FirstHeightMipsPassIndex != MAX_int32;
		}

		struct FPassChannelIndex
		{
			int32 PassIndex = INDEX_NONE;
			int32 ChannelIndex = INDEX_NONE;

			bool operator == (const FPassChannelIndex& InOther) const
			{
				return (PassIndex == InOther.PassIndex) && (ChannelIndex == InOther.ChannelIndex);
			}

			bool IsValidChannelIndex() const
			{
				return (ChannelIndex >= 0) && (ChannelIndex < 4);
			}
		};

		// Identifier for this component (use its key because it's a render-thread object so we don't want to access TObjectPtr there)
		FIntPoint ComponentKey = FIntPoint(MAX_int32, MAX_int32);
		TMap<FName, FPassChannelIndex> PerGrassNamePassChannelIndices;
		TArray<FIntVector4> PerPassPerChannelGrassWeightIndices;
		FVector2D ViewOffset = FVector2D::ZeroVector;
		int32 PixelOffsetX = 0;
		FLandscapeComponentSceneProxy* SceneProxy = nullptr;
		int32 FirstHeightMipsPassIndex = MAX_int32;
		UE::Landscape::Grass::EGrassWeightExporterFlags Flags = UE::Landscape::Grass::EGrassWeightExporterFlags::None;
	};

	int32 ComponentSizeVerts = 0;
	int32 SubsectionSizeQuads = 0;
	int32 NumSubsections = 0;
	FSceneInterface* SceneInterface = nullptr;
	TArray<FComponentInfo, TInlineAllocator<1>> ComponentInfos;
	FIntPoint TargetSize = FIntPoint(ForceInit);
	TArray<int32> HeightMips;
	float PassOffsetX = 0.0f;
	FVector ViewOrigin = FVector(ForceInit);
	UE::Landscape::Grass::EGrassWeightExporterFlags Flags = UE::Landscape::Grass::EGrassWeightExporterFlags::None;

	// game thread synchronous, do not access directly from render thread
	FLandscapeAsyncTextureReadback* GameThreadAsyncReadbackPtr = nullptr;

	FMatrix ViewRotationMatrix = FMatrix::Identity;
	FMatrix ProjectionMatrix = FMatrix::Identity;
};


// ----------------------------------------------------------------------------------

class FLandscapeGrassWeightExporter : public FLandscapeGrassWeightExporter_RenderThread
{
	friend class FLandscapeGrassMapsBuilder;

public:
	UE_DEPRECATED(5.8, "Use the new constructor with explicit flags and explicit grass names")
	LANDSCAPE_API FLandscapeGrassWeightExporter(ALandscapeProxy* InLandscapeProxy, TArrayView<ULandscapeComponent* const> InLandscapeComponents, bool bInNeedsGrassmap = true, bool bInNeedsHeightmap = true, const TArray<int32>& InHeightMips = {}, bool bInRenderImmediately = true, bool bInReadbackToCPU = true);

	/**
	 * @param InLandscape - Parent landscape. All requested component must share the same parent landscape
	 * @param InRequestedComponents - List of components for which to export weight (/height) maps
	 * @param InFlags - Flags for specializing what to export and how to read back the results
	 * @param InRequestedGrassNames - (Optional) List of grass map names requested for readback. If empty, all grass maps are exported
	 * @param InRequestedHeightMips - (Optional) List of mip levels requested for reading back heightmaps
	 */
	LANDSCAPE_API FLandscapeGrassWeightExporter(ALandscape* InLandscape, TArrayView<ULandscapeComponent* const> InRequestedComponents, UE::Landscape::Grass::EGrassWeightExporterFlags InFlags,
		TConstArrayView<FName> InRequestedGrassNames = {}, TConstArrayView<int32> InRequestedHeightMips = {});

	// If using the async readback path, check its status and update if needed. Return true when the AsyncReadbackResults are available.
	// You must call this periodically, or the async readback may not complete.
	// bInForceFinish will force the RenderThread to wait until GPU completes the readback, ensuring the readback is completed after the render thread executes the command.
	// NOTE: you may still see false returned, this just means the render thread hasn't executed the command yet.
	bool CheckAndUpdateAsyncReadback(bool& bOutRenderCommandsQueued, const bool bInForceFinish = false);

	// return true if the async readback is complete.  (Does not update the readback state)
	bool IsAsyncReadbackComplete();

	// Fetches the results from the GPU texture and translates them into FLandscapeComponentGrassDatas.
	// If using async readback, requires AsyncReadback to be complete before calling this.
	// bFreeAsyncReadback if true will call FreeAsyncReadback() to free the readback resource (otherwise you must do it manually)
	TMap<ULandscapeComponent*, TUniquePtr<FLandscapeComponentGrassData>, TInlineSetAllocator<1>> FetchResults(bool bFreeAsyncReadback);

private:
	void FreeAsyncReadback();

	// Applies the results using pre-fetched data.
	static void ApplyResults(TMap<ULandscapeComponent*, TUniquePtr<FLandscapeComponentGrassData>, TInlineSetAllocator<1>>& Results);

	// Fetches the results and applies them to the landscape components
	// If using async readback, requires AsyncReadback to be complete before calling this.
	void ApplyResults();

	void CancelAndSelfDestruct();

private:
	TObjectPtr<ALandscape> Landscape;
	TArray<TObjectPtr<ULandscapeGrassType>> GrassTypes;
};
