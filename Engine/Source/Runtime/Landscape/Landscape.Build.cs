// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class Landscape : ModuleRules
{
	public Landscape(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"TargetPlatform",
				"DerivedDataCache",
				"Shaders"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"ApplicationCore",
				"Engine",
				"RenderCore",
				"ImageCore",
				"MathCore",
				"RHI",
				"Renderer",
				"Foliage",
				"GeometryCore",
				"TraceLog",
				"DeveloperSettings"
			}
		);

		SetupModulePhysicsSupport(Target);

		if (Target.Type == TargetType.Editor || Target.Type == TargetType.Program)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"MeshDescription",
					"StaticMeshDescription",
                    "MeshUtilitiesCommon"
				}
			);
		}

		if (Target.bBuildEditor == true)
		{
			// TODO: Remove all landscape editing code from the Landscape module!!!
			PrivateIncludePathModuleNames.Add("LandscapeEditor");

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"EditorFramework",
					"UnrealEd",
					"MaterialUtilities",
					"SlateCore",
					"Slate",
					"GeometryCore",
					"MeshUtilities",
					"MeshUtilitiesCommon",
					"MeshBuilderCommon",
					"MeshBuilder",
					"ActionableMessage"
				}
			);

			DynamicallyLoadedModuleNames.Add("NaniteBuilder");
			PrivateIncludePathModuleNames.Add("NaniteBuilder");

			CircularlyReferencedDependentModules.AddRange(
				new string[] {
					"UnrealEd",
					"MaterialUtilities",
					"MeshUtilities",
				}
			);
		}
	}
}
