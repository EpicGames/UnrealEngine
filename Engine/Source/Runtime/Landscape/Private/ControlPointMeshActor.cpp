// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ControlPointMeshActor.cpp: ControlPoint mesh actor class implementation.
=============================================================================*/

#include "ControlPointMeshActor.h"
#include "ControlPointMeshComponent.h"
#include "Engine/CollisionProfile.h"
#include "Logging/TokenizedMessage.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Misc/MapErrors.h"
#include "Engine/StaticMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ControlPointMeshActor)

#define LOCTEXT_NAMESPACE "ControlPointMeshActor"

AControlPointMeshActor::AControlPointMeshActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetCanBeDamaged(false);

	ControlPointMeshComponent = ObjectInitializer.CreateDefaultSubobject<UControlPointMeshComponent>(this, TEXT("ControlPointMeshComponent0"));
	ControlPointMeshComponent->SetCollisionProfileName(UCollisionProfile::BlockAll_ProfileName);
	ControlPointMeshComponent->Mobility = EComponentMobility::Static;
	ControlPointMeshComponent->SetGenerateOverlapEvents(false);

	RootComponent = ControlPointMeshComponent;
}


FString AControlPointMeshActor::GetDetailedInfoInternal() const
{
	check(ControlPointMeshComponent != nullptr);
	return ControlPointMeshComponent->GetDetailedInfoInternal();
}

void AControlPointMeshActor::SetMobility(EComponentMobility::Type InMobility)
{
	check(ControlPointMeshComponent != nullptr);
	ControlPointMeshComponent->SetMobility(InMobility);
}

#if WITH_EDITOR

bool AControlPointMeshActor::GetReferencedContentObjects(TArray<UObject*>& Objects) const
{
	Super::GetReferencedContentObjects(Objects);

	check(ControlPointMeshComponent != nullptr);
	if (ControlPointMeshComponent->GetStaticMesh() != nullptr)
	{
		Objects.Add(ControlPointMeshComponent->GetStaticMesh());
	}
	return true;
}

void AControlPointMeshActor::CheckForErrors()
{
	Super::CheckForErrors();

	FMessageLog MapCheck("MapCheck");

	if (ControlPointMeshComponent->GetStaticMesh() == nullptr)
	{
		MapCheck.Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_ControlPointMeshNull", "ControlPoint mesh actor has NULL StaticMesh property")))
			->AddToken(FMapErrorToken::Create(FMapErrors::StaticMeshNull));
	}
}

#endif // WITH_EDITOR

/** Returns ControlPointMeshComponent subobject **/
UControlPointMeshComponent* AControlPointMeshActor::GetControlPointMeshComponent() const { return ControlPointMeshComponent; }

#undef LOCTEXT_NAMESPACE

