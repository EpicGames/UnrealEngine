// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Landscape.cpp: Terrain rendering
=============================================================================*/

#include "Landscape.h"

#include "Algo/Count.h"
#include "Algo/ForEach.h"
#include "Algo/IsSorted.h"
#include "Algo/NoneOf.h"
#include "Algo/Transform.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/BufferArchive.h"
#include "Serialization/MemoryReader.h"
#include "UObject/RenderingObjectVersion.h"
#include "UObject/UObjectIterator.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/ConstructorHelpers.h"
#include "UObject/DevObjectVersion.h"
#include "UObject/LinkerLoad.h"
#include "UObject/Package.h"
#include "Components/RuntimeVirtualTextureComponent.h"
#include "Framework/Application/SlateApplication.h"
#include "LandscapePrivate.h"
#include "LandscapeStreamingProxy.h"
#include "LandscapeInfo.h"
#include "LightMap.h"
#include "Engine/MapBuildDataRegistry.h"
#include "ShadowMap.h"
#include "LandscapeComponent.h"
#include "LandscapeLayerInfoObject.h"
#include "LandscapeInfoMap.h"
#include "EditorSupportDelegates.h"
#include "LandscapeMeshProxyComponent.h"
#include "LandscapeNaniteComponent.h"
#include "LandscapeRender.h"
#include "LandscapeModule.h"
#include "LandscapePrivate.h"
#include "Logging/StructuredLog.h"
#include "Logging/TokenizedMessage.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Misc/MapErrors.h"
#include "Misc/PackageSegment.h"
#include "DerivedDataCacheInterface.h"
#include "Interfaces/ITargetPlatform.h"
#include "LandscapeMeshCollisionComponent.h"
#include "MaterialCachedData.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "LandscapeMaterialInstanceConstant.h"
#include "Engine/CollisionProfile.h"
#include "LandscapeMeshProxyActor.h"
#include "Materials/MaterialExpressionLandscapeLayerWeight.h"
#include "Materials/MaterialExpressionLandscapeLayerSwitch.h"
#include "Materials/MaterialExpressionLandscapeLayerSample.h"
#include "Materials/MaterialExpressionLandscapeLayerBlend.h"
#include "Materials/MaterialExpressionLandscapeVisibilityMask.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "ProfilingDebugging/CookStats.h"
#include "ILandscapeSplineInterface.h"
#include "LandscapeGrassType.h"
#include "LandscapeSplineActor.h"
#include "LandscapeSplinesComponent.h"
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "EngineUtils.h"
#include "ComponentRecreateRenderStateContext.h"
#include "LandscapeWeightmapUsage.h"
#include "LandscapeSubsystem.h"
#include "LandscapeGroup.h"
#include "LandscapeGrassMapsBuilder.h"
#include "LandscapeCulling.h"
#include "ContentStreaming.h"
#include "UObject/ObjectSaveContext.h"
#include "GlobalShader.h"
#include "ShaderParameterStruct.h"
#include "PixelShaderUtils.h"
#include "Rendering/Texture2DResource.h"
#include "RenderCaptureInterface.h"
#include "VisualLogger/VisualLogger.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "NaniteSceneProxy.h"
#include "Misc/ArchiveMD5.h"
#include "LandscapeEditLayer.h"
#include "LandscapeTextureStorageProvider.h"
#include "LandscapeUtils.h"
#include "LandscapeUtilsPrivate.h"
#include "LandscapeVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/FortniteReleaseBranchCustomObjectVersion.h"
#include "UObject/EditorObjectVersion.h"
#include "UObject/UObjectThreadContext.h"
#include "LandscapeDataAccess.h"
#include "LandscapeNotification.h"
#include "LandscapeHeightfieldCollisionComponent.h"
#include "SystemTextures.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/WorldPartitionHandle.h"
#include "WorldPartition/Landscape/LandscapeActorDesc.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "Engine/Texture2DArray.h"
#include "Modules/ModuleManager.h"

#if WITH_EDITOR
#include "Cooker/CookEvents.h"
#include "Rendering/StaticLightingSystemInterface.h"
#include "Misc/ScopedSlowTask.h"
#include "UnrealEdGlobals.h"
#include "Editor/UnrealEdEngine.h"
#include "LandscapeEdit.h"
#include "LandscapeEditTypes.h"
#include "MaterialUtilities.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Selection.h"
#include "Engine/Texture2D.h"
#include "AssetCompilingManager.h"
#include "FileHelpers.h"
#include "ScopedTransaction.h"
#include "LandscapeSettings.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(Landscape)

/** Landscape stats */

DEFINE_STAT(STAT_LandscapeDynamicDrawTime);
DEFINE_STAT(STAT_LandscapeVFDrawTimeVS);
DEFINE_STAT(STAT_LandscapeVFDrawTimePS);
DEFINE_STAT(STAT_LandscapeComponentRenderPasses);
DEFINE_STAT(STAT_LandscapeDrawCalls);
DEFINE_STAT(STAT_LandscapeTriangles);
DEFINE_STAT(STAT_LandscapeLayersRegenerateDrawCalls);

#if ENABLE_COOK_STATS
namespace LandscapeCookStats
{
	static FCookStats::FDDCResourceUsageStats UsageStats;
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
	{
		UsageStats.LogStats(AddStat, TEXT("Landscape.Usage"), TEXT(""));
	});
}
#endif

#define LOCTEXT_NAMESPACE "Landscape"

static void PrintNumLandscapeShadows()
{
	int32 NumComponents = 0;
	int32 NumShadowCasters = 0;
	for (TObjectIterator<ULandscapeComponent> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
	{
		ULandscapeComponent* LC = *It;
		NumComponents++;
		if (LC->CastShadow && LC->bCastDynamicShadow)
		{
			NumShadowCasters++;
		}
	}
	UE_LOGF(LogLandscape, Log, "%d/%d landscape components cast shadows", NumShadowCasters, NumComponents);
}

FAutoConsoleCommand CmdPrintNumLandscapeShadows(
	TEXT("landscape.PrintNumLandscapeShadows"),
	TEXT("Prints the number of landscape components that cast shadows."),
	FConsoleCommandDelegate::CreateStatic(PrintNumLandscapeShadows)
	);

namespace UE::Landscape
{
int32 RenderCaptureNextMergeRenders = 0;
static FAutoConsoleVariableRef CVarRenderCaptureNextMergeRenders(
	TEXT("landscape.RenderCaptureNextMergeRenders"),
	RenderCaptureNextMergeRenders,
	TEXT("Trigger a render capture during the next N RenderHeightmap/RenderWeightmap(s) draws"));
} // namespace UE::Landscape

#if WITH_EDITOR

namespace UE::Landscape
{
	int32 NaniteExportCacheMaxQuadCount = 2048 * 2048;
	static FAutoConsoleVariableRef CVarNaniteExportCacheMaxQuadCount(
		TEXT("landscape.NaniteExportCacheMaxQuadCount"),
		NaniteExportCacheMaxQuadCount,
		TEXT("The maximum number of quads in a landscape proxy that will use the DDC cache when exporting the nanite mesh (any larger landscapes will be uncached).  Set to a negative number to always cache."));
}

float LandscapeNaniteAsyncDebugWait = 0.0f;
static FAutoConsoleVariableRef CVarNaniteAsyncDebugWait(
	TEXT("landscape.Nanite.AsyncDebugWait"),
	LandscapeNaniteAsyncDebugWait,
	TEXT("Time in seconds to pause the async Nanite build. Used for debugging"));

float LandscapeNaniteBuildLag = 0.25f;
static FAutoConsoleVariableRef CVarNaniteUpdateLag(
	TEXT("landscape.Nanite.UpdateLag"),
	LandscapeNaniteBuildLag,
	TEXT("Time to wait in seconds after the last landscape update before triggering a nanite rebuild"));

float LandscapeNaniteStallDetectionTimeout = 3.0f * 60.0f;	// 3 minutes
static FAutoConsoleVariableRef CVarNaniteStallDetectionTimeout(
	TEXT("landscape.Nanite.StallDetectionTimeout"),
	LandscapeNaniteStallDetectionTimeout,
	TEXT("Time, in seconds, after which we consider a landscape nanite async build to have stalled or deadlocked."));

static FAutoConsoleVariable CVarForceInvalidateNaniteOnLoad(
	TEXT("landscape.ForceInvalidateNaniteOnLoad"),
	false,
	TEXT("Trigger a rebuild of Nanite representation on load (for debugging purposes)"));

static FAutoConsoleVariable CVarSilenceSharedPropertyDeprecationFixup(
	TEXT("landscape.SilenceSharedPropertyDeprecationFixup"),
	true,
	TEXT("Silently performs the fixup of discrepancies in shared properties when handling data modified before the enforcement introduction."));

FAutoConsoleVariable CVarLandscapeSilenceInvalidPropertyOverridesMapCheckWarnings(
	TEXT("landscape.SilenceInvalidPropertyOverridesMapCheckWarnings"),
	false,
	TEXT("Issue MapCheck Info messages instead of warnings if some landscape property overrides are incorrectly set (e.g. overriding the LandscapeMaterial but leaving it empty)."));

static FAutoConsoleVariable CVarLandscapeSilenceMapCheckWarnings_Nanite(
	TEXT("landscape.Nanite.SilenceMapCheckWarnings"),
	false,
	TEXT("Issue MapCheck Info messages instead of warnings if Nanite Data is out of date"));

FAutoConsoleVariable CVarStripLayerTextureMipsOnLoad(
	TEXT("landscape.StripLayerMipsOnLoad"),
	false,
	TEXT("Remove (on load) the mip chain from textures used in layers which don't require them"));

static FAutoConsoleVariable CVarAllowGrassStripping(
	TEXT("landscape.AllowGrassStripping"),
	true,
	TEXT("Enables the conditional stripping of grass data during cook.  Disabling this means the bStripGrassWhenCooked* will be ignored."));

static const TCHAR* CVarLandscapeHeightmapCompressionModeName = TEXT("landscape.HeightmapCompressionMode");
int32 GLandscapeHeightmapCompressionMode = 1;
static FAutoConsoleVariableRef CVarLandscapeHeightmapCompressionMode(
	CVarLandscapeHeightmapCompressionModeName,
	GLandscapeHeightmapCompressionMode,
	TEXT("Defines whether compression is applied to landscapes.  Can be defined per platform.\n")
	TEXT(" 0: force disable heightmap compression on all landscapes\n")
	TEXT(" 1: force enable heightmap compression on all landscapes (default)\n"),
	ECVF_Preview | ECVF_ReadOnly);

static const TCHAR* CVarLandscapeHeightmapCompressionMipThresholdName = TEXT("landscape.HeightmapCompressionMipThreshold");
int32 GLandscapeHeightmapCompressionMipThreshold = 32;
static FAutoConsoleVariableRef CVarLandscapeHeightmapCompressionMipThreshold(
	CVarLandscapeHeightmapCompressionMipThresholdName,
	GLandscapeHeightmapCompressionMipThreshold,
	TEXT("Sets the minimum size for which heightmap mips are stored in a compressed layout.  Can be defined per platform.\n")
	TEXT("Below this size, mips are stored in an uncompressed layout.\n")
	TEXT("Default threshold is 32, though some slower platforms may have a higher default threshold out of the box.\n"),
	ECVF_Preview | ECVF_ReadOnly);

bool GLandscapePrioritizeDirtyRVTPages = true;
static FAutoConsoleVariableRef CVarPrioritizeDirtyRVTPages(
	TEXT("landscape.PrioritizeDirtyRVTPages"),
	GLandscapePrioritizeDirtyRVTPages,
	TEXT("Prioritize RVT pages affected by the landscape tools, so that they get updated prior to others. Improves reactiveness when invalidating large areas of the RVT.")
);
#endif // WITH_EDITOR

TAutoConsoleVariable<int32> CVarRenderNaniteLandscape(
	TEXT("landscape.RenderNanite"), 1,
	TEXT("Render Landscape using Nanite."),
	ECVF_Scalability | ECVF_RenderThreadSafe
);

extern int32 GGrassEnable;
extern int32 GGrassMapUseRuntimeGeneration;
extern FAutoConsoleVariableRef CVarGrassMapUseRuntimeGeneration;

struct FCompareULandscapeComponentClosest
{
	FCompareULandscapeComponentClosest(const FIntPoint& InCenter) : Center(InCenter) {}

	FORCEINLINE bool operator()(const ULandscapeComponent* A, const ULandscapeComponent* B) const
	{
		const FIntPoint ABase = A->GetSectionBase();
		const FIntPoint BBase = B->GetSectionBase();

		int32 DistA = (ABase - Center).SizeSquared();
		int32 DistB = (BBase - Center).SizeSquared();

		return DistA < DistB;
	}

	FIntPoint Center;

};

ULandscapeComponent::ULandscapeComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bNaniteActive(false)
#if WITH_EDITORONLY_DATA
	, LayerUpdateFlagPerMode(0)
	, bPendingCollisionDataUpdate(false)
	, bPendingLayerCollisionDataUpdate(false)
	, WeightmapsHash(0)
	, SplineHash(0)
	, PhysicalMaterialHash(0)
#endif
	, GrassData(MakeShared<FLandscapeComponentGrassData>())
	, ChangeTag(0)
{
	SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	SetGenerateOverlapEvents(false);

	bUseAsOccluder = true;
	bAllowCullDistanceVolume = false;
	CollisionMipLevel = 0;
	StaticLightingResolution = 0.f; // Default value 0 means no overriding

	MaterialInstances.AddDefaulted(); // make sure we always have a MaterialInstances[0]	
	LODIndexToMaterialIndex.AddDefaulted(); // make sure we always have a MaterialInstances[0]	

	HeightmapScaleBias = FVector4(0.0f, 0.0f, 0.0f, 1.0f);
	WeightmapScaleBias = FVector4(0.0f, 0.0f, 0.0f, 1.0f);

	bBoundsChangeTriggersStreamingDataRebuild = true;
	ForcedLOD = -1;
	LODBias = 0;
#if WITH_EDITORONLY_DATA
	LightingLODBias = -1; // -1 Means automatic LOD calculation based on ForcedLOD + LODBias
#endif

	Mobility = EComponentMobility::Static;

#if WITH_EDITORONLY_DATA
	EditToolRenderData = FLandscapeEditToolRenderData();
#endif

	// We don't want to load this on the server, this component is for graphical purposes only
	AlwaysLoadOnServer = false;

	// Default sort priority of landscape to -1 so that it will default to the first thing rendered in any runtime virtual texture
	TranslucencySortPriority = -1;
}

int32 ULandscapeComponent::GetMaterialInstanceCount(bool InDynamic) const
{
	ALandscapeProxy* Actor = GetLandscapeProxy();

	if (Actor != nullptr && Actor->bUseDynamicMaterialInstance && InDynamic)
	{
		return MaterialInstancesDynamic.Num();
	}

	return MaterialInstances.Num();
}

UMaterialInstance* ULandscapeComponent::GetMaterialInstance(int32 InIndex, bool InDynamic) const
{
	ALandscapeProxy* Actor = GetLandscapeProxy();

	if (Actor != nullptr && Actor->bUseDynamicMaterialInstance && InDynamic)
	{
		check(MaterialInstancesDynamic.IsValidIndex(InIndex));
		return MaterialInstancesDynamic[InIndex];
	}

	check(MaterialInstances.IsValidIndex(InIndex));
	return MaterialInstances[InIndex];
}

int32 ULandscapeComponent::GetCurrentRuntimeMaterialInstanceCount() const
{
	ALandscapeProxy* Proxy = GetLandscapeProxy();
	UWorld* World = GetWorld();
	const bool bIsMobile = (World != nullptr) && (World->GetFeatureLevel() == ERHIFeatureLevel::ES3_1);
	if (bIsMobile)
	{
		return MobileMaterialInterfaces.Num();
	}

	bool bDynamic = Proxy->bUseDynamicMaterialInstance;
	return GetMaterialInstanceCount(bDynamic);
}

UMaterialInterface* ULandscapeComponent::GetCurrentRuntimeMaterialInterface(int32 InIndex) const
{
	ALandscapeProxy* Proxy = GetLandscapeProxy();
	UWorld* World = GetWorld();
	bool bIsMobile = (World != nullptr) && (World->GetFeatureLevel() == ERHIFeatureLevel::ES3_1);
	if (bIsMobile)
	{
		check(MobileMaterialInterfaces.IsValidIndex(InIndex));
		return MobileMaterialInterfaces[InIndex];
	}

	bool bDynamic = Proxy->bUseDynamicMaterialInstance;
	return GetMaterialInstance(InIndex, bDynamic);
}

UMaterialInstanceDynamic* ULandscapeComponent::GetMaterialInstanceDynamic(int32 InIndex) const
{
	ALandscapeProxy* Actor = GetLandscapeProxy();

	if (Actor != nullptr && Actor->bUseDynamicMaterialInstance)
	{
		if (MaterialInstancesDynamic.IsValidIndex(InIndex))
		{
			return MaterialInstancesDynamic[InIndex];
		}
	}

	return nullptr;
}

#if WITH_EDITOR

void ULandscapeComponent::BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform)
{
	Super::BeginCacheForCookedPlatformData(TargetPlatform);

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		if (TargetPlatform->SupportsFeature(ETargetPlatformFeatures::MobileRendering))
		{
			CheckGenerateMobilePlatformData(/*bIsCooking = */ true, TargetPlatform);
		}
	}
}

void ALandscapeProxy::CheckGenerateMobilePlatformData(bool bIsCooking, const ITargetPlatform* TargetPlatform)
{
	for (ULandscapeComponent* Component : LandscapeComponents)
	{
		Component->CheckGenerateMobilePlatformData(bIsCooking, TargetPlatform);
	}
}

bool ALandscapeProxy::IsNaniteMeshUpToDate() const
{
	if (IsNaniteEnabled() && !HasAnyFlags(RF_ClassDefaultObject) && LandscapeComponents.Num() > 0)
	{
		// Consider that the Nanite mesh is up-to-date in the standalone editor (!GIsEditor) because the nanite content Id cannot be relied up in that mode and we still want to display Nanite. 
		//  This means the Nanite mesh might not be accurate, though. We could rebuild it then, just as we do on cook, but that would impact the standalone editor performance, and we don't want that. 
		if (GIsEditor)
		{
			const FGuid NaniteContentId = GetNaniteContentId();
			return AreNaniteComponentsValid(NaniteContentId);
		}
	}

	return true;
}

FGraphEventRef ALandscapeProxy::UpdateNaniteRepresentationAsync(const ITargetPlatform* InTargetPlatform)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ALandscapeProxy::UpdateNaniteRepresentationAsync);
	FGraphEventRef BatchBuildEvent;
	check(GIsEditor);

	if (IsNaniteEnabled() && !HasAnyFlags(RF_ClassDefaultObject) && LandscapeComponents.Num() > 0)
	{
		int32 NumNewNaniteComponents = NumNaniteRequiredComponents();
		if (NumNewNaniteComponents != NaniteComponents.Num())
		{
			RemoveNaniteComponents();
			CreateNaniteComponents(NumNewNaniteComponents);
		}

		const bool bNaniteContentDirty = !IsNaniteMeshUpToDate();
		if (bNaniteContentDirty && IsRunningCookCommandlet())
		{
			UE_LOGF(LogLandscape, Display, "Landscape Nanite out of date. Map requires resaving. Actor: '%ls' Package: '%ls'", *GetActorNameOrLabel(), *GetPackage()->GetName());
		}

		// When nanite content is out of date update the source landscape components for each nanite component
		if (bNaniteContentDirty)
		{
			SetSourceComponentsForNaniteComponents();
		}

		FGraphEventArray UpdateDependencies;
		for (int32 i = 0; i < NumNewNaniteComponents; ++i)
		{
			FGraphEventArray SingleProxyDependencies;

			if (bNaniteContentDirty)
			{
				FGraphEventRef ComponentProcessTask = NaniteComponents[i]->InitializeForLandscapeAsync(this, GetNaniteContentId(), NaniteComponents[i]->GetSourceLandscapeComponents(), i);
				SingleProxyDependencies.Add(ComponentProcessTask);
			}

			// TODO: Add a flag that only initializes the platform if we called InitializeForLandscape during the PreSave for this or a previous platform
			TWeakObjectPtr<ULandscapeNaniteComponent> WeakComponent = NaniteComponents[i];
			TWeakObjectPtr<ALandscapeProxy> WeakProxy = this;
			FGraphEventRef FinalizeEvent = FFunctionGraphTask::CreateAndDispatchWhenReady([WeakComponent, WeakProxy, Name = GetActorNameOrLabel(), InTargetPlatform]() {
				if (!WeakComponent.IsValid() || !WeakProxy.IsValid())
				{
					UE_LOGF(LogLandscape, Log, "UpdateNaniteRepresentationAsync Component on: '%ls' Is Invalid", *Name);
					return;
				}
				WeakComponent->InitializePlatformForLandscape(WeakProxy.Get(), InTargetPlatform);
				WeakComponent->UpdatedSharedPropertiesFromActor();
			},
				TStatId(),
				&SingleProxyDependencies,
				ENamedThreads::GameThread);

			UpdateDependencies.Add(FinalizeEvent);
		}

		BatchBuildEvent = FFunctionGraphTask::CreateAndDispatchWhenReady([] {}, TStatId(), &UpdateDependencies, ENamedThreads::GameThread);

		// Register the finalize build event so that it can be tracked globally : 
		if (UWorld* World = GetWorld())
		{
			ULandscapeSubsystem* LandscapeSubsystem = World->GetSubsystem<ULandscapeSubsystem>();
			check(LandscapeSubsystem != nullptr);
			LandscapeSubsystem->AddNaniteFinalizeBuildEvent(BatchBuildEvent);
		}
	}
	else
	{
		InvalidateNaniteRepresentation(/* bInCheckContentId = */false);
	}

	return BatchBuildEvent;
}

void ALandscapeProxy::UpdateNaniteRepresentation(const ITargetPlatform* InTargetPlatform)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ALandscapeProxy::UpdateNaniteRepresentation);
	check(IsInGameThread());

	if (!GIsEditor)
	{
		return;
	}

	// If Nanite was disabled, we want to remove Nanite components no matter what 
	if (!IsNaniteEnabled())
	{
		RemoveNaniteComponents();
		return;
	}

	FGraphEventRef GraphEvent = UpdateNaniteRepresentationAsync(InTargetPlatform);

	UWorld* World = GetWorld();
	if (World == nullptr)
	{
		return;
	}
	
	ULandscapeSubsystem* LandscapeSubsystem = World->GetSubsystem<ULandscapeSubsystem>();
	check(LandscapeSubsystem != nullptr);

	if (!GraphEvent.IsValid())
	{
		return;
	}

	if (!LandscapeSubsystem->IsMultithreadedNaniteBuildEnabled() || IsRunningCookCommandlet())
	{
		const bool bAllNaniteBuildsDone = LandscapeSubsystem->FinishAllNaniteBuildsInFlightNow(ULandscapeSubsystem::EFinishAllNaniteBuildsInFlightFlags::Default);
		// Not passing ULandscapeSubsystem::EFinishAllNaniteBuildsInFlightFlags::AllowCancel, so there should be no way that FinishAllNaniteBuildsInFlightNow returns false :
		check(bAllNaniteBuildsDone);
	}
}

void ALandscapeProxy::InvalidateNaniteRepresentation(bool bInCheckContentId)
{
	if (HasNaniteComponents())
	{
		// If Nanite was disabled, we want to remove Nanite components no matter what
		if (!IsNaniteEnabled() || !bInCheckContentId || !IsNaniteMeshUpToDate())
		{
			RemoveNaniteComponents();
		}
	}
}

void ALandscapeProxy::InvalidateOrUpdateNaniteRepresentation(bool bInCheckContentId, const ITargetPlatform* InTargetPlatform)
{
	UWorld* World = GetWorld();
	if (World == nullptr)
	{
		return;
	}
	
	ULandscapeSubsystem* Subsystem = World->GetSubsystem<ULandscapeSubsystem>();
	check(Subsystem != nullptr);

	if (Subsystem->IsLiveNaniteRebuildEnabled())
	{
		UpdateNaniteRepresentation(InTargetPlatform);
	}
	else
	{
		InvalidateNaniteRepresentation(bInCheckContentId);
	}
}

FGuid ALandscapeProxy::GetNaniteContentId() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ALandscapeProxy::GetNaniteContentId);

	// This function isn't meant to be called outside the actual editor (i.e. not in the -game editor), as it relies on ULandscapeTextureHash, which is editor-only
	// The only purpose of querying this is to detect whether Nanite landscape is out of date and regenerate it, which only doable in the editor anyway. 
	check(GIsEditor); 
	
	if (!IsNaniteEnabled())
	{
		return FGuid();
	}

	FBufferArchive ContentStateAr;

	int32 LocalNaniteLODIndex = GetNaniteLODIndex();
	ContentStateAr << LocalNaniteLODIndex;

	struct FCompareULandscapeComponentBySectionBase
	{
		FORCEINLINE bool operator()(const ULandscapeComponent* A, const ULandscapeComponent* B) const
		{
			if (!A)
			{
				return true;
			}
			if (!B)
			{
				return false;
			}
			// Sort components based on their SectionBase (i.e. 2D index relative to the entire landscape) to ensure stable ID generation
			return (A->GetSectionBase().X == B->GetSectionBase().X) ? (A->GetSectionBase().Y < B->GetSectionBase().Y) : (A->GetSectionBase().X < B->GetSectionBase().X);
		}
	};
	TArray<ULandscapeComponent*> StableOrderComponents(LandscapeComponents);
	Algo::Sort(StableOrderComponents, FCompareULandscapeComponentBySectionBase());

	for (ULandscapeComponent* Component : StableOrderComponents)
	{
		if (Component == nullptr)
		{
			continue;
		}

		// Bump if changes to ULandscapeNaniteComponent::InitializeForLandscape() need to be enforced.
		static FGuid ExportRawMeshGuid("36208D9A475B4D93B33BF84FFEDA1536");
		ContentStateAr << ExportRawMeshGuid;

		FGuid HeightmapGuid = ULandscapeTextureHash::GetHash(Component->GetHeightmap());
		ContentStateAr << HeightmapGuid;

		// Take into account the Heightmap offset per component
		ContentStateAr << Component->HeightmapScaleBias.Z;
		ContentStateAr << Component->HeightmapScaleBias.W;

		// Visibility affects the generated Nanite mesh so it has to be taken into account :
		//  Note : visibility might be different at runtime if using a masked material (per-pixel visibility) but we obviously cannot take that into account
		//  when baking the visibility into the mesh like we do with Nanite landscape
		if (Component->ComponentHasVisibilityPainted())
		{
			const TArray<UTexture2D*>& WeightmapTextures = Component->GetWeightmapTextures();
			const TArray<FWeightmapLayerAllocationInfo>& AllocInfos = Component->GetWeightmapLayerAllocations();
			for (const FWeightmapLayerAllocationInfo& AllocInfo : AllocInfos)
			{
				if (AllocInfo.IsAllocated() && AllocInfo.LayerInfo == ALandscapeProxy::VisibilityLayer)
				{
					UTexture2D* VisibilityWeightmap = WeightmapTextures[AllocInfo.WeightmapTextureIndex];
					check(VisibilityWeightmap != nullptr);

					// TODO [jonathan.bard] : technically, this is not good, we would need to only check the hash of AllocInfo.WeightmapTextureChannel. We'll leave it as is, though, for 
					//  as long as we don't store the source weightmaps individually, so that this function stays fast : 
					FGuid VisibilityWeightmapGuid = VisibilityWeightmap->Source.GetId();
					ContentStateAr << VisibilityWeightmapGuid;
				}
			}
		}
	}

	// landscape nanite settings which might affect the resultant Nanite Static Mesh.
	int32 NaniteSkirtEnabled = bNaniteSkirtEnabled;
	float NaniteSkirtDepthTest = bNaniteSkirtEnabled ? NaniteSkirtDepth : 0.0f; // The hash should only change if Skirts are enabled.
	ContentStateAr << NaniteSkirtEnabled;
	ContentStateAr << NaniteSkirtDepthTest;
	int32 NanitePositionPrecisionCopy = NanitePositionPrecision;  
	ContentStateAr << NanitePositionPrecisionCopy;
	float NaniteMaxEdgeLengthFactorCopy = NaniteMaxEdgeLengthFactor; 
	ContentStateAr << NaniteMaxEdgeLengthFactorCopy;

	uint32 Hash[5];
	FSHA1::HashBuffer(ContentStateAr.GetData(), ContentStateAr.Num(), (uint8*)Hash);
	return FGuid(Hash[0] ^ Hash[4], Hash[1], Hash[2], Hash[3]);
}

void ALandscapeProxy::CheckForErrors()
{
	Super::CheckForErrors();

	ValidateGrassTypeOverrides();
}

void ALandscapeProxy::ValidateGrassTypeOverrides() 
{
	if (GrassTypesOverrides.IsEmpty())
	{
		return;
	}

 	TSet<UMaterialInterface*> LandscapeMaterials;
	RetrieveAllLandscapeMaterials(LandscapeMaterials);

	TArray<FString> MissingGrassOverrides;
	for (const auto& ItPair : GrassTypesOverrides)
	{
		FName GrassName = ItPair.Key.Name;
		if (!GrassName.IsNone())
		{
			// If that grass name is unknown to all of our landscape materials, report : 
			if (Algo::NoneOf(LandscapeMaterials, [&GrassName](UMaterialInterface* InMaterial) 
				{ 
					return InMaterial->GetCachedExpressionData().NamedGrassTypes.Contains(GrassName);
				}))
			{ 
				MissingGrassOverrides.AddUnique(GrassName.ToString());
			}
		}
	}

	if (!MissingGrassOverrides.IsEmpty())
	{
		TArray<FString> LandscapeMaterialStrings;
		LandscapeMaterialStrings.Reserve(LandscapeMaterials.Num());
		Algo::Transform(LandscapeMaterials, LandscapeMaterialStrings, [](UMaterialInterface* InMaterial) { return InMaterial->GetName(); });

		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("NumMissingGrassOverrides"), MissingGrassOverrides.Num());
		Arguments.Add(TEXT("MissingGrassOverrides"), FText::FromString(FString::Join(MissingGrassOverrides, TEXT(","))));
		Arguments.Add(TEXT("NumLandscapeMaterials"), LandscapeMaterialStrings.Num());
		Arguments.Add(TEXT("LandscapeMaterials"), FText::FromString(FString::Join(LandscapeMaterialStrings, TEXT(","))));

		FMessageLog MessageLog("MapCheck");
		MessageLog.Warning()->AddToken(FUObjectToken::Create(this, FText::FromString(GetActorNameOrLabel())))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_InvalidLandscapeGrassTypesOverride", 
				"None of the following grass type overrides '{MissingGrassOverrides}' can be found in the proxy landscape "
				"{NumLandscapeMaterials}|plural(one=material,other=materials) '{LandscapeMaterials}'. {NumMissingGrassOverrides}|plural(one=This override,other=These overrides) will be ineffective."), Arguments)))
			->AddToken(FMapErrorToken::Create(FMapErrors::LandscapeProxyInvalidGrassTypesOverride_Warning));
	}
}

void ULandscapeComponent::CheckGenerateMobilePlatformData(bool bIsCooking, const ITargetPlatform* TargetPlatform)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeComponent::CheckGenerateMobilePlatformData);

	// Regenerate platform data only when it's missing or there is a valid hash-mismatch.
	FBufferArchive ComponentStateAr;
	SerializeStateHashes(ComponentStateAr);

	// Serialize the version guid as part of the hash so we can invalidate DDC data if needed
	FString MobileVersion = FDevSystemGuids::GetSystemGuid(FDevSystemGuids::Get().LANDSCAPE_MOBILE_COOK_VERSION).ToString();
	ComponentStateAr << MobileVersion;

	bool IsTextureArrayEnabled = UE::Landscape::Private::IsMobileWeightmapTextureArrayEnabled();
	ComponentStateAr << IsTextureArrayEnabled;

	uint32 Hash[5];
	FSHA1::HashBuffer(ComponentStateAr.GetData(), ComponentStateAr.Num(), (uint8*)Hash);
	FGuid NewSourceHash = FGuid(Hash[0] ^ Hash[4], Hash[1], Hash[2], Hash[3]);

	bool bHashMismatch = MobileDataSourceHash != NewSourceHash;
	bool bMissingPixelData = MobileMaterialInterfaces.Num() == 0 || MaterialPerLOD.Num() == 0;
	bool bRegeneratePixelData = bMissingPixelData || bHashMismatch;

	if (bRegeneratePixelData)
	{
		GenerateMobilePlatformPixelData(bIsCooking, TargetPlatform);
	}

	MobileDataSourceHash = NewSourceHash;
}

#endif // WITH_EDITOR

void ULandscapeComponent::SetForcedLOD(int32 InForcedLOD)
{
	SetLOD(/*bForced = */true, InForcedLOD);
}

void ULandscapeComponent::SetLODBias(int32 InLODBias)
{
	SetLOD(/*bForced = */false, InLODBias);
}

void ULandscapeComponent::SetLOD(bool bForcedLODChanged, int32 InLODValue)
{
	if (bForcedLODChanged)
	{
		ForcedLOD = InLODValue;
		if (ForcedLOD >= 0)
		{
			ForcedLOD = FMath::Clamp<int32>(ForcedLOD, 0, FMath::CeilLogTwo(SubsectionSizeQuads + 1) - 1);
		}
		else
		{
			ForcedLOD = -1;
		}
	}
	else
	{
		int32 MaxLOD = FMath::CeilLogTwo(SubsectionSizeQuads + 1) - 1;
		LODBias = FMath::Clamp<int32>(InLODValue, -MaxLOD, MaxLOD);
	}

	InvalidateLightingCache();
	MarkRenderStateDirty();

#if WITH_EDITOR
	// Update neighbor components for lighting cache (only relevant in the editor ATM) : 
	ULandscapeInfo* Info = GetLandscapeInfo();
	if (Info)
	{
		FIntPoint ComponentBase = GetSectionBase() / ComponentSizeQuads;
		FIntPoint LandscapeKey[8] =
		{
			ComponentBase + FIntPoint(-1, -1),
			ComponentBase + FIntPoint(+0, -1),
			ComponentBase + FIntPoint(+1, -1),
			ComponentBase + FIntPoint(-1, +0),
			ComponentBase + FIntPoint(+1, +0),
			ComponentBase + FIntPoint(-1, +1),
			ComponentBase + FIntPoint(+0, +1),
			ComponentBase + FIntPoint(+1, +1)
		};

		for (int32 Idx = 0; Idx < 8; ++Idx)
		{
			ULandscapeComponent* Comp = Info->XYtoComponentMap.FindRef(LandscapeKey[Idx]);
			if (Comp)
			{
				Comp->Modify();
				Comp->InvalidateLightingCache();
				Comp->MarkRenderStateDirty();
			}
		}
	}
#endif // WITH_EDITOR
}

void ULandscapeComponent::SetNaniteActive(bool bValue)
{
	if (bNaniteActive != bValue)
	{
		bNaniteActive = bValue;
		MarkRenderStateDirty();
	}
}

void ULandscapeComponent::Serialize(FArchive& Ar)
{
	LLM_SCOPE(ELLMTag::Landscape);
	Ar.UsingCustomVersion(FRenderingObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);

	bool bStripGrassData = false;
#if WITH_EDITOR
	if (Ar.IsCooking() && !HasAnyFlags(RF_ClassDefaultObject))
	{
		const ITargetPlatform* TargetPlatform = Ar.CookingTarget();

		// for -oldcook:
		// the old cooker calls BeginCacheForCookedPlatformData after the package export set is tagged, so the mobile material doesn't get saved, so we have to do CheckGenerateMobilePlatformData in serialize
		// the new cooker clears the texture source data before calling serialize, causing GeneratePlatformVertexData to crash, so we have to do CheckGenerateMobilePlatformData in BeginCacheForCookedPlatformData
		if (TargetPlatform->SupportsFeature(ETargetPlatformFeatures::MobileRendering))
		{
			CheckGenerateMobilePlatformData(/*bIsCooking = */ true, TargetPlatform);
		}

		// determine whether our target platform is going to need serialized grass data
		TSharedPtr<IConsoleVariable> TargetPlatformUseRuntimeGeneration =
			CVarGrassMapUseRuntimeGeneration->GetPlatformValueVariable(*TargetPlatform->IniPlatformName());
		check(TargetPlatformUseRuntimeGeneration.IsValid());
		bStripGrassData = TargetPlatformUseRuntimeGeneration->GetBool();

		if (ALandscapeProxy* Proxy = GetLandscapeProxy())
		{
			// Also strip grass data according to Proxy flags (when not cooking for editor)
			if (!TargetPlatform->AllowsEditorObjects())
			{
				if (CVarAllowGrassStripping->GetBool() &&
					((Proxy->bStripGrassWhenCookedClient && Proxy->bStripGrassWhenCookedServer) ||
					 (Proxy->bStripGrassWhenCookedClient && TargetPlatform->IsClientOnly()) ||
					 (Proxy->bStripGrassWhenCookedServer && TargetPlatform->IsServerOnly())))
				{
					bStripGrassData = true;
				}
			}
		}
	}

#if WITH_EDITOR
	// double check we never save an invalid cached local box to a cooked package (should always be recalculated in ALandscapeProxy::PreSave)
	if (Ar.IsSaving() && Ar.IsCooking() && !Ar.IsSerializingDefaults())
	{
		check(CachedLocalBox.GetVolume() > 0);
	}
#endif // WITH_EDITOR

	// Avoid the archiver in the PIE duplicate writer case because we want to share landscape textures & materials
	if (Ar.GetPortFlags() & PPF_DuplicateForPIE)
	{
		if (Ar.IsLoading())
		{
			Super::Serialize(Ar);
		}

		TArray<UObject**> TexturesAndMaterials;
		TexturesAndMaterials.Add((UObject**)&HeightmapTexture);
		for (TObjectPtr<UTexture2D>& WeightmapTexture : WeightmapTextures)
		{
			TexturesAndMaterials.Add((UObject**)&static_cast<UTexture2D*&>(WeightmapTexture));
		}
		for (TObjectPtr<UTexture2D>& MobileWeightmapTexture : MobileWeightmapTextures)
		{
			TexturesAndMaterials.Add((UObject**)&static_cast<UTexture2D*&>(MobileWeightmapTexture));
		}

		TexturesAndMaterials.Add((UObject**)&static_cast<UTexture2DArray*&>(MobileWeightmapTextureArray));

		for (auto& ItPair : LayersData)
		{
			FLandscapeLayerComponentData& LayerComponentData = ItPair.Value;
			TexturesAndMaterials.Add((UObject**)&LayerComponentData.HeightmapData.Texture);
			for (TObjectPtr<UTexture2D>& WeightmapTexture : LayerComponentData.WeightmapData.Textures)
			{
				TexturesAndMaterials.Add((UObject**)&static_cast<UTexture2D*&>(WeightmapTexture));
			}
		}
		for (TObjectPtr<UMaterialInstanceConstant>& MaterialInstance : MaterialInstances)
		{
			TexturesAndMaterials.Add((UObject**)&static_cast<UMaterialInstanceConstant*&>(MaterialInstance));
		}
		for (TObjectPtr<UMaterialInterface>& MobileMaterialInterface : MobileMaterialInterfaces)
		{
			TexturesAndMaterials.Add((UObject**)(&static_cast<UMaterialInterface*&>(MobileMaterialInterface)));
		}
		for (TObjectPtr<UMaterialInstanceConstant>& MobileCombinationMaterialInstance : MobileCombinationMaterialInstances)
		{
			TexturesAndMaterials.Add((UObject**)&static_cast<UMaterialInstanceConstant*&>(MobileCombinationMaterialInstance));
		}

		if (Ar.IsSaving())
		{
			TArray<UObject*> BackupTexturesAndMaterials;
			BackupTexturesAndMaterials.AddZeroed(TexturesAndMaterials.Num());
			for (int i = 0; i < TexturesAndMaterials.Num(); ++i)
			{
				Exchange(*TexturesAndMaterials[i], BackupTexturesAndMaterials[i]);
			}

			Super::Serialize(Ar);

			for (int i = 0; i < TexturesAndMaterials.Num(); ++i)
			{
				Exchange(*TexturesAndMaterials[i], BackupTexturesAndMaterials[i]);
			}
		}
		// Manually serialize pointers
		for (UObject** Object : TexturesAndMaterials)
		{
			Ar.Serialize(Object, sizeof(UObject*));
		}
	}
	else if (Ar.IsCooking() && !HasAnyFlags(RF_ClassDefaultObject))
	{
		if (!Ar.CookingTarget()->SupportsFeature(ETargetPlatformFeatures::DeferredRendering))
		{
			// These are used for SM5 rendering
			TArray<TObjectPtr<UMaterialInstanceConstant>> BackupMaterialInstances;
			TArray<TObjectPtr<UTexture2D>> BackupWeightmapTextures;

			Exchange(BackupMaterialInstances, MaterialInstances);
			Exchange(BackupWeightmapTextures, WeightmapTextures);

			Super::Serialize(Ar);

			Exchange(BackupMaterialInstances, MaterialInstances);
			Exchange(BackupWeightmapTextures, WeightmapTextures);
		}
		else if (!Ar.CookingTarget()->SupportsFeature(ETargetPlatformFeatures::MobileRendering))
		{
			// These properties are only for Mobile
			TArray<TObjectPtr<UMaterialInterface>> BackupMobileMaterialInterfaces;
			TArray<TObjectPtr<UTexture2D>> BackupMobileWeightmapTextures;

			Exchange(MobileMaterialInterfaces, BackupMobileMaterialInterfaces);
			Exchange(MobileWeightmapTextures, BackupMobileWeightmapTextures);
			MobileWeightmapTextureArray = TObjectPtr<UTexture2DArray>(nullptr);
			Super::Serialize(Ar);

			Exchange(MobileMaterialInterfaces, BackupMobileMaterialInterfaces);
			Exchange(MobileWeightmapTextures, BackupMobileWeightmapTextures);
		}
		else
		{
			// Serialize both mobile and SM5 properties
			Super::Serialize(Ar);
		}
	}
	else
#endif // WITH_EDITOR
	{
		Super::Serialize(Ar);
	}

	// this is a sanity check, as ALandscapeProxy::PreSave() for cook should have ensured that the cached local box has non-zero volume
	if (Ar.IsLoadingFromCookedPackage() && (CachedLocalBox.GetVolume() <= 0))
	{
		// we must set a conservative bounds as a last resort here -- if not we risk strobing flicker of landscape visibility
		FVector MinBox(0, 0, LandscapeDataAccess::GetLocalHeight(0));
		FVector MaxBox(ComponentSizeQuads + 1, ComponentSizeQuads + 1, LandscapeDataAccess::GetLocalHeight(UINT16_MAX));
		CachedLocalBox = FBox(MinBox, MaxBox);
		UE_LOGF(LogLandscape, Error, "The component %ls has an invalid CachedLocalBox. It has been set to a conservative bounds, that may result in reduced visibility culling performance", *GetName());
	}

	if (Ar.IsLoading() && Ar.CustomVer(FRenderingObjectVersion::GUID) < FRenderingObjectVersion::MapBuildDataSeparatePackage)
	{
		FMeshMapBuildData* LegacyMapBuildData = new FMeshMapBuildData();
		Ar << LegacyMapBuildData->LightMap;
		Ar << LegacyMapBuildData->ShadowMap;

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		LegacyMapBuildData->IrrelevantLights = IrrelevantLights_DEPRECATED;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA

		FMeshMapBuildLegacyData LegacyComponentData;
		LegacyComponentData.Data.Emplace(MapBuildDataId, LegacyMapBuildData);
		GComponentsWithLegacyLightmaps.AddAnnotation(this, MoveTemp(LegacyComponentData));
	}

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::NewLandscapeMaterialPerLOD)
	{
		if (MobileMaterialInterface_DEPRECATED != nullptr)
		{
			MobileMaterialInterfaces.AddUnique(MobileMaterialInterface_DEPRECATED);
		}

		if (MobileCombinationMaterialInstance_DEPRECATED != nullptr)
		{
			MobileCombinationMaterialInstances.AddUnique(MobileCombinationMaterialInstance_DEPRECATED);
		}
	}

	if (Ar.IsLoading())
	{
		if (!GrassTypes_DEPRECATED.IsEmpty())
		{
			check(NamedGrassTypes.IsEmpty()); 
			for (TObjectPtr<ULandscapeGrassType> GrassType : GrassTypes_DEPRECATED)
			{
				if (GrassType != nullptr)
				{
					// Just use the grass type's fallback name for backwards compatibility since we do the same in FMaterialCachedExpressionData::PostSerialize, 
					//  this will be updated by UpdateGrassTypes to support the actual remapping feature that comes with NamedGrassTypes
					check(!NamedGrassTypes.Contains(GrassType->GetGrassNameFallback())); // Should be unique as the old GrassTypes were forced unique by the UX code
					NamedGrassTypes.Emplace(GrassType->GetGrassNameFallback(), GrassType);
				}
			}
			GrassTypes_DEPRECATED.Empty();
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA

	if (Ar.UEVer() >= VER_UE4_SERIALIZE_LANDSCAPE_GRASS_DATA)
	{
		// Share the shared ref so PIE can share this data
		if (Ar.GetPortFlags() & PPF_DuplicateForPIE)
		{
			if (Ar.IsSaving())
			{
				PTRINT GrassDataPointer = (PTRINT)&GrassData;
				Ar << GrassDataPointer;
			}
			else
			{
				PTRINT GrassDataPointer;
				Ar << GrassDataPointer;
				// Duplicate shared reference
				GrassData = *(TSharedRef<FLandscapeComponentGrassData, ESPMode::ThreadSafe>*)GrassDataPointer;
			}
		}
		else
		{
			if (bStripGrassData)
			{
				FLandscapeComponentGrassData EmptyGrassData;
				EmptyGrassData.NumElements = 0;
				Ar << EmptyGrassData;
			}
			else
			{
				// technically on load this is doing a thread-unsafe operation by stomping the data in the existing ref
				// but we're assuming there are no async threads using this pointer yet at load...
				Ar << GrassData.Get();
			}
		}

		// When loading or saving a component, validate that grass data is valid : 
		checkf(IsTemplate() || !Ar.IsLoading() || !Ar.IsSaving() || GrassData->HasValidData(), TEXT("If this asserts, then serialization occurred on grass data that wasn't properly loaded/computed. It's a problem"));
	}

#if WITH_EDITOR
	if (Ar.IsTransacting())
	{
		Ar << EditToolRenderData.SelectedType;
	}
#endif

	bool bCooked = false;

	if (Ar.UEVer() >= VER_UE4_LANDSCAPE_PLATFORMDATA_COOKING && !HasAnyFlags(RF_ClassDefaultObject))
	{
		bCooked = Ar.IsCooking() || (FPlatformProperties::RequiresCookedData() && Ar.IsSaving());
		// This is needed when loading cooked data, to know to serialize differently
		Ar << bCooked;
	}

	if (FPlatformProperties::RequiresCookedData() && !bCooked && Ar.IsLoading())
	{
		UE_LOGF(LogLandscape, Fatal, "This platform requires cooked packages, and this landscape does not contain cooked data %ls.", *GetName());
	}

#if WITH_EDITOR
	if (Ar.IsSaving() && Ar.IsPersistent())
	{
		//Update the last saved Hash for physical material
		LastSavedPhysicalMaterialHash = PhysicalMaterialHash;
	}
#endif
}

void ULandscapeComponent::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(GrassData->GetAllocatedSize());
}

UMaterialInterface* ULandscapeComponent::GetLandscapeMaterial(int8 InLODIndex) const
{
	if (InLODIndex != INDEX_NONE)
	{
		UWorld* World = GetWorld();
		if (World != nullptr)
		{
			if (const FLandscapePerLODMaterialOverride* LocalMaterialOverride = PerLODOverrideMaterials.FindByPredicate(
				[InLODIndex](const FLandscapePerLODMaterialOverride& InOverride) { return (InOverride.LODIndex == InLODIndex) && (InOverride.Material != nullptr); }))
			{
				return LocalMaterialOverride->Material;
			}
		}
	}

	if (OverrideMaterial != nullptr)
	{
		return OverrideMaterial;
	}

	ALandscapeProxy* Proxy = GetLandscapeProxy();
	if (Proxy)
	{
		return Proxy->GetLandscapeMaterial(InLODIndex);
	}

	return UMaterial::GetDefaultMaterial(MD_Surface);
}

UMaterialInterface* ULandscapeComponent::GetLandscapeHoleMaterial() const
{
	if (OverrideHoleMaterial)
	{
		return OverrideHoleMaterial;
	}
	ALandscapeProxy* Proxy = GetLandscapeProxy();
	if (Proxy)
	{
		return Proxy->GetLandscapeHoleMaterial();
	}
	return nullptr;
}

void ULandscapeComponent::RetrieveAllLandscapeMaterials(TSet<UMaterialInterface*>& OutMaterials) const
{
	if (UMaterialInterface* Material = GetLandscapeMaterial(); 
		Material && (Material != UMaterial::GetDefaultMaterial(MD_Surface)))
	{
		OutMaterials.Add(Material);
	}
	if (UMaterialInterface* Material = GetLandscapeHoleMaterial();
		Material && (Material != UMaterial::GetDefaultMaterial(MD_Surface)))
	{
		OutMaterials.Add(Material);
	}
}

#if WITH_EDITOR
bool ULandscapeComponent::IsLandscapeHoleMaterialValid() const
{
	UMaterialInterface* HoleMaterial = GetLandscapeHoleMaterial();
	if (!HoleMaterial)
	{
		HoleMaterial = GetLandscapeMaterial();
	}

	return HoleMaterial ? HoleMaterial->GetMaterial()->HasAnyExpressionsInMaterialAndFunctionsOfType<UMaterialExpressionLandscapeVisibilityMask>() : false;
}

bool ULandscapeComponent::ComponentHasVisibilityPainted() const
{
	for (const FWeightmapLayerAllocationInfo& Allocation : WeightmapLayerAllocations)
	{
		if (Allocation.LayerInfo == ALandscapeProxy::VisibilityLayer)
		{
			return true;
		}
	}

	return false;
}

ULandscapeLayerInfoObject* ULandscapeComponent::GetVisibilityLayer() const
{
	for (const FWeightmapLayerAllocationInfo& Allocation : WeightmapLayerAllocations)
	{
		if (Allocation.LayerInfo == ALandscapeProxy::VisibilityLayer)
		{
			return Allocation.LayerInfo;
		}
	}

	return nullptr;
}

void ULandscapeComponent::GetLayerDebugColorKey(int32& R, int32& G, int32& B) const
{
	ULandscapeInfo* Info = GetLandscapeInfo();
	if (Info != nullptr)
	{
		R = INDEX_NONE, G = INDEX_NONE, B = INDEX_NONE;

		for (auto It = Info->Layers.CreateConstIterator(); It; It++)
		{
			const FLandscapeInfoLayerSettings& LayerSettings = *It;
			if (LayerSettings.DebugColorChannel > 0
				&& LayerSettings.LayerInfoObj)
			{
				const TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = GetWeightmapLayerAllocations();

				for (int32 LayerIdx = 0; LayerIdx < ComponentWeightmapLayerAllocations.Num(); LayerIdx++)
				{
					if (ComponentWeightmapLayerAllocations[LayerIdx].LayerInfo == LayerSettings.LayerInfoObj)
					{
						if (LayerSettings.DebugColorChannel & 1) // R
						{
							R = (ComponentWeightmapLayerAllocations[LayerIdx].WeightmapTextureIndex * 4 + ComponentWeightmapLayerAllocations[LayerIdx].WeightmapTextureChannel);
						}
						if (LayerSettings.DebugColorChannel & 2) // G
						{
							G = (ComponentWeightmapLayerAllocations[LayerIdx].WeightmapTextureIndex * 4 + ComponentWeightmapLayerAllocations[LayerIdx].WeightmapTextureChannel);
						}
						if (LayerSettings.DebugColorChannel & 4) // B
						{
							B = (ComponentWeightmapLayerAllocations[LayerIdx].WeightmapTextureIndex * 4 + ComponentWeightmapLayerAllocations[LayerIdx].WeightmapTextureChannel);
						}
						break;
					}
				}
			}
		}
	}
}
#endif	//WITH_EDITOR

ULandscapeInfo::ULandscapeInfo(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
#if WITH_EDITORONLY_DATA
	, bDirtyOnlyInMode(false)
#endif
	, XYComponentBounds(MAX_int32, MAX_int32, MIN_int32, MIN_int32)
{
}

#if WITH_EDITOR
void ULandscapeInfo::UpdateDebugColorMaterial()
{
	FlushRenderingCommands();
	//GWarn->BeginSlowTask( *FString::Printf(TEXT("Compiling layer color combinations for %s"), *GetName()), true);

	for (auto It = XYtoComponentMap.CreateIterator(); It; ++It)
	{
		ULandscapeComponent* Comp = It.Value();
		if (Comp)
		{
			Comp->EditToolRenderData.UpdateDebugColorMaterial(Comp);
			Comp->UpdateEditToolRenderData();
		}
	}
	FlushRenderingCommands();
	//GWarn->EndSlowTask();
}
#endif // WITH_EDITOR

void ULandscapeComponent::UpdatedSharedPropertiesFromActor()
{
	ALandscapeProxy* LandscapeProxy = GetLandscapeProxy();

	CastShadow = LandscapeProxy->CastShadow;
	bCastDynamicShadow = LandscapeProxy->bCastDynamicShadow;
	bCastStaticShadow = LandscapeProxy->bCastStaticShadow;
	bCastContactShadow = LandscapeProxy->bCastContactShadow;
	bCastFarShadow = LandscapeProxy->bCastFarShadow;
	bCastHiddenShadow = LandscapeProxy->bCastHiddenShadow;
	bCastShadowAsTwoSided = LandscapeProxy->bCastShadowAsTwoSided;
	bAffectDistanceFieldLighting = LandscapeProxy->bAffectDistanceFieldLighting;
	bAffectDynamicIndirectLighting = LandscapeProxy->bAffectDynamicIndirectLighting;
	bAffectIndirectLightingWhileHidden = LandscapeProxy->bAffectIndirectLightingWhileHidden;
	bRenderCustomDepth = LandscapeProxy->bRenderCustomDepth;
	CustomDepthStencilWriteMask = LandscapeProxy->CustomDepthStencilWriteMask;
	CustomDepthStencilValue = LandscapeProxy->CustomDepthStencilValue;
	SetCullDistance(LandscapeProxy->LDMaxDrawDistance);
	LightingChannels = LandscapeProxy->LightingChannels;
	ShadowCacheInvalidationBehavior = LandscapeProxy->ShadowCacheInvalidationBehavior;
	bHoldout = LandscapeProxy->bHoldout;

	UpdateNavigationRelevance();
	UpdateRejectNavmeshUnderneath();
}

void ULandscapeComponent::PostLoad()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeComponent::PostLoad);

	using namespace UE::Landscape;

	Super::PostLoad();

	PrecachePSOs();

#if WITH_EDITOR
	ALandscapeProxy* LandscapeProxy = GetLandscapeProxy();
	if (ensure(LandscapeProxy))
	{
		// Ensure that the component's lighting settings matches the actor's.
		UpdatedSharedPropertiesFromActor();

		// check SectionBaseX/Y are correct
		const FVector LocalRelativeLocation = GetRelativeLocation();
		int32 CheckSectionBaseX = FMath::RoundToInt32(LocalRelativeLocation.X) + LandscapeProxy->GetSectionBase().X;
		int32 CheckSectionBaseY = FMath::RoundToInt32(LocalRelativeLocation.Y) + LandscapeProxy->GetSectionBase().Y;
		if (CheckSectionBaseX != SectionBaseX ||
			CheckSectionBaseY != SectionBaseY)
		{
			UE_LOGF(LogLandscape, Warning, "LandscapeComponent SectionBaseX disagrees with its location, attempted automated fix: '%ls', %d,%d vs %d,%d.",
				*GetFullName(), SectionBaseX, SectionBaseY, CheckSectionBaseX, CheckSectionBaseY);
			SectionBaseX = CheckSectionBaseX;
			SectionBaseY = CheckSectionBaseY;
		}
	}

	if (GIsEditor && !HasAnyFlags(RF_ClassDefaultObject) && ensure(LandscapeProxy))
	{
		// This is to ensure that component relative location is exact section base offset value
		FVector LocalRelativeLocation = GetRelativeLocation();
		float CheckRelativeLocationX = float(SectionBaseX - LandscapeProxy->GetSectionBase().X);
		float CheckRelativeLocationY = float(SectionBaseY - LandscapeProxy->GetSectionBase().Y);
		if (!FMath::IsNearlyEqual(CheckRelativeLocationX, LocalRelativeLocation.X, UE_DOUBLE_KINDA_SMALL_NUMBER) ||
			!FMath::IsNearlyEqual(CheckRelativeLocationY, LocalRelativeLocation.Y, UE_DOUBLE_KINDA_SMALL_NUMBER))
		{
			UE_LOGF(LogLandscape, Warning, "LandscapeComponent RelativeLocation disagrees with its section base, attempted automated fix: '%ls', %f,%f vs %f,%f.",
				*GetFullName(), LocalRelativeLocation.X, LocalRelativeLocation.Y, CheckRelativeLocationX, CheckRelativeLocationY);
			LocalRelativeLocation.X = CheckRelativeLocationX;
			LocalRelativeLocation.Y = CheckRelativeLocationY;

			SetRelativeLocation_Direct(LocalRelativeLocation);
		}

		// Remove standalone flags from data textures to ensure data is unloaded in the editor when reverting an unsaved level.
		// Previous version of landscape set these flags on creation.
		if (HeightmapTexture)
		{
			ULandscapeTextureHash::SetInitialStateOnPostLoad(HeightmapTexture, ELandscapeTextureUsage::FinalData, ELandscapeTextureType::Heightmap);
			if (HeightmapTexture->HasAnyFlags(RF_Standalone))
			{
				HeightmapTexture->ClearFlags(RF_Standalone);
			}
		}
		for (int32 Idx = 0; Idx < WeightmapTextures.Num(); Idx++)
		{
			ULandscapeTextureHash::SetInitialStateOnPostLoad(WeightmapTextures[Idx], ELandscapeTextureUsage::FinalData, ELandscapeTextureType::Weightmap);
			if (WeightmapTextures[Idx])
			{
				if (WeightmapTextures[Idx]->HasAnyFlags(RF_Standalone))
				{
					WeightmapTextures[Idx]->ClearFlags(RF_Standalone);
				}
			}
		}

		LastSavedPhysicalMaterialHash = PhysicalMaterialHash;

		PRAGMA_DISABLE_DEPRECATION_WARNINGS;
		if (!OverrideMaterials_DEPRECATED.IsEmpty())
		{
			PerLODOverrideMaterials.Reserve(OverrideMaterials_DEPRECATED.Num());
			for (const FLandscapeComponentMaterialOverride& LocalMaterialOverride : OverrideMaterials_DEPRECATED)
			{
				PerLODOverrideMaterials.Add({ LocalMaterialOverride.LODIndex.Default, LocalMaterialOverride.Material });
			}
			OverrideMaterials_DEPRECATED.Reset();
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS;
	}

#if WITH_EDITORONLY_DATA
	// Handle old MaterialInstance

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (MaterialInstance_DEPRECATED)
	{
		MaterialInstances.Empty(1);
		MaterialInstances.Add(MaterialInstance_DEPRECATED);
		MaterialInstance_DEPRECATED = nullptr;

		if (GIsEditor && MaterialInstances.Num() > 0 && MaterialInstances[0] != nullptr)
		{
			MaterialInstances[0]->ConditionalPostLoad();
			UpdateMaterialInstances();
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (CVarStripLayerTextureMipsOnLoad->GetBool())
	{
		auto DropMipChain = [](UTexture2D* InTexture)
		{
			if (InTexture->Source.GetNumMips() <= 1)
			{
				return;
			}

			TArray64<uint8> TopMipData;
			InTexture->Source.GetMipData(TopMipData, 0);

			InTexture->PreEditChange(nullptr);
			InTexture->Source.Init(InTexture->Source.GetSizeX(), InTexture->Source.GetSizeY(), 1, 1, InTexture->Source.GetFormat(), TopMipData.GetData());
			InTexture->UpdateResource();

			InTexture->PostEditChange();
		};

		// Remove Non zero mip levels found in layer textures
		for (auto& LayerIt : LayersData)
		{
			DropMipChain(LayerIt.Value.HeightmapData.Texture);
			for (int32 i = 0; i < LayerIt.Value.WeightmapData.Textures.Num(); ++i)
			{
				DropMipChain(LayerIt.Value.WeightmapData.Textures[i]);
			}
		}
	}
	
#endif

	auto ReparentObject = [this](UObject* Object)
	{
		if (Object && !Object->HasAllFlags(RF_Public | RF_Standalone) && (Object->GetOuter() != GetOuter()) && (Object->GetOutermost() == GetOutermost()))
		{
			Object->Rename(nullptr, GetOuter(), REN_AllowPackageLinkerMismatch);
			return true;
		}
		return false;
	};

	ReparentObject(HeightmapTexture);

	for (UTexture2D* WeightmapTexture : WeightmapTextures)
	{
		ReparentObject(WeightmapTexture);
	}

	for (UTexture2D* MobileWeightmapTexture : MobileWeightmapTextures)
	{
		ReparentObject(MobileWeightmapTexture);
	}

	if (MobileWeightmapTextureArray)
	{
		ReparentObject(MobileWeightmapTextureArray.Get());
	}

	for (auto& ItPair : LayersData)
	{
		FLandscapeLayerComponentData& LayerComponentData = ItPair.Value;
		ReparentObject(LayerComponentData.HeightmapData.Texture);
		for (UTexture2D* WeightmapTexture : LayerComponentData.WeightmapData.Textures)
		{
			ReparentObject(WeightmapTexture);
		}

		// Fixup missing/mismatching edit layer names :
		if (const ULandscapeEditLayerBase* LandscapeEditLayer = GetLandscapeActor() ? GetLandscapeActor()->GetEditLayerConst(ItPair.Key) : nullptr)
		{
			if (LayerComponentData.DebugName != LandscapeEditLayer->GetName())
			{
				LayerComponentData.DebugName = LandscapeEditLayer->GetName();
			}
		}
	}

	for (UMaterialInstance* MaterialInstance : MaterialInstances)
	{
		ULandscapeMaterialInstanceConstant* CurrentMIC = Cast<ULandscapeMaterialInstanceConstant>(MaterialInstance);
		while (ReparentObject(CurrentMIC))
		{
			CurrentMIC = Cast<ULandscapeMaterialInstanceConstant>(MaterialInstance->Parent);
		}
	}

	for (UMaterialInterface* MobileMaterialInterface : MobileMaterialInterfaces)
	{
		while (ReparentObject(MobileMaterialInterface))
		{
			MobileMaterialInterface = Cast<UMaterialInstance>(MobileMaterialInterface) ? Cast<UMaterialInstance>(((UMaterialInstance*)MobileMaterialInterface)->Parent) : nullptr;
		}
	}

	for (UMaterialInstance* MobileCombinationMaterialInstance : MobileCombinationMaterialInstances)
	{
		while (ReparentObject(MobileCombinationMaterialInstance))
		{
			MobileCombinationMaterialInstance = Cast<UMaterialInstance>(MobileCombinationMaterialInstance->Parent);
		}
	}

#if !UE_BUILD_SHIPPING
	// This will fix the data in case there is mismatch between save of asset/maps
	const int8 MaxLOD = static_cast<int8>(FMath::CeilLogTwo(SubsectionSizeQuads + 1) - 1);

	TArray<ULandscapeMaterialInstanceConstant*> ResolvedMaterials;

	if (LODIndexToMaterialIndex.Num() != MaxLOD+1)
	{
		if (GIsEditor)
		{
			UpdateMaterialInstances();
		}
		else
		{
			// Correct in-place differences by applying the highest LOD value we have to the newly added items as most case will be missing items added at the end
			LODIndexToMaterialIndex.SetNumZeroed(MaxLOD + 1);

			int8 LastLODIndex = 0;

			for (int32 i = 0; i < LODIndexToMaterialIndex.Num(); ++i)
			{
				if (LODIndexToMaterialIndex[i] > LastLODIndex)
				{
					LastLODIndex = LODIndexToMaterialIndex[i];
				}

				if (LODIndexToMaterialIndex[i] == 0 && LastLODIndex != 0)
				{
					LODIndexToMaterialIndex[i] = LastLODIndex;
				}
			}
		}
	}
#endif // UE_BUILD_SHIPPING

	if (GIsEditor && !HasAnyFlags(RF_ClassDefaultObject))
	{
		// Move the MICs and Textures back to the Package if they're currently in the level
		// Moving them into the level caused them to be duplicated when running PIE, which is *very very slow*, so we've reverted that change
		// Also clear the public flag to avoid various issues, e.g. generating and saving thumbnails that can never be seen
		if (ULevel* Level = GetLevel())
		{
			TArray<UObject*> ObjectsToMoveFromLevelToPackage;
			GetGeneratedTexturesAndMaterialInstances(ObjectsToMoveFromLevelToPackage);

			UPackage* MyPackage = GetOutermost();
			for (auto* Obj : ObjectsToMoveFromLevelToPackage)
			{
				Obj->ClearFlags(RF_Public);
				if (Obj->GetOuter() == Level)
				{
					Obj->Rename(nullptr, MyPackage, REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
				}
			}
		}
	}

	if (GIsEditor && GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::LandscapeSupportPerComponentGrassTypes)
	{
		UpdateGrassTypes();
	}

#if !UE_BUILD_SHIPPING
	if (MobileCombinationMaterialInstances.Num() == 0)
	{
		if (GIsEditor)
		{
			UpdateMaterialInstances();
		}
		else
		{
			UE_LOGF(LogLandscape, Error, "Landscape component (%d, %d) Does not have a valid mobile combination material. To correct this issue, open the map in the editor and resave the map.", SectionBaseX, SectionBaseY);
		}
	}
#endif // UE_BUILD_SHIPPING


	// May have been saved without mobile layer allocations, but those are serialized now
	if (MobileWeightmapLayerAllocations.Num() == 0)
	{
		GenerateMobileWeightmapLayerAllocations();
	}

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		FSceneInterface* SceneInterface = GetScene();
		ERHIFeatureLevel::Type FeatureLevel = ((GEngine->GetDefaultWorldFeatureLevel() == ERHIFeatureLevel::ES3_1) || (SceneInterface && (SceneInterface->GetFeatureLevel() <= ERHIFeatureLevel::ES3_1)))
			? ERHIFeatureLevel::ES3_1 : GMaxRHIFeatureLevel;

		// If we're loading on a platform that doesn't require cooked data, but defaults to a mobile feature level, generate or preload data from the DDC
		if (!FPlatformProperties::RequiresCookedData() && FeatureLevel == ERHIFeatureLevel::ES3_1)
		{
			CheckGenerateMobilePlatformData(/*bIsCooking = */ false, /*TargetPlatform = */ nullptr);
		}
	}

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// If the Collision Component is not set yet and we're transferring the property from the lazy object pointer it was previously stored as to the soft object ptr it is now stored as :
	if (!CollisionComponentRef && CollisionComponent_DEPRECATED.IsValid())
	{
		CollisionComponentRef = CollisionComponent_DEPRECATED.Get();
		CollisionComponent_DEPRECATED = nullptr;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// If mip-to-mip info is missing, recompute them (they were introduced later) :
	if (MipToMipMaxDeltas.IsEmpty())
	{
		UpdateCachedBounds();
	}
#endif // !WITH_EDITORONLY_DATA

	if (GIsEditor && GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::OverridableLandscapeGrassTypes)
	{
		// Make sure grass types are up-to-date (i.e. the grass names are updated based on the material). We need to make sure the material has performed its post-load :
		if (UMaterialInterface* Material = GetLandscapeMaterial())
		{
			Material->ConditionalPostLoad();
		}
		UpdateGrassTypes();
		// Prior to this version, GrassData contained ULandscapeGrassType* entries. These have turned into GrassName (FName), as declared by the landscape material. We can do the fixup here
		//  since we have access to the material-declared grass types here (while FLandscapeComponentGrassData's operator<< cannot) :
		GrassData->FixupFallbackGrassNames(GetNamedGrassTypes());
	}

#endif // WITH_EDITOR

	GrassData->ConditionalDiscardDataOnLoad(GetNamedGrassTypes());
}

void ULandscapeComponent::CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams)
{
	FPSOPrecacheParams PrecachePSOParams = BasePrecachePSOParams;
	PrecachePSOParams.bIgnoreMaskMaterial = !UE::Landscape::UseLandscapeVertexMask();
	PrecachePSOParams.bRequireRTDynamicGeometryUpdate = true;

	FPSOPrecacheVertexFactoryDataList VertexFactoryDataList;
	VertexFactoryDataList.Add(FPSOPrecacheVertexFactoryData(&FLandscapeVertexFactory::StaticType));

	// we need the fixed grid vertex factory for both virtual texturing and grass
	if (UE::Landscape::NeedsFixedGridVertexFactory(GMaxRHIShaderPlatform))
	{
		VertexFactoryDataList.Add(FPSOPrecacheVertexFactoryData(&FLandscapeFixedGridVertexFactory::StaticType));
	}

	if (UE::Landscape::Culling::UseCulling(GMaxRHIShaderPlatform))
	{
		VertexFactoryDataList.Add(FPSOPrecacheVertexFactoryData(UE::Landscape::Culling::GetTileVertexFactoryType()));
	}

	ILandscapeModule& LandscapeModule = FModuleManager::GetModuleChecked<ILandscapeModule>("Landscape");
	const UE::Landscape::FSetupLandscapePSOPrecacheVertexFactoryDataListDelegate& SetupLandscapePSOPrecacheVertexFactoryDataListDelegate = LandscapeModule.GetSetupLandscapePSOPrecacheVertexFactoryDataListDelegate();
	if (SetupLandscapePSOPrecacheVertexFactoryDataListDelegate.IsBound())
	{
		SetupLandscapePSOPrecacheVertexFactoryDataListDelegate.Execute(this, VertexFactoryDataList);
	}

	TArray<UMaterialInterface*> Materials;
	bool bGetDebugMaterials = false;
	GetUsedMaterials(Materials, bGetDebugMaterials);
	for (UMaterialInterface* MaterialInterface : Materials)
	{
		if (MaterialInterface)
		{
			FMaterialInterfacePSOPrecacheParams& ComponentParams = OutParams[OutParams.AddDefaulted()];
			ComponentParams.Priority = EPSOPrecachePriority::High;
			ComponentParams.MaterialInterface = MaterialInterface;
			ComponentParams.VertexFactoryDataList = VertexFactoryDataList;
			ComponentParams.PSOPrecacheParams = PrecachePSOParams;
		}
	}
}

#if WITH_EDITORONLY_DATA
void ULandscapeComponent::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
{
	Super::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	OutConstructClasses.Add(FTopLevelAssetPath(ULandscapeMaterialInstanceConstant::StaticClass()));
}
#endif

#if WITH_EDITORONLY_DATA
TArray<ALandscapeProxy*> ALandscapeProxy::LandscapeProxies;
#endif

ALandscapeProxy::ALandscapeProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bReplicates = false;
	SetNetUpdateFrequency(10.0f);
	SetHidden(false);
	SetReplicatingMovement(false);
	SetCanBeDamaged(false);

	CastShadow = true;
	bCastDynamicShadow = true;
	bCastStaticShadow = true;
	bCastContactShadow = true;
	bCastFarShadow = true;
	bCastHiddenShadow = false;
	bCastShadowAsTwoSided = false;
	bAffectDistanceFieldLighting = true;
	bAffectDynamicIndirectLighting = true;
	bAffectIndirectLightingWhileHidden = false;
	bHoldout = false;

	RootComponent->SetRelativeScale3D(FVector(128.0f, 128.0f, 256.0f)); // Old default scale, preserved for compatibility. See ULandscapeEditorObject::NewLandscape_Scale
	RootComponent->Mobility = EComponentMobility::Static;

	StaticLightingResolution = 1.0f;
	StreamingDistanceMultiplier = 1.0f;
	MaxLODLevel = -1;
	bUseDynamicMaterialInstance = false;
#if WITH_EDITORONLY_DATA
	bLockLocation = true;
#endif // WITH_EDITORONLY_DATA
	bCastStaticShadow = true;
	ShadowCacheInvalidationBehavior = EShadowCacheInvalidationBehavior::Auto;
	bUsedForNavigation = true;
	bFillCollisionUnderLandscapeForNavmesh = false;
	BodyInstance.SetCollisionProfileName(UCollisionProfile::BlockAll_ProfileName);
	bGenerateOverlapEvents = false;
#if WITH_EDITORONLY_DATA
	MaxPaintedLayersPerComponent = 0;
	HLODTextureSizePolicy = ELandscapeHLODTextureSizePolicy::SpecificSize;
	HLODTextureSize = 256;
	HLODMeshSourceLODPolicy = ELandscapeHLODMeshSourceLODPolicy::LowestDetailLOD;
	HLODMeshSourceLOD = 0;
#endif

#if WITH_EDITOR
	if (VisibilityLayer == nullptr)
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<ULandscapeLayerInfoObject> LandscapeVisibilityLayerInfoFinder;
			FConstructorStatics()
				: LandscapeVisibilityLayerInfoFinder(TEXT("LandscapeLayerInfoObject'/Engine/EngineResources/LandscapeVisibilityLayerInfo.LandscapeVisibilityLayerInfo'"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		VisibilityLayer = ConstructorStatics.LandscapeVisibilityLayerInfoFinder.Get();
		check(VisibilityLayer);
		VisibilityLayer->SetLayerName(UMaterialExpressionLandscapeVisibilityMask::ParameterName, /*bInModify =*/ false);
		VisibilityLayer->SetLayerUsageDebugColor(FLinearColor(0, 0, 0, 0), /*bInModify =*/ false, /*InChangeType = */EPropertyChangeType::ValueSet);
		VisibilityLayer->AddToRoot();
	}
#endif // #if WITH_EDITOR

	static uint32 FrameOffsetForTickIntervalInc = 0;
	FrameOffsetForTickInterval = FrameOffsetForTickIntervalInc++;

#if WITH_EDITORONLY_DATA
	LandscapeProxies.Add(this);
#endif
}

void ALandscapeProxy::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITOR
	if (UWorld* World = GetWorld())
	{
		if (!IsInAsyncLoadingThread() && !HasAnyFlags(RF_ArchetypeObject | RF_ClassDefaultObject))
		{
			FOnFeatureLevelChanged::FDelegate FeatureLevelChangedDelegate = FOnFeatureLevelChanged::FDelegate::CreateUObject(this, &ALandscapeProxy::OnFeatureLevelChanged);
			// This needs to be called on the game thread. In the case of async loading, it will get called in PostLoad()
			FeatureLevelChangedDelegateHandle = World->AddOnFeatureLevelChangedHandler(FeatureLevelChangedDelegate);
		}
	}
#endif // #if WITH_EDITOR
}

#if WITH_EDITORONLY_DATA
ALandscape::FLandscapeEdModeInfo::FLandscapeEdModeInfo()
	: ViewMode(ELandscapeViewMode::Invalid)
	, ToolTarget(ELandscapeToolTargetType::Invalid)
{
}
#endif

ALandscape::ALandscape(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bLockLocation = false;
	LayerContentUpdateModes = 0;
	bSplineLayerUpdateRequested = false;
	bLandscapeLayersAreInitialized = false;
	LandscapeEdMode = nullptr;
	bGrassUpdateEnabled = true;
	bIsSpatiallyLoaded = false;
	bDefaultOutlinerExpansionState = false;
#endif // WITH_EDITORONLY_DATA
}

ALandscapeStreamingProxy::ALandscapeStreamingProxy(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bLockLocation = true;
#endif // WITH_EDITORONLY_DATA
}

const ALandscape* ALandscape::GetLandscapeActor() const
{
	return this;
}

ALandscape* ALandscape::GetLandscapeActor()
{
	return this;
}

const ALandscape* ALandscapeStreamingProxy::GetLandscapeActor() const
{
	return LandscapeActorRef.Get();
}

ALandscape* ALandscapeStreamingProxy::GetLandscapeActor()
{
	return LandscapeActorRef.Get();
}

void ALandscapeStreamingProxy::SetLandscapeActor(ALandscape* InLandscape)
{
	LandscapeActorRef = InLandscape;
}

void ALandscape::SetLODGroupKey(uint32 InLODGroupKey)
{
	SetLODGroupKeyInternal(InLODGroupKey);

	// change LODGroupKey on any proxies that are currently registered
	// (any proxies that get registered later will copy the value on registration)
	if (ULandscapeInfo* Info = GetLandscapeInfo())
	{
		Info->ForEachLandscapeProxy([InLODGroupKey](ALandscapeProxy* Proxy)
			{
				Proxy->SetLODGroupKeyInternal(InLODGroupKey);
				return true;
			});
	}
}

void ALandscapeProxy::SetLODGroupKeyInternal(uint32 InLODGroupKey)
{
	if (LODGroupKey != InLODGroupKey)
	{
		LODGroupKey = InLODGroupKey;
		MarkComponentsRenderStateDirty();
	}
}

uint32 ALandscape::GetLODGroupKey()
{
	return LODGroupKey;
}

void ALandscape::SetGrassEnabled(bool bEnabled, bool bFlushGrassMaps)
{
	if (bEnabled != bGrassEnabled)
	{
		bGrassEnabled = bEnabled;

		if (!bGrassEnabled)
		{
			FlushGrassComponents(/* OnlyForComponents = */nullptr, /* bFlushGrassMaps = */bFlushGrassMaps);
		}
	}
}

void ALandscape::MarkAllLandscapeRenderStateDirty()
{
	if (ULandscapeInfo* Info = GetLandscapeInfo())
	{
		Info->ForEachLandscapeProxy([](ALandscapeProxy* Proxy)
			{
				Proxy->MarkComponentsRenderStateDirty();
				return true;
			});
	}
}

ULandscapeInfo* ALandscapeProxy::CreateLandscapeInfo(bool bMapCheck, bool bUpdateAllAddCollisions)
{
	ULandscapeInfo* LandscapeInfo = ULandscapeInfo::FindOrCreate(GetWorld(), LandscapeGuid);
	LandscapeInfo->RegisterActor(this, bMapCheck, bUpdateAllAddCollisions);
	return LandscapeInfo;
}

ULandscapeInfo* ALandscapeProxy::GetLandscapeInfo() const
{
	return ULandscapeInfo::Find(GetWorld(), LandscapeGuid);
}

FTransform ALandscapeProxy::LandscapeActorToWorld() const
{
	FTransform TM = ActorToWorld();
	// Add this proxy landscape section offset to obtain landscape actor transform
	TM.AddToTranslation(TM.TransformVector(-FVector(SectionBase)));
	return TM;
}

void ALandscapeProxy::UpdateSharedProperties(ULandscapeInfo* InLandscapeInfo)
{
	check(LandscapeGuid == InLandscapeInfo->LandscapeGuid);
}

static TArray<float> GetLODScreenSizeArray(const ALandscapeProxy* InLandscapeProxy, const int32 InNumLODLevels)
{
	float LOD0ScreenSize;
	float LOD0Distribution;
	if (InLandscapeProxy->bUseScalableLODSettings)
	{
		const int32 LandscapeQuality = Scalability::GetQualityLevels().LandscapeQuality;
		
		LOD0ScreenSize = InLandscapeProxy->ScalableLOD0ScreenSize.GetValue(LandscapeQuality);
		LOD0Distribution = InLandscapeProxy->ScalableLOD0DistributionSetting.GetValue(LandscapeQuality);	
	}
	else
	{
		static IConsoleVariable* CVarLSLOD0DistributionScale = IConsoleManager::Get().FindConsoleVariable(TEXT("r.LandscapeLOD0DistributionScale"));
		
		LOD0ScreenSize = InLandscapeProxy->LOD0ScreenSize;
		LOD0Distribution = InLandscapeProxy->LOD0DistributionSetting * CVarLSLOD0DistributionScale->GetFloat();
	}
	
	static TConsoleVariableData<float>* CVarSMLODDistanceScale = IConsoleManager::Get().FindTConsoleVariableDataFloat(TEXT("r.StaticMeshLODDistanceScale"));
	
	float CurrentScreenSize = LOD0ScreenSize / CVarSMLODDistanceScale->GetValueOnGameThread();
	const float ScreenSizeMult = 1.f / FMath::Max(LOD0Distribution , 1.01f);

	TArray<float> Result;
	Result.Empty(InNumLODLevels);
	for (int32 Idx = 0; Idx < InNumLODLevels; ++Idx)
	{
		Result.Add(CurrentScreenSize);
		CurrentScreenSize *= ScreenSizeMult;
	}
	return Result;
}

TArray<float> ALandscapeProxy::GetLODScreenSizeArray() const
{
	const int32 MaxPossibleLOD = FMath::CeilLogTwo(SubsectionSizeQuads + 1) - 1;
	const int32 MaxLOD = MaxLODLevel != INDEX_NONE ? FMath::Min<int32>(MaxLODLevel, MaxPossibleLOD) : MaxPossibleLOD;

	const int32 NumLODLevels = MaxLOD + 1;
	return ::GetLODScreenSizeArray(this, NumLODLevels);
}


ALandscape* ULandscapeComponent::GetLandscapeActor() const
{
	if (ALandscapeProxy* LandscapeProxy = GetLandscapeProxy())
	{
		// When the proxy is a LandscapeStreamingProxy, the landscape actor may be invalid whenever ALandscape is not registered to the current LandscapeInfo
		// This may happen during operations like Undo that trigger UnregisterActor events
		return LandscapeProxy->GetLandscapeActor();
	}
	return nullptr;
}

ULevel* ULandscapeComponent::GetLevel() const
{
	AActor* MyOwner = GetOwner();
	return MyOwner ? MyOwner->GetLevel() : nullptr;
}

#if WITH_EDITOR
TArray<UTexture*> ULandscapeComponent::GetGeneratedTextures() const
{
	TArray<UTexture*> OutTextures;
	if (HeightmapTexture)
	{
		OutTextures.Add(HeightmapTexture);
	}

	for (const auto& ItPair : LayersData)
	{
		const FLandscapeLayerComponentData& LayerComponentData = ItPair.Value;

		OutTextures.Add(LayerComponentData.HeightmapData.Texture);
		OutTextures.Append(LayerComponentData.WeightmapData.Textures);
	}

	OutTextures.Append(WeightmapTextures);

	TArray<UMaterialInstance*> OutMaterials;
	for (UMaterialInstance* MaterialInstance : MaterialInstances)
	{
		for (ULandscapeMaterialInstanceConstant* CurrentMIC = Cast<ULandscapeMaterialInstanceConstant>(MaterialInstance); CurrentMIC; CurrentMIC = Cast<ULandscapeMaterialInstanceConstant>(CurrentMIC->Parent))
		{
			// Sometimes weight map is not registered in the WeightmapTextures, so
			// we need to get it from here.
			FTextureParameterValue* WeightmapPtr = CurrentMIC->TextureParameterValues.FindByPredicate(
				[](const FTextureParameterValue& ParamValue)
			{
				static const FName WeightmapParamName("Weightmap0");
				return ParamValue.ParameterInfo.Name == WeightmapParamName;
			});

			if (WeightmapPtr != nullptr)
			{
				OutTextures.AddUnique(WeightmapPtr->ParameterValue);
			}
		}
	}

	OutTextures.Remove(nullptr);

	return OutTextures;
}

TArray<UMaterialInstance*> ULandscapeComponent::GetGeneratedMaterialInstances() const
{
	TArray<UMaterialInstance*> OutMaterials;
	for (UMaterialInstance* MaterialInstance : MaterialInstances)
	{
		for (ULandscapeMaterialInstanceConstant* CurrentMIC = Cast<ULandscapeMaterialInstanceConstant>(MaterialInstance); CurrentMIC; CurrentMIC = Cast<ULandscapeMaterialInstanceConstant>(CurrentMIC->Parent))
		{
			OutMaterials.Add(CurrentMIC);
		}
	}

	for (UMaterialInstanceConstant* MaterialInstance : MobileCombinationMaterialInstances)
	{
		for (ULandscapeMaterialInstanceConstant* CurrentMIC = Cast<ULandscapeMaterialInstanceConstant>(MaterialInstance); CurrentMIC; CurrentMIC = Cast<ULandscapeMaterialInstanceConstant>(CurrentMIC->Parent))
		{
			OutMaterials.Add(CurrentMIC);
		}
	}

	return OutMaterials;
}

void ULandscapeComponent::GetGeneratedTexturesAndMaterialInstances(TArray<UObject*>& OutTexturesAndMaterials) const
{
	TArray<UTexture*> LocalTextures = GetGeneratedTextures();
	TArray<UMaterialInstance*> LocalMaterialInstances = GetGeneratedMaterialInstances();
	OutTexturesAndMaterials.Reserve(LocalTextures.Num() + LocalMaterialInstances.Num());
	OutTexturesAndMaterials.Append(LocalTextures);
	OutTexturesAndMaterials.Append(LocalMaterialInstances);
}
#endif

ALandscapeProxy* ULandscapeComponent::GetLandscapeProxy() const
{
	return CastChecked<ALandscapeProxy>(GetOuter());
}

FString ULandscapeComponent::GetDebugName() const
{
	const FIntPoint ComponentKey = GetComponentKey();
	const FIntPoint SectionBase = GetSectionBase();
	return FString::Format(TEXT("Component_{0}_{1}[{2}_{3}]"), { ComponentKey.X, ComponentKey.Y, SectionBase.X, SectionBase.Y });
}

int32 ULandscapeComponent::GetNumRelevantMips() const
{
	const int32 TextureSize = (SubsectionSizeQuads + 1) * NumSubsections;
	const int32 NumTextureMips = FMath::FloorLog2(TextureSize) + 1;
	// We actually only don't care about the last texture mip, since a 1 vertex landscape is meaningless. When using 2x2 subsections, we can even drop an additional mip 
	//  as the 4 texels of the penultimate mip will be identical (i.e. 4 sub-sections of 1 vertex are equally meaningless) :
	const int32 NumRelevantMips = (NumSubsections > 1) ? (NumTextureMips - 2) : (NumTextureMips - 1);
	check(NumRelevantMips > 0);
	return NumRelevantMips;
}

const FMeshMapBuildData* ULandscapeComponent::GetMeshMapBuildData() const
{
	AActor* Owner = GetOwner();

	if (Owner)
	{
		ULevel* OwnerLevel = Owner->GetLevel();

#if WITH_EDITOR
		if (FStaticLightingSystemInterface::GetPrimitiveMeshMapBuildData(this))
		{
			return FStaticLightingSystemInterface::GetPrimitiveMeshMapBuildData(this);
		}
#endif

		if (OwnerLevel && OwnerLevel->OwningWorld)
		{
			UMapBuildDataRegistry* MapBuildData = UMapBuildDataRegistry::Get(this);

			if (MapBuildData)
			{
				return MapBuildData->GetMeshBuildData(MapBuildDataId);
			}
		}
	}

	return NULL;
}

void ULandscapeComponent::OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType TeleportType)
{
	// if registered with a group, we need to update our registration to reflect the new transform
	if (RegisteredLandscapeGroup != nullptr)
	{
		RegisteredLandscapeGroup->OnTransformUpdated(this);
	}
}

bool ULandscapeComponent::IsPrecomputedLightingValid() const
{
	return GetMeshMapBuildData() != NULL;
}

void ULandscapeComponent::PropagateLightingScenarioChange()
{
	FComponentRecreateRenderStateContext Context(this);
}

bool ULandscapeComponent::IsHLODRelevant() const
{
	if (!CanBeHLODRelevant(this))
	{
		return false;
	}

#if WITH_EDITOR
	return bEnableAutoLODGeneration;
#else
	return false;
#endif
}

TArray<URuntimeVirtualTexture*> const& ULandscapeComponent::GetRuntimeVirtualTextures() const
{
	return GetLandscapeProxy()->RuntimeVirtualTextures;
}

ERuntimeVirtualTextureMainPassType ULandscapeComponent::GetVirtualTextureRenderPassType() const
{
	return GetLandscapeProxy()->VirtualTextureRenderPassType;
}

ULandscapeInfo* ULandscapeComponent::GetLandscapeInfo() const
{
	return GetLandscapeProxy()->GetLandscapeInfo();
}

TArray<ULandscapeGrassType*> ULandscapeComponent::GetGrassTypesBP() const
{
	TArray<ULandscapeGrassType*> GrassTypes;
	GrassTypes.Reserve(NamedGrassTypes.Num());
	for (const TPair<FName, TObjectPtr<ULandscapeGrassType>>& ItPair : NamedGrassTypes)
	{
		if (ItPair.Value != nullptr)
		{
			GrassTypes.Add(ItPair.Value);
		}
	}
	return GrassTypes;
}

TArray<TObjectPtr<ULandscapeGrassType>> ULandscapeComponent::GetGrassTypes() const
{
	return ObjectPtrWrap(GetGrassTypesBP());
}

void ULandscapeComponent::BeginDestroy()
{
	Super::BeginDestroy();

#if WITH_EDITOR
	// Ask render thread to destroy EditToolRenderData
	EditToolRenderData = FLandscapeEditToolRenderData();
	UpdateEditToolRenderData();

	if (GIsEditor && !HasAnyFlags(RF_ClassDefaultObject))
	{
		ALandscapeProxy* Proxy = GetLandscapeProxy();

		// Remove any weightmap allocations from the Landscape Actor's map
		for (int32 LayerIdx = 0; LayerIdx < WeightmapLayerAllocations.Num(); LayerIdx++)
		{
			int32 WeightmapIndex = WeightmapLayerAllocations[LayerIdx].WeightmapTextureIndex;
			if (WeightmapTextures.IsValidIndex(WeightmapIndex))
			{
				UTexture2D* WeightmapTexture = WeightmapTextures[WeightmapIndex];
				TObjectPtr<ULandscapeWeightmapUsage>* Usage = Proxy->WeightmapUsageMap.Find(WeightmapTexture);
				if (Usage != nullptr && (*Usage) != nullptr)
				{
					(*Usage)->ChannelUsage[WeightmapLayerAllocations[LayerIdx].WeightmapTextureChannel] = nullptr;

					if ((*Usage)->IsEmpty())
					{
						Proxy->WeightmapUsageMap.Remove(WeightmapTexture);
					}
				}
			}
		}

		WeightmapTexturesUsage.Reset();
	}
#endif
}

FPrimitiveSceneProxy* ULandscapeComponent::CreateSceneProxy()
{
	if (CheckPSOPrecachingAndBoostPriority(EPSOPrecachePriority::Highest) && UE::Landscape::GetLandscapePSOPrecacheProxyCreationStrategy() == EPSOPrecacheProxyCreationStrategy::DelayUntilPSOPrecached)
	{
		UE_LOGF(LogLandscape, Verbose, "Skipping CreateSceneProxy for ULandscapeComponent %ls (Its PSOs are still compiling)", *GetFullName());
		return nullptr;
	}
	
	FPrimitiveSceneProxy* LocalSceneProxy = nullptr;

	ILandscapeModule& LandscapeModule = FModuleManager::GetModuleChecked<ILandscapeModule>("Landscape");
	const UE::Landscape::FCreateLandscapeComponentSceneProxyDelegate& CreateProxyDelegate = LandscapeModule.GetCreateLandscapeComponentSceneProxyDelegate();
	if (CreateProxyDelegate.IsBound())
	{
		LocalSceneProxy = CreateProxyDelegate.Execute(this);
	}
	
	if (LocalSceneProxy == nullptr)
	{
		LocalSceneProxy = new FLandscapeComponentSceneProxy(this);
	}
	return LocalSceneProxy;
}

bool ULandscapeComponent::IsShown(const FEngineShowFlags& ShowFlags) const
{
	return ShowFlags.Landscape;
}

void ULandscapeComponent::DestroyComponent(bool bPromoteChildren/*= false*/)
{
	ALandscapeProxy* Proxy = GetLandscapeProxy();
	if (Proxy)
	{
		Proxy->LandscapeComponents.Remove(this);
	}

	Super::DestroyComponent(bPromoteChildren);
}

FBoxSphereBounds ULandscapeComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	FBox MyBounds = CachedLocalBox.TransformBy(LocalToWorld);
	MyBounds = MyBounds.ExpandBy({ 0, 0, NegativeZBoundsExtension }, { 0, 0, PositiveZBoundsExtension });

	ALandscapeProxy* Proxy = GetLandscapeProxy();
	if (Proxy)
	{
		MyBounds = MyBounds.ExpandBy({ 0, 0, Proxy->NegativeZBoundsExtension }, { 0, 0, Proxy->PositiveZBoundsExtension });
	}

	return FBoxSphereBounds(MyBounds);
}

static void OnStaticMeshLODDistanceScaleChanged()
{
	extern RENDERER_API TAutoConsoleVariable<float> CVarStaticMeshLODDistanceScale;

	static float LastValue = 1.0f;

	if (LastValue != CVarStaticMeshLODDistanceScale.GetValueOnAnyThread())
	{
		LastValue = CVarStaticMeshLODDistanceScale.GetValueOnAnyThread();

		for (auto* LandscapeComponent : TObjectRange<ULandscapeComponent>(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage))
		{
			LandscapeComponent->MarkRenderStateDirty();
		}
	}
}

FAutoConsoleVariableSink OnStaticMeshLODDistanceScaleChangedSink(FConsoleCommandDelegate::CreateStatic(&OnStaticMeshLODDistanceScaleChanged));

ULandscapeHeightmapTextureEdgeFixup* ULandscapeComponent::InstallOrUpdateTextureUserDatas(bool bUseEdgeFixup, bool bUseCompression, bool bUpdateSnapshotNow, int32 HeightmapCompressionMipThreshold)
{
	if (HeightmapTexture == nullptr)
	{
		UE_LOGF(LogLandscape, Warning, "Tried to install EdgeFixup on component %ls (proxy %ls), but it had NO heightmap", *GetPathName(), *GetLandscapeProxy()->GetPathName());
		return nullptr;
	}

	// grid scale must be updated to get proper normal calculations on the edges
	FVector LandscapeGridScale = GetLandscapeProxy()->GetRootComponent()->GetRelativeScale3D();

	// first update or install the heightmap texture edge fixup
	ULandscapeHeightmapTextureEdgeFixup* EdgeFixup = nullptr;
	if (bUseEdgeFixup && UE::Landscape::ShouldInstallEdgeFixup())
	{
		// find or create edge fixup
		EdgeFixup = ULandscapeHeightmapTextureEdgeFixup::FindOrCreateFor(HeightmapTexture);
#if WITH_EDITOR
		if (bUpdateSnapshotNow)
		{
			const bool bForceUpdate = true;
			EdgeFixup->UpdateEdgeSnapshotFromHeightmapSource(LandscapeGridScale, bForceUpdate);
		}
#endif // WITH_EDITOR
	}
	else
	{
		// remove any existing edge fixup (we will remove the factory references to the EdgeFixup below)
		HeightmapTexture->RemoveUserDataOfClass(ULandscapeHeightmapTextureEdgeFixup::StaticClass());
		EdgeFixup = nullptr;
	}

	// check if the heightmap has a ULandscapeTextureMipEdgeOverrideFactory or a ULandscapeTextureStorageProviderFactory
	ULandscapeTextureMipEdgeOverrideFactory* OverrideFactory = (ULandscapeTextureMipEdgeOverrideFactory*) HeightmapTexture->GetAssetUserDataOfClass(ULandscapeTextureMipEdgeOverrideFactory::StaticClass());
	ULandscapeTextureStorageProviderFactory* StorageFactory = (ULandscapeTextureStorageProviderFactory*) HeightmapTexture->GetAssetUserDataOfClass(ULandscapeTextureStorageProviderFactory::StaticClass());

	// we should never have both
	check(!OverrideFactory || !StorageFactory);

#if WITH_EDITOR
	if (bUseCompression || StorageFactory)
#else // !WITH_EDITOR
	check(bUseCompression == false); // cannot install compression in non-editor builds
	if (StorageFactory)
#endif // !WITH_EDITOR
	{
		// Remove any existing mip edge override factory
		if (OverrideFactory)
		{
			OverrideFactory->SetupEdgeFixup(nullptr);
			HeightmapTexture->RemoveUserDataOfClass(ULandscapeTextureMipEdgeOverrideFactory::StaticClass());
			OverrideFactory = nullptr;
		}

#if WITH_EDITOR
		// Install or Update the Storage Factory [editor only]
		if (StorageFactory == nullptr)
		{
			StorageFactory = ULandscapeTextureStorageProviderFactory::ApplyTo(HeightmapTexture, LandscapeGridScale, HeightmapCompressionMipThreshold, NumSubsections, SubsectionSizeQuads);
		}
		else
		{
			// since different platforms may change compression settings / thresholds, we should update the compression each time
			StorageFactory->UpdateCompressedDataFromSource(HeightmapTexture, LandscapeGridScale, HeightmapCompressionMipThreshold, NumSubsections, SubsectionSizeQuads);
		}
#endif // WITH_EDITOR
		StorageFactory->SetupEdgeFixup(EdgeFixup);
	}
		
	if (EdgeFixup != nullptr)
	{
		// an edge fixup requires at least one factory -- if storage factory doesn't exist, create an override factory
		if (StorageFactory == nullptr)
		{
			if (OverrideFactory == nullptr)
			{
				OverrideFactory = ULandscapeTextureMipEdgeOverrideFactory::AddTo(HeightmapTexture);
			}
			OverrideFactory->SetupEdgeFixup(EdgeFixup);
		}

	}
	else
	{
		// no edge fixup, override factory not needed -- remove any existing one
		if (OverrideFactory)
		{
			OverrideFactory->SetupEdgeFixup(nullptr);
			HeightmapTexture->RemoveUserDataOfClass(ULandscapeTextureMipEdgeOverrideFactory::StaticClass());
			OverrideFactory = nullptr;
		}
	}

#if WITH_EDITOR
	// The EdgeFixup will always require linear texture data and should not apply per platform offline processing
	HeightmapTexture->bNotOfflineProcessed = (EdgeFixup != nullptr);
#endif // WITH_EDITOR

	// double check we've achieved the desired relationships..
	if (EdgeFixup)
	{
		// EdgeFixup must have exactly one factory
		check(!!OverrideFactory == !StorageFactory);
	}
	else
	{
		// No EdgeFixup means No override factory
		// (storage factory is optional, it can exist without an EdgeFixup)
		check(!OverrideFactory);
	}

	return EdgeFixup;
}

#if WITH_EDITOR
void ALandscapeProxy::InstallOrUpdateTextureUserDatas(const ITargetPlatform* TargetPlatform)
{
	FName IniPlatformName = *TargetPlatform->IniPlatformName();

	int32 HeightmapCompressionMode = 0;
	if (IConsoleVariable* PlatformCVar = CVarLandscapeHeightmapCompressionMode->GetPlatformValueVariable(IniPlatformName).Get())
	{
		PlatformCVar->GetValue(HeightmapCompressionMode);
	}
	else
	{
		CVarLandscapeHeightmapCompressionMode->GetValue(HeightmapCompressionMode);
	}

	int32 HeightmapCompressionMipThreshold = 32;
	if (IConsoleVariable* PlatformCVar = CVarLandscapeHeightmapCompressionMipThreshold->GetPlatformValueVariable(IniPlatformName).Get())
	{
		PlatformCVar->GetValue(HeightmapCompressionMipThreshold);
	}
	else
	{
		CVarLandscapeHeightmapCompressionMipThreshold->GetValue(HeightmapCompressionMipThreshold);
	}

	const bool bShouldCompressHeightmap = (HeightmapCompressionMode > 0);
	const bool bIsStreamingProxy = this->IsA<ALandscapeStreamingProxy>();
	const bool bUseEdgeFixups = bIsStreamingProxy;
	const bool bUpdateSnapshotNow = true;
	for (ULandscapeComponent* LandscapeComponent : LandscapeComponents)
	{
		LandscapeComponent->InstallOrUpdateTextureUserDatas(bUseEdgeFixups, bShouldCompressHeightmap, bUpdateSnapshotNow, HeightmapCompressionMipThreshold);
	}
}
#endif // WITH_EDITOR

void ULandscapeComponent::OnRegister()
{
	Super::OnRegister();

	if (ALandscapeProxy* Proxy = GetLandscapeProxy())
	{
		// Generate MID representing the MIC
		if (Proxy->bUseDynamicMaterialInstance)
		{
			MaterialInstancesDynamic.Reserve(MaterialInstances.Num());

			for (int32 i = 0; i < MaterialInstances.Num(); ++i)
			{
				MaterialInstancesDynamic.Add(UMaterialInstanceDynamic::Create(MaterialInstances[i], this));
			}
		}

		// AActor::GetWorld checks for Unreachable and BeginDestroyed
		UWorld* World = Proxy->GetWorld();
		if (World)
		{
			if (ULandscapeInfo* Info = GetLandscapeInfo())
			{
				Info->RegisterActorComponent(this);
			}
			if (ULandscapeSubsystem *Subsystem = World->GetSubsystem<ULandscapeSubsystem>())
			{
				Subsystem->RegisterComponent(this);
			}
		}
	}
}

void ULandscapeComponent::OnUnregister()
{
	Super::OnUnregister();

#if WITH_EDITOR
	PhysicalMaterialTask.Release();
#endif

	if (ALandscapeProxy* Proxy = GetLandscapeProxy())
	{
		// Generate MID representing the MIC
		if (Proxy->bUseDynamicMaterialInstance)
		{
			MaterialInstancesDynamic.Empty();
		}

		// AActor::GetWorld checks for Unreachable and BeginDestroyed
		UWorld* World = Proxy->GetWorld();
		if (World)
		{
			if (ULandscapeInfo* Info = GetLandscapeInfo())
			{
				Info->UnregisterActorComponent(this);
			}
			if (ULandscapeSubsystem* Subsystem = World->GetSubsystem<ULandscapeSubsystem>())
			{
				Subsystem->UnregisterComponent(this);
			}
		}
	}
}

UTexture2D* ULandscapeComponent::GetHeightmap(bool InReturnEditingHeightmap) const
{
#if WITH_EDITORONLY_DATA
	if (InReturnEditingHeightmap)
	{
		if (const FLandscapeLayerComponentData* EditingLayer = GetEditingLayer())
		{
			return EditingLayer->HeightmapData.Texture;
		}
	}
#endif

	return HeightmapTexture;
}

UTexture2D* ULandscapeComponent::GetHeightmap(const FGuid& InLayerGuid) const
{
#if WITH_EDITORONLY_DATA
	if (InLayerGuid.IsValid())
	{
		if (const FLandscapeLayerComponentData* LayerData = GetLayerData(InLayerGuid))
		{
			return LayerData->HeightmapData.Texture;
		}
	}
#endif

	return HeightmapTexture;
}

const TArray<UTexture2D*>& ULandscapeComponent::GetWeightmapTextures(bool InReturnEditingWeightmap) const
{
#if WITH_EDITORONLY_DATA
	if (InReturnEditingWeightmap)
	{
		if (const FLandscapeLayerComponentData* EditingLayer = GetEditingLayer())
		{
			return EditingLayer->WeightmapData.Textures;
		}
	}
#endif

	return WeightmapTextures;
}

TArray<TObjectPtr<UTexture2D>>& ULandscapeComponent::GetWeightmapTextures(bool InReturnEditingWeightmap)
{
#if WITH_EDITORONLY_DATA
	if (InReturnEditingWeightmap)
	{
		if (FLandscapeLayerComponentData* EditingLayer = GetEditingLayer())
		{
			return EditingLayer->WeightmapData.Textures;
		}
	}
#endif

	return WeightmapTextures;
}

const TArray<UTexture2D*>& ULandscapeComponent::GetWeightmapTextures(const FGuid& InLayerGuid) const
{
#if WITH_EDITORONLY_DATA
	if (InLayerGuid.IsValid())
	{
		if (const FLandscapeLayerComponentData* LayerData = GetLayerData(InLayerGuid))
		{
			return LayerData->WeightmapData.Textures;
		}
	}
#endif

	return WeightmapTextures;
}

TArray<TObjectPtr<UTexture2D>>& ULandscapeComponent::GetWeightmapTextures(const FGuid& InLayerGuid)
{
#if WITH_EDITORONLY_DATA
	if (InLayerGuid.IsValid())
	{
		if (FLandscapeLayerComponentData* LayerData = GetLayerData(InLayerGuid))
		{
			return LayerData->WeightmapData.Textures;
		}
	}
#endif

	return WeightmapTextures;
}

const TArray<UTexture2D*>& ULandscapeComponent::GetRenderedWeightmapTexturesForFeatureLevel(ERHIFeatureLevel::Type FeatureLevel) const
{
	if (FeatureLevel == ERHIFeatureLevel::ES3_1)
	{
		return MobileWeightmapTextures;
	}
	else
	{
		return WeightmapTextures;
	}
}

const TArray<FWeightmapLayerAllocationInfo>& ULandscapeComponent::GetWeightmapLayerAllocations(bool InReturnEditingWeightmap) const
{
#if WITH_EDITORONLY_DATA
	if (InReturnEditingWeightmap)
	{
		if (const FLandscapeLayerComponentData* EditingLayer = GetEditingLayer())
		{
			return EditingLayer->WeightmapData.LayerAllocations;
		}
	}
#endif

	return WeightmapLayerAllocations;
}

TArray<FWeightmapLayerAllocationInfo>& ULandscapeComponent::GetWeightmapLayerAllocations(const FGuid& InLayerGuid)
{
#if WITH_EDITORONLY_DATA
	if (InLayerGuid.IsValid())
	{
		if (FLandscapeLayerComponentData* LayerData = GetLayerData(InLayerGuid))
		{
			return LayerData->WeightmapData.LayerAllocations;
		}
	}
#endif

	return WeightmapLayerAllocations;
}

const TArray<FWeightmapLayerAllocationInfo>& ULandscapeComponent::GetWeightmapLayerAllocations(const FGuid& InLayerGuid) const
{
#if WITH_EDITORONLY_DATA
	if (InLayerGuid.IsValid())
	{
		if (const FLandscapeLayerComponentData* LayerData = GetLayerData(InLayerGuid))
		{
			return LayerData->WeightmapData.LayerAllocations;
		}
	}
#endif

	return WeightmapLayerAllocations;
}

TArray<FWeightmapLayerAllocationInfo>& ULandscapeComponent::GetWeightmapLayerAllocations(bool InReturnEditingWeightmap)
{
#if WITH_EDITORONLY_DATA
	if (InReturnEditingWeightmap)
	{
		if (FLandscapeLayerComponentData* EditingLayer = GetEditingLayer())
		{
			return EditingLayer->WeightmapData.LayerAllocations;
		}
	}
#endif

	return WeightmapLayerAllocations;
}

const TArray<UTexture2D*>& ULandscapeComponent::GetCurrentRuntimeWeightmapTextures() const
{
	UWorld* World = GetWorld();
	bool bIsMobile = (World != nullptr) && (World->GetFeatureLevel() == ERHIFeatureLevel::ES3_1);
	return bIsMobile ? MobileWeightmapTextures : WeightmapTextures;
}

const TArray<FWeightmapLayerAllocationInfo>& ULandscapeComponent::GetCurrentRuntimeWeightmapLayerAllocations() const
{
	UWorld* World = GetWorld();
	bool bIsMobile = (World != nullptr) && (World->GetFeatureLevel() == ERHIFeatureLevel::ES3_1);
	return bIsMobile ? MobileWeightmapLayerAllocations : WeightmapLayerAllocations;
}

TArray<FWeightmapLayerAllocationInfo>& ULandscapeComponent::GetCurrentRuntimeWeightmapLayerAllocations()
{
	UWorld* World = GetWorld();
	bool bIsMobile = (World != nullptr) && (World->GetFeatureLevel() == ERHIFeatureLevel::ES3_1);
	return bIsMobile ? MobileWeightmapLayerAllocations : WeightmapLayerAllocations;
}

#if WITH_EDITOR

FLandscapeLayerComponentData* ULandscapeComponent::GetEditingLayer()
{
	if (ALandscape* LandscapeActor = GetLandscapeActor())
	{
		const FGuid& EditingLayerGuid = LandscapeActor->GetEditingLayer();
		return EditingLayerGuid.IsValid() ? LayersData.Find(EditingLayerGuid) : nullptr;
	}
	return nullptr;
}

const FLandscapeLayerComponentData* ULandscapeComponent::GetEditingLayer() const
{
	if (ALandscape* LandscapeActor = GetLandscapeActor())
	{
		const FGuid& EditingLayerGuid = LandscapeActor->GetEditingLayer();
		return EditingLayerGuid.IsValid() ? LayersData.Find(EditingLayerGuid) : nullptr;
	}
	return nullptr;
}

FGuid ULandscapeComponent::GetEditingLayerGUID() const
{
	ALandscape* Landscape = GetLandscapeActor();
	return Landscape != nullptr ? Landscape->GetEditingLayer() : FGuid();
}

bool ULandscapeComponent::HasLayersData() const
{
	return LayersData.Num() > 0;
}

const FLandscapeLayerComponentData* ULandscapeComponent::GetLayerData(const FGuid& InLayerGuid) const
{
	return LayersData.Find(InLayerGuid);
}

FLandscapeLayerComponentData* ULandscapeComponent::GetLayerData(const FGuid& InLayerGuid)
{
	return LayersData.Find(InLayerGuid);
}

void ULandscapeComponent::CopyComponentDataToEditLayer(const FGuid& InSourceEditLayerGuid, const FGuid& InDestEditLayerGuid, bool bInUseObsoleteLayerData, TSet<UTexture2D*>& OutProcessedHeightmaps, TSet<UTexture2D*>& OutProcessedWeightmaps)
{
	const ALandscapeProxy* LandscapeProxy = GetLandscapeProxy();
	FLandscapeLayerComponentData* DestLayerComponentData = GetLayerData(InDestEditLayerGuid);

	if (!InDestEditLayerGuid.IsValid() || !LandscapeProxy)
	{
		UE_LOGF(LogLandscape, Error, "CopyComponentDataToEditLayer: Failed to copy data to component: %ls. Destination edit layer guid is invalid or Proxy is not yet registered with ALandscape", *GetName());
		return;
	}

	if (DestLayerComponentData == nullptr || !DestLayerComponentData->IsInitialized())
	{
		UE_LOGF(LogLandscape, Error, "CopyComponentDataToEditLayer: Failed to copy data to component: %ls. Destination edit layer data does not exist or is not initialized on component", *GetName());
		return;
	}

	// Determine the source data: uses the given edit layer data when valid, or the final merged data
	const FLandscapeLayerComponentData* DataToCopy = nullptr;
	FLandscapeLayerComponentData CurrentFinalLayerData;

	if (InSourceEditLayerGuid.IsValid())
	{
		DataToCopy = bInUseObsoleteLayerData ? GetObsoleteEditLayerData(InSourceEditLayerGuid) : GetLayerData(InSourceEditLayerGuid);
	}
	else
	{
		CurrentFinalLayerData.HeightmapData.Texture = GetHeightmap();
		CurrentFinalLayerData.WeightmapData.LayerAllocations = GetWeightmapLayerAllocations();
		CurrentFinalLayerData.WeightmapData.Textures = GetWeightmapTextures();
		CurrentFinalLayerData.WeightmapData.TextureUsages = GetWeightmapTexturesUsage();

		DataToCopy = bInUseObsoleteLayerData ? GetObsoleteEditLayerData(FGuid()) : &CurrentFinalLayerData;
	}

	if (DataToCopy == nullptr || !DataToCopy->IsInitialized())
	{
		UE_LOGF(LogLandscape, Error, "CopyComponentDataToEditLayer: Failed to copy data to destination layer for %ls : Component: %ls. Source layer GUID is invalid or data is not initialized properly",
			*LandscapeProxy->GetName(), *GetName());
		return;
	}

	Modify();

	// Heightmap: 
	UTexture2D* SourceHeightmap = DataToCopy->HeightmapData.Texture;
	if (!OutProcessedHeightmaps.Contains(SourceHeightmap))
	{
		OutProcessedHeightmaps.Add(SourceHeightmap);

		UTexture* DestinationHeightmap = DestLayerComponentData->HeightmapData.Texture;
		check(DestinationHeightmap != nullptr);

		// Only copy Mip0 as other mips will get regenerated
		TArray64<uint8> ExistingMip0Data;
		SourceHeightmap->Source.GetMipData(ExistingMip0Data, 0);

		// Mark transactional for Undo operations 
		DestinationHeightmap->SetFlags(RF_Transactional);

		// Calling modify here makes sure that async texture compilation finishes so we can Lock the mip
		DestinationHeightmap->Modify();
		FColor* Mip0Data = (FColor*)DestinationHeightmap->Source.LockMip(0);
		FMemory::Memcpy(Mip0Data, ExistingMip0Data.GetData(), ExistingMip0Data.Num());
		DestinationHeightmap->Source.UnlockMip(0);

		DestinationHeightmap->UpdateResource();
		DestinationHeightmap->ClearFlags(RF_Transactional);
	}

	// Weightmaps:
	const TArray<UTexture2D*>& SourceWeightmapTextures = DataToCopy->WeightmapData.Textures;
	const TArray<FWeightmapLayerAllocationInfo>& SourceLayerAllocations = DataToCopy->WeightmapData.LayerAllocations;

	DestLayerComponentData->WeightmapData.Textures.Empty(SourceWeightmapTextures.Num());
	DestLayerComponentData->WeightmapData.LayerAllocations.Empty(SourceLayerAllocations.Num());
	DestLayerComponentData->WeightmapData.TextureUsages.Empty();

	DestLayerComponentData->WeightmapData.Textures.AddDefaulted(SourceWeightmapTextures.Num());

	for (int32 TextureIndex = 0; TextureIndex < SourceWeightmapTextures.Num(); ++TextureIndex)
	{
		UTexture2D* ComponentWeightmap = SourceWeightmapTextures[TextureIndex];
		if (OutProcessedWeightmaps.Contains(ComponentWeightmap))
		{
			DestLayerComponentData->WeightmapData.Textures[TextureIndex] = ComponentWeightmap;
		}
		else
		{
			// No need for mip chain on edit layers : 
			check(LandscapeProxy);
			UTexture2D* DestLayerWeightmapTexture = LandscapeProxy->CreateLandscapeTexture(ComponentWeightmap->Source.GetSizeX(), ComponentWeightmap->Source.GetSizeY(), TEXTUREGROUP_Terrain_Weightmap,
				ComponentWeightmap->Source.GetFormat(), /*OptionalOverrideOuter = */ nullptr, /*bCompress = */false, /*bMipChain = */false);

			// Only copy Mip0 as other mips will get regenerated
			TArray64<uint8> ExistingMip0Data;
			ComponentWeightmap->Source.GetMipData(ExistingMip0Data, 0);

			// Mark transactional for Undo operations
			DestLayerWeightmapTexture->SetFlags(RF_Transactional);

			// Calling modify here makes sure that async texture compilation finishes so we can Lock the mip
			DestLayerWeightmapTexture->Modify();
			FColor* Mip0Data = (FColor*)DestLayerWeightmapTexture->Source.LockMip(0);
			FMemory::Memcpy(Mip0Data, ExistingMip0Data.GetData(), ExistingMip0Data.Num());
			DestLayerWeightmapTexture->Source.UnlockMip(0);

			DestLayerComponentData->WeightmapData.Textures[TextureIndex] = DestLayerWeightmapTexture;

			// Copy weightmap layer allocations 
			for (const FWeightmapLayerAllocationInfo& Allocation : SourceLayerAllocations)
			{
				if (Allocation.WeightmapTextureIndex == TextureIndex)
				{
					DestLayerComponentData->WeightmapData.LayerAllocations.Add(Allocation);
				}
			}

			DestLayerWeightmapTexture->UpdateResource();
			DestLayerWeightmapTexture->ClearFlags(RF_Transactional);

			OutProcessedWeightmaps.Add(DestLayerWeightmapTexture);
		}
	}

	// Recalculate weightmap usage based on Textures/Allocations instead of relying on copies from source data
	InitializeLayersWeightmapUsage(InDestEditLayerGuid);
}

const TMap<FGuid, FLandscapeLayerComponentData>& ULandscapeComponent::GetObsoleteEditLayersData() const
{
	return ObsoleteEditLayerData;
}

const FLandscapeLayerComponentData* ULandscapeComponent::GetObsoleteEditLayerData(const FGuid& InLayerGuid) const
{
	return ObsoleteEditLayerData.Find(InLayerGuid);
}

void ULandscapeComponent::RemoveObsoleteEditLayerData(const FGuid& InLayerGuid)
{
	Modify();
	ObsoleteEditLayerData.Remove(InLayerGuid);
}

void ULandscapeComponent::InitializeObsoleteEditLayerData(const TSet<FGuid>& InParentEditLayerGuids)
{
	// Data should only be set during first registration
	check(!GetLandscapeProxy() || GetLandscapeProxy()->RegistrationCount == 0);

	// Move all obsolete layer data from the active layers data to transient storage
	ForEachLayer([&InParentEditLayerGuids, this](const FGuid& LayerGuid, FLandscapeLayerComponentData& LayerData)
	{
		if (LayerGuid.IsValid() && !InParentEditLayerGuids.Contains(LayerGuid))
		{
			ObsoleteEditLayerData.Add(LayerGuid, *GetLayerData(LayerGuid));
		}
	});

	if (!ObsoleteEditLayerData.IsEmpty())
	{
		Modify();

		// Obsolete data has been added to transient storage
		// Remove layer data from active layers list so it is not included in merge
		for (TPair<FGuid, FLandscapeLayerComponentData>& ObsoleteLayer : ObsoleteEditLayerData)
		{
			RemoveLayerData(ObsoleteLayer.Key);
		}

		// Create duplicate textures of the final data mapped to FGuid()
		FLandscapeLayerComponentData CurrentFinalLayerData;
		CurrentFinalLayerData.HeightmapData.Texture = HeightmapTexture;
		CurrentFinalLayerData.WeightmapData.LayerAllocations = WeightmapLayerAllocations;
		CurrentFinalLayerData.WeightmapData.Textures = WeightmapTextures;
		CurrentFinalLayerData.WeightmapData.TextureUsages = WeightmapTexturesUsage;

		FLandscapeLayerComponentData FinalLayerDataCopy("Obsolete Final Data OnLoad");
		UE::Landscape::Private::CreateLandscapeComponentLayerDataDuplicate(CurrentFinalLayerData, FinalLayerDataCopy);
		ObsoleteEditLayerData.Add(FGuid(), FinalLayerDataCopy);

		GetLandscapeActor()->RequestLayersContentUpdateForceAll();
	}
}

void ULandscapeComponent::ForEachLayer(TFunctionRef<void(const FGuid&, struct FLandscapeLayerComponentData&)> Fn)
{
	for (auto& Pair : LayersData)
	{
		Fn(Pair.Key, Pair.Value);
	}
}

void ULandscapeComponent::AddLayerData(const FGuid& InLayerGuid, const FLandscapeLayerComponentData& InData)
{
	Modify();
	FLandscapeLayerComponentData& Data = LayersData.FindOrAdd(InLayerGuid);
	Data = InData;
}

// TODO [jonathan.bard] : When removing texture sharing, this can be simplified : 
//  - InComponentsUsingHeightmap is only ever used in 1 case : ALandscapeProxy::InitializeLayerWithEmptyContent. This is the only place where texture sharing should actually occur. 
//   In all other places, we pass a single component to this array : the component itself
//  - InOutCreatedHeightmapTextures isn't being used anywhere and can be removed safely as well
//  - We should also assert or early-out in the cases where !LandscapeEditLayer->NeedsPersistentTextures() : in this case, we don't need any texture at all for this component on this edit layer
void ULandscapeComponent::AddDefaultLayerData(const FGuid& InLayerGuid, const TArray<ULandscapeComponent*>& InComponentsUsingHeightmap, TMap<UTexture2D*, UTexture2D*>& InOutCreatedHeightmapTextures)
{
	Modify();

	UTexture2D* ComponentHeightmap = GetHeightmap();

	// Compute per layer data
	FLandscapeLayerComponentData* LayerData = GetLayerData(InLayerGuid);

	if (LayerData == nullptr || !LayerData->IsInitialized())
	{
		const ULandscapeEditLayerBase* LandscapeEditLayer = GetLandscapeActor() ? GetLandscapeActor()->GetEditLayerConst(InLayerGuid) : nullptr;
		FLandscapeLayerComponentData NewData(LandscapeEditLayer ? LandscapeEditLayer->GetName() : FName());

		// Setup Heightmap data
		UTexture2D** LayerHeightmap = InOutCreatedHeightmapTextures.Find(ComponentHeightmap);
		const FColor DefaultHeightColor = LandscapeDataAccess::GetDefaultPackedHeightColor();

		if (LayerHeightmap == nullptr)
		{
			// No mipchain required as these layer weight maps are used in layer compositing to generate a final set of weight maps to be used for rendering
			UTexture2D* NewLayerHeightmap = GetLandscapeProxy()->CreateLandscapeTexture(ComponentHeightmap->Source.GetSizeX(), ComponentHeightmap->Source.GetSizeY(), TEXTUREGROUP_Terrain_Heightmap, ComponentHeightmap->Source.GetFormat(), /* OptionalOverrideOuter = */ nullptr, /* bCompress = */ false, /* bMipChain = */ false);
			LayerHeightmap = &InOutCreatedHeightmapTextures.Add(ComponentHeightmap, NewLayerHeightmap);

			// Init Mip0 to be at 32768 which is equal to "0"
			TArrayView<FColor> Mip0Data((FColor*)NewLayerHeightmap->Source.LockMip(0), NewLayerHeightmap->Source.GetSizeX() * NewLayerHeightmap->Source.GetSizeY());

			for (ULandscapeComponent* ComponentUsingHeightmap : InComponentsUsingHeightmap)
			{
				int32 HeightmapComponentOffsetX = FMath::RoundToInt32(NewLayerHeightmap->Source.GetSizeX() * ComponentUsingHeightmap->HeightmapScaleBias.Z);
				int32 HeightmapComponentOffsetY = FMath::RoundToInt32(NewLayerHeightmap->Source.GetSizeY() * ComponentUsingHeightmap->HeightmapScaleBias.W);

				for (int32 SubsectionY = 0; SubsectionY < NumSubsections; SubsectionY++)
				{
					for (int32 SubsectionX = 0; SubsectionX < NumSubsections; SubsectionX++)
					{
						for (int32 SubY = 0; SubY <= SubsectionSizeQuads; SubY++)
						{
							for (int32 SubX = 0; SubX <= SubsectionSizeQuads; SubX++)
							{
								// X/Y of the vertex we're looking at in component's coordinates.
								const int32 CompX = SubsectionSizeQuads * SubsectionX + SubX;
								const int32 CompY = SubsectionSizeQuads * SubsectionY + SubY;

								// X/Y of the vertex we're looking indexed into the texture data
								const int32 TexX = (SubsectionSizeQuads + 1) * SubsectionX + SubX;
								const int32 TexY = (SubsectionSizeQuads + 1) * SubsectionY + SubY;

								const int32 HeightTexDataIdx = (HeightmapComponentOffsetX + TexX) + (HeightmapComponentOffsetY + TexY) * NewLayerHeightmap->Source.GetSizeX();

								// Copy default height and normal data. Normals will be calculated later
								Mip0Data[HeightTexDataIdx] = DefaultHeightColor;
							}
						}
					}
				}
			}

			NewLayerHeightmap->Source.UnlockMip(0);
			ULandscapeTextureHash::UpdateHash(NewLayerHeightmap, ELandscapeTextureUsage::EditLayerData, ELandscapeTextureType::Heightmap);
			NewLayerHeightmap->UpdateResource();
		}

		NewData.HeightmapData.Texture = *LayerHeightmap;

		// Nothing to do for Weightmap by default

		AddLayerData(InLayerGuid, MoveTemp(NewData));
	}
}

void ULandscapeComponent::RemoveLayerData(const FGuid& InLayerGuid)
{
	Modify();
	LayersData.Remove(InLayerGuid);
}

#endif // WITH_EDITOR

void ULandscapeComponent::SetHeightmap(UTexture2D* NewHeightmap)
{
	check(NewHeightmap != nullptr);
	HeightmapTexture = NewHeightmap;
}

void ULandscapeComponent::SetWeightmapTextures(const TArray<UTexture2D*>& InNewWeightmapTextures, bool InApplyToEditingWeightmap)
{
#if WITH_EDITOR
	FLandscapeLayerComponentData* EditingLayer = GetEditingLayer();

	if (InApplyToEditingWeightmap && EditingLayer != nullptr)
	{
		EditingLayer->WeightmapData.Textures.Reset(InNewWeightmapTextures.Num());
		EditingLayer->WeightmapData.Textures.Append(InNewWeightmapTextures);
	}
	else
#endif // WITH_EDITOR
	{
		WeightmapTextures = InNewWeightmapTextures;
	}
}

// Note that there is a slight difference in behavior with the Internal function:
// unlike SetWeightmapTextures, this function will never set the runtime WeightmapTextures when you intended to set an edit layer's WeightmapData.Textures
void ULandscapeComponent::SetWeightmapTexturesInternal(const TArray<UTexture2D*>& InNewWeightmapTextures, const FGuid& InEditLayerGuid)
{
	if (InEditLayerGuid.IsValid())
	{
#if WITH_EDITOR
		FLandscapeLayerComponentData* EditingLayer = GetLayerData(InEditLayerGuid);
		if (ensure(EditingLayer))
		{
			EditingLayer->WeightmapData.Textures.Reset(InNewWeightmapTextures.Num());
			EditingLayer->WeightmapData.Textures.Append(InNewWeightmapTextures);
		}
#endif // WITH_EDITOR
	}
	else
	{
		WeightmapTextures = InNewWeightmapTextures;
	}
}


#if WITH_EDITOR
void ULandscapeComponent::SetWeightmapLayerAllocations(const TArray<FWeightmapLayerAllocationInfo>& InNewWeightmapLayerAllocations)
{
	WeightmapLayerAllocations = InNewWeightmapLayerAllocations;
}

TArray<TObjectPtr<ULandscapeWeightmapUsage>>& ULandscapeComponent::GetWeightmapTexturesUsage(bool InReturnEditingWeightmap)
{
	if (InReturnEditingWeightmap)
	{
		if (FLandscapeLayerComponentData* EditingLayer = GetEditingLayer())
		{
			return EditingLayer->WeightmapData.TextureUsages;
		}
	}

	return WeightmapTexturesUsage;
}

const TArray<ULandscapeWeightmapUsage*>& ULandscapeComponent::GetWeightmapTexturesUsage(bool InReturnEditingWeightmap) const
{
	if (InReturnEditingWeightmap)
	{
		if (const FLandscapeLayerComponentData* EditingLayer = GetEditingLayer())
		{
			return EditingLayer->WeightmapData.TextureUsages;
		}
	}

	return WeightmapTexturesUsage;
}

TArray<TObjectPtr<ULandscapeWeightmapUsage>>& ULandscapeComponent::GetWeightmapTexturesUsage(const FGuid& InLayerGuid)
{
	if (InLayerGuid.IsValid())
	{
		if (FLandscapeLayerComponentData* LayerData = GetLayerData(InLayerGuid))
		{
			return LayerData->WeightmapData.TextureUsages;
		}
	}

	return WeightmapTexturesUsage;
}

const TArray<ULandscapeWeightmapUsage*>& ULandscapeComponent::GetWeightmapTexturesUsage(const FGuid& InLayerGuid) const
{
	if (InLayerGuid.IsValid())
	{
		if (const FLandscapeLayerComponentData* LayerData = GetLayerData(InLayerGuid))
		{
			return LayerData->WeightmapData.TextureUsages;
		}
	}

	return WeightmapTexturesUsage;
}

void ULandscapeComponent::SetWeightmapTexturesUsage(const TArray<ULandscapeWeightmapUsage*>& InNewWeightmapTexturesUsage, bool InApplyToEditingWeightmap)
{
	FLandscapeLayerComponentData* EditingLayer = GetEditingLayer();

	if (InApplyToEditingWeightmap && EditingLayer != nullptr)
	{
		EditingLayer->WeightmapData.TextureUsages.Reset(InNewWeightmapTexturesUsage.Num());
		EditingLayer->WeightmapData.TextureUsages.Append(InNewWeightmapTexturesUsage);
	}
	else
	{
		WeightmapTexturesUsage = InNewWeightmapTexturesUsage;
	}
}

void ULandscapeComponent::SetWeightmapTexturesUsageInternal(const TArray<ULandscapeWeightmapUsage*>& InNewWeightmapTexturesUsage, const FGuid& InEditLayerGuid)
{
	if (InEditLayerGuid.IsValid())
	{
#if WITH_EDITOR
		FLandscapeLayerComponentData* EditingLayer = GetLayerData(InEditLayerGuid);
		if (ensure(EditingLayer))
		{
			EditingLayer->WeightmapData.TextureUsages.Reset(InNewWeightmapTexturesUsage.Num());
			EditingLayer->WeightmapData.TextureUsages.Append(InNewWeightmapTexturesUsage);
		}
#endif // WITH_EDITOR
	}
	else
	{
		WeightmapTexturesUsage = InNewWeightmapTexturesUsage;
	}
}

void ULandscapeComponent::DeleteLayerAllocation(const FGuid& InEditLayerGuid, int32 InLayerAllocationIdx, bool bInShouldDirtyPackage)
{
	TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = GetWeightmapLayerAllocations(InEditLayerGuid);
	TArray<TObjectPtr<UTexture2D>>& ComponentWeightmapTextures = GetWeightmapTextures(InEditLayerGuid);
	TArray<TObjectPtr<ULandscapeWeightmapUsage>>& ComponentWeightmapTexturesUsage = GetWeightmapTexturesUsage(InEditLayerGuid);

	const FWeightmapLayerAllocationInfo& LayerAllocation = ComponentWeightmapLayerAllocations[InLayerAllocationIdx];
	const int32 DeleteLayerWeightmapTextureIndex = LayerAllocation.WeightmapTextureIndex;

	ALandscapeProxy* Proxy = GetLandscapeProxy();
	Modify(bInShouldDirtyPackage);
	Proxy->Modify(bInShouldDirtyPackage);

	// Mark the weightmap channel as unallocated, so we can reuse it later
	ULandscapeWeightmapUsage* Usage = ComponentWeightmapTexturesUsage.IsValidIndex(DeleteLayerWeightmapTextureIndex) ? ComponentWeightmapTexturesUsage[DeleteLayerWeightmapTextureIndex] : nullptr;
	if (Usage) // can be null if WeightmapUsageMap hasn't been built yet
	{
		Usage->ChannelUsage[LayerAllocation.WeightmapTextureChannel] = nullptr;
	}

	// Remove the layer:
	ComponentWeightmapLayerAllocations.RemoveAt(InLayerAllocationIdx);

	// Check if the weightmap texture used by the material layer we just removed is used by any other material layer -- if not then we can remove the texture from the local list (as it's not used locally)
	bool bCanRemoveLayerTexture = !ComponentWeightmapLayerAllocations.ContainsByPredicate([DeleteLayerWeightmapTextureIndex](const FWeightmapLayerAllocationInfo& Allocation) { return Allocation.WeightmapTextureIndex == DeleteLayerWeightmapTextureIndex; });
	if (bCanRemoveLayerTexture)
	{
	    // Make sure the texture can be garbage collected, if necessary
		ComponentWeightmapTextures[DeleteLayerWeightmapTextureIndex]->ClearFlags(RF_Standalone);

		// Remove from our local list of textures and usages
		ComponentWeightmapTextures.RemoveAt(DeleteLayerWeightmapTextureIndex);
		if (Usage)
		{
			ComponentWeightmapTexturesUsage.RemoveAt(DeleteLayerWeightmapTextureIndex);
		}

		// Adjust WeightmapTextureIndex for other allocations (as we just reordered the Weightmap list with the deletions above)
		for (FWeightmapLayerAllocationInfo& Allocation : ComponentWeightmapLayerAllocations)
		{
			if (Allocation.WeightmapTextureIndex > DeleteLayerWeightmapTextureIndex)
			{
				Allocation.WeightmapTextureIndex--;
			}
			check(Allocation.WeightmapTextureIndex < ComponentWeightmapTextures.Num());
		}
	}

	Proxy->ValidateProxyLayersWeightmapUsage();
}

#endif // WITH_EDITOR

void ALandscapeProxy::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

	ULandscapeInfo* LandscapeInfo = nullptr;
	if (!IsPendingKillPending())
	{
		// Duplicated or newly spawned Landscapes don't have a valid guid until PostEditImport is called, we'll register then
		if (LandscapeGuid.IsValid())
		{
			LandscapeInfo = GetLandscapeInfo();

			// Depending what action triggered this callback, we may have already registered.  If not register now with LandscapeInfo.
			if ((LandscapeInfo == nullptr) || !LandscapeInfo->IsRegistered(this))
			{
				LandscapeInfo = CreateLandscapeInfo(true);
			}
		}

		if (UWorld* OwningWorld = GetWorld())
		{
			if (ULandscapeSubsystem* LandscapeSubsystem = OwningWorld->GetSubsystem<ULandscapeSubsystem>())
			{
				LandscapeSubsystem->RegisterActor(this);
			}
		}

		UpdateRenderingMethod();
	}
#if WITH_EDITOR
	if ((LandscapeInfo != nullptr) && !IsPendingKillPending() && LandscapeGuid.IsValid())
	{
		LandscapeInfo->FixupProxiesTransform();

		if (LandscapeInfo->IsLandscapeEditableWorld())
		{
			// Register to data change events on the layer info object so that we can update the landscape and react to changes : 
			Algo::ForEach(GetValidTargetLayerObjects(), [this](ULandscapeLayerInfoObject* InLayerInfoObject)
				{
					InLayerInfoObject->OnLayerInfoChanged().AddUObject(this, &ALandscapeProxy::OnLayerInfoObjectDataChanged);
				});
		}
	}
#endif // WITH_EDITOR
}

void ALandscapeProxy::UnregisterAllComponents(const bool bForReregister)
{
	// On shutdown the world will be unreachable
	if (GetWorld() && IsValidChecked(GetWorld()) && !GetWorld()->IsUnreachable() &&
		// When redoing the creation of a landscape we may get UnregisterAllComponents called when
		// we are in a "pre-initialized" state (empty guid, etc)
		LandscapeGuid.IsValid())
	{
		ULandscapeInfo* LandscapeInfo = GetLandscapeInfo();
		if (LandscapeInfo)
		{
#if WITH_EDITOR
			if (LandscapeInfo->IsLandscapeEditableWorld())
			{
				// Unregister from data change events on the layer info objects
				Algo::ForEach(GetValidTargetLayerObjects(), [this](ULandscapeLayerInfoObject* InLayerInfoObject)
					{
						InLayerInfoObject->OnLayerInfoChanged().RemoveAll(this);
					});
			}
#endif // WITH_EDITOR
			LandscapeInfo->UnregisterActor(this);
		}

		if (ULandscapeSubsystem* LandscapeSubsystem = GetWorld()->GetSubsystem<ULandscapeSubsystem>())
		{
			LandscapeSubsystem->UnregisterActor(this);
		}
	}

	Super::UnregisterAllComponents(bForReregister);
}


FArchive& operator<<(FArchive& Ar, FWeightmapLayerAllocationInfo& U)
{
	return Ar << U.LayerInfo << U.WeightmapTextureChannel << U.WeightmapTextureIndex;
}

#if WITH_EDITORONLY_DATA
FArchive& operator<<(FArchive& Ar, FLandscapeAddCollision& U)
{
	return Ar << U.Corners[0] << U.Corners[1] << U.Corners[2] << U.Corners[3];
}
#endif // WITH_EDITORONLY_DATA

void ULandscapeInfo::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsTransacting())
	{
		Ar << XYtoComponentMap;
#if WITH_EDITORONLY_DATA
		Ar << XYtoAddCollisionMap;
#endif
		Ar << SelectedComponents;
		Ar << SelectedRegion;
		Ar << SelectedRegionComponents;
	}
}

void ALandscape::PostLoad()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ALandscape::PostLoad);

	if (!LandscapeGuid.IsValid())
	{
		LandscapeGuid = FGuid::NewGuid();
	}
	else
	{
#if WITH_EDITOR
		UWorld* CurrentWorld = GetWorld();
		for (ALandscape* Landscape : TObjectRange<ALandscape>(RF_ClassDefaultObject | RF_BeginDestroyed))
		{
			if (Landscape && Landscape != this && Landscape->LandscapeGuid == LandscapeGuid && Landscape->GetWorld() == CurrentWorld)
			{
				// Duplicated landscape level, need to generate new GUID. This can happen during PIE or gameplay when streaming the same landscape actor.
				Modify();
				LandscapeGuid = FGuid::NewGuid();
				break;
			}
		}
#endif // WITH_EDITOR
	}

#if WITH_EDITOR
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::IntroduceLandscapeEditLayerClass)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		LandscapeEditLayers = LandscapeLayers_DEPRECATED;

		for (FLandscapeLayer& Layer : LandscapeEditLayers)
		{
			UClass* EditLayerClass = ULandscapeEditLayer::StaticClass();

			if (Layer.Guid_DEPRECATED == LandscapeSplinesTargetLayerGuid_DEPRECATED)
			{
				EditLayerClass = ULandscapeEditLayerSplines::StaticClass();
			}
			check(Layer.EditLayer == nullptr);
			Layer.EditLayer = NewObject<ULandscapeEditLayerBase>(this, EditLayerClass, MakeUniqueObjectName(this, EditLayerClass), RF_Transactional);
		}

		// Empty the old property now that we've moved them over, else we'll accidentally keep references to brushes etc.
		LandscapeLayers_DEPRECATED.Empty();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	for (int32 LayerIndex = 0; LayerIndex < LandscapeEditLayers.Num(); ++LayerIndex)
	{
		FLandscapeLayer& Layer = LandscapeEditLayers[LayerIndex];
		if (Layer.EditLayer != nullptr)
		{
			if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::MigrateLandscapeEditLayerProperties)
			{
				// Set Owning Landscape before other fields so Setter checks succeed
				Layer.EditLayer->SetBackPointer(this);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
				Layer.EditLayer->SetFlags(RF_Transactional); // Bug fix, this flag was forgotten in the initial version
				Layer.EditLayer->SetGuid(Layer.Guid_DEPRECATED, /*bInModify = */false);
				Layer.EditLayer->SetName(Layer.Name_DEPRECATED, /*bInModify = */false);
				Layer.EditLayer->SetVisible(Layer.bVisible_DEPRECATED, /*bInModify = */false);
				Layer.EditLayer->SetLocked(Layer.bLocked_DEPRECATED, /*bInModify = */false);
				Layer.EditLayer->SetAlphaForTargetType(ELandscapeToolTargetType::Heightmap, Layer.HeightmapAlpha_DEPRECATED, /*bInModify = */false, EPropertyChangeType::ValueSet);
				Layer.EditLayer->SetAlphaForTargetType(ELandscapeToolTargetType::Weightmap, Layer.WeightmapAlpha_DEPRECATED, /*bInModify = */false, EPropertyChangeType::ValueSet);
				Layer.EditLayer->SetWeightmapLayerAllocationBlend(Layer.WeightmapLayerAllocationBlend_DEPRECATED, /*bInModify = */false);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}

			for (FLandscapeLayerBrush& Brush : Layer.Brushes)
			{
				Brush.SetOwner(this);
			}
		}
		else
		{
			UE_LOGF(LogLandscape, Error, "Couldn't load edit layer object associated with layer at index %i for landscape %ls. This may happen when the edit layer class cannot be found "
				"(for example, when a plugin is removed from the project). The layer will be deleted.", 
				LayerIndex, *GetFullName());
			ensure(DeleteLayer(LayerIndex));
		}
	}

	bool bIsEditLayerLandscape = true;

	// Non-edit layer landscapes are marked for auto-conversion in ALandscapeProxy::PostLoad. Run validations on serialized edit layer landscapes only
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::MigrateLandscapeNonEditLayerToEditLayer)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bIsEditLayerLandscape = bCanHaveLayersContent_DEPRECATED;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	// In case we're a landscape with edit layers but we actually lack a layer (e.g. it was removed by the test above, because its edit layer class is unknown),
	// Create a default layer because we're always supposed to have at least 1, then copy over relevant data if needed :
	if (bIsEditLayerLandscape)
	{
		if (LandscapeEditLayers.IsEmpty())
		{
			// Create a default layer
			CreateDefaultLayer();
		}

		ensure(!LandscapeEditLayers.IsEmpty());
		ensure(SelectedEditLayerIndex == 0);
	}
#endif // WITH_EDITOR

	Super::PostLoad();
}

FBox ALandscape::GetLoadedBounds() const
{
	return GetLandscapeInfo()->GetLoadedBounds();
}


// ----------------------------------------------------------------------------------

// This shader allows to render parts of the heightmaps/weightmaps (all pixels except the redundant ones on the right/bottom edges) in an atlas render target (uncompressed height for heightmaps)
class FLandscapeMergeTexturesPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FLandscapeMergeTexturesPS);
	SHADER_USE_PARAMETER_STRUCT(FLandscapeMergeTexturesPS, FGlobalShader);

public:
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FUintVector4, InAtlasSubregion)
		SHADER_PARAMETER(FUintVector4, InSourceTextureSubregion)
		SHADER_PARAMETER(int32, InSourceTextureChannel)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, InSourceTexture)
		RENDER_TARGET_BINDING_SLOTS()
		END_SHADER_PARAMETER_STRUCT()

	class FIsHeightmap : SHADER_PERMUTATION_BOOL("IS_HEIGHTMAP");

	using FPermutationDomain = TShaderPermutationDomain<FIsHeightmap>;

	static FPermutationDomain GetPermutationVector(bool bInIsHeighmap)
	{
		FPermutationDomain PermutationVector;
		PermutationVector.Set<FIsHeightmap>(bInIsHeighmap);
		return PermutationVector;
	}


	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& InParameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		UE::Landscape::ModifyShaderCompilerEnvironmentForDebug(OutEnvironment);

		OutEnvironment.SetDefine(TEXT("MERGE_TEXTURE"), 1);
	}

	static void MergeTexture(FRDGBuilder& GraphBuilder, FParameters* InParameters, const FIntRect& InRenderTargetArea, bool bInIsHeightmap)
	{
		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

		const FLandscapeMergeTexturesPS::FPermutationDomain PixelPermutationVector = FLandscapeMergeTexturesPS::GetPermutationVector(bInIsHeightmap);
		TShaderMapRef<FLandscapeMergeTexturesPS> PixelShader(ShaderMap, PixelPermutationVector);

		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			ShaderMap,
			RDG_EVENT_NAME("LandscapeMergeTexture"),
			PixelShader,
			InParameters,
			InRenderTargetArea);
	}
};

IMPLEMENT_GLOBAL_SHADER(FLandscapeMergeTexturesPS, "/Engine/Private/Landscape/LandscapeMergeTexturesPS.usf", "MergeTexture", SF_Pixel);


// ----------------------------------------------------------------------------------

// This shader allows to resample the heightmap/weightmap (bilinear interpolation) from a given atlas usually produced by FLandscapeMergeTexturesPS :
//  For heightmap, the output can be either compressed or uncompressed depending on the render target format (8 bits/channel for the former, 16/32 bits/channel for the latter)
class FLandscapeResampleMergedTexturePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FLandscapeResampleMergedTexturePS);
	SHADER_USE_PARAMETER_STRUCT(FLandscapeResampleMergedTexturePS, FGlobalShader);

public:
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FMatrix44f, InOutputUVToMergedTextureUV)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float>, InMergedTexture)
		SHADER_PARAMETER_SAMPLER(SamplerState, InMergedTextureSampler)
		SHADER_PARAMETER(FUintVector2, InRenderAreaSize)
		RENDER_TARGET_BINDING_SLOTS()
		END_SHADER_PARAMETER_STRUCT()

	class FIsHeightmap : SHADER_PERMUTATION_BOOL("IS_HEIGHTMAP");
	class FCompressHeight : SHADER_PERMUTATION_BOOL("COMPRESS_HEIGHT");

	using FPermutationDomain = TShaderPermutationDomain<FIsHeightmap, FCompressHeight>;

	static FPermutationDomain GetPermutationVector(bool bInIsHeighmap, bool bInCompressHeight)
	{
		FPermutationDomain PermutationVector;
		PermutationVector.Set<FIsHeightmap>(bInIsHeighmap);
		PermutationVector.Set<FCompressHeight>(bInCompressHeight);
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& InParameters)
	{
		FPermutationDomain PermutationVector(InParameters.PermutationId);
		bool bIsHeightmap = PermutationVector.Get<FIsHeightmap>();
		bool bCompressHeight = PermutationVector.Get<FCompressHeight>();
		// No need for heightmap compression for weightmaps
		return (bIsHeightmap || !bCompressHeight);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		UE::Landscape::ModifyShaderCompilerEnvironmentForDebug(OutEnvironment);

		OutEnvironment.SetDefine(TEXT("RESAMPLE_MERGED_TEXTURE"), 1);
	}

	static void ResampleMergedTexture(FRDGBuilder& GraphBuilder, FParameters* InParameters, bool bInIsHeightmap, bool bInCompressHeight)
	{
		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

		const FLandscapeResampleMergedTexturePS::FPermutationDomain PixelPermutationVector = FLandscapeResampleMergedTexturePS::GetPermutationVector(bInIsHeightmap, bInCompressHeight);
		TShaderMapRef<FLandscapeResampleMergedTexturePS> PixelShader(ShaderMap, PixelPermutationVector);

		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			ShaderMap,
			RDG_EVENT_NAME("ResampleMergedTexture"),
			PixelShader,
			InParameters,
			FIntRect(0, 0, InParameters->InRenderAreaSize.X, InParameters->InRenderAreaSize.Y));
	}
};

IMPLEMENT_GLOBAL_SHADER(FLandscapeResampleMergedTexturePS, "/Engine/Private/Landscape/LandscapeMergeTexturesPS.usf", "ResampleMergedTexture", SF_Pixel);


// ----------------------------------------------------------------------------------

// This shader allows to pack up to 4 single-channel textures onto a single rgba one
class FLandscapePackRGBAChannelsPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FLandscapePackRGBAChannelsPS);
	SHADER_USE_PARAMETER_STRUCT(FLandscapePackRGBAChannelsPS, FGlobalShader);

public:
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(int32, InNumChannels)
		SHADER_PARAMETER_RDG_TEXTURE_SRV_ARRAY(Texture2D<float>, InSourceTextures, [4])
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& InParameters)
	{
		return true;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		UE::Landscape::ModifyShaderCompilerEnvironmentForDebug(OutEnvironment);

		OutEnvironment.SetDefine(TEXT("PACK_RGBA_CHANNELS"), 1);
	}

	static void PackRGBAChannels(FRDGBuilder& GraphBuilder, FParameters* InParameters, const FIntRect& InRenderTargetArea)
	{
		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

		TShaderMapRef<FLandscapePackRGBAChannelsPS> PixelShader(ShaderMap);

		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			ShaderMap,
			RDG_EVENT_NAME("PackRGBAChannels"),
			PixelShader,
			InParameters,
			InRenderTargetArea);
	}
};

IMPLEMENT_GLOBAL_SHADER(FLandscapePackRGBAChannelsPS, "/Engine/Private/Landscape/LandscapeMergeTexturesPS.usf", "PackRGBAChannels", SF_Pixel);

// Render-thread version of the data / functions we need for the local merge of edit layers : 
namespace UE::Landscape::Private::RenderMergedTexture_RenderThread
{
	// Describes a texture and an associated subregion (for heightmap texture sharing), and, optionally, a single channel (for weightmap texture sharing) :
	struct FTexture2DResourceSubregion
	{
		FTexture2DResourceSubregion() = default;

		FTexture2DResourceSubregion(FTexture2DResource* InTexture, const FIntRect& InSubregion, int32 InChannelIndex = INDEX_NONE)
			: Texture(InTexture)
			, Subregion(InSubregion)
			, ChannelIndex(InChannelIndex)
		{
		}

		FTexture2DResource* Texture = nullptr;
		FIntRect Subregion;
		int32 ChannelIndex = INDEX_NONE;
	};

	struct FRenderInfo
	{
		// Transform to go from the output render area space ((0,0) in the lower left corner, (1,1) in the upper-right) to the temporary render target space
		FMatrix OutputUVToMergedTextureUV;
		FIntPoint SubsectionSizeQuads;
		int32 NumSubsections = 1;
		bool bIsHeightmap = true;
		bool bCompressHeight = false;
		FName TargetLayerName;

		TMap<FIntPoint, FTexture2DResourceSubregion> ComponentTexturesToRender;
	};

	void RenderMergedTexture(const FRenderInfo& InRenderInfo, FRDGBuilder& GraphBuilder, const FRenderTargetBinding& InOutputRenderTargetBinding)
	{
		RDG_EVENT_SCOPE(GraphBuilder, "RenderMergedTexture %s", *InRenderInfo.TargetLayerName.ToString());

		// Find the total area that those components need to be rendered to :
		FIntRect ComponentKeyRect;
		for (auto Iter = InRenderInfo.ComponentTexturesToRender.CreateConstIterator(); Iter; ++Iter)
		{
			ComponentKeyRect.Include(Iter.Key());
		}

		ComponentKeyRect.Max += FIntPoint(1, 1);
		FIntPoint NumComponentsToRender(ComponentKeyRect.Width(), ComponentKeyRect.Height());
		FIntPoint NumSubsectionsToRender = NumComponentsToRender * InRenderInfo.NumSubsections;
		FIntPoint RenderTargetSize = NumSubsectionsToRender * InRenderInfo.SubsectionSizeQuads + 1; // add one for the end vertex
		FIntPoint ComponentSizeQuads = InRenderInfo.SubsectionSizeQuads * InRenderInfo.NumSubsections;

		// We need a temporary render target that can contain all textures. 
		// For heightmaps, use PF_G16 (decoded height) as this will be resampled using bilinear sampling :
		EPixelFormat AtlasTextureFormat = InRenderInfo.bIsHeightmap ? PF_G16 : PF_G8;
		FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(RenderTargetSize, AtlasTextureFormat, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource);
		FRDGTextureRef AtlasTexture = GraphBuilder.CreateTexture(Desc, TEXT("LandscapeMergedTextureAtlas"));
		FRDGTextureSRVRef AtlasTextureSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(AtlasTexture));
		// Start with a cleared atlas : 
		FRenderTargetBinding AtlasTextureRT(AtlasTexture, ERenderTargetLoadAction::ENoAction);
		FRDGTextureClearInfo ClearInfo;
		AddClearRenderTargetPass(GraphBuilder, AtlasTexture, ClearInfo);

		TMap<FTexture2DResource*, FRDGTextureSRVRef> SourceTextureSRVs;

		// Fill that render target subsection by subsection, in order to bypass the redundant columns/lines on the subsection edges:
		for (int32 ComponentY = ComponentKeyRect.Min.Y; ComponentY < ComponentKeyRect.Max.Y; ++ComponentY)
		{
			for (int32 ComponentX = ComponentKeyRect.Min.X; ComponentX < ComponentKeyRect.Max.X; ++ComponentX)
			{
				FIntPoint LandscapeComponentKey(ComponentX, ComponentY);
				if (const FTexture2DResourceSubregion* SourceTextureResourceSubregion = InRenderInfo.ComponentTexturesToRender.Find(LandscapeComponentKey))
				{
					FIntPoint SubsectionSubregionSize = SourceTextureResourceSubregion->Subregion.Size() / InRenderInfo.NumSubsections;
					FRDGTextureSRVRef* SourceTextureSRV = SourceTextureSRVs.Find(SourceTextureResourceSubregion->Texture);
					if (SourceTextureSRV == nullptr)
					{
						FString* DebugString = GraphBuilder.AllocObject<FString>(SourceTextureResourceSubregion->Texture->GetTextureName().ToString());
						FRDGTextureRef TextureRef = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(SourceTextureResourceSubregion->Texture->TextureRHI, **DebugString));
						SourceTextureSRV = &SourceTextureSRVs.Add(SourceTextureResourceSubregion->Texture, GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(TextureRef)));
					}

					for (int32 SubsectionY = 0; SubsectionY < InRenderInfo.NumSubsections; ++SubsectionY)
					{
						for (int32 SubsectionX = 0; SubsectionX < InRenderInfo.NumSubsections; ++SubsectionX)
						{
							FIntPoint SubsectionLocalKey(SubsectionX, SubsectionY);
							FIntPoint SubsectionKey = LandscapeComponentKey * InRenderInfo.NumSubsections + SubsectionLocalKey;

							FIntRect AtlasTextureSubregion;
							AtlasTextureSubregion.Min = SubsectionKey * InRenderInfo.SubsectionSizeQuads;
							// We only really need the +1 on the very last subsection to get the last row/column, since we end up overwriting the other end
							// rows/columns when we proceed to the next tile. However it's much easier to add the +1 here and do a small amount of duplicate
							// writes, because otherwise we would have to adjust SubsectionSubregion to align with the region we're writing, which would get
							// messy in cases of different mip levels.
							AtlasTextureSubregion.Max = AtlasTextureSubregion.Min + InRenderInfo.SubsectionSizeQuads + 1;

							FIntRect SubsectionSubregion;
							SubsectionSubregion.Min = SourceTextureResourceSubregion->Subregion.Min + SubsectionLocalKey * SubsectionSubregionSize;
							SubsectionSubregion.Max = SubsectionSubregion.Min + SubsectionSubregionSize;

							FLandscapeMergeTexturesPS::FParameters* MergeTexturesPSParams = GraphBuilder.AllocParameters<FLandscapeMergeTexturesPS::FParameters>();
							MergeTexturesPSParams->InAtlasSubregion = FUintVector4(AtlasTextureSubregion.Min.X, AtlasTextureSubregion.Min.Y, AtlasTextureSubregion.Max.X, AtlasTextureSubregion.Max.Y);
							MergeTexturesPSParams->InSourceTexture = *SourceTextureSRV;
							MergeTexturesPSParams->InSourceTextureSubregion = FUintVector4(SubsectionSubregion.Min.X, SubsectionSubregion.Min.Y, SubsectionSubregion.Max.X, SubsectionSubregion.Max.Y);
							check(InRenderInfo.bIsHeightmap || ((SourceTextureResourceSubregion->ChannelIndex >= 0) && (SourceTextureResourceSubregion->ChannelIndex < 4)));
							MergeTexturesPSParams->InSourceTextureChannel = SourceTextureResourceSubregion->ChannelIndex;
							MergeTexturesPSParams->RenderTargets[0] = AtlasTextureRT;

							FLandscapeMergeTexturesPS::MergeTexture(GraphBuilder, MergeTexturesPSParams, AtlasTextureSubregion, InRenderInfo.bIsHeightmap);
						}
					}
				}
			}
		}

		{
			FRDGTexture* OutputTexture = InOutputRenderTargetBinding.GetTexture();
			check(OutputTexture != nullptr);
			FIntVector RenderAreaSize = OutputTexture->Desc.GetSize();
			FLandscapeResampleMergedTexturePS::FParameters* ResampleMergedTexturePSParams = GraphBuilder.AllocParameters<FLandscapeResampleMergedTexturePS::FParameters>();
			ResampleMergedTexturePSParams->InOutputUVToMergedTextureUV = FMatrix44f(InRenderInfo.OutputUVToMergedTextureUV);
			ResampleMergedTexturePSParams->InMergedTexture = AtlasTextureSRV;
			ResampleMergedTexturePSParams->InMergedTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
			ResampleMergedTexturePSParams->InRenderAreaSize = FUintVector2((uint32)RenderAreaSize.X, (uint32)RenderAreaSize.Y);
			ResampleMergedTexturePSParams->RenderTargets[0] = InOutputRenderTargetBinding;

			// We now need to resample the atlas texture where the render area is : 
			FLandscapeResampleMergedTexturePS::ResampleMergedTexture(GraphBuilder, ResampleMergedTexturePSParams, InRenderInfo.bIsHeightmap, InRenderInfo.bCompressHeight);
	}
}
} // namespace UE::Landscape::Private::RenderMergedTexture_RenderThread

bool ALandscape::IsValidRenderTargetFormatHeightmap(EPixelFormat InRenderTargetFormat, bool& bOutCompressHeight)
{
	bOutCompressHeight = false;
	switch (InRenderTargetFormat)
	{
		// 8 bits formats : need compression
	case PF_A8R8G8B8:
	case PF_R8G8B8A8:
	case PF_R8G8:
	case PF_B8G8R8A8:
	{
		bOutCompressHeight = true;
		return true;
	}
	// 16 bits formats :
	case PF_G16:
	// We don't use 16 bit float formats because they will have precision issues
	// (we need 16 bits of mantissa)

	// TODO: We can support 32 bit floating point formats, but for these, we probably
	// want to output the height as an unpacked, signed values. We'll add support for
	// that in a later CL.
	//case PF_R32_FLOAT:
	//case PF_G32R32F:
	//case PF_R32G32B32F:
	//case PF_A32B32G32R32F:
	{
		return true;
	}
	default:
		break;
	}

	return false;
}

bool ALandscape::IsValidRenderTargetFormatWeightmap(EPixelFormat InRenderTargetFormat, int32& OutNumChannels)
{
	OutNumChannels = 0;
	switch (InRenderTargetFormat)
	{
		// TODO [jonathan.bard] : for now, we only support 8 bits formats as they're the weightmap format but possibly we could handle the conversion to other formats
	case PF_G8:
	case PF_A8:
	case PF_R8G8:
	case PF_A8R8G8B8:
	case PF_R8G8B8A8:
	case PF_B8G8R8A8:
	{
		OutNumChannels = GPixelFormats[InRenderTargetFormat].NumComponents;
		return true;
	}
	default:
		break;
	}

	return false;
}

bool ALandscape::RenderMergedTextureInternal(const FTransform& InRenderAreaWorldTransform, const FBox2D& InRenderAreaExtents, const TArray<FName>& InWeightmapLayerNames, UTextureRenderTarget* OutRenderTarget)
{
	// TODO: We may want a version of this function that returns a lambda that can be passed to the render thread and run
	// there to add the pass to an existing FRDGBuilder, in case the user wants this to be a part of a render graph with
	// other passes. In that case RenderMergedTextureInternal would just use that function.

	using namespace UE::Landscape;
	using namespace UE::Landscape::Private;

	TRACE_CPUPROFILER_EVENT_SCOPE(Landscape_RenderMergedTextureInternal);

	if (!FApp::CanEverRender())
	{
		UE_LOGF(LogLandscape, Log, "RenderMergedTexture : Cannot render anything on landscape (%ls) : the app doesn't have a valid rendering device ", *GetFullName());
		return false;
	}

	ULandscapeInfo* Info = GetLandscapeInfo();
	if (Info == nullptr)
	{
		UE_LOGF(LogLandscape, Error, "RenderMergedTexture : Cannot render anything if there's no associated landscape info with this landscape (%ls)", *GetFullName());
		return false;
	}

	// Check render target validity :
	if (OutRenderTarget == nullptr)
	{
		UE_LOGF(LogLandscape, Error, "RenderMergedTexture : Missing render target");
		return false;
	}

	// Check Render target format :
	const bool bIsHeightmap = InWeightmapLayerNames.IsEmpty();
	bool bCompressHeight = false;
	UTextureRenderTarget2D* RenderTarget2D = Cast<UTextureRenderTarget2D>(OutRenderTarget);
	UTextureRenderTarget2DArray* RenderTarget2DArray = Cast<UTextureRenderTarget2DArray>(OutRenderTarget);
	EPixelFormat RenderTargetFormat = (RenderTarget2DArray != nullptr) ? RenderTarget2DArray->GetFormat() : (RenderTarget2D != nullptr) ? RenderTarget2D->GetFormat() : PF_Unknown;
	if (bIsHeightmap)
	{
		if (RenderTarget2D == nullptr)
		{
			UE_LOGF(LogLandscape, Error, "RenderMergedTexture : Heightmap capture requires a UTextureRenderTarget2D");
			return false;
		}

		if (!IsValidRenderTargetFormatHeightmap(RenderTargetFormat, bCompressHeight))
	{
			UE_LOGF(LogLandscape, Warning, "RenderMergedTexture : invalid render target format for rendering heightmap (%ls)", GetPixelFormatString(RenderTargetFormat));
			return false;
		}
	}
	else
	{
		// If more than 1 weightmaps are requested, we expected a texture array or at the very least a texture 2D with enough channels to fit all weightmaps :
		int32 NumChannels = 0;
		if (!IsValidRenderTargetFormatWeightmap(RenderTargetFormat, NumChannels))
		{
			UE_LOGF(LogLandscape, Warning, "RenderMergedTexture : invalid render target format for rendering weightmap (%ls)", GetPixelFormatString(RenderTargetFormat));
			return false;
		}

		if (InWeightmapLayerNames.Num() > 1)
		{
			if ((RenderTarget2D != nullptr) && (NumChannels < InWeightmapLayerNames.Num()))
			{
				UE_LOGF(LogLandscape, Warning, "RenderMergedTexture : Not enough channels available (%i) in render target to accomodate for all requested weightmaps (%i)", NumChannels, InWeightmapLayerNames.Num());
				return false;
			}
			else if ((RenderTarget2DArray != nullptr) && ((NumChannels * RenderTarget2DArray->Slices) < InWeightmapLayerNames.Num()))
			{
				UE_LOGF(LogLandscape, Warning, "RenderMergedTexture : Not enough channels available (%i) in render target array to accomodate for all requested weightmaps (%i)", NumChannels * RenderTarget2DArray->Slices, InWeightmapLayerNames.Num());
				return false;
			}
		}
	}

	// If the requested extents are invalid, use the entire loaded landscape are as extents and transform : 
	const FTransform& LandscapeTransform = GetTransform();
	FTransform FinalRenderAreaWorldTransform = InRenderAreaWorldTransform;
	FBox2D FinalRenderAreaExtents = InRenderAreaExtents;
	if (!InRenderAreaExtents.bIsValid || InRenderAreaExtents.GetExtent().IsZero())
	{
		FinalRenderAreaWorldTransform = LandscapeTransform;
		FBox LoadedBounds = Info->GetLoadedBounds();
		FinalRenderAreaExtents = FBox2D(FVector2D(LandscapeTransform.InverseTransformPosition(LoadedBounds.Min)), FVector2D(LandscapeTransform.InverseTransformPosition(LoadedBounds.Max)));
	}

	// It can be helpful to visualize where the render happened so leave a visual log for that: 
	UE_VLOG_OBOX(this, LogLandscape, Log, FBox(FVector(FinalRenderAreaExtents.Min, 0.0), FVector(FinalRenderAreaExtents.Max, 0.0)), FinalRenderAreaWorldTransform.ToMatrixWithScale(), FColor::Blue, TEXT("LandscapeRenderMergedTexture"));

	// Don't do anything if this render area overlaps with no landscape component :
	TMap<FIntPoint, ULandscapeComponent*> OverlappedComponents;
	FIntRect OverlappedComponentIndicesBoundingRect;
	if (!Info->GetOverlappedComponents(FinalRenderAreaWorldTransform, FinalRenderAreaExtents, OverlappedComponents, OverlappedComponentIndicesBoundingRect))
	{
		UE_LOGF(LogLandscape, Log, "RenderMergedTexture : nothing to render");
		return true;
	}

	RenderCaptureInterface::FScopedCapture RenderCapture((RenderCaptureNextMergeRenders != 0), TEXT("LandscapeRenderMergedTextureCapture"));
	RenderCaptureNextMergeRenders = FMath::Max(RenderCaptureNextMergeRenders - 1, 0);


	// We'll want to perform one merge per target layer (i.e. as many as there are weightmaps, or just 1 in the case of heightmap) : 
	const int32 NumTargetLayers = bIsHeightmap ? 1 : InWeightmapLayerNames.Num();

	TArray<RenderMergedTexture_RenderThread::FRenderInfo> MergeTextureRenderInfos;
	MergeTextureRenderInfos.Reserve(NumTargetLayers);

	for (int32 TargetLayerIndex = 0; TargetLayerIndex < NumTargetLayers; ++TargetLayerIndex)
	{
		const FName TargetLayerName = bIsHeightmap ? FName("Heightmap") : InWeightmapLayerNames[TargetLayerIndex];

		RenderMergedTexture_RenderThread::FRenderInfo& MergeTextureRenderInfo = MergeTextureRenderInfos.Emplace_GetRef();
		// For now, merge the texture at max resolution :
		MergeTextureRenderInfo.SubsectionSizeQuads = SubsectionSizeQuads;
		MergeTextureRenderInfo.NumSubsections = NumSubsections;
		MergeTextureRenderInfo.bIsHeightmap = bIsHeightmap;
		MergeTextureRenderInfo.bCompressHeight = bCompressHeight;
		MergeTextureRenderInfo.TargetLayerName = TargetLayerName;

		// Indices of the components being rendered by this target layer : 
		FIntRect RenderTargetComponentIndicesBoundingRect;

		for (auto It : OverlappedComponents)
		{
			ULandscapeComponent* Component = It.Value;
			FIntPoint ComponentKey = It.Key;

			UTexture2D* SourceTexture = nullptr;
			FVector2D SourceTextureBias = FVector2D::ZeroVector;
			int32 SourceTextureChannel = INDEX_NONE;

			if (bIsHeightmap)
			{
				SourceTexture = Component->GetHeightmap();
				SourceTextureBias = FVector2D(Component->HeightmapScaleBias.Z, Component->HeightmapScaleBias.W);
			}
			else
			{
				const TArray<UTexture2D*>& WeightmapTextures = Component->GetCurrentRuntimeWeightmapTextures();
				const TArray<FWeightmapLayerAllocationInfo>& AllocInfos = Component->GetCurrentRuntimeWeightmapLayerAllocations();
				const FWeightmapLayerAllocationInfo* AllocInfo = AllocInfos.FindByPredicate([TargetLayerName](const FWeightmapLayerAllocationInfo& InAllocInfo) { return InAllocInfo.IsAllocated() && (InAllocInfo.GetLayerName() == TargetLayerName); });
				if (AllocInfo != nullptr)
				{
					SourceTexture = WeightmapTextures[AllocInfo->WeightmapTextureIndex];
					check(SourceTexture != nullptr);
					// Note : don't use WeightmapScaleBias here, it has a different meaning than HeightmapScaleBias (very conveniently!) : this is compensated by the FloorToInt32 later on, 
					//  but still, let's set this to zero here and use the fact that there's no texture sharing on weightmaps : 
					SourceTextureBias = FVector2D::ZeroVector;
					SourceTextureChannel = AllocInfo->WeightmapTextureChannel;
				}
			}

			if (SourceTexture != nullptr)
			{
				// Get the subregion of the source texture that this component uses (differs due to texture sharing).
				// SourceTextureBias values give us the offset of the component in a shared texture
				int32 ComponentSize = Component->NumSubsections * (Component->SubsectionSizeQuads + 1);

				FIntPoint SourceTextureOffset(0, 0);
				FTextureResource* SourceTextureResource = SourceTexture->GetResource();
				if (ensure(SourceTextureResource))
				{
					// We get the overall source texture size via the resource instead of direct GetSizeX/Y calls because the latter are unreliable while the texture is being built.
					SourceTextureOffset = FIntPoint(
						FMath::FloorToInt32(SourceTextureBias.X * SourceTextureResource->GetSizeX()),
						FMath::FloorToInt32(SourceTextureBias.Y * SourceTextureResource->GetSizeY()));
				}

				// When mips are partially loaded, we need to take that into consideration when merging the source texture :
				uint32 MipBias = SourceTexture->GetNumMips() - SourceTexture->GetNumResidentMips();

				// Theoretically speaking, all of our component source textures should be powers of two when we include the duplicated
				// rows/columns across subsections, so we shouldn't get weird truncation results here...
				SourceTextureOffset.X >>= MipBias;
				SourceTextureOffset.Y >>= MipBias;
				ComponentSize >>= MipBias;

				// Effective area of the texture affecting this component (because of texture sharing) :
				FIntRect SourceTextureSubregion(SourceTextureOffset, SourceTextureOffset + ComponentSize);
				MergeTextureRenderInfo.ComponentTexturesToRender.Add(ComponentKey, RenderMergedTexture_RenderThread::FTexture2DResourceSubregion(SourceTextureResource->GetTexture2DResource(), SourceTextureSubregion, SourceTextureChannel));

				// Since this component will be rendered in the render target, we can now expand the render target's bounds :
				RenderTargetComponentIndicesBoundingRect.Union(FIntRect(ComponentKey, ComponentKey + FIntPoint(1)));
			}
		}

		// Create the transform that will go from output target UVs to world space: 
		FVector OutputUVOrigin = FinalRenderAreaWorldTransform.TransformPosition(FVector(FinalRenderAreaExtents.Min.X, FinalRenderAreaExtents.Min.Y, 0.0));
		FVector OutputUVScale = FinalRenderAreaWorldTransform.GetScale3D() * FVector(FinalRenderAreaExtents.GetSize(), 1.0);
		FTransform OutputUVToWorld(FinalRenderAreaWorldTransform.GetRotation(), OutputUVOrigin, OutputUVScale);

		// Create the transform that will go from the merged texture (atlas) UVs to world space. Note that this is slightly trickier because
		// vertices in the landscape correspond to pixel centers. So UV (0,0) is not at the minimal landscape vertex, but is instead 
		// half a quad further (one pixel is one quad in size, so the center of the first pixel ends up at the minimal vertex).
		// For related reasons, the size of the merged texture in world coordinates is actually one quad bigger in each direction.
		check(RenderTargetComponentIndicesBoundingRect.IsEmpty()
			|| (RenderTargetComponentIndicesBoundingRect.Min.X < RenderTargetComponentIndicesBoundingRect.Max.X) && (RenderTargetComponentIndicesBoundingRect.Min.Y < RenderTargetComponentIndicesBoundingRect.Max.Y));
		FVector MergedTextureScale = (FVector(RenderTargetComponentIndicesBoundingRect.Max - RenderTargetComponentIndicesBoundingRect.Min) * static_cast<double>(ComponentSizeQuads) + 1)
			* LandscapeTransform.GetScale3D();
		MergedTextureScale.Z = 1.0f;
		FVector MergedTextureUVOrigin = LandscapeTransform.TransformPosition(FVector(RenderTargetComponentIndicesBoundingRect.Min) * (double)ComponentSizeQuads - FVector(0.5, 0.5, 0));
		FTransform MergedTextureUVToWorld(LandscapeTransform.GetRotation(), MergedTextureUVOrigin, MergedTextureScale);

		MergeTextureRenderInfo.OutputUVToMergedTextureUV = OutputUVToWorld.ToMatrixWithScale() * MergedTextureUVToWorld.ToInverseMatrixWithScale();
	}

	// Extract the render thread version of the output render target :
	FTextureRenderTargetResource* OutputRenderTargetResource = OutRenderTarget->GameThread_GetRenderTargetResource();
	check(OutputRenderTargetResource != nullptr);

	ENQUEUE_RENDER_COMMAND(RenderMergedTexture)([MergeTextureRenderInfos, OutputRenderTargetResource, bIsHeightmap, RenderTargetFormat](FRHICommandListImmediate& RHICmdList)
	{
		FRDGBuilder GraphBuilder(RHICmdList, RDG_EVENT_NAME("RenderMergedTexture"));

		FTextureRenderTarget2DResource* OutputRenderTarget2DResource = OutputRenderTargetResource->GetTextureRenderTarget2DResource();
		FTextureRenderTarget2DArrayResource* OutputRenderTarget2DArrayResource = OutputRenderTargetResource->GetTextureRenderTarget2DArrayResource();
		check((OutputRenderTarget2DResource != nullptr) || (OutputRenderTarget2DArrayResource != nullptr)); // either a render target 2D array or a render target 2D

		FRDGTextureRef OutputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(OutputRenderTargetResource->GetTextureRHI(), TEXT("MergedTexture")));

		// If we perform a single merge, we can simply render to the final texture : 
		const int32 NumTargetLayers = MergeTextureRenderInfos.Num();
		if (NumTargetLayers == 1)
		{
			// If it's a texture array, we need to specify the slice index
			const int16 ArraySlice = (OutputRenderTarget2DArrayResource != nullptr) ? 0 : -1;
			FRenderTargetBinding RenderTargetBinding(OutputTexture, ERenderTargetLoadAction::ENoAction, /*InMipIndex = */0, ArraySlice);
			RenderMergedTexture_RenderThread::RenderMergedTexture(MergeTextureRenderInfos[0], GraphBuilder, RenderTargetBinding);
		}
		// In the case of multiple target layers, we'll render them one by one and pack them on the available output channels : 
		else
		{
			const int32 NumChannels = GPixelFormats[RenderTargetFormat].NumComponents;
			const int32 NumChannelPackingOperations = FMath::DivideAndRoundUp<int32>(NumTargetLayers, NumChannels);
			check(NumChannelPackingOperations > 0);
			checkf((OutputRenderTarget2DArrayResource != nullptr) || (NumTargetLayers <= NumChannels), TEXT("Trying to merge %i weightmaps onto a 2D texture of %i channels only"), NumTargetLayers, NumChannels);
			checkf(!bIsHeightmap, TEXT("We should only be able to merge multiple textures in the case of weightmaps"));
			FRDGTextureSRVRef DummyBlackTextureSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(GSystemTextures.GetBlackDummy(GraphBuilder)));

			// We'll need temporary 1 channel-texture for each weightmap that will then be packed onto the needed channels. This is for weightmaps only so PF_G8 pixel format is what we need for 
			FIntPoint OutputTextureSize(OutputTexture->Desc.GetSize().X, OutputTexture->Desc.GetSize().Y);
			FRDGTextureDesc SingleChannelTextureDesc = FRDGTextureDesc::Create2D(OutputTextureSize, PF_G8, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource);
			int32 TargetLayerIndex = 0; 
			const int32 NumSlices = (OutputRenderTarget2DArrayResource != nullptr) ? OutputTexture->Desc.ArraySize : 1;
			for (int32 SliceIndex = 0; SliceIndex < NumSlices; ++SliceIndex)
			{
				RDG_EVENT_SCOPE(GraphBuilder, "RenderMergedTexture Slice %d", SliceIndex);

				// The last slice might have to render less than the actual number of channels of the texture : 
				const int32 NumEffectiveChannels = FMath::Min(NumChannels, (NumTargetLayers - SliceIndex * NumChannels));
				check((NumEffectiveChannels >= 0) && (NumEffectiveChannels <= NumChannels));
				TArray<FRDGTextureRef, TInlineAllocator<4>> SingleChannelTextures;

				// First, render the each channel independently : 
				for (int32 ChannelIndex = 0; ChannelIndex < NumEffectiveChannels; ++ChannelIndex)
				{
					FRDGTextureRef& SingleChannelTexture = SingleChannelTextures.Add_GetRef(GraphBuilder.CreateTexture(SingleChannelTextureDesc, TEXT("LandscapeMergedTextureTargetLayer")));
					FRenderTargetBinding SingleChannelRenderTargetBinding(SingleChannelTexture, ERenderTargetLoadAction::ENoAction);
					RenderMergedTexture_RenderThread::RenderMergedTexture(MergeTextureRenderInfos[TargetLayerIndex], GraphBuilder, SingleChannelRenderTargetBinding);
					// We have rendered a new target layer, move on to the next :
					++TargetLayerIndex;
					check(TargetLayerIndex <= NumTargetLayers);
				}

				// Now pack the channels directly to the final render target (slice)
				{
					FLandscapePackRGBAChannelsPS::FParameters* PackRGBAChannelsParams = GraphBuilder.AllocParameters<FLandscapePackRGBAChannelsPS::FParameters>();
					PackRGBAChannelsParams->InNumChannels = NumEffectiveChannels;
					for (int32 ChannelIndex = 0; ChannelIndex < 4; ++ChannelIndex)
					{
						PackRGBAChannelsParams->InSourceTextures[ChannelIndex] = (ChannelIndex < NumEffectiveChannels) ? GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(SingleChannelTextures[ChannelIndex])) : DummyBlackTextureSRV;
					}
					const int16 ArraySlice = (OutputRenderTarget2DArrayResource != nullptr) ? SliceIndex : -1;
					// If it's a texture 2D or a texture 2D array with individually targetable slices, we can pack directly using the slice's RTV :
					if ((OutputRenderTarget2DArrayResource == nullptr) || EnumHasAnyFlags(OutputTexture->Desc.Flags, TexCreate_TargetArraySlicesIndependently))
					{
						FRenderTargetBinding RenderTargetBinding(OutputTexture, ERenderTargetLoadAction::ENoAction, /*InMipIndex = */0, ArraySlice);
						PackRGBAChannelsParams->RenderTargets[0] = RenderTargetBinding;
						FLandscapePackRGBAChannelsPS::PackRGBAChannels(GraphBuilder, PackRGBAChannelsParams, FIntRect(FIntPoint::ZeroValue, OutputTextureSize));
					}
					else
					{
						// Otherwise (2D array but with non-individually targetable slices), we need to render to another render target and use a copy : 
						FRDGTextureDesc IntermediateRenderTargetDesc = FRDGTextureDesc::Create2D(OutputTextureSize, OutputTexture->Desc.Format, FClearValueBinding::Black, TexCreate_RenderTargetable);
						FRDGTextureRef IntermediateRenderTarget = GraphBuilder.CreateTexture(IntermediateRenderTargetDesc, TEXT("PackedRGBASlice"));
						FRenderTargetBinding RenderTargetBinding(IntermediateRenderTarget, ERenderTargetLoadAction::ENoAction, /*InMipIndex = */0, /*InArraySlice = */-1);
						PackRGBAChannelsParams->RenderTargets[0] = RenderTargetBinding;
						FLandscapePackRGBAChannelsPS::PackRGBAChannels(GraphBuilder, PackRGBAChannelsParams, FIntRect(FIntPoint::ZeroValue, OutputTextureSize));

						FRHICopyTextureInfo CopyTextureInfo;
						CopyTextureInfo.DestSliceIndex = ArraySlice;
						AddCopyTexturePass(GraphBuilder, IntermediateRenderTarget, OutputTexture, CopyTextureInfo);
					}
				}
			}
		}

		GraphBuilder.Execute();
	});

	return true;
}

bool ALandscape::RenderHeightmap(FTransform InRenderAreaWorldTransform, FBox2D InRenderAreaExtents, UTextureRenderTarget2D* OutRenderTarget)
{
	return RenderMergedTextureInternal(InRenderAreaWorldTransform, InRenderAreaExtents, /*InWeightmapLayerNames = */{}, OutRenderTarget);
}

bool ALandscape::RenderWeightmap(FTransform InRenderAreaWorldTransform, FBox2D InRenderAreaExtents, FName InWeightmapLayerName, UTextureRenderTarget2D* OutRenderTarget)
{
	return RenderMergedTextureInternal(InRenderAreaWorldTransform, InRenderAreaExtents, /*InWeightmapLayerNames = */{ InWeightmapLayerName }, OutRenderTarget);
}

bool ALandscape::RenderWeightmaps(FTransform InRenderAreaWorldTransform, FBox2D InRenderAreaExtents, const TArray<FName>& InWeightmapLayerNames, UTextureRenderTarget* OutRenderTarget)
{
	return RenderMergedTextureInternal(InRenderAreaWorldTransform, InRenderAreaExtents, InWeightmapLayerNames, OutRenderTarget);
}

#if WITH_EDITOR

FBox ALandscape::GetCompleteBounds() const
{
	if (GetLandscapeInfo())
	{
		return GetLandscapeInfo()->GetCompleteBounds();
	}
	else
	{
		return FBox(EForceInit::ForceInit);
	}
}

void ALandscape::SetUseGeneratedLandscapeSplineMeshesActors(bool bInEnabled)
{
		bUseGeneratedLandscapeSplineMeshesActors = bInEnabled;
	}

bool ALandscape::GetUseGeneratedLandscapeSplineMeshesActors() const
{
	return bUseGeneratedLandscapeSplineMeshesActors;
}

void ALandscape::EnableNaniteSkirts(bool bInEnable, float InSkirtDepth, bool bInShouldDirtyPackage)
{
	FProperty* bNaniteSkirtEnabledProperty = StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ALandscapeProxy, bNaniteSkirtEnabled));
	FProperty* NaniteSkirtDepthProperty = StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ALandscapeProxy, NaniteSkirtDepth));
	check(bNaniteSkirtEnabledProperty && NaniteSkirtDepthProperty);

	// PostEditChangeProperty propagates to proxies and invalidates nanite/render state
	if (bNaniteSkirtEnabled != bInEnable)
	{
		Modify(bInShouldDirtyPackage);
		bNaniteSkirtEnabled = bInEnable;

		FPropertyChangedEvent PropertyChangedEvent(bNaniteSkirtEnabledProperty);
		PostEditChangeProperty(PropertyChangedEvent);
	}

	if (NaniteSkirtDepth != InSkirtDepth)
	{
		Modify(bInShouldDirtyPackage);
		NaniteSkirtDepth = InSkirtDepth;

		FPropertyChangedEvent PropertyChangedEvent(NaniteSkirtDepthProperty);
		PostEditChangeProperty(PropertyChangedEvent);
	}
}

void ALandscape::SetDisableRuntimeGrassMapGeneration(bool bInDisableRuntimeGrassMapGeneration)
{
	bDisableRuntimeGrassMapGeneration = bInDisableRuntimeGrassMapGeneration;
	if (ULandscapeInfo* LandscapeInfo = GetLandscapeInfo())
	{
		LandscapeInfo->ForEachLandscapeProxy([bInDisableRuntimeGrassMapGeneration](ALandscapeProxy* Proxy) -> bool
		{
			Proxy->SetDisableRuntimeGrassMapGenerationProxyOnly(bInDisableRuntimeGrassMapGeneration);
			return true;
		});
	}
}

void ALandscapeProxy::OnFeatureLevelChanged(ERHIFeatureLevel::Type NewFeatureLevel)
{
	FlushGrassComponents(nullptr, /*bFlushGrassMaps=*/ false); // rebuild grass instances, but keep the grass maps

	UpdateAllComponentMaterialInstances();

	if (NewFeatureLevel == ERHIFeatureLevel::ES3_1)
	{
		for (ULandscapeComponent* Component : LandscapeComponents)
		{
			if (Component != nullptr)
			{
				Component->CheckGenerateMobilePlatformData(/*bIsCooking = */ false, /*TargetPlatform = */ nullptr);
			}
		}
	}

	UpdateRenderingMethod();
}

void ALandscapeProxy::BeginCacheForCookedPlatformData(const ITargetPlatform* TargetPlatform)
{
	InstallOrUpdateTextureUserDatas(TargetPlatform);
	Super::BeginCacheForCookedPlatformData(TargetPlatform);
}

void ALandscapeProxy::OnCookEvent(UE::Cook::ECookEvent CookEvent, UE::Cook::FCookEventContext& CookContext)
{
	using namespace UE::Cook;
	Super::OnCookEvent(CookEvent, CookContext);

	if (CookEvent == ECookEvent::PlatformCookDependencies && CookContext.IsCooking())
	{
		CookContext.AddLoadBuildDependency(FCookDependency::ConsoleVariable(FStringView(CVarLandscapeHeightmapCompressionModeName), CookContext.GetTargetPlatform(), /*bFallbackToNonPlatform*/true));
		CookContext.AddLoadBuildDependency(FCookDependency::ConsoleVariable(FStringView(CVarLandscapeHeightmapCompressionMipThresholdName), CookContext.GetTargetPlatform(), /*bFallbackToNonPlatform*/true));
	}
}

#endif


void ALandscapeProxy::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

#if WITH_EDITOR
	if (!ObjectSaveContext.IsProceduralSave()) // only finalize grass in a true editor save (when a GPU is available).
	{
		ALandscape* Landscape = GetLandscapeActor();
		if (Landscape)
		{
			check(ObjectSaveContext.IsFirstConcurrentSave());  //if PreSave ever actually becomes concurrent, this will need some change to make it safe.
			Landscape->FlushLayerContentThisFrame();
		}

		// It would be nice to strip grass data at editor save time to reduce asset size on disk.
		// Unfortunately we can't easily know if there is a platform out there that may need to use the serialized grass map path.
		// And future cook processes may not have a GPU available to build the grass data themselves.
		// So for now, we always build all grass maps on editor save, just in case.
		// The grass maps will get stripped later in BeginCacheForCookedPlatformData for cooked builds that don't need them.
		{
			// generate all of the grass data
			BuildGrassMaps();

			int32 ValidGrassCount = 0;
			for (ULandscapeComponent* Component : LandscapeComponents)
			{
				if (Component->GrassData->HasValidData())
				{
					ValidGrassCount++;
				}
			}

			UE_LOGF(LogGrass, Verbose, "PRESAVE: landscape %ls has %d / %d valid grass components (UseRuntimeGeneration %d Disable %d)", *GetName(), ValidGrassCount, LandscapeComponents.Num(), GGrassMapUseRuntimeGeneration, bDisableRuntimeGrassMapGeneration);
		}
	}

	// Update nanite (and block to wait for it).  Don't update nanite on auto-save, since its so slow.
	if (!ObjectSaveContext.IsFromAutoSave())
	{
		if (ULandscapeInfo* LandscapeInfo = GetLandscapeInfo())
		{
			LandscapeInfo->UpdateNanite(ObjectSaveContext.GetTargetPlatform());
		}
	}

	if (ALandscape* Landscape = GetLandscapeActor())
	{
		for (ULandscapeComponent* LandscapeComponent : LandscapeComponents)
		{
			Landscape->ClearDirtyData(LandscapeComponent);

			// Make sure edit layer debug names are synchronized upon save :
			LandscapeComponent->ForEachLayer([&](const FGuid& LayerGuid, FLandscapeLayerComponentData& LayerData)
			{
				if (const ULandscapeEditLayerBase* EditLayer = Landscape->GetEditLayerConst(LayerGuid))
				{
					LayerData.DebugName = EditLayer->GetName();
				}
			});
		}
		UpdateRenderingMethod();
	}

	for (ULandscapeComponent* LandscapeComponent : LandscapeComponents)
	{
		if (LandscapeComponent->CanUpdatePhysicalMaterial() && LandscapeComponent->PhysicalMaterialTask.IsValid())
		{
			UE_LOGF(LogLandscape, Display, "Completing landscape physical material before saving.  %ls", *LandscapeComponent->GetFullName());
			if (LandscapeComponent->PhysicalMaterialTask.IsInProgress())
			{
				LandscapeComponent->PhysicalMaterialTask.Flush();
			}

			if (LandscapeComponent->PhysicalMaterialTask.IsComplete())
			{
				// IsInProgress tests the render thread status.  Now finish the last steps on the game thread.
				LandscapeComponent->UpdatePhysicalMaterialTasks();
			}
			else
			{
				UE_LOGF(LogLandscape, Error, "Landscape physical material failed to complete before saving. That could be due to the landscape material failing to compile on this component (%ls)", *LandscapeComponent->GetFullName());
			}
		}

		// Ensure the component's cached bounds are correct
		FBox OldCachedLocalBox = LandscapeComponent->CachedLocalBox;
		if (LandscapeComponent->UpdateCachedBounds(/* bInApproximateBounds= */ false))
		{
			// conservative bounds are true bounding boxes, just not as tight/optimal as they could be
			// if it's not conservative, then visibility flashing issues can occur because of self-occlusion in culling
			bool bOldBoxIsConservative = LandscapeComponent->CachedLocalBox.IsInsideOrOn(OldCachedLocalBox);
			if (bOldBoxIsConservative)
			{
				UE_LOGF(LogLandscape, Display, "The component %ls had non-optimal bounds.  The bounds have been recalculated (old CachedLocalBox: %ls, new CachedLocalBox: %ls)", *LandscapeComponent->GetPathName(), *OldCachedLocalBox.ToString(), *LandscapeComponent->CachedLocalBox.ToString());
			}
			else
			{
				UE_LOGF(LogLandscape, Display, "The component %ls had incorrect bounds.  The bounds have been recalculated (old CachedLocalBox: %ls, new CachedLocalBox: %ls)", *LandscapeComponent->GetPathName(), *OldCachedLocalBox.ToString(), *LandscapeComponent->CachedLocalBox.ToString());
			}
			check(LandscapeComponent->CachedLocalBox.GetVolume() > 0.0);
		}
	}

	if (LandscapeGuid.IsValid())
	{
		if (ULandscapeInfo* LandscapeInfo = GetLandscapeInfo())
		{
			LandscapeInfo->OnModifiedPackageSaved(GetPackage());
		}
	}

	if (ObjectSaveContext.IsCooking())
	{
		InstallOrUpdateTextureUserDatas(ObjectSaveContext.GetTargetPlatform());
	}
#endif // WITH_EDITOR
}

void ALandscapeProxy::Serialize(FArchive& Ar)
{
	FGuid InstanceLandscapeGuid = this->LandscapeGuid;
	if (Ar.IsSaving() && Ar.IsPersistent())
	{
		// if we're using an instance-modified landscape guid, we need to restore the original before saving to persistent storage
		// (this can happen when you are cooking a level containing level instances in a commandlet)
		if ((LandscapeGuid != OriginalLandscapeGuid) && OriginalLandscapeGuid.IsValid())
		{
			this->LandscapeGuid = this->OriginalLandscapeGuid;
		}
	}

	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FLandscapeCustomVersion::GUID);
	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading() && Ar.CustomVer(FLandscapeCustomVersion::GUID) < FLandscapeCustomVersion::MigrateOldPropertiesToNewRenderingProperties)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (LODDistanceFactor_DEPRECATED > 0)
		{
			const float LOD0LinearDistributionSettingMigrationTable[11] = { 1.75f, 1.75f, 1.75f, 1.75f, 1.75f, 1.68f, 1.55f, 1.4f, 1.25f, 1.25f, 1.25f };
			const float LODDLinearDistributionSettingMigrationTable[11] = { 2.0f, 2.0f, 2.0f, 1.65f, 1.35f, 1.25f, 1.25f, 1.25f, 1.25f, 1.25f, 1.25f };
			const float LOD0SquareRootDistributionSettingMigrationTable[11] = { 1.75f, 1.6f, 1.25f, 1.25f, 1.25f, 1.25f, 1.25f, 1.25f, 1.25f, 1.25f, 1.25f };
			const float LODDSquareRootDistributionSettingMigrationTable[11] = { 2.0f, 1.8f, 1.55f, 1.3f, 1.25f, 1.25f, 1.25f, 1.25f, 1.25f, 1.25f, 1.25f };

			if (LODFalloff_DEPRECATED == ELandscapeLODFalloff::Type::Linear)
			{
				LOD0DistributionSetting = LOD0LinearDistributionSettingMigrationTable[FMath::RoundToInt(LODDistanceFactor_DEPRECATED)];
				LODDistributionSetting = LODDLinearDistributionSettingMigrationTable[FMath::RoundToInt(LODDistanceFactor_DEPRECATED)];
			}
			else if (LODFalloff_DEPRECATED == ELandscapeLODFalloff::Type::SquareRoot)
			{
				LOD0DistributionSetting = LOD0SquareRootDistributionSettingMigrationTable[FMath::RoundToInt(LODDistanceFactor_DEPRECATED)];
				LODDistributionSetting = LODDSquareRootDistributionSettingMigrationTable[FMath::RoundToInt(LODDistanceFactor_DEPRECATED)];
			}
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif

	if (Ar.IsSaving() && Ar.IsPersistent())
	{
		// restore the instance guid
		this->LandscapeGuid = InstanceLandscapeGuid;
	}
}

void ALandscapeProxy::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	ALandscapeProxy* This = CastChecked<ALandscapeProxy>(InThis);

	Super::AddReferencedObjects(InThis, Collector);

#if WITH_EDITORONLY_DATA
	Collector.AddReferencedObjects(This->MaterialInstanceConstantMap, This);
#endif
}

#if WITH_EDITOR

FName FLandscapeInfoLayerSettings::GetLayerName() const
{
	checkSlow(LayerInfoObj == nullptr || LayerInfoObj->GetLayerName() == LayerName);

	return LayerName;
}

const FLandscapeTargetLayerSettings& FLandscapeInfoLayerSettings::GetTargetLayerSettings() const
{
	check(LayerInfoObj);

	ULandscapeInfo* LandscapeInfo = Owner->GetLandscapeInfo();
	return LandscapeInfo->GetTargetLayerSettings(LayerInfoObj);
}

const FLandscapeTargetLayerSettings& ULandscapeInfo::GetTargetLayerSettings(ULandscapeLayerInfoObject* LayerInfo) const
{
	ALandscapeProxy* Proxy = GetLandscapeProxy();
	const FName* LayerName = Proxy->GetTargetLayers().FindKey(FLandscapeTargetLayerSettings(LayerInfo));
	if (LayerName)
	{
		return *Proxy->GetTargetLayers().Find(*LayerName);
	}
	else
	{
		return Proxy->AddTargetLayer(LayerInfo->GetLayerName(), FLandscapeTargetLayerSettings(LayerInfo));
	}
}

void ULandscapeInfo::CreateTargetLayerSettingsFor(ULandscapeLayerInfoObject* LayerInfo)
{
	ForEachLandscapeProxy([LayerInfo](ALandscapeProxy* Proxy)
	{
		if (Proxy->HasTargetLayer(LayerInfo->GetLayerName()))
		{
			Proxy->UpdateTargetLayer(LayerInfo->GetLayerName(), FLandscapeTargetLayerSettings(LayerInfo));
		}
		else
		{
			Proxy->AddTargetLayer(LayerInfo->GetLayerName(), FLandscapeTargetLayerSettings(LayerInfo));
		}

		return true;
	});
}

ULandscapeLayerInfoObject* ULandscapeInfo::GetLayerInfoByName(FName LayerName, ALandscapeProxy* Owner /*= nullptr*/) const
{
	ULandscapeLayerInfoObject* LayerInfo = nullptr;
	for (int32 j = 0; j < Layers.Num(); j++)
	{
		if (Layers[j].LayerInfoObj && Layers[j].LayerInfoObj->GetLayerName() == LayerName
			&& (Owner == nullptr || Layers[j].Owner == Owner))
		{
			LayerInfo = Layers[j].LayerInfoObj;
		}
	}
	return LayerInfo;
}

int32 ULandscapeInfo::GetLayerInfoIndex(ULandscapeLayerInfoObject* LayerInfo, ALandscapeProxy* Owner /*= nullptr*/) const
{
	for (int32 j = 0; j < Layers.Num(); j++)
	{
		if (Layers[j].LayerInfoObj && Layers[j].LayerInfoObj == LayerInfo
			&& (Owner == nullptr || Layers[j].Owner == Owner))
		{
			return j;
		}
	}

	return INDEX_NONE;
}

int32 ULandscapeInfo::GetLayerInfoIndex(FName LayerName, ALandscapeProxy* Owner /*= nullptr*/) const
{
	for (int32 j = 0; j < Layers.Num(); j++)
	{
		if (Layers[j].GetLayerName() == LayerName
			&& (Owner == nullptr || Layers[j].Owner == Owner))
		{
			return j;
		}
	}

	return INDEX_NONE;
}


bool ULandscapeInfo::UpdateLayerInfoMapInternal(ALandscapeProxy* Proxy)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeInfo::UpdateLayerInfoMapInternal);

	bool bLayerInfoMapChanged = false;
	if( !LandscapeActor.IsValid())
	{
		return false;
	}

	// Perform a delayed (see where TargetLayersForFixup is set), one-time deprecation of the landscape layer data based on the content of the materials in the components (see FixupLandscapeTargetLayersInLandscapeActor)
	if (GIsEditor && Proxy && !Proxy->TargetLayersForFixup.IsEmpty())
	{
		// Go through the list of layer names / info to fixup and declare new or update existing layers in the main landscape actor if we have one that the main landscape doesn't know about :
		for (const auto& It : Proxy->TargetLayersForFixup)
		{
			FName LayerName = It.Key;
			ULandscapeLayerInfoObject* LayerInfo = It.Value;
			check(LayerName.IsValid());

			const FLandscapeTargetLayerSettings* LayerSettingsInLandscapeActor = LandscapeActor->GetTargetLayers().Find(LayerName);
			// If the layer isn't known to the main landscape, add it now : 
			if (LayerSettingsInLandscapeActor == nullptr)
			{
				// Mark the parent landscape actor dirty with bInForceResave == true so that the parent actor is put into the list of files to save even if we do this fixup on load :
				MarkObjectDirty(/*InObject = */LandscapeActor.Get(), /*bInForceResave = */true);

				LandscapeActor->AddTargetLayer(LayerName, FLandscapeTargetLayerSettings(LayerInfo), false);
				bLayerInfoMapChanged = true;
			}
			// If the layer name is known to the main landscape but it hasn't got a landscape info associated to it yet, update it to use this LayerInfo :
			else if ((LayerInfo != nullptr) && (LayerSettingsInLandscapeActor->LayerInfoObj == nullptr))
			{
				// Mark the parent landscape actor dirty with bInForceResave == true so that the parent actor is put into the list of files to save even if we do this fixup on load :
				MarkObjectDirty(/*InObject = */LandscapeActor.Get(), /*bInForceResave = */true);

				LandscapeActor->UpdateTargetLayer(LayerName, FLandscapeTargetLayerSettings(LayerInfo), false);
				bLayerInfoMapChanged = true;
			}
		}

		Proxy->TargetLayersForFixup.Empty();
	}

	// Keep a temp copy of the previous layers to keep the thumbnail MICs alive : 
	TArray<FLandscapeInfoLayerSettings> PreviousLayers;
	Swap(Layers, PreviousLayers);

	for (const TTuple<FName, FLandscapeTargetLayerSettings>& TargetLayer : LandscapeActor->GetTargetLayers())
	{
		FLandscapeInfoLayerSettings InfoLayerSettings (TargetLayer.Key, LandscapeActor.Get());
		InfoLayerSettings.LayerInfoObj = TargetLayer.Value.LayerInfoObj;
		if (const FLandscapeInfoLayerSettings* PreviousLayer = Algo::FindBy(PreviousLayers, TargetLayer.Key, &FLandscapeInfoLayerSettings::LayerName))
		{
			InfoLayerSettings.ThumbnailMIC = PreviousLayer->ThumbnailMIC;
		}

		Layers.Add(InfoLayerSettings);
	}
	
	// Add Visibility Layer info if not initialized
	if (ALandscapeProxy::VisibilityLayer != nullptr)
	{
		int32 LayerInfoIndex = GetLayerInfoIndex(ALandscapeProxy::VisibilityLayer->GetLayerName());

		if ((LayerInfoIndex != INDEX_NONE) && (Layers[LayerInfoIndex].LayerInfoObj == nullptr))
		{
			Layers[LayerInfoIndex].LayerInfoObj = ALandscapeProxy::VisibilityLayer;
		}
	}

	return bLayerInfoMapChanged;
}

bool ULandscapeInfo::UpdateLayerInfoMap(ALandscapeProxy* Proxy /*= nullptr*/, bool bInvalidate /*= false*/)
{
	bool bLayerInfoMapChanged = UpdateLayerInfoMapInternal(Proxy);

	if (GIsEditor)
	{
		ALandscape* Landscape = LandscapeActor.Get();
		if (Landscape && !Landscape->IsTemplate())
		{
			Landscape->RequestLayersInitialization(/*bInRequestContentUpdate*/false, /*bInForceLayerResourceReset*/false);
		}
	}
	return bLayerInfoMapChanged;

}

#endif // WITH_EDITOR

/* if the outer world is instanced, we need to change our landscape guid(in a deterministic way)
 * this avoids guid collisions when you instance a world (and its landscapes) multiple times,
 * while maintaining the same GUID between landscape proxy objects within an instance
 */
void ChangeLandscapeGuidIfObjectIsInstanced(FGuid& InOutGuid, UObject* InObject)
{
	// we shouldn't be dealing with any instanced landscapes in these cases, early out
	if (InObject->IsTemplate())
	{
		return;
	}

	UWorldPartition* WorldPartition = FWorldPartitionHelpers::GetWorldPartition(InObject);

#if WITH_EDITOR
	// In PIE, Actors that are part of a Unsaved cluster of actors can end up being duplicated through a UActorContainer. 
	// In this case we need to resolve the WorldPartition differently. This could be fixed in a more generic way but would require a lot more testing (Jira: tbd)
	if (!WorldPartition)
	{
		if (UActorContainer* Container = InObject->GetTypedOuter<UActorContainer>())
		{
			WorldPartition = FWorldPartitionHelpers::GetWorldPartition(Container->RuntimeLevel.Get());
		}
	}
#endif

	UWorld* OuterWorld = WorldPartition ? WorldPartition->GetTypedOuter<UWorld>() : InObject->GetTypedOuter<UWorld>();

	// TODO [chris.tchou] : Note this is not 100% correct, IsInstanced() returns TRUE when using PIE on non-instanced landscapes.
	// That is generally ok however, as the GUID remaps are still deterministic and landscape still works.
	if (OuterWorld && OuterWorld->IsInstanced())
	{
		FArchiveMD5 Ar;
		FGuid OldLandscapeGuid = InOutGuid;
		Ar << OldLandscapeGuid;

		UPackage* OuterWorldPackage = OuterWorld->GetPackage();
		if (ensure(OuterWorldPackage))
		{
			FName PackageName = OuterWorldPackage->GetFName();
			Ar << PackageName;
		}

		InOutGuid = Ar.GetGuidFromHash();
	}
}

void ALandscapeProxy::PostLoadFixupLandscapeGuidsIfInstanced()
{
	// record the original value before modification
	check(!OriginalLandscapeGuid.IsValid() || (OriginalLandscapeGuid == LandscapeGuid));
	OriginalLandscapeGuid = this->LandscapeGuid;

	ChangeLandscapeGuidIfObjectIsInstanced(this->LandscapeGuid, this);
}

void ALandscapeProxy::PostLoad()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ALandscapeProxy::PostLoad);


#if WITH_EDITOR
	// Not sure that this can ever happen without someone deliberately changing the root component but a landscape without a root component is 
	//  worthless and will lead to pain and crash, so attempt to fix it up on load here : 
	if (GetRootComponent() == nullptr)
	{
		TInlineComponentArray<USceneComponent*> SceneComponents;
		GetComponents<USceneComponent>(SceneComponents, /*bIncludeFromChildActors = */false);
		UClass* SceneComponentClass = USceneComponent::StaticClass();
		if (USceneComponent** RootComponentCandidate = Algo::FindByPredicate(SceneComponents, [SceneComponentClass](USceneComponent* InComponent) { return (InComponent->GetClass() == SceneComponentClass); }))
		{
			SetRootComponent(*RootComponentCandidate);
		}
		else
		{
			UE_LOGF(LogLandscape, Error, "Unable to retrieve a root component for landscape proxy %ls. The landscape will not work properly.", *GetFullName());
		}
	}

	// Store edit layer state before fixup edit layer data is run. If edit layers do not exist, proxy is a legacy non-edit layer landscape
	bHasEditLayersOnLoad = !LandscapeComponents.IsEmpty() && (LandscapeComponents[0] != nullptr) && LandscapeComponents[0]->HasLayersData();
#endif // WITH_EDITOR

	Super::PostLoad();

	PostLoadFixupLandscapeGuidsIfInstanced();

#if WITH_EDITOR
	// Silently add a visibility layer to each proxy. The cost is minimal, since components do not store weightmap info until data is written to the layer
	// Target layers are no longer auto generated during material compilation so this guarantees the visibility layer is always found
	// This needs to be done before the potential call to FixupSharedData so that we don't report an non-synchronized "TargetLayers" property in there because it 
	//  would mean the main ALandscape is already available and therefore has already been upgraded to include the visibility layer while the proxy, not yet
	const FName VisibilityLayerName = (ALandscapeProxy::VisibilityLayer != nullptr) ? ALandscapeProxy::VisibilityLayer->GetLayerName() : FName();
	if (!IsTemplate() && !VisibilityLayerName.IsNone() && !HasTargetLayer(VisibilityLayerName))
	{
		AddTargetLayer(VisibilityLayerName, FLandscapeTargetLayerSettings(), /*bPostEditChange =*/false);
	}

	FixupOverriddenSharedProperties();

	ALandscape* LandscapeActor = GetLandscapeActor();

	// Try to fixup shared properties if everything is ready for it as some postload process may depend on it.
	if ((GetLandscapeInfo() != nullptr) && (LandscapeActor != nullptr) && (LandscapeActor != this))
	{
		const bool bMapCheck = true;
		FixupSharedData(LandscapeActor, bMapCheck, false /*bInExecutePostEditChangeProperty*/);
	}
#endif // WITH_EDITOR

	// Temporary
	if (ComponentSizeQuads == 0 && LandscapeComponents.Num() > 0)
	{
		ULandscapeComponent* Comp = LandscapeComponents[0];
		if (Comp)
		{
			ComponentSizeQuads = Comp->ComponentSizeQuads;
			SubsectionSizeQuads = Comp->SubsectionSizeQuads;
			NumSubsections = Comp->NumSubsections;
		}
	}

	if (IsTemplate() == false)
	{
		BodyInstance.FixupData(this);
	}

	for (ULandscapeComponent* Comp : LandscapeComponents)
	{
		if (Comp == nullptr)
		{
			continue;
		}

		UE_LOGF(LogGrass, Verbose, "POSTLOAD: component %ls on landscape %ls UseRuntimeGeneration %d Disable: %d data: %d",
			*Comp->GetName(),
			*GetName(),
			GGrassMapUseRuntimeGeneration, bDisableRuntimeGrassMapGeneration, Comp->GrassData->NumElements);

#if !WITH_EDITOR
		// if using runtime grass gen, it should have been cleared out in PreSave
		if (GGrassMapUseRuntimeGeneration && !bDisableRuntimeGrassMapGeneration)
		{
			if (Comp->GrassData->HasData())
			{
				UE_LOGF(LogGrass, Warning, "grass.GrassMap.UseRuntimeGeneration is enabled, but component %ls on landscape %ls has unnecessary grass data saved.  Ensure grass.GrassMap.UseRuntimeGeneration is enabled at cook time to reduce cooked data size.",
					*Comp->GetName(),
					*GetName());

				// Free the memory, so at least we will save the space at runtime.
				Comp->GrassData = MakeShared<FLandscapeComponentGrassData>();
			}
		}
#endif // !WITH_EDITOR
	}

#if WITH_EDITOR
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!LandscapeMaterialsOverride_DEPRECATED.IsEmpty())
	{
		PerLODOverrideMaterials.Reserve(LandscapeMaterialsOverride_DEPRECATED.Num());
		for (const FLandscapeProxyMaterialOverride& LocalMaterialOverride : LandscapeMaterialsOverride_DEPRECATED)
		{
			PerLODOverrideMaterials.Add({ LocalMaterialOverride.LODIndex.Default, LocalMaterialOverride.Material });
		}
		LandscapeMaterialsOverride_DEPRECATED.Reset();
	}

	if (!EditorLayerSettings_DEPRECATED.IsEmpty())
	{
		// If we still have access to EditorLayerSettings_DEPRECATED because it's the first time we deprecate this proxy since FFortniteMainBranchObjectVersion::LandscapeTargetLayersInLandscapeActor, 
		//  fill the list of target layers to fixup based on the original property because it's the most accurate (it has layer info assignment even if there's no weightmap allocation for a given layer) :
		TargetLayersForFixup.Reserve(EditorLayerSettings_DEPRECATED.Num());
		for (const FLandscapeEditorLayerSettings& EditorLayerSetting : EditorLayerSettings_DEPRECATED)
		{
			if (EditorLayerSetting.LayerInfoObj != nullptr)
			{
				TargetLayersForFixup.Add(EditorLayerSetting.LayerInfoObj->GetLayerName(), EditorLayerSetting.LayerInfoObj);
			}
		}
		EditorLayerSettings_DEPRECATED.Reset();
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	const int32 LinkerVersion = GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	// With and before LandscapeTargetLayersInLandscapeActor and until FixupLandscapeTargetLayersInLandscapeActor, some layer info objects have been incorrectly unassigned and we now have to go through 
	//  all materials of all streaming proxies to gather the missing landscape layer info objects and this can only be done after proxies are united with their parent landscape by their ULandscapeInfo 
	//  (since TargetLayers is a LandscapeInherited property of the parent landscape, propagated to the child proxies), so we must delay this operation until then. What we do here is prepare a list of 
	//  layers to fixup in the main landscape actor (TargetLayersForFixup) and when the proxy is registered to the parent landscape, we'll go through that list and update the landscape's TargetLayers list,
	//  which will then be synchronized with all proxies if necessary:
	if (LinkerVersion < FFortniteMainBranchObjectVersion::FixupLandscapeTargetLayersInLandscapeActor)
	{
		// Go through the list of materials and weightmap allocations to gather potential layer name / layer info associations :
		{
			TMap<FName, ULandscapeLayerInfoObject*> LayerInfosFromAllocations = RetrieveTargetLayerInfosFromAllocations();
			for (const auto& It : LayerInfosFromAllocations)
			{
				FName LayerName = It.Key;
				ULandscapeLayerInfoObject* LayerInfo = It.Value;
				TObjectPtr<ULandscapeLayerInfoObject>* LayerInfoInFixupMap = TargetLayersForFixup.Find(LayerName);
				// Unknown layer name yet, let's add a layer name / info association :
				if (LayerInfoInFixupMap == nullptr)
				{
					TargetLayersForFixup.Add(LayerName, LayerInfo);
				}
				// Known layer name, but we have no valid layer info associated with it yet, update it :
				else if (*LayerInfoInFixupMap == nullptr)
				{
					*LayerInfoInFixupMap = LayerInfo;
				}
				// Otherwise, don't touch it, we consider that TargetLayersForFixup has the authority over this layer already
			}
		}
	}

	if (GIsEditor)
	{
		// We may not have run PostLoad on LandscapeComponents yet
		for (TObjectPtr<ULandscapeComponent>& LandscapeComponent : LandscapeComponents)
		{
			if (LandscapeComponent)
			{
				LandscapeComponent->ConditionalPostLoad();
			}
		}

		// We may not have run PostLoad on CollisionComponent yet
		for (TObjectPtr<ULandscapeHeightfieldCollisionComponent>& CollisionComponent : CollisionComponents)
		{
			if (CollisionComponent)
			{
				CollisionComponent->ConditionalPostLoad();
			}
		}

		if ((GetLinker() && (GetLinker()->UEVer() < VER_UE4_LANDSCAPE_COMPONENT_LAZY_REFERENCES)) ||
			LandscapeComponents.Num() != CollisionComponents.Num() ||
			LandscapeComponents.ContainsByPredicate([](ULandscapeComponent* Comp) { return ((Comp != nullptr) && (Comp->GetCollisionComponent() == nullptr)); }) ||
			CollisionComponents.ContainsByPredicate([](ULandscapeHeightfieldCollisionComponent* Comp) { return ((Comp != nullptr) && (Comp->GetRenderComponent() == nullptr)); }))
		{
			// Need to clean up invalid collision and render components
			RecreateCollisionComponents();
		}
	}
	
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (EditorCachedLayerInfos_DEPRECATED.Num() > 0)
	{
		for (int32 i = 0; i < EditorCachedLayerInfos_DEPRECATED.Num(); i++)
		{
			TargetLayers.Add(EditorCachedLayerInfos_DEPRECATED[i]->LayerName, FLandscapeTargetLayerSettings(EditorCachedLayerInfos_DEPRECATED[i]));
		}
		EditorCachedLayerInfos_DEPRECATED.Empty();
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	bool bFixedUpInvalidMaterialInstances = false;
	for (ULandscapeComponent* Comp : LandscapeComponents)
	{
		if (Comp == nullptr)
		{
			continue;
		}

		// Validate the layer combination and store it in the MaterialInstanceConstantMap
		UMaterialInstance* MaterialInstance = Comp->GetMaterialInstance(0, false);

		if (MaterialInstance == nullptr)
		{
			continue;
		}

		UMaterialInstanceConstant* CombinationMaterialInstance = Cast<UMaterialInstanceConstant>(MaterialInstance->Parent);
		// Only validate if uncooked and in the editor/commandlet mode (we cannot re-build material instance constants if this is not the case : see UMaterialInstance::CacheResourceShadersForRendering, which is only called if FApp::CanEverRender() returns true) 
		if (!Comp->GetOutermost()->HasAnyPackageFlags(PKG_FilterEditorOnly) && (GIsEditor && FApp::CanEverRender()))
		{
			UMaterial* BaseLandscapeMaterial = Comp->GetLandscapeMaterial()->GetMaterial();
			// MaterialInstance is different from the used LandscapeMaterial, we need to update the material as we cannot properly validate used combinations.
			if (MaterialInstance->GetMaterial() != BaseLandscapeMaterial)
			{
				Comp->UpdateMaterialInstances();
				bFixedUpInvalidMaterialInstances = true;
				continue;
			}

			if (Comp->ValidateCombinationMaterial(CombinationMaterialInstance))
			{
				MaterialInstanceConstantMap.Add(*ULandscapeComponent::GetLayerAllocationKey(Comp->GetWeightmapLayerAllocations(), CombinationMaterialInstance->Parent), CombinationMaterialInstance);
			}
			else
			{
				// There was a problem with the loaded material : it doesn't match the expected material combination, we need to regenerate the material instances : 
				Comp->UpdateMaterialInstances();
				bFixedUpInvalidMaterialInstances = true;
			}
		}
		else if (CombinationMaterialInstance)
		{
			// Skip ValidateCombinationMaterial
			MaterialInstanceConstantMap.Add(*ULandscapeComponent::GetLayerAllocationKey(Comp->GetWeightmapLayerAllocations(), CombinationMaterialInstance->Parent), CombinationMaterialInstance);
		}
	}

	if (bFixedUpInvalidMaterialInstances)
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("ProxyPackage"), FText::FromString(GetOutermost()->GetName()));
		FMessageLog("MapCheck").Info()
			->AddToken(FUObjectToken::Create(this, FText::FromString(GetActorNameOrLabel())))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_FixedUpInvalidLandscapeMaterialInstances", "Fixed up invalid landscape material instances. Please re-save {ProxyPackage}."), Arguments)))
			->AddToken(FMapErrorToken::Create(FMapErrors::FixedUpInvalidLandscapeMaterialInstances));
	}

	// Display a MapCheck warning if the Nanite data is stale with the option to trigger a rebuild & Save
	if (!IsNaniteMeshUpToDate() && !IsRunningCookCommandlet())
	{
		auto CreateMapCheckMessage = [](FMessageLog& MessageLog)
		{
			if (CVarLandscapeSilenceMapCheckWarnings_Nanite->GetBool())
			{
				return MessageLog.Info();
			}
			return MessageLog.Warning();
		};

		TWeakObjectPtr<ALandscapeProxy> WeakLandscapeProxy(this);

		FMessageLog MessageLog("MapCheck");
		CreateMapCheckMessage(MessageLog)
			->AddToken(FUObjectToken::Create(this, FText::FromString(GetActorNameOrLabel())))
			->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_LandscapeRebuildNanite", "Landscape Nanite is enabled but the saved mesh data is out of date. ")))
			->AddToken(FActionToken::Create(LOCTEXT("MapCheck_SaveFixedUpData", "Save Modified Landscapes"), LOCTEXT("MapCheck_SaveFixedUpData_Desc", "Saves the modified landscape proxy actors"),
				FOnActionTokenExecuted::CreateLambda([WeakLandscapeProxy]()
					{
						if (!WeakLandscapeProxy.IsValid())
						{
							return;
						}
						ULandscapeInfo* Info = WeakLandscapeProxy->GetLandscapeInfo();
						check(Info);

						TSet<UPackage*> DirtyNanitePackages;
						Info->ForEachLandscapeProxy([&DirtyNanitePackages](const ALandscapeProxy* Proxy)
							{
								if (!Proxy->IsNaniteMeshUpToDate())
								{
									DirtyNanitePackages.Add(Proxy->GetOutermost());
								}
								return true;
							});

						Info->UpdateNanite(nullptr);

						constexpr bool bPromptUserToSave = true;
						constexpr bool bSaveMapPackages = true;
						constexpr bool bSaveContentPackages = true;
						constexpr bool bFastSave = false;
						constexpr bool bNotifyNoPackagesSaved = false;
						constexpr bool bCanBeDeclined = true;

						FEditorFileUtils::SaveDirtyPackages(bPromptUserToSave, bSaveMapPackages, bSaveContentPackages, bFastSave, bNotifyNoPackagesSaved, bCanBeDeclined, nullptr,
							[&DirtyNanitePackages](const UPackage* Package) { return !DirtyNanitePackages.Contains(Package); });

					}), FCanExecuteActionToken::CreateLambda([WeakLandscapeProxy]() { return WeakLandscapeProxy.IsValid() ? !WeakLandscapeProxy->IsNaniteMeshUpToDate() : false; }))
			);
	}

	UWorld* World = GetWorld();

	if (World)
	{
		FOnFeatureLevelChanged::FDelegate FeatureLevelChangedDelegate = FOnFeatureLevelChanged::FDelegate::CreateUObject(this, &ALandscapeProxy::OnFeatureLevelChanged);
		FeatureLevelChangedDelegateHandle = World->AddOnFeatureLevelChangedHandler(FeatureLevelChangedDelegate);
	}
	RepairInvalidTextures();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (NaniteComponent_DEPRECATED)
	{
		NaniteComponents.Add(NaniteComponent_DEPRECATED);
		NaniteComponent_DEPRECATED = nullptr;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (GIsEditor)
	{
		// Fix for empty nanite components that have lost their link to their source landscape components
		SetSourceComponentsForNaniteComponents(/*bSetEmptyComponentsOnly=*/ true);
	}
	
	// Handle Nanite representation invalidation on load: 
	if (!HasAnyFlags(RF_ClassDefaultObject) && !FPlatformProperties::RequiresCookedData())
	{
		// FFortniteReleaseBranchCustomObjectVersion::FixupNaniteLandscapeMeshes : Fixup Nanite meshes which were using the wrong material and didn't have proper UVs
		// FFortniteReleaseBranchCustomObjectVersion::RemoveUselessLandscapeMeshesCookedCollisionData : Remove cooked collision data from Nanite landscape meshes, since collisions are handled by ULandscapeHeightfieldCollisionComponent
		// FFortniteReleaseBranchCustomObjectVersion::FixNaniteLandscapeMeshNames : Fix the names of the generated Nanite landscape UStaticMesh so that it's unique in a given package
		// FFortniteMainBranchObjectVersion::FixNaniteLandscapeMeshDDCKey : Fix the non-deterministic hash being used by the generated Nanite landscape UStaticMesh so that it can benefit from DDC sharing if it's identical to a previously uploaded mesh derived data
		if ((GetLinkerCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID) < FFortniteReleaseBranchCustomObjectVersion::FixNaniteLandscapeMeshNames)
			|| (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::FixNaniteLandscapeMeshDDCKey)
			|| CVarForceInvalidateNaniteOnLoad->GetBool())
		{
			// This will force the Nanite meshes to be properly regenerated during the next save :
			InvalidateNaniteRepresentation(/* bCheckContentId = */ false);
		}
		else
		{
			// On load, get rid of the Nanite representation if it's not up-to-date so that it's marked as needing an update and will get fixed by the user when building Nanite data
			InvalidateNaniteRepresentation(/* bCheckContentId = */ true);
		}

		// Remove RF_Transactional from Nanite components : they're re-created upon transacting now : 
		ClearNaniteTransactional();
	}

	// Keep previous behavior of landscape HLODs if created before the settings were added
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::LandscapeAddedHLODSettings)
	{
		HLODTextureSizePolicy = ELandscapeHLODTextureSizePolicy::AutomaticSize;
		HLODMeshSourceLODPolicy = ELandscapeHLODMeshSourceLODPolicy::AutomaticLOD;
	}
#endif // WITH_EDITOR
}

// Deprecated
FIntPoint ALandscapeProxy::GetSectionBaseOffset() const
{
	return SectionBase;
}

FIntPoint ALandscapeProxy::GetSectionBase() const
{
	return SectionBase;
}

#if WITH_EDITOR
void ALandscapeProxy::SetSectionBase(FIntPoint InSectionBase)
{
	FIntPoint Difference = InSectionBase - SectionBase;
	SectionBase = InSectionBase;

	RecreateComponentsRenderState([Difference](ULandscapeComponent* Comp)
		{
			FIntPoint AbsoluteSectionBase = Comp->GetSectionBase() + Difference;
			Comp->SetSectionBase(AbsoluteSectionBase);
		});

	for (int32 CompIdx = 0; CompIdx < CollisionComponents.Num(); CompIdx++)
	{
		ULandscapeHeightfieldCollisionComponent* Comp = CollisionComponents[CompIdx];
		if (Comp)
		{
			FIntPoint AbsoluteSectionBase = Comp->GetSectionBase() + Difference;
			Comp->SetSectionBase(AbsoluteSectionBase);
		}
	}

	// LandscapeInfo sorts on section base
	ULandscapeInfo* LandscapeInfo = GetLandscapeInfo();
	check(LandscapeInfo);
	LandscapeInfo->SortStreamingProxies();
}

void ALandscapeProxy::Destroyed()
{
	Super::Destroyed();

	UWorld* World = GetWorld();

	if (GIsEditor)
	{
		ULandscapeInfo::RecreateLandscapeInfo(World, false);

		if (SplineComponent)
		{
			SplineComponent->ModifySplines();
		}
	}

	// Destroy the Nanite component when we get destroyed so that we don't restore a garbage Nanite component (it's non-transactional and will get regenerated anyway)
	InvalidateNaniteRepresentation(/* bInCheckContentId = */false);

	// unregister feature level changed handler for grass
	if (FeatureLevelChangedDelegateHandle.IsValid())
	{
		World->RemoveOnFeatureLevelChangedHandler(FeatureLevelChangedDelegateHandle);
		FeatureLevelChangedDelegateHandle.Reset();
	}
}

namespace UE::Landscape::Private
{
	static bool CopyProperty(FProperty* InProperty, UObject* InSourceObject, UObject* InDestinationObject, bool bInExecutePostEditChangeProperty)
	{
		// Do not execute the post edit change if the containing level has not finished post loading.  
		// This will trigger Registration of the components, before the owning level of the world has been determined.
		// This avoids registring landscape proxies in the incorrect landscape info object.
		if(bInExecutePostEditChangeProperty)
		{
			ULevel* Level = nullptr;
			if (ALandscapeProxy* LandscapeProxy = Cast<ALandscapeProxy>(InDestinationObject))
			{
				Level = LandscapeProxy->GetLevel();
			}
			else if (USceneComponent* SceneComponent = Cast<USceneComponent>(InDestinationObject))
			{
				Level = SceneComponent->GetOwner()->GetLevel();
			}

			bInExecutePostEditChangeProperty = Level == nullptr || !Level->HasAnyFlags(EObjectFlags::RF_NeedPostLoad);
			ensureAlwaysMsgf(bInExecutePostEditChangeProperty, TEXT("Executing CopyProperty with bInExecutePostEditChangeProperty set to true before the level has post loaded would trigger registration in the world before it's owning world has been determined."));
		}

		void* SrcValuePtr = InProperty->ContainerPtrToValuePtr<void>(InSourceObject);
		void* DestValuePtr = InProperty->ContainerPtrToValuePtr<void>(InDestinationObject);

		if ((DestValuePtr == nullptr) || (SrcValuePtr == nullptr))
		{
			return false;
		}

		InProperty->CopyCompleteValue(DestValuePtr, SrcValuePtr);
		if (bInExecutePostEditChangeProperty)
		{
			// Some properties may need additional processing (ex: LandscapeMaterial), notify the proxy of the change.
			FPropertyChangedEvent PropertyChangedEvent(InProperty);
			InDestinationObject->PostEditChangeProperty(PropertyChangedEvent);
			InDestinationObject->Modify();
		}

		return true;
	}

	static bool CopyPropertyByName(const TWeakObjectPtr<ALandscapeProxy>& InLandscapeProxy, const TWeakObjectPtr<ALandscape>& InParentLandscape, const FName& InPropertyName, bool bInExecutePostEditChangeProperty)
	{
		if (!InLandscapeProxy.IsValid() || !InParentLandscape.IsValid())
		{
			return false;
		}

		UClass* LandscapeProxyClass = InLandscapeProxy->GetClass();

		if (LandscapeProxyClass == nullptr)
		{
			return false;
		}

		FProperty* PropertyToCopy = LandscapeProxyClass->FindPropertyByName(InPropertyName);

		if (PropertyToCopy == nullptr)
		{
			return false;
		}

		CopyProperty(PropertyToCopy, InParentLandscape.Get(), InLandscapeProxy.Get(), bInExecutePostEditChangeProperty);
		return true;
	}

	static void DisplaySynchronizedPropertiesMapcheckWarning(const TArray<FName>& InSynchronizedProperties, const ALandscapeProxy& InSynchronizedProxy, const ALandscapeProxy& InParentLandscape, const bool bAddSilencingMessage = false)
	{
		check(!InSynchronizedProperties.IsEmpty());

		TStringBuilder<1024> SynchronizedPropertiesStringBuilder;
		ULandscapeSubsystem* LandscapeSubsystem = InSynchronizedProxy.GetWorld() ? InSynchronizedProxy.GetWorld()->GetSubsystem<ULandscapeSubsystem>() : nullptr;
		checkf(LandscapeSubsystem != nullptr, TEXT("DisplaySynchronizedPropertiesMapcheckWarning can only be called when a subsystem is available"));

		for (const FName& SynchronizedProperty : InSynchronizedProperties)
		{
			if (SynchronizedPropertiesStringBuilder.Len() > 0)
			{
				SynchronizedPropertiesStringBuilder.Append(TEXT(", "));
			}

			SynchronizedPropertiesStringBuilder.Append(SynchronizedProperty.ToString());
		}

		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("Proxy"), FText::FromString(InSynchronizedProxy.GetActorNameOrLabel()));
		Arguments.Add(TEXT("Landscape"), FText::FromString(InParentLandscape.GetActorNameOrLabel()));
		TSharedRef<FTokenizedMessage> Message = FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(&InSynchronizedProxy, FText::FromString(InSynchronizedProxy.GetActorNameOrLabel())))
			->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_LandscapeProxy_FixupSharedData", "had some shared properties not in sync with its parent landscape actor. This has been fixed but the proxy needs to be saved in order to ensure cooking behaves as expected. ")))
			->AddToken(FActionToken::Create(LOCTEXT("MapCheck_SaveFixedUpData", "Save Modified Landscapes"), LOCTEXT("MapCheck_SaveFixedUpData_Desc", "Saves the modified landscape proxy actors"),
				FOnActionTokenExecuted::CreateUObject(LandscapeSubsystem, &ULandscapeSubsystem::SaveModifiedLandscapes, UE::Landscape::EBuildFlags::WriteFinalLog),
				FCanExecuteActionToken::CreateUObject(LandscapeSubsystem, &ULandscapeSubsystem::HasModifiedLandscapes),
				/*bInSingleUse = */false))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_LandscapeProxy_FixupSharedData_SharedProperties", "The following properties were synchronized: {0}."), FText::FromString(SynchronizedPropertiesStringBuilder.ToString()))));

		if (bAddSilencingMessage)
		{
			Message->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_LandscapeProxy_SilenceWarning", "You can silence this warning and perform the deprecation silently using the landscape.SilenceSharedPropertyDeprecationFixup CVar. ")));
		}

		Message->AddToken(FMapErrorToken::Create(FMapErrors::LandscapeComponentPostLoad_Warning));

		// Show MapCheck window
		FMessageLog("MapCheck").Open(EMessageSeverity::Warning);
	}
} // namespace UE::Landscape::Private

void ALandscapeProxy::CopySharedProperties(ALandscapeProxy* InLandscape, bool bInExecutePostEditChangeProperty)
{
	SynchronizeUnmarkedSharedProperties(InLandscape);

	for (TFieldIterator<FProperty> PropertyIterator(GetClass()); PropertyIterator; ++PropertyIterator)
	{
		FProperty* Property = *PropertyIterator;

		if (Property == nullptr)
		{
			continue;
		}

		if (IsSharedProperty(Property))
		{
			UE::Landscape::Private::CopyProperty(Property, InLandscape, this, bInExecutePostEditChangeProperty);
		}
	}
}

TArray<FName> ALandscapeProxy::SynchronizeSharedProperties(ALandscapeProxy* InLandscape, bool bInExecutePostEditChangeProperty)
{
	TArray<FName> SynchronizedProperties = SynchronizeUnmarkedSharedProperties(InLandscape);

	for (TFieldIterator<FProperty> PropertyIterator(GetClass()); PropertyIterator; ++PropertyIterator)
	{
		FProperty* Property = *PropertyIterator;

		if (Property == nullptr)
		{
			continue;
		}

		if ((IsPropertyInherited(Property) ||
			(IsPropertyOverridable(Property) && !IsSharedPropertyOverridden(Property->GetFName()))) &&
			!Property->Identical_InContainer(this, InLandscape))
		{
			SynchronizedProperties.Emplace(Property->GetFName());

			// Caller may need a PostEditChangeProperty to trigger other landscape updates (Eg. Dirty Render state, nanite, materials)
			// PostEditChangeProperty updates are unnecessary when syncing properties during new landscape creation
			UE::Landscape::Private::CopyProperty(Property, InLandscape, this, bInExecutePostEditChangeProperty);
		}
	}

	if (!SynchronizedProperties.IsEmpty())
	{
		Modify();
	}

	return SynchronizedProperties;
}

bool ALandscapeProxy::IsSharedProperty(const FName& InPropertyName) const
{
	FProperty* Property = StaticClass()->FindPropertyByName(InPropertyName);

	return IsSharedProperty(Property);
}

bool ALandscapeProxy::IsSharedProperty(const FProperty* InProperty) const
{
	return IsPropertyInherited(InProperty) || IsPropertyOverridable(InProperty);
}

bool ALandscapeProxy::IsPropertyInherited(const FProperty* InProperty) const
{
	if (InProperty == nullptr)
	{
		return false;
	}

	return InProperty->HasMetaData(LandscapeInheritedTag);
}

bool ALandscapeProxy::IsPropertyOverridable(const FProperty* InProperty) const
{
	if (InProperty == nullptr)
	{
		return false;
	}

	return InProperty->HasMetaData(LandscapeOverridableTag);
}
#endif // WITH_EDITOR

UMaterialInterface* ALandscapeProxy::GetLandscapeMaterial(int8 InLODIndex) const
{
	if (InLODIndex != INDEX_NONE)
	{
		UWorld* World = GetWorld();

		if (World != nullptr)
		{
			if (const FLandscapePerLODMaterialOverride* LocalMaterialOverride = PerLODOverrideMaterials.FindByPredicate(
				[InLODIndex](const FLandscapePerLODMaterialOverride& InOverride) { return (InOverride.LODIndex == InLODIndex) && (InOverride.Material != nullptr); }))
			{
				return LocalMaterialOverride->Material;
			}
		}
	}

	return LandscapeMaterial != nullptr ? LandscapeMaterial : UMaterial::GetDefaultMaterial(MD_Surface);
}

UMaterialInterface* ALandscapeProxy::GetLandscapeHoleMaterial() const
{
	return LandscapeHoleMaterial;
}

UMaterialInterface* ALandscapeStreamingProxy::GetLandscapeMaterial(int8 InLODIndex) const
{
	if (InLODIndex != INDEX_NONE)
	{
		UWorld* World = GetWorld();

		if (World != nullptr)
		{
			if (const FLandscapePerLODMaterialOverride* LocalMaterialOverride = PerLODOverrideMaterials.FindByPredicate(
				[InLODIndex](const FLandscapePerLODMaterialOverride& InOverride) { return (InOverride.LODIndex == InLODIndex) && (InOverride.Material != nullptr); }))
			{
				return LocalMaterialOverride->Material;
			}
		}
	}

	if (LandscapeMaterial != nullptr)
	{
		return LandscapeMaterial;
	}

	// Note : we don't fallback to the parent ALandscape anymore, since this is only reliable in the editor. The LandscapeMaterial property is now a LandscapeOverridable one,
	//  which means that overriding the material is now done explicitly via OverriddenSharedProperties rather than by the LandscapeMaterial property value being not null. It means it's now a
	//  "valid" thing to override the material property and set it to "None" (probably a user mistake, though, hence the MapCheck warning if data is setup this way). 
	//  In order to make the error plain visible, we therefore return an invalid material now, so that the editor doesn't show the parent ALandscape's material, while cooking
	//  potentially returns an invalid material depending on whether ALandscape is loaded. 

	return UMaterial::GetDefaultMaterial(MD_Surface);
}

UMaterialInterface* ALandscapeStreamingProxy::GetLandscapeHoleMaterial() const
{
	if (LandscapeHoleMaterial)
	{
		return LandscapeHoleMaterial;
	}

	// Note : we don't fallback to the parent ALandscape anymore, since this is only reliable in the editor. The LandscapeHoleMaterial property is now a LandscapeOverridable one,
	//  which means that overriding the material is now done explicitly via OverriddenSharedProperties rather than by the LandscapeHoleMaterial property value being not null. It means it's now a
	//  "valid" thing to override the material property and set it to "None" (probably a user mistake, though, hence the MapCheck warning if data is setup this way). 
	//  In order to make the error plain visible, we therefore return a null material now, so that the editor doesn't show the parent ALandscape's hole material, while cooking
	//  potentially returns an invalid material depending on whether ALandscape is loaded. 

	return nullptr;
}

void ALandscapeProxy::RetrieveAllLandscapeMaterials(TSet<UMaterialInterface*>& OutMaterials) const
{
	// Retrieve the materials specified in the proxy or its parent landscape (for a streaming proxy)
	if (UMaterialInterface* Material = GetLandscapeMaterial();
		Material && (Material != UMaterial::GetDefaultMaterial(MD_Surface)))
	{
		OutMaterials.Add(Material);
	}

	if (UMaterialInterface* Material = GetLandscapeHoleMaterial();
		Material && (Material != UMaterial::GetDefaultMaterial(MD_Surface)))
	{
		OutMaterials.Add(Material);
	}

	// Also retrieve the materials from the components, as they may override the proxy's materials : 
	for (ULandscapeComponent* Component : LandscapeComponents)
	{
		if (Component != nullptr)
		{
			Component->RetrieveAllLandscapeMaterials(OutMaterials);
		}
	}
}

#if WITH_EDITOR

bool ALandscapeStreamingProxy::IsSharedPropertyOverridden(const FName& InPropertyName) const
{
	return OverriddenSharedProperties.Contains(InPropertyName);
}

void ALandscapeStreamingProxy::SetSharedPropertyOverride(const FName& InPropertyName, const bool bIsOverridden, const bool bInExecutePostEditChangeProperty)
{
	check(IsSharedProperty(InPropertyName));

	Modify();

	if (bIsOverridden)
	{
		OverriddenSharedProperties.Add(InPropertyName);
	}
	else
	{
		TWeakObjectPtr<ALandscapeProxy> LandscapeProxy = this;
		TWeakObjectPtr<ALandscape> ParentLandscape = GetLandscapeActor();
		
		if (!ParentLandscape.IsValid())
		{
			UE_LOGF(LogLandscape, Warning, "Unable to retrieve the parent landscape's shared property value (ALandscapeStreamingProxy: %ls, Property: %ls). The proper value will be fixedup when reloading this proxy.",
				   *GetFullName(), *InPropertyName.ToString());
		}
		else
		{
			UE::Landscape::Private::CopyPropertyByName(LandscapeProxy, ParentLandscape, InPropertyName, bInExecutePostEditChangeProperty);
		}

		OverriddenSharedProperties.Remove(InPropertyName);
	}
}

void ALandscapeStreamingProxy::FixupOverriddenSharedProperties()
{
	const UClass* StreamingProxyClass = StaticClass();

	for (const FName& PropertyName : OverriddenSharedProperties)
	{
		const FProperty* Property = StreamingProxyClass->FindPropertyByName(PropertyName);
		checkf(Property != nullptr, TEXT("An overridden property is referenced but cannot be found. Please check this property hasn't been renamed or deprecated and/or provide the proper adapting mechanism."));
	}
}

void ALandscapeStreamingProxy::RequestDeferredLayerContentUpdateModes(ELandscapeLayerUpdateMode InLayerContentUpdateModes)
{
	DeferredLayerContentUpdateModes |= InLayerContentUpdateModes;
}

void ALandscapeStreamingProxy::PerformDeferredLayerContentUpdateModesRequest()
{
	if (DeferredLayerContentUpdateModes != 0)
	{
		check(LandscapeActorRef != nullptr);
		LandscapeActorRef->RequestLayersContentUpdate((ELandscapeLayerUpdateMode)DeferredLayerContentUpdateModes);

		// Reset deferred update modes
		DeferredLayerContentUpdateModes = 0;
	}
}

void ALandscapeProxy::UpgradeSharedProperties(ALandscape* InParentLandscape, const bool bInExecutePostEditChangeProperty)
{
	TArray<FName> SynchronizedProperties;
	bool bOpenMapCheckWindow = false;
	ULandscapeInfo* LandscapeInfo = GetLandscapeInfo();
	checkf(LandscapeInfo != nullptr, TEXT("UpgradeSharedProperties can only be called after the proxies are registered to ULandscapeInfo"));

	for (TFieldIterator<FProperty> PropertyIterator(GetClass()); PropertyIterator; ++PropertyIterator)
	{
		FProperty* Property = *PropertyIterator;

		if (Property == nullptr)
		{
			continue;
		}

		if (IsPropertyInherited(Property) && !Property->Identical_InContainer(this, InParentLandscape))
		{
			SynchronizedProperties.Emplace(Property->GetFName());
			UE::Landscape::Private::CopyProperty(Property, InParentLandscape, this, bInExecutePostEditChangeProperty);
		}
		else if (IsPropertyOverridable(Property) && !IsSharedPropertyOverridden(Property->GetFName()) && !Property->Identical_InContainer(this, InParentLandscape))
		{
			if (CVarSilenceSharedPropertyDeprecationFixup->GetBool())
			{
				SetSharedPropertyOverride(Property->GetFName(), true, bInExecutePostEditChangeProperty);
			}
			else
			{
				FFormatNamedArguments Arguments;
				TWeakObjectPtr<ALandscapeProxy> LandscapeProxy = this;
				TWeakObjectPtr<ALandscape> ParentLandscape = InParentLandscape;
				const FName PropertyName = Property->GetFName();

				bOpenMapCheckWindow = true;

				Arguments.Add(TEXT("Proxy"), FText::FromString(GetActorNameOrLabel()));
				Arguments.Add(TEXT("Landscape"), FText::FromString(InParentLandscape->GetActorNameOrLabel()));
				FMessageLog("MapCheck").Warning()
					->AddToken(FUObjectToken::Create(this, FText::FromString(GetActorNameOrLabel())))
					->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_LandscapeProxy_UpgradeSharedProperties", "Contains a property ({0}) different from parent's landscape actor. Please select between "), FText::FromString(PropertyName.ToString()))))
					->AddToken(FActionToken::Create(LOCTEXT("MapCheck_OverrideProperty", "Override property"), LOCTEXT("MapCheck_OverrideProperty_Desc", "Keeping the current value and marking the property as overriding the parent landscape's value."),
						FOnActionTokenExecuted::CreateLambda([LandscapeProxy, PropertyName, bInExecutePostEditChangeProperty]()
							{
								if (LandscapeProxy.IsValid())
								{
									LandscapeProxy->SetSharedPropertyOverride(PropertyName, true, bInExecutePostEditChangeProperty);
								}
							}),
						/*bInSingleUse = */true))
					->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_LandscapeProxy_UpgradeSharedProperties_Or", " or ")))
					->AddToken(FActionToken::Create(LOCTEXT("MapCheck_InheritProperty", "Inherit from parent landscape"), LOCTEXT("MapCheck_InheritProperty_Desc", "Copying the parent landscape's value for this property."),
							FOnActionTokenExecuted::CreateLambda([LandscapeProxy, ParentLandscape, PropertyName, bInExecutePostEditChangeProperty]()
								{
									UE::Landscape::Private::CopyPropertyByName(LandscapeProxy, ParentLandscape, PropertyName, bInExecutePostEditChangeProperty);
								}),
							/*bInSingleUse = */true))
					->AddToken(FMapErrorToken::Create(FMapErrors::LandscapeComponentPostLoad_Warning));
			}
		}
	}

	if (!SynchronizedProperties.IsEmpty())
	{
		// This function may be called from PostLoad, in which case InParentLandscape will be non-null. Pass it along to LandscapeInfo so that if the landscape actor has not registered to the 
		//  landscape info yet, it can still retrieve it via this direct pointer : 
		LandscapeInfo->MarkObjectDirty(/*InObject = */this, /*bInForceResave = */true, InParentLandscape);

		if (!CVarSilenceSharedPropertyDeprecationFixup->GetBool())
		{
			UE::Landscape::Private::DisplaySynchronizedPropertiesMapcheckWarning(SynchronizedProperties, /*InSynchronizedProxy = */*this, *InParentLandscape, /*bAddSilencingMessage = */true);
		}
	}

	if (bOpenMapCheckWindow)
	{
		// Show MapCheck window
		FMessageLog("MapCheck").Open(EMessageSeverity::Warning);
	}
}

void ALandscapeProxy::DisplayObsoleteEditLayerMapCheck(ALandscape* InParentLandscape)
{
	ULandscapeInfo* LandscapeInfo = GetLandscapeInfo();
	checkf(LandscapeInfo != nullptr, TEXT("DisplayObsoleteEditLayerMapCheck can only be called after the proxies are registered to ULandscapeInfo"));

	TMap<FGuid, FName> ObsoleteEditLayers = GetObsoleteEditLayersInComponents();

	if (!ObsoleteEditLayers.IsEmpty())
	{
		TWeakObjectPtr<ALandscapeProxy> LandscapeProxy = this;
		TWeakObjectPtr<ALandscape> ParentLandscape = InParentLandscape;

		auto CanUpdateObsoleteLayersLamda = [LandscapeProxy]()
		{
			if (LandscapeProxy.IsValid() && !LandscapeProxy->GetObsoleteEditLayersInComponents().IsEmpty())
			{
				ULandscapeInfo* LandscapeInfo = LandscapeProxy->GetLandscapeInfo();
				return LandscapeInfo && LandscapeInfo->IsLandscapeEditableWorld();
			}

			return false;
		};

		// TODO [jared.ritchie] in the future we can show an entry for each obsolete edit layer (instead of only allowing copy of final data) 
		// TODO [jared.ritchie] in the future, add options for Copy to Layer X (instead of default layer only)
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("Proxy"), FText::FromString(LandscapeProxy->GetActorNameOrLabel()));
		Arguments.Add(TEXT("Landscape"), FText::FromString(ParentLandscape->GetActorNameOrLabel()));
		TSharedRef<FTokenizedMessage> Message = FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(LandscapeProxy.Get(), FText::FromString(LandscapeProxy->GetActorNameOrLabel())))
			->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_LandscapeProxy_CheckForObsoleteEditLayers", "has data on edit layers that do not exist on it's parent landscape actor. Saving will delete this data for good unless copied to a valid edit layer.")))
			->AddToken(FActionToken::Create(LOCTEXT("MapCheck_CheckForObsoleteEditLayers_Copy_Default", "Copy Data to Default Edit Layer"), LOCTEXT("MapCheck_CheckForObsoleteEditLayers_Copy_Default_Desc", "Copy the final merged data from all obsolete layers to the default edit layer. Marks proxy dirty"),
				FOnActionTokenExecuted::CreateLambda([LandscapeProxy, ParentLandscape]()
				{
					if (LandscapeProxy.IsValid() && ParentLandscape.IsValid())
					{
						FScopedTransaction Transaction(LOCTEXT("Landscape_CopyObsoleteLayerToDefault", "Copy Obsolete Edit Layer Data to Default Layer"));
						// First Copy the final merged data from all obsolete layers to the default edit layer
						ParentLandscape->CopyDataToEditLayer(LandscapeProxy.Get(), FGuid(), FGuid(), /*bInUseObsoleteLayerData =*/ true); 
						LandscapeProxy->RemoveObsoleteEditLayers(/*bShowMapCheckWarning =*/ false);
					}
				}),
				FCanExecuteActionToken::CreateLambda(CanUpdateObsoleteLayersLamda)))
			->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_LandscapeProxy_CheckForObsoleteEditLayers_Or", " or ")))
			->AddToken(FActionToken::Create(LOCTEXT("MapCheck_CheckForObsoleteEditLayers_Delete", "Delete Obsolete Edit Layer Data"), LOCTEXT("MapCheck_CheckForObsoleteEditLayers_Delete_Desc", "Deletes all of this proxy's obsolete edit layer data for good. Marks proxy dirty"),
				FOnActionTokenExecuted::CreateLambda([LandscapeProxy]()
				{
					if (LandscapeProxy.IsValid())
					{
						FScopedTransaction Transaction(LOCTEXT("Landscape_CopyObsoleteDeleteLayers", "Delete Obsolete Layer Data"));
						LandscapeProxy->RemoveObsoleteEditLayers(/*bShowMapCheckWarning =*/ true);
					}
				}),
				FCanExecuteActionToken::CreateLambda(CanUpdateObsoleteLayersLamda)));

		Message->AddToken(FMapErrorToken::Create(FMapErrors::LandscapeComponentPostLoad_Warning));

		// Show MapCheck window
		FMessageLog("MapCheck").Open(EMessageSeverity::Warning);
	}
}

void ALandscapeProxy::FixupSharedData(ALandscape* Landscape, const bool bMapCheck, const bool bInExecutePostEditChangeProperty)
{
	if (Landscape == nullptr)
	{
		UE_LOGF(LogLandscape, Error, "FixupSharedData: Invalid parent ALandscape. Proxy failed to synchronize parent properties.");
		return;
	}

	if (Landscape == this)
	{
		return;
	}
	
	ULandscapeInfo* LandscapeInfo = GetLandscapeInfo();
	checkf(LandscapeInfo != nullptr, TEXT("FixupSharedData can only be called after the proxies are registered to ULandscapeInfo"));

	bool bUpdated = false;
	TArray<FName> SynchronizedProperties;

	const bool bUpgradeSharedPropertiesPerformedBefore = bUpgradeSharedPropertiesPerformed;
	if (!bUpgradeSharedPropertiesPerformed && 
		((GetLinkerCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID) < FFortniteReleaseBranchCustomObjectVersion::LandscapeSharedPropertiesEnforcement)
		|| (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::LandscapeBodyInstanceAsSharedProperty)))
	{
		UpgradeSharedProperties(Landscape, bInExecutePostEditChangeProperty);
		bUpgradeSharedPropertiesPerformed = true;
	}
	else
	{
		SynchronizedProperties = SynchronizeSharedProperties(Landscape, bInExecutePostEditChangeProperty);
		bUpdated |= !SynchronizedProperties.IsEmpty();
	}

	for (const ULandscapeEditLayerBase* EditLayer : Landscape->GetEditLayersConst())
	{
		bUpdated |= AddLayer(EditLayer->GetGuid());
	}

	const bool bIsLandscapeInfoReady = LandscapeInfo->LandscapeActor != nullptr;

	// Check for invalid target layers (batched merge checks all target layer names). Global merge did not validate this condition, so legacy maps can be serialized with bad data
	// A bug in Create Layers From Materials allowed invalid layers to be generated unintentionally. Maps saved with invalid target layers crash on first merge
	if (const FLandscapeTargetLayerSettings* InvalidTargetLayerSettings = TargetLayers.Find(FName()))
	{
		UE_LOGF(LogLandscape, Warning, "FixupSharedData: Removed invalid target layer. Batched merge requires all target layers to have a valid FName. Actor: '%ls' Package: '%ls'", *GetActorNameOrLabel(), *GetPackage()->GetName());
		if (bIsLandscapeInfoReady)
		{
			LandscapeInfo->DeleteLayer(InvalidTargetLayerSettings->LayerInfoObj, FName());
		}
		else
		{
			// Let each proxy handle target layer removal when the landscape info is not ready
			RemoveTargetLayer(FName(), /*bPostEditChange = */false);
		}
		bUpdated |= true;
		SynchronizedProperties.Add(GET_MEMBER_NAME_CHECKED(ALandscapeProxy, TargetLayers));
	}

	if (bUpdated)
	{
		// In cases where LandscapeInfo is not fully ready yet, we forward the provided ALandscape. If ALandscape is available in LandscapeInfo, we let the object function naturally.
		const ALandscape* LandscapeActorOverride = bIsLandscapeInfoReady ? nullptr : Landscape;

		// Force resave the proxy through the modified landscape system, so that the user can then use the Build > Save Modified Landscapes (or Build > Build Landscape) button and therefore manually trigger the re-save of all modified proxies. * /
		bool bNeedsManualResave = LandscapeInfo->MarkObjectDirty(/*InObject = */this, /*bInForceResave = */true, LandscapeActorOverride);

		if (bMapCheck && bNeedsManualResave && !SynchronizedProperties.IsEmpty())
		{
			UE::Landscape::Private::DisplaySynchronizedPropertiesMapcheckWarning(SynchronizedProperties, /*InSynchronizedProxy = */*this, *Landscape);
		}
	}

	// Validate that property overrides are properly setup
	CheckOverriddenSharedProperties();

	OnLandscapeProxyFixupSharedDataDelegate.Broadcast(/*Proxy = */this, FOnLandscapeProxyFixupSharedDataParams { .Landscape = Landscape, .bUpgradeSharedPropertiesPerformed = bUpgradeSharedPropertiesPerformedBefore });
}

// Deprecated
void ALandscapeProxy::SetAbsoluteSectionBase(FIntPoint InSectionBase)
{
}

void ALandscapeProxy::RecreateComponentsState()
{
	RecreateComponentsRenderState([](ULandscapeComponent* Comp)
	{
		Comp->UpdateComponentToWorld();
		Comp->UpdateCachedBounds();
		Comp->UpdateBounds();
	});

	for (int32 ComponentIndex = 0; ComponentIndex < CollisionComponents.Num(); ComponentIndex++)
	{
		ULandscapeHeightfieldCollisionComponent* Comp = CollisionComponents[ComponentIndex];
		if (Comp)
		{
			Comp->UpdateComponentToWorld();
			Comp->RecreatePhysicsState();
		}
	}
}

void ALandscapeProxy::RecreateComponentsRenderState(TFunctionRef<void(ULandscapeComponent*)> Fn)
{
	// Batch component render state recreation
	TArray<FComponentRecreateRenderStateContext> ComponentRecreateRenderStates;
	ComponentRecreateRenderStates.Reserve(LandscapeComponents.Num());

	for (int32 ComponentIndex = 0; ComponentIndex < LandscapeComponents.Num(); ComponentIndex++)
	{
		ULandscapeComponent* Comp = LandscapeComponents[ComponentIndex];
		if (Comp)
		{
			Fn(Comp);
			ComponentRecreateRenderStates.Emplace(Comp);
		}
	}
}

bool ULandscapeInfo::GetDirtyOnlyInMode() const
{
	if (const ALandscape* Landscape = LandscapeActor.Get())
	{
		if (const UWorld* World = Landscape->GetWorld())
		{
			if (const ULandscapeSubsystem* LandscapeSubsystem = World->GetSubsystem<ULandscapeSubsystem>())
			{
				return LandscapeSubsystem->GetDirtyOnlyInMode();
			}
		}
		return false;
	}

	return false;
}

FLandscapeDirtyOnlyInModeScope::FLandscapeDirtyOnlyInModeScope(ULandscapeInfo* InLandscapeInfo)
	: LandscapeInfo(InLandscapeInfo)
	, bDirtyOnlyInModePrevious(InLandscapeInfo->bDirtyOnlyInMode)
{
	LandscapeInfo->bDirtyOnlyInMode = LandscapeInfo->GetDirtyOnlyInMode();
}


FLandscapeDirtyOnlyInModeScope::FLandscapeDirtyOnlyInModeScope(ULandscapeInfo* InLandscapeInfo, bool bInOverrideDirtyMode)
	: LandscapeInfo(InLandscapeInfo)
	, bDirtyOnlyInModePrevious(InLandscapeInfo->bDirtyOnlyInMode)
{
	LandscapeInfo->bDirtyOnlyInMode = bInOverrideDirtyMode;
}

FLandscapeDirtyOnlyInModeScope::~FLandscapeDirtyOnlyInModeScope()
{
	LandscapeInfo->bDirtyOnlyInMode = bDirtyOnlyInModePrevious;
}

void ULandscapeInfo::OnModifiedPackageSaved(UPackage* InPackage)
{
	ModifiedPackages.Remove(InPackage);
}

TArray<UPackage*> ULandscapeInfo::GetModifiedPackages() const
{
	TArray<UPackage*> LocalModifiedPackages;
	LocalModifiedPackages.Reserve(ModifiedPackages.Num());
	Algo::TransformIf(ModifiedPackages, LocalModifiedPackages, 
		[](const TWeakObjectPtr<UPackage>& InWeakPackagePtr) { return InWeakPackagePtr.IsValid(); }, 
		[](const TWeakObjectPtr<UPackage>& InWeakPackagePtr) { return InWeakPackagePtr.Get(); });
	return LocalModifiedPackages;
}

bool ULandscapeInfo::IsPackageModified(UPackage* InPackage) const
{
	return ModifiedPackages.Contains(InPackage);
}

int32 ULandscapeInfo::MarkModifiedPackagesAsDirty()
{
	int32 NumDirtied = 0;
	// Move into a local set to avoid OnMarkPackageDirty triggering from this loop and changing it during iteration.
	TSet<TWeakObjectPtr<UPackage>, TWeakObjectPtrSetKeyFuncs<TWeakObjectPtr<UPackage>>> LocalModifiedPackages(MoveTemp(ModifiedPackages));
	ModifiedPackages.Empty();
	for (TWeakObjectPtr<UPackage> WeakPackagePtr : LocalModifiedPackages)
	{
		if (UPackage* Package = WeakPackagePtr.Get())
		{
			const bool bWasDirty = Package->IsDirty();
			const bool bIsDirty = Package->MarkPackageDirty();
			NumDirtied += (!bWasDirty && bIsDirty) ? 1 : 0;
		}
	}

	ProcessDeferredDeletions();

	return NumDirtied;
}

void ULandscapeInfo::OnMarkPackageDirty(UPackage* InPackage, bool bIsDirty)
{
	// Looking for the landscape actor's package to be marked dirty.
	ALandscape* Landscape = LandscapeActor.Get();
	if (!Landscape || Landscape->GetPackage() != InPackage)
	{
		return;
	}

	if (IsPackageModified(InPackage))
	{
		// LandscapeActor is on the soft-dirty list and the package is becoming actual-dirty. Also process the rest of the
		// ModifiedPackages list.  Try to avoid saving the landscape out-of-sync with dependent objects (patches can have
		// a problem with this).  At least provide the proper prompt to the user with all relevant actors on it.

		ModifiedPackages.Remove(InPackage);  // Remove the main landscape package first to avoid infinitely looping.
		MarkModifiedPackagesAsDirty();
	}
}

int32 ULandscapeInfo::GetModifiedPackageCount() const
{
	return IntCastChecked<int32>(Algo::CountIf(ModifiedPackages, [](const TWeakObjectPtr<UPackage>& InWeakPackagePtr) { return InWeakPackagePtr.IsValid(); }));
}

bool ULandscapeInfo::TryAddToModifiedPackages(UPackage* InPackage, const ALandscape* InLandscapeOverride)
{
	const ALandscape* LocalLandscapeActor = (InLandscapeOverride != nullptr) ? InLandscapeOverride : LandscapeActor.Get();
	check(LocalLandscapeActor != nullptr);

	UWorld* World = LocalLandscapeActor->GetWorld();
	check(World != nullptr);

	// We don't want to bother with packages being marked dirty for anything else than the Editor world 
	if (World->WorldType != EWorldType::Editor)
	{
		return false;
	}

	// Also don't track packages when rolling back a transaction because they are already dirty anyway 
	if (GIsTransacting)
	{
		return false;
	}

	// No need to add the package to ModifiedPackages if it's already dirty.
	if (InPackage->IsDirty())
	{
		return false;
	}

	// Don't consider unsaved packages as modified/not dirty because they will be saved later on anyway. What we're really after are existing packages made dirty on load
	if (FPackageName::IsTempPackage(InPackage->GetName()))
	{
		return false;
	}

	ModifiedPackages.Add(InPackage);
	return true;
}

bool ULandscapeInfo::MarkObjectDirty(UObject* InObject, bool bInForceResave, const ALandscape* InLandscapeOverride)
{
	check(InObject);
		
	bool bWasAddedToModifiedPackages = false;
	if (bInForceResave)
	{
		if (!InObject->MarkPackageDirty())
		{
			// When force-resaving (e.g. when syncing must-sync properties on load), unconditionally add the package to the list of packages to save if we couldn't mark it dirty already, so that 
			//  the user can manually resave all that needs to be saved with the Build > Save Modified Landscapes (or Build > Build Landscape) button :
			bWasAddedToModifiedPackages = TryAddToModifiedPackages(InObject->GetPackage(), InLandscapeOverride);
		}
	}
	else if (bDirtyOnlyInMode)
	{
		const ALandscape* LocalLandscapeActor = (InLandscapeOverride != nullptr) ? InLandscapeOverride : LandscapeActor.Get();
		check(LocalLandscapeActor);
		if (LocalLandscapeActor->HasLandscapeEdMode())
		{
			InObject->MarkPackageDirty();
		}
		else
		{
			bWasAddedToModifiedPackages = TryAddToModifiedPackages(InObject->GetPackage(), InLandscapeOverride);
		}
	}
	else
	{
		InObject->MarkPackageDirty();
	}

	return bWasAddedToModifiedPackages;
}

bool ULandscapeInfo::ModifyObject(UObject* InObject, bool bAlwaysMarkDirty)
{
	check(InObject && (InObject->IsA<ALandscapeProxy>() || InObject->GetTypedOuter<ALandscapeProxy>() != nullptr));
	bool bWasAddedToModifiedPackages = false;
	
	if (!bAlwaysMarkDirty)
	{
		InObject->Modify(false);
	}
	else if(!bDirtyOnlyInMode)
	{
		InObject->Modify(true);
	}
	else
	{
		ALandscape* LocalLandscapeActor = LandscapeActor.Get();
		check(LocalLandscapeActor);
		if (LocalLandscapeActor->HasLandscapeEdMode())
		{
			InObject->Modify(true);
			// We just marked the package dirty, no need to keep track of it with ModifiedPackages.
			ModifiedPackages.Remove(InObject->GetPackage());
		}
		else 
		{
			InObject->Modify(false);
			bWasAddedToModifiedPackages = TryAddToModifiedPackages(InObject->GetPackage());
		}
	}

	return bWasAddedToModifiedPackages;
}

void ULandscapeInfo::DeleteActorWhenApplyingModifiedStatus(AActor* InActor, bool bInAllowUI)
{
	UWorld* World = InActor->GetWorld();
	check((World != nullptr) && (World->WorldType == EWorldType::Editor));

	// If we can mark the package dirty, then we can also delete it right away.  If we can't, then enqueue it for deletion at the same time as other deferred package dirtying.
	if (InActor->MarkPackageDirty())
	{
		UE::Landscape::DeleteActors({ InActor }, World, bInAllowUI);
	}
	else
	{
		ActorsToDelete.Add(InActor);
	}
}

void ULandscapeInfo::ProcessDeferredDeletions()
{
	TArray<AActor*> FinalActorsToDelete;
	FinalActorsToDelete.Reserve(ActorsToDelete.Num());
	for (TWeakObjectPtr<AActor> WeakActorPtr : ActorsToDelete)
	{
		if (AActor* Actor = WeakActorPtr.Get())
		{
			FinalActorsToDelete.Add(Actor);
		}
	}
	if (!FinalActorsToDelete.IsEmpty())
	{
		ensure(UE::Landscape::DeleteActors(FinalActorsToDelete, FinalActorsToDelete[0]->GetWorld(), /* bInAllowUI = */true));
	}
	ActorsToDelete.Empty();
}

void ULandscapeInfo::DirtyRuntimeVirtualTextureForLandscapeArea(int32 X1, int32 Y1, int32 X2, int32 Y2) const
{
	FBox DirtyWorldBounds(ForceInit);

	// Iterate touched components to find touched runtime virtual textures.
	TSet<ULandscapeComponent*> Components;
	GetComponentsInRegion(X1 + 1, Y1 + 1, X2 - 1, Y2 - 1, Components);

	TArray<URuntimeVirtualTexture*> RuntimeVirtualTextures;
	for (ULandscapeComponent* Component : Components)
	{
		ALandscapeProxy* Landscape = Component->GetLandscapeProxy();
		if (Landscape != nullptr && Landscape->RuntimeVirtualTextures.Num() > 0)
		{
			for (URuntimeVirtualTexture* RuntimeVirtualTexture : Landscape->RuntimeVirtualTextures)
			{
				RuntimeVirtualTextures.AddUnique(RuntimeVirtualTexture);
			}

			// Also accumulate bounds in world space.
			const int32 LocalX1 = FMath::Max(X1, Component->GetSectionBase().X) - Component->GetSectionBase().X;
			const int32 LocalY1 = FMath::Max(Y1, Component->GetSectionBase().Y) - Component->GetSectionBase().Y;
			const int32 LocalX2 = FMath::Min(X2, Component->GetSectionBase().X + ComponentSizeQuads) - Component->GetSectionBase().X;
			const int32 LocalY2 = FMath::Min(Y2, Component->GetSectionBase().Y + ComponentSizeQuads) - Component->GetSectionBase().Y;
			const FBox LocalDirtyBounds(FVector(LocalX1, LocalY1, 0), FVector(LocalX2, LocalY2, 1));

			DirtyWorldBounds += LocalDirtyBounds.TransformBy(Component->GetComponentToWorld());
		}
	}

	// Find matching runtime virtual texture components and invalidate dirty region.
	if (RuntimeVirtualTextures.Num() > 0)
	{
		for (TObjectIterator<URuntimeVirtualTextureComponent> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
		{
			if (It->GetVirtualTexture() != nullptr && RuntimeVirtualTextures.Contains(It->GetVirtualTexture()))
			{
				It->Invalidate(FBoxSphereBounds(DirtyWorldBounds), GLandscapePrioritizeDirtyRVTPages ? EVTInvalidatePriority::High : EVTInvalidatePriority::Normal);
			}
		}
	}
}

ALandscapeProxy* ULandscapeInfo::GetLandscapeProxyForLevel(ULevel* Level) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeInfo::GetLandscapeProxyForLevel);
	ALandscapeProxy* LandscapeProxy = nullptr;
	ForEachLandscapeProxy([&LandscapeProxy, Level](ALandscapeProxy* Proxy) -> bool 
	{
		if (Proxy->GetLevel() == Level)
		{
			LandscapeProxy = Proxy;
			return false;
		}
		return true;
	});
	return LandscapeProxy;
}
      
#endif // WITH_EDITOR

ALandscapeProxy* ULandscapeInfo::GetCurrentLevelLandscapeProxy(bool bRegistered) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeInfo::GetCurrentLevelLandscapeProxy);

	ALandscapeProxy* LandscapeProxy = nullptr;
	ForEachLandscapeProxy([&LandscapeProxy, bRegistered](ALandscapeProxy* Proxy) -> bool
	{
		if (!bRegistered || (Proxy->GetRootComponent() && Proxy->GetRootComponent()->IsRegistered()))
		{
			UWorld* ProxyWorld = Proxy->GetWorld();
			if (ProxyWorld &&
				ProxyWorld->GetCurrentLevel() == Proxy->GetOuter())
			{
				LandscapeProxy = Proxy;
				return false;
			}
		}
		return true;
	});
	return LandscapeProxy;
}

ALandscapeProxy* ULandscapeInfo::GetLandscapeProxy() const
{
	// Mostly this Proxy used to calculate transformations
	// in Editor all proxies of same landscape actor have root components in same locations
	// so it doesn't really matter which proxy we return here

	// prefer LandscapeActor in case it is loaded
	if (LandscapeActor.IsValid())
	{
		ALandscape* Landscape = LandscapeActor.Get();
		USceneComponent* LandscapeRootComponent = (Landscape != nullptr) ? Landscape->GetRootComponent() : nullptr;

		if ((LandscapeRootComponent != nullptr) && (LandscapeRootComponent->IsRegistered()))
		{
			return Landscape;
		}
	}

	// prefer current level proxy 
	if (ALandscapeProxy* Proxy = GetCurrentLevelLandscapeProxy(true))
	{
		return Proxy;
	}

	// any proxy in the world
	for (TWeakObjectPtr<ALandscapeStreamingProxy> ProxyPtr : SortedStreamingProxies)
	{
		ALandscapeStreamingProxy* Proxy = ProxyPtr.Get();
		USceneComponent* ProxyRootComponent = (Proxy != nullptr) ? Proxy->GetRootComponent() : nullptr;

		if ((ProxyRootComponent != nullptr) && (ProxyRootComponent->IsRegistered()))
		{
			return Proxy;
		}
	}

	return nullptr;
}

#if WITH_EDITOR

void ULandscapeInfo::Reset()
{
	LandscapeActor.Reset();

	SortedStreamingProxies.Empty();
	XYtoComponentMap.Empty();
	XYtoAddCollisionMap.Empty();

	//SelectedComponents.Empty();
	//SelectedRegionComponents.Empty();
	//SelectedRegion.Empty();
}

void ULandscapeInfo::FixupProxiesTransform(bool bDirty)
{
	ALandscape* Landscape = LandscapeActor.Get();

	if ((Landscape == nullptr)
		|| (Landscape->GetRootComponent() == nullptr)
		|| !Landscape->GetRootComponent()->IsRegistered())
	{
		return;
	}

	// Make sure section offset of all proxies is multiple of ALandscapeProxy::ComponentSizeQuads
	for (TWeakObjectPtr<ALandscapeStreamingProxy> ProxyPtr : SortedStreamingProxies)
	{
		ALandscapeProxy* Proxy = ProxyPtr.Get();
		if (!Proxy)
		{
			continue;
		}

		if (bDirty)
		{
			Proxy->Modify();
		}

		FIntPoint LandscapeSectionOffset = Proxy->GetSectionBase() - Landscape->GetSectionBase();
		FIntPoint LandscapeSectionOffsetRem(
			LandscapeSectionOffset.X % Proxy->ComponentSizeQuads,
			LandscapeSectionOffset.Y % Proxy->ComponentSizeQuads);

		if (LandscapeSectionOffsetRem.X != 0 || LandscapeSectionOffsetRem.Y != 0)
		{
			FIntPoint NewLandscapeSectionOffset = Proxy->GetSectionBase() - LandscapeSectionOffsetRem;

			UE_LOGF(LogLandscape, Warning, "Landscape section base is not multiple of component size, attempted automated fix: '%ls', %d,%d vs %d,%d.",
				*Proxy->GetFullName(), Proxy->GetSectionBase().X, Proxy->GetSectionBase().Y, NewLandscapeSectionOffset.X, NewLandscapeSectionOffset.Y);

			Proxy->SetSectionBase(NewLandscapeSectionOffset);
		}
	}

	FTransform LandscapeTM = Landscape->LandscapeActorToWorld();
	// Update transformations of all linked landscape proxies
	for (TWeakObjectPtr<ALandscapeStreamingProxy> ProxyPtr : SortedStreamingProxies)
	{
		ALandscapeProxy* Proxy = ProxyPtr.Get();
		if (!Proxy)
		{
			continue;
		}

		FTransform ProxyRelativeTM(FVector(Proxy->GetSectionBase()));
		FTransform ProxyTransform = ProxyRelativeTM * LandscapeTM;

		if (!Proxy->GetTransform().Equals(ProxyTransform))
		{
			Proxy->SetActorTransform(ProxyTransform);

			// Let other systems know that an actor was moved
			GEngine->BroadcastOnActorMoved(Proxy);
		}
	}
}

void ULandscapeInfo::UpdateComponentLayerAllowList()
{
	ForEachLandscapeProxy([](ALandscapeProxy* Proxy)
	{
		for (ULandscapeComponent* Comp : Proxy->LandscapeComponents)
		{
			Comp->UpdateLayerAllowListFromPaintedLayers();
		}
		return true;
	});
}

void ULandscapeInfo::RecreateLandscapeInfo(UWorld* InWorld, bool bMapCheck, bool bKeepRegistrationStatus)
{
	check(InWorld);

	ULandscapeInfoMap& LandscapeInfoMap = ULandscapeInfoMap::GetLandscapeInfoMap(InWorld);
	LandscapeInfoMap.Modify();

	// reset all LandscapeInfo objects
	for (auto& LandscapeInfoPair : LandscapeInfoMap.Map)
	{
		ULandscapeInfo* LandscapeInfo = LandscapeInfoPair.Value;

		if (LandscapeInfo != nullptr)
		{
			LandscapeInfo->Modify();

			// this effectively unregisters all proxies, but does not flag them as unregistered
			// so we can use the flags below to determine what was previously registered
			LandscapeInfo->Reset();
		}
	}

	TMap<FGuid, TArray<ALandscapeProxy*>> ValidLandscapesMap;
	// Gather all valid landscapes in the world
	for (ALandscapeProxy* Proxy : TActorRange<ALandscapeProxy>(InWorld))
	{
		if (Proxy->GetLevel() &&
			Proxy->GetLevel()->bIsVisible &&
			!Proxy->HasAnyFlags(RF_BeginDestroyed) &&
			IsValid(Proxy) &&
			(!bKeepRegistrationStatus || Proxy->bIsRegisteredWithLandscapeInfo) &&
			!Proxy->IsPendingKillPending())
		{
			ValidLandscapesMap.FindOrAdd(Proxy->GetLandscapeGuid()).Add(Proxy);
		}
	}

	// Register landscapes in global landscape map
	for (auto& ValidLandscapesPair : ValidLandscapesMap)
	{
		auto& LandscapeList = ValidLandscapesPair.Value;
		for (ALandscapeProxy* Proxy : LandscapeList)
		{
			// note this may re-register already registered actors
			Proxy->CreateLandscapeInfo()->RegisterActor(Proxy, bMapCheck);
		}
	}

	// Remove empty entries from global LandscapeInfo map
	for (auto It = LandscapeInfoMap.Map.CreateIterator(); It; ++It)
	{
		ULandscapeInfo* Info = It.Value();

		if (Info != nullptr && Info->GetLandscapeProxy() == nullptr)
		{
			Info->MarkAsGarbage();
			It.RemoveCurrent();
		}
		else if (Info == nullptr) // remove invalid entry
		{
			It.RemoveCurrent();
		}
	}

	// We need to inform Landscape editor tools about LandscapeInfo updates
	FEditorSupportDelegates::WorldChange.Broadcast();
}


#endif // WITH_EDITOR

ULandscapeInfo* ULandscapeInfo::Find(UWorld* InWorld, const FGuid& LandscapeGuid)
{
	ULandscapeInfo* LandscapeInfo = nullptr;
	if (InWorld != nullptr && LandscapeGuid.IsValid())
	{
		ULandscapeInfoMap& LandscapeInfoMap = ULandscapeInfoMap::GetLandscapeInfoMap(InWorld);
		LandscapeInfo = LandscapeInfoMap.Map.FindRef(LandscapeGuid);
	}
	return LandscapeInfo;
}

ULandscapeInfo* ULandscapeInfo::FindOrCreate(UWorld* InWorld, const FGuid& LandscapeGuid)
{
	ULandscapeInfo* LandscapeInfo = nullptr;

	check(LandscapeGuid.IsValid());
	check(InWorld);

	ULandscapeInfoMap& LandscapeInfoMap = ULandscapeInfoMap::GetLandscapeInfoMap(InWorld);
	LandscapeInfo = LandscapeInfoMap.Map.FindRef(LandscapeGuid);

	if (!LandscapeInfo)
	{
		LandscapeInfo = NewObject<ULandscapeInfo>(GetTransientPackage(), NAME_None, RF_Transactional | RF_Transient);
		LandscapeInfoMap.Modify(false);
		LandscapeInfo->Initialize(InWorld, LandscapeGuid);
		LandscapeInfoMap.Map.Add(LandscapeGuid, LandscapeInfo);
	}
	check(LandscapeInfo);
	return LandscapeInfo;
}

int32 ULandscapeInfo::RemoveLandscapeInfo(UWorld* InWorld, const FGuid& LandscapeGuid)
{
	check(LandscapeGuid.IsValid());
	check(InWorld);

	ULandscapeInfoMap& LandscapeInfoMap = ULandscapeInfoMap::GetLandscapeInfoMap(InWorld);
	return LandscapeInfoMap.Map.Remove(LandscapeGuid);
}

void ULandscapeInfo::Initialize(UWorld* InWorld, const FGuid& InLandscapeGuid)
{
	LandscapeGuid = InLandscapeGuid;
}

const TArray<TWeakObjectPtr<ALandscapeStreamingProxy>>& ULandscapeInfo::GetSortedStreamingProxies() const
{
	check(Algo::IsSorted(SortedStreamingProxies, UE::Landscape::LandscapeProxySortPredicate));
	return SortedStreamingProxies;
}

void ULandscapeInfo::SortStreamingProxies()
{
	SortedStreamingProxies.Sort(UE::Landscape::LandscapeProxySortPredicate);
}

void ULandscapeInfo::ForEachLandscapeProxy(TFunctionRef<bool(ALandscapeProxy*)> Fn) const
{
	if (ALandscape* Landscape = LandscapeActor.Get())
	{
		if (!Landscape->IsPendingKillPending())
		{
			if ( !Fn(Landscape))
			{
				return;
			}	
		}
		
	}

	// Gather the proxies in a local list so it won't be updated through a re-register while iterating 
	const TArray <TWeakObjectPtr<ALandscapeStreamingProxy>> SortedStreamingProxiesCopy(SortedStreamingProxies);
	for (TWeakObjectPtr<ALandscapeStreamingProxy> StreamingProxyPtr : SortedStreamingProxiesCopy)
	{
		if (ALandscapeProxy* LandscapeProxy = StreamingProxyPtr.Get())
		{
			if (!LandscapeProxy->IsPendingKillPending())
			{
				if (!Fn(LandscapeProxy))
				{
					return;
				}	
			}
			
		}
	}
}

void ULandscapeInfo::UpdateNanite(const ITargetPlatform* InTargetPlatform)
{
	ALandscape* Landscape = LandscapeActor.Get();
	if (!Landscape)
	{
		return;
	}

#if WITH_EDITOR
	if (!Landscape->IsNaniteEnabled() || !GIsEditor)
	{
		return;
	}

	UWorld* World = Landscape->GetWorld();
	bool bDoFinishAllNaniteBuildsInFlightNow = false;
	ForEachLandscapeProxy([&bDoFinishAllNaniteBuildsInFlightNow, InTargetPlatform](ALandscapeProxy* LandscapeProxy)
		{
			FGraphEventRef GraphEvent = LandscapeProxy->UpdateNaniteRepresentationAsync(InTargetPlatform);
			bDoFinishAllNaniteBuildsInFlightNow |= GraphEvent.IsValid();

			return true;
		});


	if ((World != nullptr) && bDoFinishAllNaniteBuildsInFlightNow)
	{
		ULandscapeSubsystem* LandscapeSubsystem = World->GetSubsystem<ULandscapeSubsystem>();
		const bool bAllNaniteBuildsDone = LandscapeSubsystem->FinishAllNaniteBuildsInFlightNow(ULandscapeSubsystem::EFinishAllNaniteBuildsInFlightFlags::Default);
		// Not passing ULandscapeSubsystem::EFinishAllNaniteBuildsInFlightFlags::AllowCancel, so there should be no way that FinishAllNaniteBuildsInFlightNow returns false :
		check(bAllNaniteBuildsDone);
	}
#endif //WITH_EDITOR

}

bool ULandscapeInfo::IsRegistered(const ALandscapeProxy* Proxy) const
{
	if (Proxy == nullptr)
		return false;

	bool bResult = false;
	if (Proxy->IsA<ALandscape>())
	{
		bResult = (LandscapeActor.Get() == Proxy);
	}
	else if (const ALandscapeStreamingProxy *StreamingProxy = Cast<ALandscapeStreamingProxy>(Proxy))
	{
		TWeakObjectPtr<const ALandscapeStreamingProxy> StreamingProxyPtr = StreamingProxy;
		bResult = SortedStreamingProxies.Contains(StreamingProxyPtr);
	}

#if WITH_EDITORONLY_DATA
	// NOTE: during an Undo operation, the LandscapeActor/StreamingProxies are transacted, and the registration status may be restored
	// however, in that case, the Proxy is NOT fully registered yet, because some other data in LandscapeInfo still needs to be updated (XY maps for instance are not transacted)
	// so we trust the bIsRegisteredWithLandscapeInfo flag over the actual pointers.

	// at minimum, if the proxy flag says it is registered, then the pointers should definitely be valid
	if (Proxy->bIsRegisteredWithLandscapeInfo)
	{
		check(bResult == Proxy->bIsRegisteredWithLandscapeInfo);
	}

	// trust the proxy flag over the landscape info pointers
	bResult = Proxy->bIsRegisteredWithLandscapeInfo;
#endif // WITH_EDITORONLY_DATA

	return bResult;
}

// this function contains all of the registration code that requires the ALandscape actor to be present
void ULandscapeInfo::RegisterLandscapeActorWithProxyInternal(ALandscapeProxy* Proxy, bool bMapCheck)
{
	ALandscape* Landscape = LandscapeActor.Get();
	check(Landscape);

	if (ALandscapeStreamingProxy* StreamingProxy = Cast<ALandscapeStreamingProxy>(Proxy))
	{
		// If Guids mismatch, PostLoad has failed to fixup properties and this proxy is being registered to the wrong ULandscapeInfo
		check(Landscape->GetLandscapeGuid() == StreamingProxy->GetLandscapeGuid());

		// streaming proxy specific setup here
		StreamingProxy->SetLandscapeActor(Landscape);

#if WITH_EDITOR
		// Proxy may have a deferred update request. This occurs if a component requests an update when the LandscapeActorRef is invalid.
		// This can happen on Undo, when the component requests an update while ALandscape is unregistered 
		StreamingProxy->PerformDeferredLayerContentUpdateModesRequest();

		StreamingProxy->FixupSharedData(Landscape, bMapCheck, true /*bInExecutePostEditChangeProperty*/);
#endif // WITH_EDITOR

		Proxy->bIsLandscapeActorRegisteredWithLandscapeInfo = true;
		FLandscapeGroup::RegisterAllComponentsOnStreamingProxy(StreamingProxy);
	}

#if WITH_EDITOR
	// generic proxy setup (that requires ALandscape actor) here
	if (bool bLayerInfoMapChanged = UpdateLayerInfoMap(Proxy))
	{
		// The layer info map is part of the main landscape so if it has changed, we need to do another round of shared data fixup on all proxies, so all proxies have their TargetLayers list synchronized. 
		//  This is a one-time thing because at some point during development, the target layer data was deprecated and the deprecation turned somewhat sour :S
		ForEachLandscapeProxy([this, Landscape, bMapCheck](ALandscapeProxy* Proxy)
		{
			Proxy->FixupSharedData(Landscape, bMapCheck, true /*bInExecutePostEditChangeProperty*/);
			return true;
		});
	}

	if (GIsEditor)
	{
		// Note: This can happen when loading certain cooked assets in an editor
		// Todo: Determine the root cause of this and fix it at a higher level!
		if (Proxy->LandscapeComponents.Num() > 0 && Proxy->LandscapeComponents[0] == nullptr)
		{
			Proxy->LandscapeComponents.Empty();
		}

		if (Proxy->WeightmapFixupVersion != Proxy->CurrentVersion)
		{
			Proxy->FixupWeightmaps();
		}
		
		// Check for legacy non-edit layer landscapes and obsolete edit layer data on first registration only
		if (Proxy->RegistrationCount == 0)
		{
			TSet<FGuid> ParentEditLayerGuids;
			Algo::Transform(Landscape->GetEditLayersConst(), ParentEditLayerGuids, [](const ULandscapeEditLayerBase* EditLayer) { return EditLayer->GetGuid(); });

			// If there are obsolete layers on initial registration, move them out of the active LayersData and store duplicate of the current final layer data
			UWorld* World = Landscape->GetWorld();
			
			for (ULandscapeComponent* Component : Proxy->LandscapeComponents)
			{
				if (Component != nullptr)
				{
					if (IsLandscapeEditableWorld())
					{
						Component->InitializeObsoleteEditLayerData(ParentEditLayerGuids);
					}
				}
			}
			
			if (IsLandscapeEditableWorld())
			{
				Proxy->DisplayObsoleteEditLayerMapCheck(Landscape);
			}
			
			// Non-edit layer auto conversion can only happen once on initial load
			// All proxy data fixup is complete - CopyOldDataToDefaultLayer will only migrate data once this flag is set
			Proxy->bIsLandscapeActorRegisteredWithLandscapeInfo = true;

			/*
			Automatically enable edit layers for non-edit layer landscapes here (since FixupSharedData can stomp updates in PostLoad)

			There are two possibilities for the ALandscape on registration
			1. ALandscape has layers content -> No conversion needed
			2. ALandscape does not have layers content -> Trigger full auto conversion to an edit layer based landscape

			There are two possibilities for any ALandscapeProxy on registration
			1. Proxy does not have layers content -> Proxy needs to copy data to default edit layer
				* Conditions - the entire landscape had edit layers disabled, or edit layers were enabled and serialized only on the ALandscape in a WP setup
			2. Proxy has layers content -> If ALandscape does not support layers content on load, copy data to the default edit layer
				* Conditions - the entire landscape had edit layers enabled, or edit layers are enabled and serialized on just the ALandscapeProxy in a WP setup
				* If the proxy has edit layer content but the ALandscape doesn't, the data exists on an obsolete layer. MapCheck will display allowing user to decide if data is copied
			*/

			const bool bIsLandscapeActor = Proxy == LandscapeActor;
			const bool bProxyRequiresNonEditLayerConversion = bIsLandscapeActor ? LandscapeActor->GetEditLayersConst().IsEmpty() : (!Proxy->HasEditLayersDataOnLoad() && !Proxy->LandscapeComponents.IsEmpty());

			if (!LandscapeActor->IsTemplate() && bProxyRequiresNonEditLayerConversion)
			{
				if (bIsLandscapeActor)
				{
					UE_LOGF(LogLandscape, Log, "RegisterLandscapeActorWithProxyInternal: Automatically enabling edit layers on ALandscape: %ls", *Proxy->GetFullName());

					// Create a default edit layer and copy the ALandscape data
					// Streaming proxy data gets copied as proxies complete registration
					LandscapeActor->ConvertNonEditLayerLandscape();
					check(!LandscapeActor->GetEditLayers().IsEmpty());
				}
				else
				{
					UE_LOGF(LogLandscape, Log, "RegisterLandscapeActorWithProxyInternal: Automatically enabling edit layers on Proxy: %ls", *Proxy->GetFullName());

					// As proxies are registered, copy the data to edit layer based landscape. Proxies with edit layer data will be flattened to the default layer
					// Proxy data should not be copied before this point in registration to ensure data is fixed up
					LandscapeActor->CopyOldDataToDefaultLayer(Proxy);

					// After copying, ensure the InProxy's components are valid and have an edit layer in their LayersData map (Default edit layer)
					check(!Proxy->LandscapeComponents.IsEmpty() && (Proxy->LandscapeComponents[0] != nullptr) && Proxy->LandscapeComponents[0]->HasLayersData());
				}

				Proxy->RemoveXYOffsets();
				Proxy->ValidateProxyLayersWeightmapUsage();

				// Mark the proxy dirty to display the 'Requires Save' warning to the user
				MarkObjectDirty(/*InObject = */ Proxy, /*bInForceResave =*/ true, /*InLandscapeOverride =*/ Landscape);
			}
		}
	}

	Proxy->bIsLandscapeActorRegisteredWithLandscapeInfo = true;
	Proxy->RegistrationCount++;
#endif // WITH_EDITOR
}

void ULandscapeInfo::RegisterActor(ALandscapeProxy* Proxy, bool bMapCheck, bool bUpdateAllAddCollisions)
{
	UWorld* OwningWorld = Proxy->GetWorld();
	// do not pass here invalid actors
	checkSlow(Proxy);
	check(Proxy->GetLandscapeGuid().IsValid());
	check(LandscapeGuid.IsValid());
	
	// in case this Info object is not initialized yet
	// initialized it with properties from passed actor
	if (GetLandscapeProxy() == nullptr)
	{
		ComponentSizeQuads = Proxy->ComponentSizeQuads;
		ComponentNumSubsections = Proxy->NumSubsections;
		SubsectionSizeQuads = Proxy->SubsectionSizeQuads;
	}

	// check that passed actor matches all shared parameters
	check(LandscapeGuid == Proxy->GetLandscapeGuid());
	check(ComponentSizeQuads == Proxy->ComponentSizeQuads);
	check(ComponentNumSubsections == Proxy->NumSubsections);
	check(SubsectionSizeQuads == Proxy->SubsectionSizeQuads);

	// register
	if (ALandscape* Landscape = Cast<ALandscape>(Proxy))
	{
#if WITH_EDITORONLY_DATA
		USceneComponent* Root = Proxy->GetRootComponent();
		if (Root)
		{
			DrawScale = Root->GetRelativeScale3D();
			bDrawScaleSetByActor = true;
		}
#endif // WITH_EDITORONLY_DATA

		if (!LandscapeActor.IsValid())
		{
			LandscapeActor = Landscape;

#if WITH_EDITOR
			UPackage::PackageMarkedDirtyEvent.AddWeakLambda(this, [this](UPackage* Pkg, bool bIsDirty) { OnMarkPackageDirty(Pkg, bIsDirty); });
			// Now we have associated a LandscapeActor with this info
			// we can ask for the WeightMaps
			UpdateLayerInfoMap(LandscapeActor.Get());
			
			// Update registered splines so they can pull the actor pointer
			for (TScriptInterface<ILandscapeSplineInterface> SplineActor : SplineActors)
			{
				SplineActor->UpdateSharedProperties(this);
			}

			// In world composition user is not allowed to move landscape in editor, only through WorldBrowser 
			bool bIsLockLocation = LandscapeActor->IsLockLocation();
			bIsLockLocation |= OwningWorld != nullptr ? OwningWorld->WorldComposition != nullptr : false;
			LandscapeActor->SetLockLocation(bIsLockLocation);
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
			Landscape->bIsRegisteredWithLandscapeInfo = true;
#endif // WITH_EDITORONLY_DATA

			// run post-landscape actor registration on the LandscapeActor first, then on each streaming proxy
			RegisterLandscapeActorWithProxyInternal(Landscape, bMapCheck);
			for (TWeakObjectPtr<ALandscapeStreamingProxy> StreamingProxyPtr : SortedStreamingProxies)
			{
				if (ALandscapeStreamingProxy* StreamingProxy = StreamingProxyPtr.Get())
				{
					RegisterLandscapeActorWithProxyInternal(StreamingProxy, bMapCheck);
				}
			}
		}
		else if (LandscapeActor != Landscape)
		{
			UE_LOGF(LogLandscape, Warning, "Multiple landscape actors with the same GUID detected: %ls vs %ls", *LandscapeActor->GetPathName(), *Landscape->GetPathName());
		}
#if WITH_EDITORONLY_DATA
		Landscape->bIsRegisteredWithLandscapeInfo = true;
#endif // WITH_EDITORONLY_DATA
	}
	else
	{
#if WITH_EDITORONLY_DATA
		if (!bDrawScaleSetByActor)
		{
			USceneComponent* Root = Proxy->GetRootComponent();
			if (Root)
			{
				DrawScale = Root->GetRelativeScale3D();
			}
		}
#endif // WITH_EDITORONLY_DATA

		// Insert Proxies in a sorted fashion into the landscape info Proxies list, for generating deterministic results in the Layer system
		ALandscapeStreamingProxy* StreamingProxy = CastChecked<ALandscapeStreamingProxy>(Proxy);
		TWeakObjectPtr<ALandscapeStreamingProxy> StreamingProxyPtr = StreamingProxy;
		if (!SortedStreamingProxies.Contains(StreamingProxyPtr))
		{
			// NOTE: if a streaming proxy somehow gets garbage collected without de-registering from the Proxies list, then
			// this search may return a non-deterministic index because the Proxies list will contain a null
			uint32 InsertIndex = Algo::LowerBound(SortedStreamingProxies, StreamingProxyPtr, UE::Landscape::LandscapeProxySortPredicate);
			SortedStreamingProxies.Insert(StreamingProxyPtr, InsertIndex);
		}

#if WITH_EDITORONLY_DATA
		StreamingProxy->bIsRegisteredWithLandscapeInfo = true;
#endif // WITH_EDITORONLY_DATA

		// If we have a LandscapeActor, register it with the streaming proxy.  If not, it is deferred until a LandscapeActor is registered.
		if (LandscapeActor.IsValid())
		{
			RegisterLandscapeActorWithProxyInternal(StreamingProxy, bMapCheck);
		}
	}

#if WITH_EDITOR
	if(bUpdateAllAddCollisions)
	{
		UpdateAllAddCollisions();
	}
	RegisterSplineActor(Proxy);
#endif // WITH_EDITOR

	//
	// add proxy components to the XY map
	//
	for (int32 CompIdx = 0; CompIdx < Proxy->LandscapeComponents.Num(); ++CompIdx)
	{
		RegisterActorComponent(Proxy->LandscapeComponents[CompIdx], bMapCheck);
	}

	for (ULandscapeHeightfieldCollisionComponent* CollComp : Proxy->CollisionComponents)
	{
		RegisterCollisionComponent(CollComp);
	}
}

void ULandscapeInfo::UnregisterActor(ALandscapeProxy* Proxy)
{
	UWorld* OwningWorld = Proxy->GetWorld();
	if (ALandscape* Landscape = Cast<ALandscape>(Proxy))
	{
		// Note: UnregisterActor sometimes gets triggered twice, e.g. it has been observed to happen during undo/ redo
		// Note: In some cases LandscapeActor could be updated to a new landscape actor before the old landscape is unregistered/destroyed
		// e.g. this has been observed when merging levels in the editor

		if (LandscapeActor.Get() == Landscape)
		{
			LandscapeActor = nullptr;
			UPackage::PackageMarkedDirtyEvent.RemoveAll(this);
		}

		// update proxies reference to landscape actor
		for (TWeakObjectPtr<ALandscapeStreamingProxy> StreamingProxyPtr : SortedStreamingProxies)
		{
			if (ALandscapeStreamingProxy* StreamingProxy = StreamingProxyPtr.Get())
			{
				StreamingProxy->SetLandscapeActor(LandscapeActor.Get());
			}
		}
	}
	else
	{
		ALandscapeStreamingProxy* StreamingProxy = CastChecked<ALandscapeStreamingProxy>(Proxy);
		TWeakObjectPtr<ALandscapeStreamingProxy> StreamingProxyPtr = StreamingProxy;
		SortedStreamingProxies.Remove(StreamingProxyPtr);

		FLandscapeGroup::UnregisterAllComponentsOnStreamingProxy(StreamingProxy);
	}

#if WITH_EDITOR
	UnregisterSplineActor(Proxy);
#endif // WITH_EDITOR

	// remove proxy components from the XY map
	for (int32 CompIdx = 0; CompIdx < Proxy->LandscapeComponents.Num(); ++CompIdx)
	{
		ULandscapeComponent* Component = Proxy->LandscapeComponents[CompIdx];
		if (Component) // When a landscape actor is being GC'd it's possible the components were already GC'd and are null
		{
			UnregisterActorComponent(Component);
		}
	}
	XYtoComponentMap.Compact();

	for (ULandscapeHeightfieldCollisionComponent* CollComp : Proxy->CollisionComponents)
	{
		if (CollComp)
		{
			UnregisterCollisionComponent(CollComp);
		}
	}
	XYtoCollisionComponentMap.Compact();

#if WITH_EDITOR
	UpdateLayerInfoMap();
	UpdateAllAddCollisions();
#endif

#if WITH_EDITORONLY_DATA
	Proxy->bIsRegisteredWithLandscapeInfo = false;
#endif // WITH_EDITORONLY_DATA
	Proxy->bIsLandscapeActorRegisteredWithLandscapeInfo = false;
}

#if WITH_EDITOR
TArray<FTargetLayerBlendGroup> ULandscapeInfo::GetTargetLayerBlendGroups() const
{
	TArray<FTargetLayerBlendGroup> BlendGroups;
	for (const FLandscapeInfoLayerSettings& LayerSettings : Layers)
	{
		if (LayerSettings.LayerInfoObj != nullptr)
		{
			if (LayerSettings.LayerInfoObj->GetBlendMethod() != ELandscapeTargetLayerBlendMethod::None)
			{
				FTargetLayerBlendGroup* BlendGroup = BlendGroups.FindByPredicate([&LayerSettings](const FTargetLayerBlendGroup& InGroup)
					{
						return (InGroup.GetBlendMethod() == LayerSettings.LayerInfoObj->GetBlendMethod())
							&& (InGroup.GetName() == LayerSettings.LayerInfoObj->GetBlendGroup());
					});

				if (BlendGroup == nullptr)
				{
					BlendGroup = &BlendGroups.Add_GetRef(FTargetLayerBlendGroup(LayerSettings.LayerInfoObj->GetBlendGroup(), LayerSettings.LayerInfoObj->GetBlendMethod(), { LayerSettings.LayerInfoObj }));
				}

				BlendGroup->TargetLayers.Add(LayerSettings.LayerInfoObj);
			}

		}
	}

	return BlendGroups;
}

TOptional<FTargetLayerBlendGroup> ULandscapeInfo::GetBlendGroupForTargetLayer(ULandscapeLayerInfoObject* InLayerInfo) const
{
	TOptional<FTargetLayerBlendGroup> BlendGroup;

	TArray<FTargetLayerBlendGroup> BlendGroups = GetTargetLayerBlendGroups();
	if (FTargetLayerBlendGroup* FoundBlendGroup = BlendGroups.FindByPredicate([InLayerInfo](const FTargetLayerBlendGroup& InGroup)
		{
			return InGroup.GetTargetLayers().Contains(InLayerInfo);
		}))
	{
		BlendGroup = *FoundBlendGroup;
	}

	return BlendGroup;
}

ALandscapeSplineActor* ULandscapeInfo::CreateSplineActor(const FVector& Location)
{
	check(LandscapeActor.Get());
	UWorld* World = LandscapeActor->GetWorld();
	check(World);
	FActorSpawnParameters SpawnParams;
	SpawnParams.OverrideLevel = LandscapeActor->GetLevel();
	SpawnParams.bNoFail = true;
	SpawnParams.ObjectFlags |= RF_Transactional;
	ALandscapeSplineActor* SplineActor = World->SpawnActor<ALandscapeSplineActor>(Location, FRotator::ZeroRotator, SpawnParams);
	SplineActor->GetSharedProperties(this);
	SplineActor->GetSplinesComponent()->ShowSplineEditorMesh(true);
	SplineActor->SetIsSpatiallyLoaded(AreNewLandscapeActorsSpatiallyLoaded());
	
	FActorLabelUtilities::SetActorLabelUnique(SplineActor, ALandscapeSplineActor::StaticClass()->GetName());

	RegisterSplineActor(SplineActor);
	return SplineActor;
}

void ULandscapeInfo::ForAllSplineActors(TFunctionRef<void(TScriptInterface<ILandscapeSplineInterface>)> Fn) const
{
	for (const TScriptInterface<ILandscapeSplineInterface>& SplineActor : SplineActors)
	{
		Fn(SplineActor);
	}
}

TArray<TScriptInterface<ILandscapeSplineInterface>> ULandscapeInfo::GetSplineActors() const
{
	TArray<TScriptInterface<ILandscapeSplineInterface>> CopySplineActors(SplineActors);
	return MoveTemp(CopySplineActors);
}

void ULandscapeInfo::RegisterSplineActor(TScriptInterface<ILandscapeSplineInterface> SplineActor)
{
	Modify();

	// Sort on insert to ensure spline actors are always processed in the same order, regardless of variation in the
	// sub level streaming/registration sequence.
	auto SortPredicate = [](const TScriptInterface<ILandscapeSplineInterface>& A, const TScriptInterface<ILandscapeSplineInterface>& B)
	{
		return Cast<UObject>(A.GetInterface())->GetPathName() < Cast<UObject>(B.GetInterface())->GetPathName();
	};

	// Add a unique entry, sorted
	const int32 LBoundIdx = Algo::LowerBound(SplineActors, SplineActor, SortPredicate);
	if (LBoundIdx == SplineActors.Num() || SplineActors[LBoundIdx] != SplineActor)
	{
		SplineActors.Insert(SplineActor, LBoundIdx);
	}

	SplineActor->UpdateSharedProperties(this);

	if (SplineActor->GetSplinesComponent())
	{
		RequestSplineLayerUpdate();
	}
}

void ULandscapeInfo::UnregisterSplineActor(TScriptInterface<ILandscapeSplineInterface> SplineActor)
{
	Modify();
	SplineActors.Remove(SplineActor);

	if (SplineActor->GetSplinesComponent())
	{
		RequestSplineLayerUpdate();
	}
}

void ULandscapeInfo::UpdateRegistrationForSplineActor(UWorld* InWorld, TScriptInterface<ILandscapeSplineInterface> InSplineActor)
{
	if (InWorld == nullptr)
		return;

	ULandscapeInfoMap& LandscapeInfoMap = ULandscapeInfoMap::GetLandscapeInfoMap(InWorld);
	FGuid SplineLandscapeGUID = InSplineActor->GetLandscapeGuid();

	// first let's unregister from any landscapes that have it (incorrectly) registered
	for (const auto& pair : LandscapeInfoMap.Map)
	{
		ULandscapeInfo* LandscapeInfo = pair.Value;

		// only unregister if the landscape guids don't match
		if ((LandscapeInfo->LandscapeGuid != SplineLandscapeGUID) &&
			LandscapeInfo->SplineActors.Contains(InSplineActor))
		{
			LandscapeInfo->UnregisterSplineActor(InSplineActor);
		}
	}

	// then let's make sure it is registered with the correct landscape info
	if (SplineLandscapeGUID.IsValid())
	{
		ULandscapeInfo* LandscapeInfo = InSplineActor->GetLandscapeInfo();
		check(LandscapeInfo);
		if (!LandscapeInfo->SplineActors.Contains(InSplineActor))
		{
			LandscapeInfo->RegisterSplineActor(InSplineActor);
		}
	}
}

void ULandscapeInfo::RequestSplineLayerUpdate()
{
	if (LandscapeActor.IsValid())
	{
		LandscapeActor->RequestSplineLayerUpdate();
	}
}

void ULandscapeInfo::ForceLayersFullUpdate()
{
	if (LandscapeActor.IsValid())
	{
		LandscapeActor->ForceLayersFullUpdate();
	}
}
#endif

void ULandscapeInfo::RegisterCollisionComponent(ULandscapeHeightfieldCollisionComponent* Component)
{
	if (Component == nullptr || !Component->IsRegistered())
	{
		return;
	}

	FIntPoint ComponentKey = Component->GetSectionBase() / Component->CollisionSizeQuads;
	auto RegisteredComponent = XYtoCollisionComponentMap.FindRef(ComponentKey);

	if (RegisteredComponent != Component)
	{
		if (RegisteredComponent == nullptr)
		{
			XYtoCollisionComponentMap.Add(ComponentKey, Component);
		}
	}
}

void ULandscapeInfo::UnregisterCollisionComponent(ULandscapeHeightfieldCollisionComponent* Component)
{
	if (ensure(Component))
	{
		FIntPoint ComponentKey = Component->GetSectionBase() / Component->CollisionSizeQuads;
		auto RegisteredComponent = XYtoCollisionComponentMap.FindRef(ComponentKey);

		if (RegisteredComponent == Component)
		{
			XYtoCollisionComponentMap.Remove(ComponentKey);
		}
	}
}

// TODO [jonathan.bard] : improve this function or create another one to take into account unloaded proxies : 
bool ULandscapeInfo::GetOverlappedComponents(const FTransform& InAreaWorldTransform, const FBox2D& InAreaExtents, 
	TMap<FIntPoint, ULandscapeComponent*>& OutOverlappedComponents, FIntRect& OutComponentIndicesBoundingRect)
{
	if (!LandscapeActor.IsValid())
	{ 
		return false;
	}

	FIntRect EffectiveBoundingIndices;

	// Consider invalid extents as meaning "infinite", in which case, return all loaded components : 
	if (!InAreaExtents.bIsValid)
	{
		OutOverlappedComponents.Reserve(XYtoComponentMap.Num());
		for (const TPair<FIntPoint, ULandscapeComponent*>& XYComponentPair : XYtoComponentMap)
		{
			EffectiveBoundingIndices.Union(FIntRect(XYComponentPair.Key, XYComponentPair.Key + FIntPoint(1)));
			OutOverlappedComponents.Add(XYComponentPair);
		}
	}
	else
	{
		// Compute the AABB for this area in landscape space to find which of the landscape components are overlapping :
		FVector Extremas[4];
		const FTransform& LandscapeTransform = LandscapeActor->GetTransform();
		Extremas[0] = LandscapeTransform.InverseTransformPosition(InAreaWorldTransform.TransformPosition(FVector(InAreaExtents.Min.X, InAreaExtents.Min.Y, 0.0)));
		Extremas[1] = LandscapeTransform.InverseTransformPosition(InAreaWorldTransform.TransformPosition(FVector(InAreaExtents.Min.X, InAreaExtents.Max.Y, 0.0)));
		Extremas[2] = LandscapeTransform.InverseTransformPosition(InAreaWorldTransform.TransformPosition(FVector(InAreaExtents.Max.X, InAreaExtents.Min.Y, 0.0)));
		Extremas[3] = LandscapeTransform.InverseTransformPosition(InAreaWorldTransform.TransformPosition(FVector(InAreaExtents.Max.X, InAreaExtents.Max.Y, 0.0)));
		FBox LocalExtents(Extremas, 4);

		// Indices of the landscape components needed for rendering this area : 
		FIntRect BoundingIndices;
		BoundingIndices.Min = FIntPoint(FMath::FloorToInt32(LocalExtents.Min.X / ComponentSizeQuads), FMath::FloorToInt32(LocalExtents.Min.Y / ComponentSizeQuads));
		// The max here is meant to be an exclusive bound, hence the +1
		BoundingIndices.Max = FIntPoint(FMath::FloorToInt32(LocalExtents.Max.X / ComponentSizeQuads), FMath::FloorToInt32(LocalExtents.Max.Y / ComponentSizeQuads)) + FIntPoint(1);

		// Go through each loaded component and find out the actual bounds of the area we need to render :
		for (int32 KeyY = BoundingIndices.Min.Y; KeyY < BoundingIndices.Max.Y; ++KeyY)
		{
			for (int32 KeyX = BoundingIndices.Min.X; KeyX < BoundingIndices.Max.X; ++KeyX)
			{
				FIntPoint Key(KeyX, KeyY);
				if (ULandscapeComponent* Component = XYtoComponentMap.FindRef(Key))
				{
					EffectiveBoundingIndices.Union(FIntRect(Key, Key + FIntPoint(1)));
					OutOverlappedComponents.Add(Key, Component);
				}
			}
		}
	}

	if (OutOverlappedComponents.IsEmpty())
	{
		return false;
	}

	OutComponentIndicesBoundingRect = EffectiveBoundingIndices;
	return true;
}

void ULandscapeInfo::RegisterActorComponent(ULandscapeComponent* Component, bool bMapCheck)
{
	// Do not register components which are not part of the world
	if (Component == nullptr ||
		Component->IsRegistered() == false)
	{
		return;
	}

	check(Component);

	FIntPoint ComponentKey = Component->GetSectionBase() / Component->ComponentSizeQuads;
	ULandscapeComponent* RegisteredComponent = XYtoComponentMap.FindRef(ComponentKey);

	if (RegisteredComponent != Component)
	{
		if (RegisteredComponent == nullptr)
		{
			XYtoComponentMap.Add(ComponentKey, Component);
		}
		else if (bMapCheck)
		{
#if WITH_EDITOR
			ALandscapeProxy* OurProxy = Component->GetLandscapeProxy();
			ALandscapeProxy* ExistingProxy = RegisteredComponent->GetLandscapeProxy();
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("ProxyName1"), FText::FromString(OurProxy->GetName()));
			Arguments.Add(TEXT("LevelName1"), FText::FromString(OurProxy->GetLevel()->GetOutermost()->GetName()));
			Arguments.Add(TEXT("ProxyName2"), FText::FromString(ExistingProxy->GetName()));
			Arguments.Add(TEXT("LevelName2"), FText::FromString(ExistingProxy->GetLevel()->GetOutermost()->GetName()));
			Arguments.Add(TEXT("XLocation"), Component->GetSectionBase().X);
			Arguments.Add(TEXT("YLocation"), Component->GetSectionBase().Y);
			FMessageLog("MapCheck").Warning()
				->AddToken(FUObjectToken::Create(OurProxy, FText::FromString(OurProxy->GetActorNameOrLabel())))
				->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_LandscapeComponentPostLoad_Warning", "Landscape {ProxyName1} of {LevelName1} has overlapping render components with {ProxyName2} of {LevelName2} at location ({XLocation}, {YLocation})."), Arguments)))
				->AddToken(FActionToken::Create(LOCTEXT("MapCheck_RemoveDuplicateLandscapeComponent", "Delete Duplicate"), LOCTEXT("MapCheck_RemoveDuplicateLandscapeComponentDesc", "Deletes the duplicate landscape component."), FOnActionTokenExecuted::CreateUObject(OurProxy, &ALandscapeProxy::RemoveOverlappingComponent, Component), true))
				->AddToken(FMapErrorToken::Create(FMapErrors::LandscapeComponentPostLoad_Warning));

			// Show MapCheck window
			FMessageLog("MapCheck").Open(EMessageSeverity::Warning);
#endif
		}
	}


#if WITH_EDITOR
	// Update Selected Components/Regions
	if (Component->EditToolRenderData.SelectedType)
	{
		if (Component->EditToolRenderData.SelectedType & FLandscapeEditToolRenderData::ST_COMPONENT)
		{
			SelectedComponents.Add(Component);
		}
		else if (Component->EditToolRenderData.SelectedType & FLandscapeEditToolRenderData::ST_REGION)
		{
			SelectedRegionComponents.Add(Component);
		}
	}
#endif

	XYComponentBounds.Include(ComponentKey);
}

void ULandscapeInfo::UnregisterActorComponent(ULandscapeComponent* Component)
{
	if (ensure(Component))
	{
		FIntPoint ComponentKey = Component->GetSectionBase() / Component->ComponentSizeQuads;
		ULandscapeComponent* RegisteredComponent = XYtoComponentMap.FindRef(ComponentKey);

		if (RegisteredComponent == Component)
		{
			XYtoComponentMap.Remove(ComponentKey);
		}

		SelectedComponents.Remove(Component);
		SelectedRegionComponents.Remove(Component);

		// When removing a key, we need to iterate to find the new bounds
		XYComponentBounds = FIntRect(MAX_int32, MAX_int32, MIN_int32, MIN_int32);

		for (const TPair<FIntPoint, ULandscapeComponent*>& XYComponentPair : XYtoComponentMap)
	{
			XYComponentBounds.Include(XYComponentPair.Key);
	}
	}
}

namespace LandscapeInfoBoundsHelper
{
	template<class ComponentClass>
	void AccumulateComponentBounds(TConstArrayView<TObjectPtr<ComponentClass>> InComponents, FBox& InOutBounds)
	{
		for (TObjectPtr<ComponentClass> Component : InComponents)
		{
			if ((Component != nullptr) && Component->IsRegistered())
			{
				// Reject invalid bounds
				const FBox& ComponentBox = Component->Bounds.GetBox();
				if (ComponentBox.IsValid && ComponentBox.GetSize() != FVector::Zero())
				{
					InOutBounds += ComponentBox;
				}
			}
		};
	}

	void AccumulateBounds(const ALandscapeProxy* InLandscapeProxy, FBox& InOutBounds)
	{
		FBox LandscapeProxyBounds = InLandscapeProxy->GetProxyBounds();
		if (LandscapeProxyBounds.IsValid && LandscapeProxyBounds.GetSize() != FVector::Zero())
		{
			InOutBounds += LandscapeProxyBounds;
		}
	}
}

FBox ALandscapeProxy::GetProxyBounds() const
{
	FBox Bounds(EForceInit::ForceInit);
	
	// Only accumulate bounds from landscape components, so that bounds don't get artificially inflated by other components like spawned HISM (grass) components.
	//  We include collision components, so that if this ever gets called on the server, where landscape components are missing, this is consistent
	LandscapeInfoBoundsHelper::AccumulateComponentBounds(MakeConstArrayView(LandscapeComponents), Bounds);
	LandscapeInfoBoundsHelper::AccumulateComponentBounds(MakeConstArrayView(CollisionComponents), Bounds);

	return Bounds;
}

FBox ULandscapeInfo::GetLoadedBounds() const
{
	FBox Bounds(EForceInit::ForceInit);

	if (LandscapeActor.IsValid())
	{
		LandscapeInfoBoundsHelper::AccumulateBounds(LandscapeActor.Get(), Bounds);
	}

	// Since in PIE/in-game the Proxies aren't populated, we must iterate through the loaded components
	// but this is functionally equivalent to calling ForAllLandscapeProxies
	TSet<ALandscapeProxy*> LoadedProxies;
	for (auto It = XYtoComponentMap.CreateConstIterator(); It; ++It)
	{
		if (!It.Value())
		{
			continue;
		}

		if (ALandscapeProxy* Proxy = Cast<ALandscapeProxy>(It.Value()->GetOwner()))
		{
			LoadedProxies.Add(Proxy);
		}
	}

	for (ALandscapeProxy* Proxy : LoadedProxies)
	{
		LandscapeInfoBoundsHelper::AccumulateBounds(Proxy, Bounds);
	}

	return Bounds;
}

#if WITH_EDITOR
FBox ULandscapeInfo::GetCompleteBounds() const
{
	ALandscape* Landscape = LandscapeActor.Get();

	// In a non-WP situation, the current actor's bounds will do.
	if(!Landscape || !Landscape->GetWorld() || !Landscape->GetWorld()->GetWorldPartition())
	{
		return GetLoadedBounds();
	}

	FBox Bounds(EForceInit::ForceInit);

	FWorldPartitionHelpers::ForEachActorDescInstance<ALandscapeProxy>(Landscape->GetWorld()->GetWorldPartition(), [this, &Bounds, Landscape](const FWorldPartitionActorDescInstance* ActorDescInstance)
	{
		FLandscapeActorDesc* LandscapeActorDesc = (FLandscapeActorDesc*)ActorDescInstance->GetActorDesc();
		ALandscapeProxy* LandscapeProxy = Cast<ALandscapeProxy>(ActorDescInstance->GetActor());

		// Prioritize loaded bounds, as the bounds in the actor desc might not be up-to-date
		if(LandscapeProxy && (LandscapeProxy->GetGridGuid() == LandscapeGuid))
		{
			LandscapeInfoBoundsHelper::AccumulateBounds(LandscapeProxy, Bounds);
		}
		else if (LandscapeActorDesc->GridGuid == LandscapeGuid)
		{
			Bounds += LandscapeActorDesc->GetEditorBounds();
		}

		return true;
	});

	return Bounds;
}
#endif

void ULandscapeComponent::PostInitProperties()
{
	Super::PostInitProperties();

	// Initialize MapBuildDataId to something unique, in case this is a new ULandscapeComponent
	MapBuildDataId = FGuid::NewGuid();
}

ULandscapeWeightmapUsage::ULandscapeWeightmapUsage(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ClearUsage();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
ALandscapeProxy::~ALandscapeProxy()
{
	for (int32 Index = 0; Index < AsyncFoliageTasks.Num(); Index++)
	{
		FAsyncTask<FAsyncGrassTask>* Task = AsyncFoliageTasks[Index];
		Task->EnsureCompletion(true);
		FAsyncGrassTask& Inner = Task->GetTask();
		delete Task;
	}
	AsyncFoliageTasks.Empty();

#if WITH_EDITORONLY_DATA
	LandscapeProxies.Remove(this);
#endif
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

//
// ALandscapeMeshProxyActor
//
ALandscapeMeshProxyActor::ALandscapeMeshProxyActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetCanBeDamaged(false);

	LandscapeMeshProxyComponent = CreateDefaultSubobject<ULandscapeMeshProxyComponent>(TEXT("LandscapeMeshProxyComponent0"));
	LandscapeMeshProxyComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	LandscapeMeshProxyComponent->Mobility = EComponentMobility::Static;
	LandscapeMeshProxyComponent->SetGenerateOverlapEvents(false);

	RootComponent = LandscapeMeshProxyComponent;
}

//
// ULandscapeMeshProxyComponent
//
ULandscapeMeshProxyComponent::ULandscapeMeshProxyComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void ULandscapeMeshProxyComponent::PostLoad()
{
	Super::PostLoad();

	ChangeLandscapeGuidIfObjectIsInstanced(this->LandscapeGuid, this);
}

void ULandscapeMeshProxyComponent::InitializeForLandscape(ALandscapeProxy* Landscape, int8 InProxyLOD)
{
	LandscapeGuid = Landscape->GetLandscapeGuid();
	LODGroupKey = Landscape->LODGroupKey;

	FTransform WorldToLocal = GetComponentTransform().Inverse();

	bool bFirst = true;
	for (ULandscapeComponent* Component : Landscape->LandscapeComponents)
	{
		if (Component)
		{
			const FTransform& ComponentLocalToWorld = Component->GetComponentTransform();

			if (bFirst)
			{
				bFirst = false;
				ComponentResolution = Component->ComponentSizeQuads + 1;
				FVector ComponentXVectorWorldSpace = ComponentLocalToWorld.TransformVector(FVector::XAxisVector) * ComponentResolution;
				FVector ComponentYVectorWorldSpace = ComponentLocalToWorld.TransformVector(FVector::YAxisVector) * ComponentResolution;
				ComponentXVectorObjectSpace = WorldToLocal.TransformVector(ComponentXVectorWorldSpace);
				ComponentYVectorObjectSpace = WorldToLocal.TransformVector(ComponentYVectorWorldSpace);
			}
			else
			{
				// assume it's the same resolution and orientation as the first component... (we only record one resolution and orientation)
			}

			// record the component coordinate
			ProxyComponentBases.Add(Component->GetSectionBase() / Component->ComponentSizeQuads);
			
			// record the component center position (in the space of the ULandscapeMeshProxyComponent)
			FBoxSphereBounds ComponentLocalBounds = Component->CalcBounds(FTransform::Identity);
			FVector ComponentOriginWorld = ComponentLocalToWorld.TransformPosition(ComponentLocalBounds.Origin);
			FVector LocalOrigin = WorldToLocal.TransformPosition(ComponentOriginWorld);
			ProxyComponentCentersObjectSpace.Add(LocalOrigin);
		}
	}

	if (InProxyLOD != INDEX_NONE)
	{
		ProxyLOD = FMath::Clamp<int8>(InProxyLOD, 0, static_cast<int8>(FMath::CeilLogTwo(Landscape->SubsectionSizeQuads + 1) - 1));
	}
}

#if WITH_EDITOR

void  ALandscapeProxy::CreateNaniteComponents(int32 InNumComponents) 
{
	for (int32 i = 0; i < InNumComponents; ++i)
	{
		ULandscapeNaniteComponent* NaniteComponent = NewObject<ULandscapeNaniteComponent>(this, *FString::Format(TEXT("LandscapeNaniteComponent_{0}"), {{ i }}));
		NaniteComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
		NaniteComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
		NaniteComponent->SetMobility(EComponentMobility::Static);
		NaniteComponent->SetGenerateOverlapEvents(false);
		NaniteComponent->SetCanEverAffectNavigation(false);
		NaniteComponent->CanCharacterStepUpOn = ECanBeCharacterBase::ECB_No;
		NaniteComponent->bSelectable = false;
		NaniteComponent->DepthPriorityGroup = SDPG_World;
		NaniteComponent->bForceNaniteForMasked = true;
		NaniteComponent->RegisterComponent();
		NaniteComponent->AttachToComponent(GetRootComponent(), FAttachmentTransformRules::KeepRelativeTransform);

		NaniteComponents.Add(NaniteComponent);
	}
}

void ALandscapeProxy::SerializeStateHashes(FArchive& Ar)
{
	for (FLandscapePerLODMaterialOverride& MaterialOverride : PerLODOverrideMaterials)
	{
		if (MaterialOverride.Material != nullptr)
		{
			FGuid LocalStateId = MaterialOverride.Material->GetMaterial_Concurrent()->StateId;
			Ar << LocalStateId;
			Ar << MaterialOverride.LODIndex;
		}
	}
}

void ULandscapeComponent::SerializeStateHashes(FArchive& Ar)
{
	FGuid HeightmapGuid = ULandscapeTextureHash::GetHash(HeightmapTexture);
	Ar << HeightmapGuid;
	for (auto WeightmapTexture : WeightmapTextures)
	{
		FGuid WeightmapGuid = WeightmapTexture->Source.GetId();
		Ar << WeightmapGuid;
	}

	bool bEnableNanite = GetLandscapeProxy()->IsNaniteEnabled();
	Ar << bEnableNanite;

	if (GetLandscapeHoleMaterial() && ComponentHasVisibilityPainted())
	{
		FGuid LocalStateId = GetLandscapeHoleMaterial()->GetMaterial_Concurrent()->StateId;
		Ar << LocalStateId;
	}

	// Take into account the Heightmap offset per component
	Ar << HeightmapScaleBias.Z;
	Ar << HeightmapScaleBias.W;

	if (OverrideMaterial != nullptr)
	{
		FGuid LocalStateId = OverrideMaterial->GetMaterial_Concurrent()->StateId;
		Ar << LocalStateId;
	}

	for (FLandscapePerLODMaterialOverride& MaterialOverride : PerLODOverrideMaterials)
	{
		if (MaterialOverride.Material != nullptr)
		{
			FGuid LocalStateId = MaterialOverride.Material->GetMaterial_Concurrent()->StateId;
			Ar << LocalStateId;
			Ar << MaterialOverride.LODIndex;
		}
	}

	ALandscapeProxy* Proxy = GetLandscapeProxy();

	if (Proxy->LandscapeMaterial != nullptr)
	{
		FGuid LocalStateId = Proxy->LandscapeMaterial->GetMaterial_Concurrent()->StateId;
		Ar << LocalStateId;
	}

	Proxy->SerializeStateHashes(Ar);
}

FLandscapePhysicalMaterialBuilder::FLandscapePhysicalMaterialBuilder(UWorld* InWorld)
	:World(InWorld)
	,OudatedPhysicalMaterialComponentsCount(0)
{
}

void FLandscapePhysicalMaterialBuilder::Build(UE::Landscape::EBuildFlags InBuildFlags)
{
	if (World)
	{
		int32 NumBuilt = 0;
		for (TActorIterator<ALandscapeProxy> ProxyIt(World); ProxyIt; ++ProxyIt)
		{
			if (EnumHasAnyFlags(InBuildFlags, UE::Landscape::EBuildFlags::ForceRebuild))
			{
				ProxyIt->InvalidatePhysicalMaterial();
			}
			NumBuilt += ProxyIt->BuildPhysicalMaterial() ? 1 : 0;
		}

		if (EnumHasAnyFlags(InBuildFlags, UE::Landscape::EBuildFlags::WriteFinalLog))
		{
			UE_LOGFMT_LOC(LogLandscape, Log, "BuildPhysicalMaterialFinalLog", "Build Physical Materials: {NumProxies} landscape {NumProxies}|plural(one=proxy,other=proxies) built", ("NumProxies", NumBuilt));
		}
	}
}

int32 FLandscapePhysicalMaterialBuilder::GetOudatedPhysicalMaterialComponentsCount()
{
	if (World)
	{
		OudatedPhysicalMaterialComponentsCount = 0;
		for (TActorIterator<ALandscapeProxy> ProxyIt(World); ProxyIt; ++ProxyIt)
		{
			OudatedPhysicalMaterialComponentsCount += ProxyIt->GetOudatedPhysicalMaterialComponentsCount();
		}
	}
	return OudatedPhysicalMaterialComponentsCount;
}

int32 ALandscapeProxy::GetOudatedPhysicalMaterialComponentsCount() const
{
	int32 OudatedPhysicalMaterialComponentsCount = 0;
	UpdatePhysicalMaterialTasksStatus(nullptr, &OudatedPhysicalMaterialComponentsCount);
	return OudatedPhysicalMaterialComponentsCount;
}

UE::Landscape::EOutdatedDataFlags ALandscapeProxy::GetOutdatedDataFlags() const
{
	UE::Landscape::EOutdatedDataFlags OutdatedDataFlags = UE::Landscape::EOutdatedDataFlags::None;

	if (GetOutdatedGrassMapCount() > 0)
	{
		OutdatedDataFlags |= UE::Landscape::EOutdatedDataFlags::GrassMaps;
	}

	if (GetOudatedPhysicalMaterialComponentsCount() > 0)
	{
		OutdatedDataFlags |= UE::Landscape::EOutdatedDataFlags::PhysicalMaterials;
	}

	if (!IsNaniteMeshUpToDate())
	{
		OutdatedDataFlags |= UE::Landscape::EOutdatedDataFlags::NaniteMeshes;
	}

	if (ULandscapeInfo* Info = GetLandscapeInfo())
	{
		if (Info->IsPackageModified(GetPackage()))
		{
			OutdatedDataFlags |= UE::Landscape::EOutdatedDataFlags::PackageModified;
		}
	}

	return OutdatedDataFlags;
}

void ALandscapeProxy::ClearNaniteTransactional()
{
	for (ULandscapeNaniteComponent* NaniteComponent : NaniteComponents)
	{
		if (NaniteComponent)
		{
			NaniteComponent->ClearFlags(RF_Transactional);
		}
	}
}

void ALandscapeProxy::UpdateNaniteSharedPropertiesFromActor()
{
	for (ULandscapeNaniteComponent* NaniteComponent : NaniteComponents)
	{
		if (NaniteComponent)
		{
			NaniteComponent->UpdatedSharedPropertiesFromActor();
		}
	}
}

void ALandscapeProxy::SetSourceComponentsForNaniteComponents(bool bSetEmptyComponentsOnly) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ALandscapeProxy::SetSourceComponentsForNaniteComponents);

	// Nanite components are reset after invalidation, early out until new components are created
	if (!IsNaniteEnabled() || NaniteComponents.IsEmpty())
	{
		return;
	}
	// Ensure we have the correct number of nanite components based on the proxy component count
	check(NaniteComponents.Num() == NumNaniteRequiredComponents());

	TArray<ULandscapeComponent*> StableOrderComponents(LandscapeComponents);
	for (int32 NaniteComponentIndex = 0; NaniteComponentIndex < NaniteComponents.Num(); ++NaniteComponentIndex)
	{
		const int32 StartComponentIndex = NaniteComponentIndex * NaniteMaxComponents;
		const int32 EndComponentIndex = FMath::Min(LandscapeComponents.Num(), (NaniteComponentIndex + 1) * NaniteMaxComponents);
		const int32 NumComponents = EndComponentIndex - StartComponentIndex;
		ULandscapeNaniteComponent* NaniteComponent = NaniteComponents[NaniteComponentIndex];
		if (NaniteComponent == nullptr)
		{
			UE_LOGF(LogLandscape, Warning, "Landscape Nanite failed to set source components for NaniteComponents[%d]. Landscape Nanite requires rebuild. Actor: '%ls' Package: '%ls'", NaniteComponentIndex, *GetActorNameOrLabel(), *GetPackage()->GetName());
			continue;
		}
		// Optionally skip nanite components that already have valid source components set
		if (NumComponents <= 0 || !LandscapeComponents.IsValidIndex(StartComponentIndex) || (!NaniteComponent->GetSourceLandscapeComponents().IsEmpty() && bSetEmptyComponentsOnly))
		{
			continue;
		}
		// Select the next available components from our distance sorted component pool each iteration
		TArrayView<ULandscapeComponent*> StableOrderComponentsView = TArrayView<ULandscapeComponent*>(&StableOrderComponents[StartComponentIndex], LandscapeComponents.Num() - StartComponentIndex);
		ULandscapeComponent** const MinComponent = Algo::MinElementBy(StableOrderComponentsView,
			[](const ULandscapeComponent* Component) { return Component->GetSectionBase(); },
			[](const FIntPoint& A, const FIntPoint& B) { return (A.Y == B.Y) ? (A.X < B.X) : (A.Y < B.Y); }
		);
		check(MinComponent);
		Algo::Sort(StableOrderComponentsView, FCompareULandscapeComponentClosest((*MinComponent)->GetSectionBase()));
		TArray<ULandscapeComponent*> Result(StableOrderComponentsView.GetData(), NumComponents);
		NaniteComponent->SetSourceLandscapeComponents(Result);
	}
}

void ALandscapeProxy::OnLayerInfoObjectDataChanged(const FOnLandscapeLayerInfoDataChangedParams& InParams)
{
	const ULandscapeSettings* LandscapeSettings = GetDefault<ULandscapeSettings>();
	ALandscape* LandscapeActor = GetLandscapeActor();
	check(LandscapeSettings != nullptr && LandscapeActor);
	const bool bIsLandscapeActor = LandscapeActor == this;

	const FName MemberPropertyName = InParams.PropertyChangedEvent.MemberProperty ? InParams.PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;
	// On Undo, we do not have access to the property. Force and update of everything
	const bool bUpdateAll = MemberPropertyName.IsNone();

	if (MemberPropertyName == ULandscapeLayerInfoObject::GetPhysicalMaterialMemberName()
		|| MemberPropertyName == ULandscapeLayerInfoObject::GetMinimumCollisionRelevanceWeightMemberName()
		|| bUpdateAll)
	{
		ChangedPhysMaterial();
	}

	if (MemberPropertyName == ULandscapeLayerInfoObject::GetLayerUsageDebugColorMemberName() || bUpdateAll)
	{
		if (GetWorld() && !GetWorld()->IsPlayInEditor())
		{
			MarkComponentsRenderStateDirty();
		}
	}

#if WITH_EDITORONLY_DATA
	if (MemberPropertyName == ULandscapeLayerInfoObject::GetSplineFalloffModulationTextureMemberName()
		|| MemberPropertyName == ULandscapeLayerInfoObject::GetSplineFalloffModulationColorMaskMemberName()
		|| MemberPropertyName == ULandscapeLayerInfoObject::GetSplineFalloffModulationBiasMemberName()
		|| MemberPropertyName == ULandscapeLayerInfoObject::GetSplineFalloffModulationScaleMemberName()
		|| MemberPropertyName == ULandscapeLayerInfoObject::GetSplineFalloffModulationTilingMemberName()
		|| bUpdateAll)
	{
		const bool bAllowLandscapeSplineFalloffModulationUpdate = (InParams.PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive) || LandscapeSettings->GetShouldUpdateEditLayersDuringInteractiveChanges();
		if (bIsLandscapeActor && bAllowLandscapeSplineFalloffModulationUpdate)
		{
			check(InParams.LayerInfoObject != nullptr);
			LandscapeActor->OnLayerInfoSplineFalloffModulationChanged(InParams.LayerInfoObject);
		}
	}
#endif // WITH_EDITORONLY_DATA

	// Request landscape update if needed
	const bool bAllowLandscapeUpdate = (InParams.PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive) || LandscapeSettings->GetShouldUpdateEditLayersDuringInteractiveChanges();
	if (InParams.bRequiresLandscapeUpdate && bAllowLandscapeUpdate)
	{
		LandscapeActor->RequestLayersContentUpdateForceAll(ELandscapeLayerUpdateMode::Update_All, InParams.bUserTriggered);
	}

	if ((InParams.PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive) 
		&& ((MemberPropertyName == ULandscapeLayerInfoObject::GetBlendMethodMemberName())
			|| (MemberPropertyName == ULandscapeLayerInfoObject::GetBlendGroupMemberName())
			|| bUpdateAll))
	{
		// There's no real hook to attach to when a custom thumbnail is generated (or cleared) for a given asset and we need to regenerate the thumbnails when that happens.
		//  PostEditChange is called on the layer info object when this happens, though, so we can ask for regenerating thumbnails when that happens. Not the most efficient
		//  thing but landscape layer changes are pretty rare anyway
		ILandscapeModule& LandscapeModule = FModuleManager::GetModuleChecked<ILandscapeModule>("Landscape");
		LandscapeModule.GetLandscapeEditorServices()->RegenerateLayerThumbnails();
		LandscapeModule.GetLandscapeEditorServices()->RefreshDetailPanel();
	}
}

void ALandscapeProxy::InvalidatePhysicalMaterial()
{
	for (ULandscapeComponent* Component : LandscapeComponents)
	{
		Component->InvalidatePhysicalMaterial();
	}
}

bool ALandscapeProxy::BuildPhysicalMaterial()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		const bool bShouldMarkDirty = true;
		return UpdatePhysicalMaterialTasks(bShouldMarkDirty);
	}

	return false;
}

void ALandscapeProxy::UpdatePhysicalMaterialTasksStatus(TSet<ULandscapeComponent*>* OutdatedComponents, int32* OutdatedComponentsCount) const
{
	int32 OutdatedCount = 0;
	for (ULandscapeComponent* Component : LandscapeComponents)
	{
		uint32 Hash = Component->CalculatePhysicalMaterialTaskHash();
		if (Component->PhysicalMaterialHash != Hash || Component->PhysicalMaterialTask.IsValid())
		{
			OutdatedCount++;
			if (OutdatedComponents)
			{
				OutdatedComponents->Add(Component);
			}
		}
	}

	if (OutdatedCount == 0)
	{
		for (ULandscapeComponent* Component : LandscapeComponents)
		{
			const bool bIsDirty = Component->GetPackage()->IsDirty();
			if (Component->LastSavedPhysicalMaterialHash != Component->PhysicalMaterialHash && !bIsDirty)
			{
				OutdatedCount++;
			}
		}
	}

	if (OutdatedComponentsCount)
	{
		*OutdatedComponentsCount = OutdatedCount;
	}
}

bool ALandscapeProxy::UpdatePhysicalMaterialTasks(bool bInShouldMarkDirty)
{
	if (ALandscape* Landscape = GetLandscapeActor())
	{
		// Physical material depends on the weightmaps, so make sure they are controlled by the texture streaming manager.
		Landscape->PrepareTextureResourcesLimited();
	}

	TSet<ULandscapeComponent*> OutdatedComponents;
	int32 PendingComponentsToBeSaved = 0;
	UpdatePhysicalMaterialTasksStatus(&OutdatedComponents, &PendingComponentsToBeSaved);
	for (ULandscapeComponent* Component : OutdatedComponents)
	{
		Component->UpdatePhysicalMaterialTasks();
	}
	if (bInShouldMarkDirty && PendingComponentsToBeSaved > 0)
	{
		MarkPackageDirty();
	}

	return (PendingComponentsToBeSaved > 0);
}

void ALandscapeProxy::RemoveNaniteComponents()
{
	for (ULandscapeNaniteComponent* NaniteComponent : NaniteComponents)
	{
		if (NaniteComponent)
		{
			// Don't call modify when detaching the nanite component, this is non-transactional "derived data", regenerated any time the source landscape data changes. This prevents needlessly dirtying the package :
			NaniteComponent->DetachFromComponent(FDetachmentTransformRules(EDetachmentRule::KeepRelative, /*bInCallModify = */false));
			NaniteComponent->DestroyComponent();
		}
	}

	NaniteComponents.Empty();

}
#endif // WITH_EDITOR

void ALandscapeProxy::EnableNaniteComponents(bool bInNaniteActive)
{
	for (ULandscapeNaniteComponent* NaniteComponent : NaniteComponents)
	{
		if (NaniteComponent)
		{
			NaniteComponent->SetEnabled(bInNaniteActive);
		}
	}
}

#if WITH_EDITOR
bool ALandscapeProxy::RemoveTargetLayer(const FName& Name,  bool bPostEditChange)
{
	Modify();
	
	// Remove data changed event listeners. The LayerInfoObjectAsset will not be reachable from the TargetLayers list in the next UnregisterAllComponents
	if (FLandscapeTargetLayerSettings* LayerSettings = TargetLayers.Find(Name))
	{
		if (LayerSettings && LayerSettings->LayerInfoObj)
		{
			LayerSettings->LayerInfoObj->OnLayerInfoChanged().RemoveAll(this);
		}
	}

	int32 NumItemsRemoved = TargetLayers.Remove(Name);
	if (FProperty* Property = StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ALandscapeProxy, TargetLayers)); Property != nullptr && bPostEditChange)
	{
		FPropertyChangedEvent PropertyChangedEvent(Property);
		PostEditChangeProperty(PropertyChangedEvent);	
	}
	
	return NumItemsRemoved > 0;
}

FLandscapeTargetLayerSettings& ALandscapeProxy::AddTargetLayer()
{
	return AddTargetLayer(GenerateUniqueTargetLayerName(), FLandscapeTargetLayerSettings());
}
	
FLandscapeTargetLayerSettings& ALandscapeProxy::AddTargetLayer(const FName& Name, const FLandscapeTargetLayerSettings& TargetLayerSettings, bool bPostEditChange)
{
	Modify();

	// LayerInfoObjectAsset events will be registered in PostRegisterAllComponents after PostEditChangeProperty runs
	FLandscapeTargetLayerSettings& Settings = TargetLayers.Add(Name.IsNone() ? GenerateUniqueTargetLayerName() : Name, TargetLayerSettings);
	if (FProperty* Property = StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ALandscapeProxy, TargetLayers)); Property != nullptr && bPostEditChange)
	{
		FPropertyChangedEvent PropertyChangedEvent(Property);
		PostEditChangeProperty(PropertyChangedEvent);	
	}
	
	return Settings;
}

const FName ALandscapeProxy::GenerateUniqueTargetLayerName() const
{
	int32 StartIndex = GetTargetLayers().Num();
	FName NewName;
	do
	{
		NewName = FName(FString::Format(TEXT("Layer_{0}"), { StartIndex++ }));
	} while (HasTargetLayer(NewName));

	return NewName;
}

bool ALandscapeProxy::UpdateTargetLayer(const FName& Name, const FLandscapeTargetLayerSettings& InTargetLayerSettings, bool bPostEditChange)
{
	FLandscapeTargetLayerSettings* TargetLayerSettings = TargetLayers.Find(Name);

	check(TargetLayerSettings);
	if (TargetLayerSettings)
	{
		Modify();

		// We may be updating the layer info asset here. The removed LayerInfoObjectAsset will not be reachable from the TargetLayers list in the next UnregisterAllComponents
		if (TargetLayerSettings->LayerInfoObj != nullptr)
		{
			TargetLayerSettings->LayerInfoObj->OnLayerInfoChanged().RemoveAll(this);
		}

		*TargetLayerSettings = InTargetLayerSettings;
		
		// LayerInfoObjectAsset event listeners will be re-registered in PostRegisterAllComponents after PostEditChangeProperty runs
		if (FProperty* Property = StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ALandscapeProxy, TargetLayers));  Property != nullptr && bPostEditChange)
		{
			FPropertyChangedEvent PropertyChangedEvent(Property);
			PostEditChangeProperty(PropertyChangedEvent);
		}

		return true;
	}

	return false;
}

bool ALandscapeProxy::HasTargetLayer(const FName& Name) const
{
	return TargetLayers.Find(Name) != nullptr; 
}

bool ALandscapeProxy::HasTargetLayer(const FLandscapeTargetLayerSettings& TargetLayerSettings) const
{
	return TargetLayers.FindKey(TargetLayerSettings) != nullptr;
}

bool ALandscapeProxy::HasTargetLayer(const ULandscapeLayerInfoObject* LayerInfoObject) const
{
	for (const auto& It : TargetLayers)
	{
		if (It.Value.LayerInfoObj == LayerInfoObject)
		{
			return true;
		}
	}
	return false;
}

const TMap<FName, FLandscapeTargetLayerSettings>& ALandscapeProxy::GetTargetLayers() const
{
	return TargetLayers;
}

TSet<ULandscapeLayerInfoObject*> ALandscapeProxy::GetValidTargetLayerObjects() const
{
	TSet<ULandscapeLayerInfoObject*> ValidLayerInfoObjects;
	Algo::ForEach(TargetLayers, [&ValidLayerInfoObjects](const TPair<FName, FLandscapeTargetLayerSettings>& InPair)
		{
			if (!InPair.Key.IsNone() && (InPair.Value.LayerInfoObj != nullptr))
			{
				check(!ValidLayerInfoObjects.Contains(InPair.Value.LayerInfoObj)); // There should be no duplicate
				check(InPair.Value.LayerInfoObj->GetLayerName() == InPair.Key); // And the name in the layer info object should match with the key
				ValidLayerInfoObjects.Add(InPair.Value.LayerInfoObj);
			}
		});
	return ValidLayerInfoObjects;
}
#endif

bool ALandscapeProxy::AreNaniteComponentsValid(const FGuid& InProxyContentId) const
{
	if (NaniteComponents.IsEmpty())
	{
		return false;
	}

	for (const ULandscapeNaniteComponent* NaniteComponent : NaniteComponents)
	{
		if (!NaniteComponent)
		{
			return false;
		}

		if (NaniteComponent->GetProxyContentId() != InProxyContentId)
		{
			return false;
		}
	}

	return true;
}

TSet<FPrimitiveComponentId> ALandscapeProxy::GetNanitePrimitiveComponentIds() const
{
	TSet<FPrimitiveComponentId> PrimitiveComponentIds;
	for (const ULandscapeNaniteComponent* NaniteComponent : NaniteComponents)
	{
		if (NaniteComponent && NaniteComponent->SceneProxy)
		{
			PrimitiveComponentIds.Add(NaniteComponent->SceneProxy->GetPrimitiveComponentId());
		}

	}
	return PrimitiveComponentIds;
}

bool ALandscapeProxy::AuditNaniteMaterials() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ALandscapeProxy::AuditMaterials);
	for (const ULandscapeNaniteComponent* NaniteComponent : NaniteComponents)
	{
		if (!NaniteComponent)
		{
			return false;
		}

		Nanite::FMaterialAudit NaniteMaterials;
		Nanite::AuditMaterials(NaniteComponent, NaniteMaterials);

		const bool bIsMaskingAllowed = Nanite::IsMaskingAllowed(GetWorld(), NaniteComponent->bForceNaniteForMasked);
		if (!NaniteMaterials.IsValid(bIsMaskingAllowed))
		{
			return false;
		}
	}
	return true;
}

void ALandscapeProxy::InvalidateGeneratedComponentData(bool bInvalidateLightingCache)
{
	InvalidateGeneratedComponentData(LandscapeComponents, bInvalidateLightingCache);
}

void ALandscapeProxy::InvalidateGeneratedComponentData(const TArray<ULandscapeComponent*>& Components, bool bInvalidateLightingCache)
{
	TMap<ALandscapeProxy*, TSet<ULandscapeComponent*>> ByProxy;
	for (auto Iter = Components.CreateConstIterator(); Iter; ++Iter)
	{
		ULandscapeComponent* Component = *Iter;
		if (bInvalidateLightingCache)
		{
			Component->InvalidateLightingCache();
		}
		ByProxy.FindOrAdd(Component->GetLandscapeProxy()).Add(Component);
	}

	for (auto Iter = ByProxy.CreateConstIterator(); Iter; ++Iter)
	{
		ALandscapeProxy* Proxy = Iter.Key();
		Proxy->FlushGrassComponents(&Iter.Value());

#if WITH_EDITOR
		FLandscapeProxyComponentDataChangedParams ChangeParams(Iter.Value());
		if (UWorld* ProxyWorld = Proxy->GetWorld())
		{
			ULandscapeSubsystem* Subsystem = ProxyWorld->GetSubsystem<ULandscapeSubsystem>();
			check(Subsystem != nullptr);
			if (Subsystem->IsLiveNaniteRebuildEnabled())
			{
				Proxy->GetAsyncWorkMonitor().SetDelayedUpdateTimer(FAsyncWorkMonitor::EAsyncWorkType::BuildNaniteMeshes, LandscapeNaniteBuildLag);
			}
			else
			{
				Proxy->InvalidateOrUpdateNaniteRepresentation(/* bInCheckContentId = */true, /*InTargetPlatform = */nullptr);
			}

			Subsystem->GetDelegateAccess().OnLandscapeProxyComponentDataChangedDelegate.Broadcast(Iter.Key(), ChangeParams);
		}
#endif // WITH_EDITOR

		Proxy->UpdateRenderingMethod();
	}
}

void ALandscapeProxy::InvalidateGeneratedComponentData(const TSet<ULandscapeComponent*>& Components, bool bInvalidateLightingCache)
{
	InvalidateGeneratedComponentData(Components.Array(), bInvalidateLightingCache);
}

void ALandscapeProxy::UpdateRenderingMethod()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ALandscapeProxy::UpdateRenderingMethod);
	if (LandscapeComponents.Num() == 0)
	{
		return;
	}

	bool bNaniteActive = false;
	if ((CVarRenderNaniteLandscape.GetValueOnGameThread() != 0) && HasNaniteComponents())
	{
		bNaniteActive = UseNanite(GShaderPlatformForFeatureLevel[GEngine->GetDefaultWorldFeatureLevel()]);
#if WITH_EDITOR
		if (ALandscape* LandscapeActor = GetLandscapeActor())
		{
			if (UWorld* World = LandscapeActor->GetWorld())
			{
				bNaniteActive = UseNanite(GShaderPlatformForFeatureLevel[World->GetFeatureLevel()]);
			}
		}
#endif //WITH_EDITOR
	}

#if WITH_EDITOR
	if (bNaniteActive)
	{
		bNaniteActive = IsNaniteMeshUpToDate();
	}
#endif //WITH_EDITOR

	if (bNaniteActive)
	{
		bNaniteActive = AuditNaniteMaterials();
	}

	for (ULandscapeComponent* Component : LandscapeComponents)
	{
		if (Component)
		{
			Component->SetNaniteActive(bNaniteActive);
		}
	}

	EnableNaniteComponents(bNaniteActive);
}

ULandscapeLODStreamingProxy_DEPRECATED::ULandscapeLODStreamingProxy_DEPRECATED(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR
FLandscapeProxyComponentDataChangedParams::FLandscapeProxyComponentDataChangedParams(const TSet<ULandscapeComponent*>& InComponents)
	: Components(InComponents.Array())
{
}

void FLandscapeProxyComponentDataChangedParams::ForEachComponent(TFunctionRef<void(const ULandscapeComponent*)> Func) const
{
	for (ULandscapeComponent* Component : Components)
	{
		Func(Component);
	}
}



bool FAsyncWorkMonitor::CheckIfUpdateTriggeredAndClear(EAsyncWorkType WorkType)
{
	bool& bUpdateTriggered = WorkTypeInfos[static_cast<uint32>(WorkType)].bUpdateTriggered;

	bool bReturn = bUpdateTriggered;
	bUpdateTriggered = false;
	return bReturn;
}

void FAsyncWorkMonitor::SetDelayedUpdateTimer(EAsyncWorkType WorkType, float InSecondsUntilDelayedUpdateTrigger)
{
	FAsyncWorkTypeInfo& Info = WorkTypeInfos[static_cast<uint32>(WorkType)];
	Info.SecondsUntilDelayedUpdateTrigger = InSecondsUntilDelayedUpdateTrigger;
}

void FAsyncWorkMonitor::Tick(float Detaltime)
{
	for (FAsyncWorkTypeInfo& Info : WorkTypeInfos)
	{
		if (Info.SecondsUntilDelayedUpdateTrigger > 0.0f)
		{
			Info.SecondsUntilDelayedUpdateTrigger -= Detaltime;

			if (Info.SecondsUntilDelayedUpdateTrigger <= 0.0f)
			{
				Info.SecondsUntilDelayedUpdateTrigger = 0.0f;
				Info.bUpdateTriggered = true;
			}
		}
	}
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
