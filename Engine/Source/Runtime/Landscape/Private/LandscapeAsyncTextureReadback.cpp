// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeAsyncTextureReadback.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "LandscapePrivate.h"

void FLandscapeAsyncTextureReadback::StartReadback_RenderThread(FRDGBuilder& GraphBuilder, FRDGTextureRef RDGTexture)
{
	check(!bStartedOnRenderThread && !AsyncReadback);
	check(RDGTexture->Desc.Format == PF_B8G8R8A8);
	AsyncReadback = MakeUnique<FRHIGPUTextureReadback>(TEXT("LandscapeGrassReadback"));
	AddEnqueueCopyPass(GraphBuilder, AsyncReadback.Get(), RDGTexture);
	FIntVector Size = RDGTexture->Desc.GetSize();
	TextureWidth = Size.X;
	TextureHeight = Size.Y;
	check(Size.Z == 1);

	bStartedOnRenderThread = true;
}

void FLandscapeAsyncTextureReadback::CheckAndUpdate_RenderThread(const bool bInForceFinish)
{
	// StartReadback_RenderThread() must execute before CheckAndUpdate_RenderThread()
	check(bStartedOnRenderThread);
	if (bFinishedOnRenderThread)
	{
		// already finished, nothing to do!
		return;
	}
	check(AsyncReadback.IsValid());
	if (bInForceFinish || AsyncReadback->IsReady())
	{
		FinishReadback_RenderThread();
	}
}

void FLandscapeAsyncTextureReadback::FinishReadback_RenderThread()
{
	check(bStartedOnRenderThread && AsyncReadback.IsValid());

	// always run the lock first -- even if we are cancelling the operation
	// this ensures that we wait until the readback is complete
	int32 RowPitchInPixels = 0;
	int32 BufferHeight = 0;
	void* SrcData = AsyncReadback->Lock(RowPitchInPixels, &BufferHeight);	// this will block if the readback is not yet ready
	check(SrcData);
	check(RowPitchInPixels >= TextureWidth);
	check(BufferHeight >= TextureHeight);

	if (!bCancel)	// we can skip the copy work if we're cancelling
	{
		// copy into ReadbackResults
		ReadbackResults.SetNumUninitialized(TextureWidth * TextureHeight);

		// OpenGL does not really support BGRA images and uses channnel swizzling to emulate them
		// so when we read them back we get internal RGBA representation
		const bool bSwapRBChannels = IsOpenGLPlatform(GMaxRHIShaderPlatform);

		if (!bSwapRBChannels && TextureWidth == RowPitchInPixels)
		{
			memcpy(ReadbackResults.GetData(), SrcData, TextureWidth * TextureHeight * sizeof(FColor));
		}
		else
		{
			// copy row by row
			FColor* Dst = ReadbackResults.GetData();
			FColor* Src = (FColor*)SrcData;
			if (bSwapRBChannels)
			{
				for (int y = 0; y < TextureHeight; y++)
				{
					for (int x = 0; x < TextureWidth; x++)
					{
						// swap B and R channels when copying
						Dst->B = Src->R;
						Dst->G = Src->G;
						Dst->R = Src->B;
						Dst->A = Src->A;
						Dst++;
						Src++;
					}
					Src += RowPitchInPixels - TextureWidth;
				}
			}
			else
			{
				for (int y = 0; y < TextureHeight; y++)
				{
					memcpy(Dst, Src, TextureWidth * sizeof(FColor));
					Dst += TextureWidth;
					Src += RowPitchInPixels;
				}
			}
		}
	}

	AsyncReadback->Unlock();
	AsyncReadback.Reset();

	bFinishedOnRenderThread = true;
}

bool FLandscapeAsyncTextureReadback::CheckAndUpdate(bool& bOutRenderCommandQueued, const bool bInForceFinish)
{
	// if we're already finished, then nothing to do!
	if (bFinishedOnRenderThread)
	{
		return true;
	}
	
	// We can only safely check the status on the render thread, so queue a command to check it
	FLandscapeAsyncTextureReadback* Readback = this;
	PendingRenderThreadCommands++;
	ENQUEUE_RENDER_COMMAND(FLandscapeAsyncTextureReadback_CheckAndUpdate)(
		[Readback, bInForceFinish](FRHICommandListImmediate& RHICmdList)
		{
			Readback->CheckAndUpdate_RenderThread(bInForceFinish);
			Readback->PendingRenderThreadCommands--;
		});

	bOutRenderCommandQueued = true;

	return false;
}

void FLandscapeAsyncTextureReadback::CancelAndSelfDestruct()
{
	// set the cancel flag, which will reduce work done by the finish command
	bCancel = true;

	FLandscapeAsyncTextureReadback* Readback = this;
	PendingRenderThreadCommands++;
	ENQUEUE_RENDER_COMMAND(FLandscapeAsyncTextureReadback_CancelAndSelfDestruct)(
		[Readback](FRHICommandListImmediate& RHICmdList)
		{
			check(Readback->bCancel);
			// check no commands were queued AFTER calling CancelAndSelfDestruct()
			check(Readback->PendingRenderThreadCommands == 1);

			if (Readback->bStartedOnRenderThread)
			{
				if (!Readback->bFinishedOnRenderThread)
				{
					// not yet finished, force run the finish command (may stall til completion)
					Readback->FinishReadback_RenderThread();
				}
				check(Readback->bFinishedOnRenderThread);
			}
			else
			{
				// it was never started - there shouldn't be any async readback allocated if nothing has started
				if (!ensure(!Readback->AsyncReadback.IsValid()))
				{
					UE_LOGF(LogLandscape, Warning, "In FLandscapeAsyncTextureReadback::CancelAndSelfDestruct(), readback not started, but AsyncReadback structure is unexpectedly allocated (%d, %d, %d, %d, %p, %d, %d, %d).  Attempting to clean it up.",
						Readback->bStartedOnRenderThread.load(),
						Readback->bFinishedOnRenderThread.load(),
						Readback->PendingRenderThreadCommands.load(),
						Readback->bCancel.load(),
						Readback->AsyncReadback.Get(),
						Readback->TextureWidth,
						Readback->TextureHeight,
						Readback->ReadbackResults.Num()
					);

					// but if there is, try to clean it up
					Readback->AsyncReadback.Reset();
				}
			}

			// self destruct (from render thread)
			delete Readback;
		});
}


void FLandscapeAsyncTextureReadback::QueueDeletionFromGameThread()
{
	check(IsInGameThread());
	check(bFinishedOnRenderThread);

	FLandscapeAsyncTextureReadback* Readback = this;
	PendingRenderThreadCommands++;
	ENQUEUE_RENDER_COMMAND(FLandscapeAsyncTextureReadback_QueueDeletion)(
		[Readback](FRHICommandListImmediate& RHICmdList)
		{
			// check no commands were queued AFTER calling QueueDeletionFromGameThread()
			check(Readback->PendingRenderThreadCommands == 1);

			// self destruct (from render thread)
			delete Readback;
		});
}
