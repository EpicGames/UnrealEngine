// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "RHIGPUReadback.h"
#include "RenderGraphFwd.h"

#include <atomic>

// Class that performs an async GPU texture readback for landscape purposes.
// Only supports BGRA8 texture format, and returns the results as an array of FColors.
class FLandscapeAsyncTextureReadback
{
private:
	// game thread / render thread shared state
	std::atomic_int PendingRenderThreadCommands = 0;			// number of render thread commands queued and not yet executed (for sanity checks)
	std::atomic_bool bCancel = false;							// set by game thread to signal cancel readback: we don't need to produce valid results

	// render thread state
	std::atomic_bool bStartedOnRenderThread = false;			// render thread start command has sent the readback command to the GPU
	std::atomic_bool bFinishedOnRenderThread = false;			// render thread finish command has made the data available to game thread

	TUniquePtr<FRHIGPUTextureReadback> AsyncReadback;			// render thread managed async readback structure

	// results - readable by game thread when bFinishedOnRenderThread
	int32 TextureWidth = 0;
	int32 TextureHeight = 0;
	TArray<FColor> ReadbackResults;

public:
	// construct on the game thread
	FLandscapeAsyncTextureReadback() {}

	// destruct on the render thread.  From game thread, call QueueDeletion() instead
	~FLandscapeAsyncTextureReadback()
	{
		check(IsInRenderingThread());
	}

	// use this to start an async readback operation from the render thread (on a render graph texture)
	void StartReadback_RenderThread(FRDGBuilder& GraphBuilder, FRDGTextureRef RDGTexture);

	// checks if the operation is ready to finish, and if it is, calls FinishReadback_RenderThread() to complete the readback.
	// bInForceFinish will force the readback to finish by stalling the render thread until it is ready
	void CheckAndUpdate_RenderThread(const bool bInForceFinish);

	// use this to finish an async readback operation from the render thread
	// Calling this when !IsComplete() will cause a stall until the GPU has completed the readback
	void FinishReadback_RenderThread();

	// Non-blocking call from the game thread to check readback status and start the finish command if needed.
	// Returns true when the AsyncReadbackResults are available to the game thread, at which point you can call TakeResults() to access them.
	// bOutRenderCommandQueued is set to true if any render command was queued in this call, othwerwise unchanged.
	// You must call this function occasionally or the readback will never complete, as the finish command is required.
	// bInForceFinish will force the finish process to be queued on the render thread (potentially stalling render thread, but forcing it to finish the readback)
	// bInForceFinish DOES NOT generally make the results immediately available, but it will ensure they are available after the render thread executes the command.
	bool CheckAndUpdate(bool& bOutRenderCommandQueued, const bool bInForceFinish);

	// Call from game thread to terminate any readback in flight and queue deletion of this (FLandscapeAsyncTextureReadback) on the render thread
	void CancelAndSelfDestruct();

	// Returns true when async readback results are available.  Call TakeResults() to retrieve them.
	bool IsComplete()
	{
		return bFinishedOnRenderThread;
	}

	// Retrieve the async readback results.  Requires readback to be complete.
	// This function returns its internal memory buffer, relinquishing control over it, so this function can only be called once.
	TArray<FColor> TakeResults(FIntPoint* OutSize)
	{
		check(bFinishedOnRenderThread);
		if (OutSize)
		{
			*OutSize = FIntPoint(TextureWidth, TextureHeight);
		}
		return MoveTemp(ReadbackResults);
	}

	FString ToString()
	{
		FString Result;
		Result.Appendf(TEXT("FLandscapeAsyncTextureReadback { RTStart: %d RTComplete: %d PendingCommands: %d Cancel: %d AsyncReadback: %p }"), bStartedOnRenderThread.load(), bFinishedOnRenderThread.load(), PendingRenderThreadCommands.load(), bCancel.load(), AsyncReadback.Get());
		return Result;
	}

	// once complete, call this to queue deletion of the readback object on the render thread
	// (this must be deleted on the render thread to avoid other render-thread queued commands from accessing a deallocated pointer)
	void QueueDeletionFromGameThread();
};

