// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeBlueprintBrushBase.h"
#include "CoreMinimal.h"
#include "GlobalShader.h"
#include "Engine/Engine.h"
#include "LandscapeProxy.h"
#include "Landscape.h"
#include "LandscapeEditLayerMergeRenderContext.h"
#include "LandscapeEditResourcesSubsystem.h"
#include "LandscapeEditTypes.h"
#include "LandscapeInfo.h"
#include "LandscapeLayerInfoObject.h"
#include "LandscapePrivate.h"
#include "Logging/MessageLog.h"
#include "Logging/TokenizedMessage.h"
#include "Materials/MaterialExpressionLandscapeVisibilityMask.h"
#include "Misc/MapErrors.h"
#include "Misc/UObjectToken.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "PixelShaderUtils.h"
#include "RHITransition.h"
#include "ShaderParameterStruct.h"
#include "TextureResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapeBlueprintBrushBase)

#define LOCTEXT_NAMESPACE "Landscape"

#if WITH_EDITOR
static const uint32 InvalidLastRequestLayersContentUpdateFrameNumber = 0;

static TAutoConsoleVariable<int32> CVarLandscapeBrushPadding(
	TEXT("landscape.BrushFramePadding"),
	5,
	TEXT("The number of frames to wait before pushing a full Landscape update when a brush is calling RequestLandscapeUpdate"));
#endif

// ----------------------------------------------------------------------------------

class FLandscapeEditLayersResolveLayerDataPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FLandscapeEditLayersResolveLayerDataPS);
	SHADER_USE_PARAMETER_STRUCT(FLandscapeEditLayersResolveLayerDataPS, FGlobalShader);

public:
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, InSourceTexture)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	class FResolveWeightmap : SHADER_PERMUTATION_BOOL("RESOLVE_WEIGHTMAP");
	using FPermutationDomain = TShaderPermutationDomain<FResolveWeightmap>;

	static FPermutationDomain GetPermutationVector(bool bResolveWeightmap)
	{
		FPermutationDomain PermutationVector;
		PermutationVector.Set<FResolveWeightmap>(bResolveWeightmap);
		return PermutationVector;
	}

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& InParameters)
	{
		return UE::Landscape::DoesPlatformSupportEditLayers(InParameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		UE::Landscape::ModifyShaderCompilerEnvironmentForDebug(OutEnvironment);

		OutEnvironment.SetDefine(TEXT("RESOLVE_LAYER_DATA"), 1);
	}

	static void ResolveLayerData(FRDGBuilder& GraphBuilder, FParameters* InParameters, const FIntPoint& InTextureSize, bool bResolveWeightmap)
	{
		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

		const FLandscapeEditLayersResolveLayerDataPS::FPermutationDomain PixelPermutationVector = FLandscapeEditLayersResolveLayerDataPS::GetPermutationVector(bResolveWeightmap);
		TShaderMapRef<FLandscapeEditLayersResolveLayerDataPS> PixelShader(ShaderMap, PixelPermutationVector);

		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			ShaderMap,
			RDG_EVENT_NAME("LandscapeLayers_ResolveLayerData"),
			PixelShader,
			InParameters,
			FIntRect(0, 0, InTextureSize.X, InTextureSize.Y));
	}
};

IMPLEMENT_GLOBAL_SHADER(FLandscapeEditLayersResolveLayerDataPS, "/Engine/Private/Landscape/LandscapeEditLayersUtils.usf", "ResolveLayerData", SF_Pixel);


// ----------------------------------------------------------------------------------

FLandscapeBrushParameters::FLandscapeBrushParameters(bool bInIsHeightmapMerge, const FTransform& InRenderAreaWorldTransform, const FIntPoint& InRenderAreaSize, UTextureRenderTarget2D* InCombinedResult, const FName& InWeightmapLayerName /*= FName()*/)
	: RenderAreaWorldTransform(InRenderAreaWorldTransform)
	, RenderAreaSize(InRenderAreaSize)
	, CombinedResult(InCombinedResult)
	, LayerType(bInIsHeightmapMerge ? ELandscapeToolTargetType::Heightmap : (InWeightmapLayerName == UMaterialExpressionLandscapeVisibilityMask::ParameterName) ? ELandscapeToolTargetType::Visibility : ELandscapeToolTargetType::Weightmap)
	, WeightmapLayerName(InWeightmapLayerName)
{}


// ----------------------------------------------------------------------------------

ALandscapeBlueprintBrushBase::ALandscapeBlueprintBrushBase(const FObjectInitializer& ObjectInitializer)
	: UpdateOnPropertyChange(true)
	, AffectHeightmap(false)
	, AffectWeightmap(false)
	, AffectVisibilityLayer(false)
#if WITH_EDITORONLY_DATA
	, OwningLandscape(nullptr)
	, bIsVisible(true)
	, LastRequestLayersContentUpdateFrameNumber(InvalidLastRequestLayersContentUpdateFrameNumber)
#endif // WITH_EDITORONLY_DATA
{
#if WITH_EDITOR
	USceneComponent* SceneComp = CreateDefaultSubobject<USceneComponent>(TEXT("RootComponent"));
	RootComponent = SceneComp;

	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.TickGroup = TG_DuringPhysics;
	PrimaryActorTick.bStartWithTickEnabled = true;
	PrimaryActorTick.SetTickFunctionEnable(true);
	bIsEditorOnlyActor = true;
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	bIsSpatiallyLoaded = false;
#endif // WITH_EDITORONLY_DATA
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS

#if WITH_EDITOR
UTextureRenderTarget2D* ALandscapeBlueprintBrushBase::Execute(const FLandscapeBrushParameters& InParameters)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ALandscapeBlueprintBrushBase::Execute);
	if ((InParameters.CombinedResult == nullptr) || (OwningLandscape == nullptr))
	{
		return nullptr;
	}

	// Do the render params require a new call to initialize?
	const FIntPoint NewLandscapeRenderTargetSize = FIntPoint(InParameters.CombinedResult->SizeX, InParameters.CombinedResult->SizeY);
	if (!CurrentRenderAreaWorldTransform.Equals(InParameters.RenderAreaWorldTransform) || (CurrentRenderAreaSize != InParameters.RenderAreaSize) || CurrentRenderTargetSize != NewLandscapeRenderTargetSize)
	{
		CurrentRenderAreaWorldTransform = InParameters.RenderAreaWorldTransform;
		CurrentRenderAreaSize = InParameters.RenderAreaSize;
		CurrentRenderTargetSize = NewLandscapeRenderTargetSize;

		TGuardValue<bool> AutoRestore(GAllowActorScriptExecutionInEditor, true);
		Initialize(CurrentRenderAreaWorldTransform, CurrentRenderAreaSize, CurrentRenderTargetSize);
	}

	// Time to render :
	FString LayerDetailString;
	if (InParameters.LayerType != ELandscapeToolTargetType::Heightmap)
	{
		LayerDetailString = FString::Format(TEXT(" ({0})"), { *InParameters.WeightmapLayerName.ToString() });
	}
	UTextureRenderTarget2D* Result = nullptr;
	{
		RHI_BREADCRUMB_EVENT_GAMETHREAD_F("BP Render", "BP Render %s (%s): %s", GetActorNameOrLabel(), UEnum::GetDisplayValueAsText(InParameters.LayerType).ToString(), LayerDetailString);

		TGuardValue<bool> AutoRestore(GAllowActorScriptExecutionInEditor, true);
		Result = RenderLayer(InParameters);
	}

	// If the BP brush failed to render, we still need to passthrough from the read RT to the write RT in order not to lose what has been merged so far : 
	if ((Result != nullptr) &&
		((Result->SizeX != InParameters.CombinedResult->SizeX) || (Result->SizeY != InParameters.CombinedResult->SizeY)))
	{
		UE_LOGF(LogLandscape, Warning, "In landscape %ls, the BP brush %ls failed to render for (%ls%ls). Make sure the brush properly implements RenderLayer and returns a render target of the appropriate size: expected (%i, %i), actual (%i, %i). This brush will be skipped until then.", 
			*OwningLandscape->GetActorLabel(), *GetActorLabel(), *UEnum::GetDisplayValueAsText(InParameters.LayerType).ToString(), *LayerDetailString, InParameters.CombinedResult->SizeX, InParameters.CombinedResult->SizeY, Result->SizeX, Result->SizeY);
		Result = nullptr;
	}

	return Result;
}
#endif // WITH_EDITOR

// Deprecated
UTextureRenderTarget2D* ALandscapeBlueprintBrushBase::Render_Implementation(bool InIsHeightmap, UTextureRenderTarget2D* InCombinedResult, const FName& InWeightmapLayerName)
{
	return nullptr;
}

UTextureRenderTarget2D* ALandscapeBlueprintBrushBase::RenderLayer_Implementation(const FLandscapeBrushParameters& InParameters)
{
	return RenderLayer_Native(InParameters);
}

UTextureRenderTarget2D* ALandscapeBlueprintBrushBase::RenderLayer_Native(const FLandscapeBrushParameters& InParameters)
{
	const bool bIsHeightmap = InParameters.LayerType == ELandscapeToolTargetType::Heightmap;

	// Without any implementation, we call the former Render method so content created before the deprecation will still work as expected.
	return Render(bIsHeightmap, InParameters.CombinedResult, InParameters.WeightmapLayerName);
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

void ALandscapeBlueprintBrushBase::Initialize_Implementation(const FTransform& InLandscapeTransform, const FIntPoint& InLandscapeSize, const FIntPoint& InLandscapeRenderTargetSize)
{
	Initialize_Native(InLandscapeTransform, InLandscapeSize, InLandscapeRenderTargetSize);
}

void ALandscapeBlueprintBrushBase::RequestLandscapeUpdate(bool bInUserTriggered)
{
#if WITH_EDITOR
	UE_LOGF(LogLandscape, Verbose, "ALandscapeBlueprintBrushBase::RequestLandscapeUpdate");
	if (OwningLandscape)
	{
		uint32 ModeMask = 0;
		if (CanAffectHeightmap())
		{
			ModeMask |= ELandscapeLayerUpdateMode::Update_Heightmap_Editing_NoCollision;
		}
		if (CanAffectWeightmap() || CanAffectVisibilityLayer())
		{
			ModeMask |= ELandscapeLayerUpdateMode::Update_Weightmap_Editing_NoCollision;
		}
		if (ModeMask)
		{
			OwningLandscape->RequestLayersContentUpdateForceAll((ELandscapeLayerUpdateMode)ModeMask, bInUserTriggered);
			// Just in case differentiate between 0 (default value and frame number)
			LastRequestLayersContentUpdateFrameNumber = GFrameNumber == InvalidLastRequestLayersContentUpdateFrameNumber ? GFrameNumber + 1 : GFrameNumber;
		}
	}
#endif // WITH_EDITOR
}

void ALandscapeBlueprintBrushBase::SetCanAffectHeightmap(bool bInCanAffectHeightmap)
{
#if WITH_EDITORONLY_DATA
	if (bInCanAffectHeightmap != AffectHeightmap)
	{
		Modify();
		AffectHeightmap = bInCanAffectHeightmap;
		if (OwningLandscape)
		{
			OwningLandscape->OnBlueprintBrushChanged();
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void ALandscapeBlueprintBrushBase::SetCanAffectWeightmap(bool bInCanAffectWeightmap)
{
#if WITH_EDITORONLY_DATA
	if (bInCanAffectWeightmap != AffectWeightmap)
	{
		Modify();
		AffectWeightmap = bInCanAffectWeightmap;
		if (OwningLandscape)
		{
			OwningLandscape->OnBlueprintBrushChanged();
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void ALandscapeBlueprintBrushBase::SetCanAffectVisibilityLayer(bool bInCanAffectVisibilityLayer)
{
#if WITH_EDITORONLY_DATA
	if (bInCanAffectVisibilityLayer != AffectVisibilityLayer)
	{
		Modify();
		AffectVisibilityLayer = bInCanAffectVisibilityLayer;
		if (OwningLandscape)
		{
			OwningLandscape->OnBlueprintBrushChanged();
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void ALandscapeBlueprintBrushBase::SetUsePowerOfTwoRenderTarget(bool bInUsePowerOfTwoRenderTarget)
{
#if WITH_EDITORONLY_DATA
	if (bInUsePowerOfTwoRenderTarget != bUsePowerOfTwoRenderTarget)
	{
		Modify();
		bUsePowerOfTwoRenderTarget = bInUsePowerOfTwoRenderTarget;
		if (OwningLandscape)
		{
			OwningLandscape->OnBlueprintBrushChanged();
		}
	}
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR

void ALandscapeBlueprintBrushBase::GetRendererStateInfo(const UE::Landscape::EditLayers::FMergeContext* InMergeContext,
	UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutSupportedTargetTypeState, UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutEnabledTargetTypeState, 
	TArray<UE::Landscape::EditLayers::FTargetLayerGroup>& OutTargetLayerGroups) const
{
	// What can the brush do?
	if (CanAffectHeightmap())
	{
		OutSupportedTargetTypeState.AddTargetTypeMask(ELandscapeToolTargetTypeFlags::Heightmap);
	}
	if (CanAffectWeightmap())
	{
		OutSupportedTargetTypeState.AddTargetTypeMask(ELandscapeToolTargetTypeFlags::Weightmap);
	}
	if (CanAffectVisibilityLayer())
	{
		OutSupportedTargetTypeState.AddTargetTypeMask(ELandscapeToolTargetTypeFlags::Visibility);
	}

	// What does it currently do?
	if (AffectsHeightmap())
	{
		OutEnabledTargetTypeState.AddTargetTypeMask(ELandscapeToolTargetTypeFlags::Heightmap);
	}
	if (AffectsWeightmap())
	{
		OutEnabledTargetTypeState.AddTargetTypeMask(ELandscapeToolTargetTypeFlags::Weightmap);
	}
	if (AffectsVisibilityLayer())
	{
		OutEnabledTargetTypeState.AddTargetTypeMask(ELandscapeToolTargetTypeFlags::Visibility);
	}

	// Mark which weightmap is supported/enabled : 
	if (!InMergeContext->IsHeightmapMerge() && CanAffectWeightmap())
	{
		InMergeContext->ForEachValidTargetLayer(
			[InMergeContext, this, &OutSupportedTargetTypeState, &OutEnabledTargetTypeState](int32 InTargetLayerIndex, const FName& InTargetLayerName, ULandscapeLayerInfoObject* InWeightmapLayerInfo)
			{
				if (CanAffectWeightmapLayer(InTargetLayerName) && InMergeContext->IsValidTargetLayerName(InTargetLayerName))
				{
					OutSupportedTargetTypeState.AddWeightmap(InTargetLayerIndex);
					if (AffectsWeightmapLayer(InTargetLayerName))
					{
						OutEnabledTargetTypeState.AddWeightmap(InTargetLayerIndex);
					}
				}
				return true;
			});
	}
}

TArray<UE::Landscape::EditLayers::FEditLayerRenderItem> ALandscapeBlueprintBrushBase::GetRenderItems(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const
{
	using namespace UE::Landscape::EditLayers;

	FEditLayerTargetTypeState SupportedTargetTypeState(InMergeContext);
	FEditLayerTargetTypeState EnabledTargetTypeState(InMergeContext);
	TArray<FTargetLayerGroup> DummyTargetLayerGroups;
	GetRendererStateInfo(InMergeContext, SupportedTargetTypeState, EnabledTargetTypeState, DummyTargetLayerGroups);

	// By default, for landscape BP brushes, we use FInputWorldArea::EType::Infinite, to indicate they can only reliably work when applied globally on the entire landscape
	//  This allows full backwards-compatibility but will prevent landscapes from benefiting from batched merge. Users will be able to indicate their brush works in a local fashion
	//  by overriding this and using another type of input world area
	FInputWorldArea InputWorldArea(FInputWorldArea::CreateInfinite());
	// By default, the brush only writes into the component itself (i.e. it renders to the area that it's currently being asked to render to):
	FOutputWorldArea OutputWorldArea(FOutputWorldArea::CreateLocalComponent());
	
	// Use EnabledTargetTypeState because we only want to tell what we'll actually be able to render to (instead of what we'd potentially be able to render to, i.e. what is "supported" by the brush) : 
	return { FEditLayerRenderItem(EnabledTargetTypeState, InputWorldArea, OutputWorldArea, /*bModifyExistingWeightmapsOnly = */false)};
}

bool ALandscapeBlueprintBrushBase::RenderLayer(UE::Landscape::EditLayers::FRenderParams& RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder)
{
	using namespace UE::Landscape;

	checkf(!RDGBuilderRecorder.IsRecording(), TEXT("ERenderFlags::RenderMode_Immediate means the command recorder should be recording at this point"));

	// By default, use the old way of rendering BP brushes : 

	// Swap the render targets so that the layer's input RT is the latest combined result :
	RenderParams.MergeRenderContext->CycleBlendRenderTargets(RDGBuilderRecorder);
	ULandscapeScratchRenderTarget* WriteRT = RenderParams.MergeRenderContext->GetBlendRenderTargetWrite();
	ULandscapeScratchRenderTarget* CurrentLayerReadRT = RenderParams.MergeRenderContext->GetBlendRenderTargetRead();

	WriteRT->TransitionTo(ERHIAccess::RTV, RDGBuilderRecorder);
	CurrentLayerReadRT->TransitionTo(ERHIAccess::SRVMask, RDGBuilderRecorder);

	const bool bIsHeightmapMerge = RenderParams.MergeRenderContext->IsHeightmapMerge();
	const TArray<FName> EnabledWeightmaps = RenderParams.RendererState.GetActiveTargetWeightmaps();
	
	ULandscapeEditResourcesSubsystem* LandscapeEditResourcesSubsystem = GEngine->GetEngineSubsystem<ULandscapeEditResourcesSubsystem>();
	check(LandscapeEditResourcesSubsystem != nullptr);
	ULandscapeScratchRenderTarget* CurrentLayerReadRT2D = CurrentLayerReadRT;

	// Because we only expose UTextureRenderTarget2D to BP, we need an additional scratch render target 2D that we will copy the current result of each target layer into, 
	//  so that the BP can use it as its source. We'd be inclined not to do it for bIsHeightmapMerge, where it is a texture 2D (i.e. exposed to BP), rather than a texture 
	//  2D array for weightmaps, but we still use a scratch texture for heightmaps in order to ensure the BP's render target size doesn't change too often (more details in the following comments)
	check((bIsHeightmapMerge && (CurrentLayerReadRT->IsTexture2D() && WriteRT->IsTexture2D()))
		|| (!bIsHeightmapMerge && (CurrentLayerReadRT->IsTexture2DArray() && WriteRT->IsTexture2DArray())));

	// The ideal size for the render target to use for this brush may differ from the requested render area. The reason is that the brush's Initialize function depends on the size of the 
	//  render target, so if, from one call to another, this size is changing because there happens to be a scratch render target big enough, we will have to call Initialize again 
	//  on the brush, which might have performance implications. So using a size that matches what we've already used in the past here allows such brushes not to uselessly go 
	//  through the initialization process again 
	FIntPoint RenderTargetIdealSize = RenderParams.RenderAreaSectionRect.Size().ComponentMax(CurrentRenderAreaSize);
	if (bUsePowerOfTwoRenderTarget)
	{
		// For backwards-compatibility with how it used to be, round up the RT size to the next power of 2 : 
		RenderTargetIdealSize.X = FMath::RoundUpToPowerOfTwo(RenderTargetIdealSize.X);
		RenderTargetIdealSize.Y = FMath::RoundUpToPowerOfTwo(RenderTargetIdealSize.Y);
	}

	// Use exact dimensions for the scratch texture here for the reason explained above (see RenderTargetIdealSize) :
	FScratchRenderTargetParams ScratchRenderTargetParams(TEXT("BPBrushScratchRT"), /*bInExactDimensions = */true, /*bInUseUAV = */false, /*bInTargetArraySlicesIndependently = */false,
		RenderTargetIdealSize, /*InNumSlices = */0, CurrentLayerReadRT->GetFormat(), CurrentLayerReadRT->GetClearColor(), ERHIAccess::CopyDest);
	FScratchRenderTargetScope ScratchTexture(ScratchRenderTargetParams);
	CurrentLayerReadRT2D = ScratchTexture.RenderTarget;

	// The original texture array will be accessed as ERHIAccess::CopySrc all along : 
	CurrentLayerReadRT->TransitionTo(ERHIAccess::CopySrc, RDGBuilderRecorder);

	const int32 NumTargetLayersInGroup = RenderParams.TargetLayerGroupLayerNames.Num();
	for (int32 TargetLayerIndexInGroup = 0; TargetLayerIndexInGroup < NumTargetLayersInGroup; ++TargetLayerIndexInGroup)
	{
		const FName TargetLayerName = RenderParams.TargetLayerGroupLayerNames[TargetLayerIndexInGroup];
		RHI_BREADCRUMB_EVENT_GAMETHREAD_F("Render Layer", "Render %s", TargetLayerName);

		// If necessary, copy from the source render target (slice) to the scratch render target 2D : 
		{
			RHI_BREADCRUMB_EVENT_GAMETHREAD_F("Copy Source", "Copy Source (slice %i) -> %s", TargetLayerIndexInGroup, CurrentLayerReadRT2D->GetDebugName());
			ULandscapeScratchRenderTarget::FCopyFromScratchRenderTargetParams CopyParams(CurrentLayerReadRT);
			// It's important to copy only the needed size, because CurrentLayerReadRT2D comes from ScratchTexture, whose size may be larger than RenderParams.RenderAreaSectionRect)
			CopyParams.CopySize = RenderParams.RenderAreaSectionRect.Size();
			// Copy from the proper slice in the texture array if any : 
			CopyParams.SourceSliceIndex = TargetLayerIndexInGroup;
			CurrentLayerReadRT2D->CopyFrom(CopyParams, RDGBuilderRecorder);
		}
		CurrentLayerReadRT2D->TransitionTo(ERHIAccess::SRVMask, RDGBuilderRecorder);

		check(CurrentLayerReadRT2D->GetCurrentState() == ERHIAccess::SRVMask);

		UTextureRenderTarget2D* ReadRT2D = CurrentLayerReadRT2D->GetRenderTarget2D();
		// If the BP brush failed to render, we still need to passthrough from the read RT to the write RT in order not to lose what has been merged so far : 
		UTextureRenderTarget2D* OutputRT2D = ReadRT2D;

		// Only render the target layer if it's effectively enabled for this merge : it's possible there are target layers in the group that we don't support or are not enabled so we have to 
		//  do the validation here first :
		if (bIsHeightmapMerge || EnabledWeightmaps.Contains(TargetLayerName))
		{
			// Execute (i.e. (Initialize/)Render the BP brush) : 
			FLandscapeBrushParameters BrushParameters(bIsHeightmapMerge, RenderParams.RenderAreaWorldTransform, RenderParams.RenderAreaSectionRect.Size(), ReadRT2D, TargetLayerName);
			if (UTextureRenderTarget2D* BrushOutputRT2D = Execute(BrushParameters))
			{
				// Only consider the BP brush's result if it's valid :
				OutputRT2D = BrushOutputRT2D;
			}
		}

		// TODO: handle conversion/handling of RT not same size as internal size
		check((OutputRT2D->SizeX == ReadRT2D->SizeX) && (OutputRT2D->SizeY == ReadRT2D->SizeY));

		// Resolve back to the write RT 
		{
			RHI_BREADCRUMB_EVENT_GAMETHREAD_F("Resolve BP Render Result", "Resolve BP Render Result -> %s (slice %i)", WriteRT->GetDebugName(), TargetLayerIndexInGroup);

			ENQUEUE_RENDER_COMMAND(ResolveLayerData)(
				[ InputResource = OutputRT2D->GetResource()
				, OutputResource = WriteRT->GetRenderTarget()->GetResource()
				, SliceIndex = TargetLayerIndexInGroup
				, bIsHeightmapMerge
				// It's important to copy only the needed size, because OutputRT2D is currently sized after ReadRT2D, which can be a render target larger than it needs to be (because 
				//  it can come from ScratchTexture, whose size may be larger than RenderParams.RenderAreaSectionRect)
				, ResolveTextureSize = RenderParams.RenderAreaSectionRect.Size()](FRHICommandListImmediate& InRHICmdList) mutable
			{
					FRDGBuilder GraphBuilder(InRHICmdList, RDG_EVENT_NAME("ResolveLayerData"));
					FRDGTextureRef InputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(InputResource->GetTextureRHI(), TEXT("InputTexture")));
					FRDGTextureRef OutputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(OutputResource->GetTextureRHI(), TEXT("OutputTexture")));

					FLandscapeEditLayersResolveLayerDataPS::FParameters* PassParameters = GraphBuilder.AllocParameters<FLandscapeEditLayersResolveLayerDataPS::FParameters>();
					PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputTexture, ERenderTargetLoadAction::ELoad, /*InMipIndex = */0, SliceIndex);
					PassParameters->InSourceTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(InputTexture));

					FLandscapeEditLayersResolveLayerDataPS::ResolveLayerData(GraphBuilder, PassParameters, ResolveTextureSize, /*bResolveWeightmap = */!bIsHeightmapMerge);

					// Don't let the graph builder transition the output texture back to SRVMask
					GraphBuilder.SetTextureAccessFinal(OutputTexture, ERHIAccess::RTV);
					GraphBuilder.Execute();
			});
		}
	}

	// Leave the render targets in the state they're expected to be in: 
	WriteRT->TransitionTo(ERHIAccess::RTV, RDGBuilderRecorder);
	CurrentLayerReadRT->TransitionTo(ERHIAccess::SRVMask, RDGBuilderRecorder);

	return true;
}

FString ALandscapeBlueprintBrushBase::GetEditLayerRendererDebugName() const
{
	return GetActorNameOrLabel();
}

TArray<UE::Landscape::EditLayers::FEditLayerRendererState> ALandscapeBlueprintBrushBase::GetEditLayerRendererStates(const UE::Landscape::EditLayers::FMergeContext* InMergeContext)
{
	using namespace UE::Landscape::EditLayers;

	if (OwningLandscape == nullptr)
	{
		return {};
	}

	FEditLayerRendererState RendererState(InMergeContext, this);
	// Force the renderer to be fully disabled in case we are asked to skip the brush :
	if (InMergeContext->ShouldSkipProceduralRenderers())
	{
		RendererState.DisableTargetTypeMask(ELandscapeToolTargetTypeFlags::All);
	}
	return { RendererState };
}

void ALandscapeBlueprintBrushBase::PushDeferredLayersContentUpdate()
{
	// Avoid computing collision and client updates every frame
	// Wait until we didn't trigger any more landscape update requests (padding of a couple of frames)
	if (OwningLandscape != nullptr &&
		LastRequestLayersContentUpdateFrameNumber != InvalidLastRequestLayersContentUpdateFrameNumber &&
		LastRequestLayersContentUpdateFrameNumber + CVarLandscapeBrushPadding.GetValueOnAnyThread() <= GFrameNumber)
	{
		uint32 ModeMask = 0;
		if (AffectsHeightmap())
		{
			ModeMask |= ELandscapeLayerUpdateMode::Update_Heightmap_All;
		}
		if (AffectsWeightmap() || AffectsVisibilityLayer())
		{
			ModeMask |= ELandscapeLayerUpdateMode::Update_Weightmap_All;
		}
		if (ModeMask)
		{
			OwningLandscape->RequestLayersContentUpdateForceAll((ELandscapeLayerUpdateMode)ModeMask);
		}
		LastRequestLayersContentUpdateFrameNumber = InvalidLastRequestLayersContentUpdateFrameNumber;
	}
}

void ALandscapeBlueprintBrushBase::Tick(float DeltaSeconds)
{
	// Forward the Tick to the instances class of this BP
	if (GetClass()->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
	{
		TGuardValue<bool> AutoRestore(GAllowActorScriptExecutionInEditor, true);
		ReceiveTick(DeltaSeconds);
	}

	Super::Tick(DeltaSeconds);
}

bool ALandscapeBlueprintBrushBase::ShouldTickIfViewportsOnly() const
{
	return true;
}

bool ALandscapeBlueprintBrushBase::IsLayerUpdatePending() const
{
	return GFrameNumber < LastRequestLayersContentUpdateFrameNumber + CVarLandscapeBrushPadding.GetValueOnAnyThread();
}

void ALandscapeBlueprintBrushBase::SetIsVisible(bool bInIsVisible)
{
	Modify();
	bIsVisible = bInIsVisible;
	if (OwningLandscape)
	{
		OwningLandscape->OnBlueprintBrushChanged();
	}
}

bool ALandscapeBlueprintBrushBase::CanAffectWeightmapLayer(const FName& InLayerName) const
{
	if (!CanAffectWeightmap())
	{
		return false;
	}

	// By default, it's the same implementation as AffectsWeightmapLayer : if the weightmap layer name is in our list, consider we can affect it :
	//  CanAffectWeightmapLayer can be overridden in child classes that don't use AffectedWeightmapLayers to list the weightmaps they can affect
	return AffectedWeightmapLayers.Contains(InLayerName);
}

bool ALandscapeBlueprintBrushBase::AffectsHeightmap() const
{
	return CanAffectHeightmap() && IsVisible();
}

bool ALandscapeBlueprintBrushBase::AffectsWeightmap() const
{
	return CanAffectWeightmap() && IsVisible();
}

bool ALandscapeBlueprintBrushBase::AffectsWeightmapLayer(const FName& InLayerName) const
{
	if (!AffectsWeightmap())
	{
		return false;
	}

	// By default, it's the same implementation as CanAffectWeightmapLayer : if the weightmap layer name is in our list, consider we do affect it :
	//  AffectsWeightmapLayer can be overridden in child classes that don't use AffectedWeightmapLayers to list the weightmaps they're currently affecting :
	return AffectedWeightmapLayers.Contains(InLayerName);
}

bool ALandscapeBlueprintBrushBase::AffectsVisibilityLayer() const
{
	return CanAffectVisibilityLayer() && IsVisible();
}

void ALandscapeBlueprintBrushBase::PostEditMove(bool bFinished)
{
	Super::PostEditMove(bFinished);
	RequestLandscapeUpdate();
}

void ALandscapeBlueprintBrushBase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	if (OwningLandscape && UpdateOnPropertyChange)
	{
		OwningLandscape->OnBlueprintBrushChanged();
	}
}

void ALandscapeBlueprintBrushBase::PostEditUndo()
{
	RequestLandscapeUpdate();
}

void ALandscapeBlueprintBrushBase::Destroyed()
{
	Super::Destroyed();
	if (OwningLandscape && !GIsReinstancing)
	{
		OwningLandscape->RemoveBrush(this);
	}
	OwningLandscape = nullptr;
}

void ALandscapeBlueprintBrushBase::CheckForErrors()
{
	Super::CheckForErrors();

	if (GetWorld() && !IsTemplate())
	{
		if (OwningLandscape == nullptr)
		{
			FMessageLog("MapCheck").Error()
				->AddToken(FUObjectToken::Create(this, FText::FromString(GetActorNameOrLabel())))
				->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_MissingLandscape", "This brush requires a Landscape. Add one to the map or remove the brush actor.")))
				->AddToken(FMapErrorToken::Create(TEXT("LandscapeBrushMissingLandscape")));
		}
	}
}

void ALandscapeBlueprintBrushBase::GetRenderDependencies(TSet<UObject*>& OutDependencies)
{
	TArray<UObject*> BPDependencies;
	GetBlueprintRenderDependencies(BPDependencies);

	OutDependencies.Append(BPDependencies);
}

void ALandscapeBlueprintBrushBase::SetOwningLandscape(ALandscape* InOwningLandscape)
{
	if (OwningLandscape == InOwningLandscape)
	{
		return;
	}

	const bool bAlwaysMarkDirty = false;
	Modify(bAlwaysMarkDirty);

	if (OwningLandscape)
	{
		OwningLandscape->OnBlueprintBrushChanged();
	}

	OwningLandscape = InOwningLandscape;

	if (OwningLandscape)
	{
		OwningLandscape->OnBlueprintBrushChanged();
	}
}

ALandscape* ALandscapeBlueprintBrushBase::GetOwningLandscape() const
{
	return OwningLandscape;
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
