// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LandscapeBlueprintSupport.cpp: Landscape blueprint functions
  =============================================================================*/

#include "CoreMinimal.h"
#include "Engine/World.h"
#include "LandscapeProxy.h"
#include "LandscapeSplineSegment.h"
#include "LandscapeSplineRaster.h"
#include "Components/SplineComponent.h"
#include "LandscapeComponent.h"
#include "Landscape.h"
#include "LandscapeEditLayer.h"
#include "LandscapePrivate.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "LandscapeProxy.h"

void ALandscapeProxy::EditorApplySpline(USplineComponent* InSplineComponent, float StartWidth, float EndWidth, float StartSideFalloff, float EndSideFalloff, float StartRoll, float EndRoll, int32 NumSubdivisions, bool bRaiseHeights, bool bLowerHeights, ULandscapeLayerInfoObject* PaintLayer, FName EditLayerName)
{
#if WITH_EDITOR
	if (InSplineComponent && !GetWorld()->IsGameWorld())
	{
		if (ALandscape* Landscape = GetLandscapeInfo()->LandscapeActor.Get())
		{
			const ULandscapeEditLayerBase* EditLayer = Landscape->GetEditLayerConst(EditLayerName);
			if (EditLayer == nullptr)
			{
				UE_LOGF(LogLandscape, Error, "Invalid landscape edit layer name (\"%ls\") for Edit Layers-enabled landscape. Cannot apply spline. ", *EditLayerName.ToString());
				return;
			}

			FScopedSetLandscapeEditingLayer Scope(Landscape, EditLayer ? EditLayer->GetGuid() : FGuid(), [=] { Landscape->RequestLayersContentUpdate(ELandscapeLayerUpdateMode::Update_All); });

			TArray<FLandscapeSplineInterpPoint> Points;
			LandscapeSplineRaster::FPointifyFalloffs Falloffs(StartSideFalloff, EndSideFalloff);
			LandscapeSplineRaster::Pointify(StaticCastPtr<const USplineComponent, USplineComponent>(InSplineComponent)->GetSplinePointsPosition(), Points, NumSubdivisions, 0.0f, 0.0f, StartWidth, EndWidth, StartWidth, EndWidth, Falloffs, StartRoll, EndRoll);

			FTransform SplineToWorld = InSplineComponent->GetComponentTransform();
			LandscapeSplineRaster::RasterizeSegmentPoints(GetLandscapeInfo(), MoveTemp(Points), SplineToWorld, bRaiseHeights, bLowerHeights, PaintLayer);
		}
	}
#endif
}

void ALandscapeProxy::SetLandscapeMaterialTextureParameterValue(FName ParameterName, class UTexture* Value)
{	
	if (bUseDynamicMaterialInstance)
	{
		for (ULandscapeComponent* Component : LandscapeComponents)
		{
			for (UMaterialInstanceDynamic* MaterialInstance : Component->MaterialInstancesDynamic)
			{
				if (MaterialInstance != nullptr)
				{
					MaterialInstance->SetTextureParameterValue(ParameterName, Value);
				}
			}
		}
	}
}

void ALandscapeProxy::SetLandscapeMaterialVectorParameterValue(FName ParameterName, FLinearColor Value)
{
	if (bUseDynamicMaterialInstance)
	{
		for (ULandscapeComponent* Component : LandscapeComponents)
		{
			for (UMaterialInstanceDynamic* MaterialInstance : Component->MaterialInstancesDynamic)
			{
				if (MaterialInstance != nullptr)
				{
					MaterialInstance->SetVectorParameterValue(ParameterName, Value);
				}
			}
		}		
	}
}

void ALandscapeProxy::SetLandscapeMaterialScalarParameterValue(FName ParameterName, float Value)
{
	if (bUseDynamicMaterialInstance)
	{
		for (ULandscapeComponent* Component : LandscapeComponents)
		{
			for (UMaterialInstanceDynamic* MaterialInstance : Component->MaterialInstancesDynamic)
			{
				if (MaterialInstance != nullptr)
				{
					MaterialInstance->SetScalarParameterValue(ParameterName, Value);
				}
			}
		}			
	}
}

void ALandscapeProxy::EditorSetLandscapeMaterial(UMaterialInterface* NewLandscapeMaterial)
{
#if WITH_EDITOR
	if (!GetWorld()->IsGameWorld())
	{
		LandscapeMaterial = NewLandscapeMaterial;
		FPropertyChangedEvent PropertyChangedEvent(FindFieldChecked<FProperty>(GetClass(), FName("LandscapeMaterial")));
		PostEditChangeProperty(PropertyChangedEvent);
	}
#endif
}

float ULandscapeComponent::EditorGetPaintLayerWeightByNameAtLocation(const FVector& InLocation, const FName InPaintLayerName)
{
#if WITH_EDITOR
	ULandscapeInfo* LandscapeInfo = GetLandscapeInfo();
	ULandscapeLayerInfoObject* PaintLayer = LandscapeInfo ? LandscapeInfo->GetLayerInfoByName(InPaintLayerName) : nullptr;
	return GetLayerWeightAtLocation(InLocation, PaintLayer);
#else
	return 0.f;
#endif
}

float ULandscapeComponent::EditorGetPaintLayerWeightAtLocation(const FVector& InLocation, ULandscapeLayerInfoObject* PaintLayer)
{
#if WITH_EDITOR
	if (PaintLayer)
	{
		return GetLayerWeightAtLocation(InLocation, PaintLayer);
	}
#endif
	return 0.f;
}

void ALandscapeProxy::SetVirtualTextureRenderPassType(ERuntimeVirtualTextureMainPassType InType)
{
	if (InType != VirtualTextureRenderPassType)
	{
		VirtualTextureRenderPassType = InType;

		for (ULandscapeComponent* Component : LandscapeComponents)
		{
			if (Component != nullptr)
			{
				Component->MarkRenderStateDirty();
			}
		}
	}
}
