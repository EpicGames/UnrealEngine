// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "AutoRTFM.h"
#include "GenericPlatform/GenericPlatformStackWalk.h"
#include "Materials/Material.h"
#include "Misc/Guid.h"
#include "SceneView.h"
#include "Serialization/MemoryReader.h"
#include "Stats/Stats.h"
#include "Serialization/BufferArchive.h"
#include "Misc/FeedbackContext.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopeLock.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"
#include "UObject/PropertyPortFlags.h"
#include "EngineDefines.h"
#include "Engine/EngineTypes.h"
#include "Components/SceneComponent.h"
#include "GameFramework/Actor.h"
#include "AI/Navigation/NavigationTypes.h"
#include "Misc/SecureHash.h"
#include "CollisionQueryParams.h"
#include "Engine/World.h"
#include "LandscapeSubsystem.h"
#include "LandscapeGrassMapsBuilder.h"
#include "LandscapeRender.h"
#include "LandscapeProxy.h"
#include "LandscapeSettings.h"
#include "LandscapeInfo.h"
#include "Interfaces/Interface_CollisionDataProvider.h"
#include "AI/NavigationSystemBase.h"
#include "LandscapeComponent.h"
#include "LandscapeLayerInfoObject.h"
#include "LandscapePrivate.h"
#include "PhysicsPublic.h"
#include "LandscapeDataAccess.h"
#include "DerivedDataCacheInterface.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "LandscapeHeightfieldCollisionComponent.h"
#include "LandscapeMeshCollisionComponent.h"
#include "FoliageInstanceBase.h"
#include "InstancedFoliageActor.h"
#include "InstancedFoliage.h"
#include "AI/NavigationSystemHelpers.h"
#include "Engine/CollisionProfile.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ProfilingDebugging/CookStats.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "EngineGlobals.h"
#include "EngineUtils.h"
#include "Engine/Engine.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Physics/PhysicsFiltering.h"
#include "Physics/PhysicsInterfaceCore.h"
#include "Physics/PhysicsInterfaceScene.h"
#include "Physics/PhysicsInterfaceUtils.h"
#include "PrimitiveSceneProxy.h"
#include "DynamicMeshBuilder.h"
#include "VisualLogger/VisualLogger.h"

#include "Chaos/ParticleHandle.h"
#include "Chaos/Vector.h"
#include "Chaos/Core.h"
#include "Chaos/HeightField.h"
#include "Chaos/ImplicitObjectTransformed.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/Experimental/ChaosCooking.h"
#include "Chaos/ChaosArchive.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "Chaos/Framework/PhysicsSolverBase.h"
#include "Chaos/Defines.h"
#include "PBDRigidsSolver.h"

using namespace PhysicsInterfaceTypes;

// Global switch for whether to read/write to DDC for landscape cooked data
// It's a lot faster to compute than to request from DDC, so always skip.
bool GLandscapeCollisionSkipDDC = true;


// Callback to flag scene proxy as dirty when cvars changes
static void OnCVarLandscapeShowCollisionMeshChanged(IConsoleVariable*)
{
	for (ULandscapeHeightfieldCollisionComponent* LandscapeHeightfieldCollisionComponent : TObjectRange<ULandscapeHeightfieldCollisionComponent>(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage))
	{
		LandscapeHeightfieldCollisionComponent->MarkRenderStateDirty();
	}
}

static TAutoConsoleVariable<int32> CVarLandscapeCollisionMeshShow(
	TEXT("landscape.CollisionMesh.Show"),
	static_cast<int>(EHeightfieldSource::Simple),
	TEXT("Selects which heightfield to visualize when ShowFlags.Collision is used. 0 to disable, 1 for simple, 2 for complex, 3 for editor only."),
	FConsoleVariableDelegate::CreateStatic(OnCVarLandscapeShowCollisionMeshChanged),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<float> CVarLandscapeCollisionMeshHeightOffset(
	TEXT("landscape.CollisionMesh.HeightOffset"),
	0.0f,
	TEXT("Offsets the collision mesh wireframe to assist in viewing from distances where the lower landscape lods might hide it."),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<bool> CVarLandscapeCollisionMeshShowPhysicalMaterial(
	TEXT("landscape.CollisionMesh.ShowPhysicalMaterial"),
	false,
	TEXT("When enabled, vertex colors of the collision mesh are chosen based on the physical material"),
	ECVF_RenderThreadSafe);

static FAutoConsoleVariable CVarAllowPhysicsStripping(
	TEXT("landscape.AllowPhysicsStripping"),
	true,
	TEXT("Enables the conditional stripping of physics data during cook.  Disabling this means the bStripPhysicsWhenCooked* will be ignored."));

static FAutoConsoleVariable CVarVisualLogShowFoliageSnapping(
		TEXT("landscape.VisualLog.ShowFoliageSnapping"),
		false,
		TEXT("When visual log is active, allows to leave a visual log of the areas where foliage snapping occurs."));


#if ENABLE_COOK_STATS
namespace LandscapeCollisionCookStats
{
	static FCookStats::FDDCResourceUsageStats HeightfieldUsageStats;
	static FCookStats::FDDCResourceUsageStats MeshUsageStats;
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
	{
		if (!GLandscapeCollisionSkipDDC)
		{
			HeightfieldUsageStats.LogStats(AddStat, TEXT("LandscapeCollision.Usage"), TEXT("Heightfield"));
			MeshUsageStats.LogStats(AddStat, TEXT("LandscapeCollision.Usage"), TEXT("Mesh"));
		}
	});
}
#endif

// Used for MT access to GSharedHeightfieldRefs.
// This is necessary when p.Chaos.EnableAsyncInitBody = true and ULandscapeHeightfieldCollisionComponent::AllowsAsyncPhysicsStateCreation returns true.
// Since ULandscapeHeightfieldCollisionComponent::AllowsAsyncPhysicsStateCreation only returns true when !WITH_EDITOR, there's no point to lock when WITH_EDITOR.
#if WITH_EDITOR
#define UE_SCOPELOCK_SHARED_HEIGHTFIELD_REFS()
#else
FTransactionallySafeCriticalSection GSharedHeightfieldRefsCriticalSection;
#define UE_SCOPELOCK_SHARED_HEIGHTFIELD_REFS() UE::TScopeLock Lock(GSharedHeightfieldRefsCriticalSection);
#endif
TMap<FGuid, ULandscapeHeightfieldCollisionComponent::FHeightfieldGeometryRef* > GSharedHeightfieldRefs;

ULandscapeHeightfieldCollisionComponent::FHeightfieldGeometryRef::FHeightfieldGeometryRef(FGuid& InGuid)
	: Guid(InGuid)
{
}

ULandscapeHeightfieldCollisionComponent::FHeightfieldGeometryRef::~FHeightfieldGeometryRef()
{
	// Remove ourselves from the shared map.
	UE_SCOPELOCK_SHARED_HEIGHTFIELD_REFS();
	GSharedHeightfieldRefs.Remove(Guid);
}

void ULandscapeHeightfieldCollisionComponent::FHeightfieldGeometryRef::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(sizeof(*this));

	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(UsedChaosMaterials.GetAllocatedSize());

	if (HeightfieldGeometry.IsValid())
	{
		TArray<uint8> Data;
		FMemoryWriter MemAr(Data);
		Chaos::FChaosArchive ChaosAr(MemAr);
		HeightfieldGeometry->Serialize(ChaosAr);
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(Data.Num());
	}

	if (HeightfieldSimpleGeometry.IsValid())
	{
		TArray<uint8> Data;
		FMemoryWriter MemAr(Data);
		Chaos::FChaosArchive ChaosAr(MemAr);
		HeightfieldSimpleGeometry->Serialize(ChaosAr);
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(Data.Num());
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
// Deprecated
ULandscapeMeshCollisionComponent_DEPRECATED::FTriMeshGeometryRef::FTriMeshGeometryRef()
{
}

// Deprecated
ULandscapeMeshCollisionComponent_DEPRECATED::FTriMeshGeometryRef::FTriMeshGeometryRef(FGuid& InGuid)
	: Guid(InGuid)
{
}

// Deprecated
ULandscapeMeshCollisionComponent_DEPRECATED::FTriMeshGeometryRef::~FTriMeshGeometryRef()
{
}

// Deprecated
void ULandscapeMeshCollisionComponent_DEPRECATED::FTriMeshGeometryRef::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

// Generate a new guid to force a recache of landscape collison derived data
#define LANDSCAPE_COLLISION_DERIVEDDATA_VER	TEXT("75E2F3A08BE44420813DD2F2AD34021D")

static FString GetHFDDCKeyString(const FName& Format, bool bDefMaterial, const FGuid& StateId, const TArray<UPhysicalMaterial*>& PhysicalMaterials)
{
	FGuid CombinedStateId;
	
	ensure(StateId.IsValid());

	if (bDefMaterial)
	{
		CombinedStateId = StateId;
	}
	else
	{
		// Build a combined state ID based on both the heightfield state and all physical materials.
		FBufferArchive CombinedStateAr;

		// Add main heightfield state
		FGuid HeightfieldState = StateId;
		CombinedStateAr << HeightfieldState;

		// Add physical materials
		for (UPhysicalMaterial* PhysicalMaterial : PhysicalMaterials)
		{
			FString PhysicalMaterialName = PhysicalMaterial->GetPathName().ToUpper();
			CombinedStateAr << PhysicalMaterialName;
		}

		uint32 Hash[5];
		FSHA1::HashBuffer(CombinedStateAr.GetData(), CombinedStateAr.Num(), (uint8*)Hash);
		CombinedStateId = FGuid(Hash[0] ^ Hash[4], Hash[1], Hash[2], Hash[3]);
	}

	FString InterfacePrefix = FString::Printf(TEXT("%s_%s"), TEXT("CHAOS"), Chaos::ChaosVersionGUID);

#if PLATFORM_CPU_ARM_FAMILY
	// Separate out arm keys as x64 and arm64 clang do not generate the same data for a given
	// input. Add the arm specifically so that a) we avoid rebuilding the current DDC and
	// b) we can remove it once we get arm64 to be consistent.
	InterfacePrefix.Append(TEXT("_arm64"));
#endif

	const FString KeyPrefix = FString::Printf(TEXT("%s_%s_%s"), *InterfacePrefix, *Format.ToString(), (bDefMaterial ? TEXT("VIS") : TEXT("FULL")));
	return FDerivedDataCacheInterface::BuildCacheKey(*KeyPrefix, LANDSCAPE_COLLISION_DERIVEDDATA_VER, *CombinedStateId.ToString());
}

void ULandscapeHeightfieldCollisionComponent::OnRegister()
{
	Super::OnRegister();

	if (GetLandscapeProxy())
	{
		// AActor::GetWorld checks for Unreachable and BeginDestroyed
		UWorld* World = GetLandscapeProxy()->GetWorld();
		if (World)
		{
			ULandscapeInfo* Info = GetLandscapeInfo();
			if (Info)
			{
				Info->RegisterCollisionComponent(this);
			}
		}
	}
}

void ULandscapeHeightfieldCollisionComponent::OnUnregister()
{
	Super::OnUnregister();

	// Save off the Heightfields for potential re-use later, because the original cooked data was deleted in OnRegister.
	// These heightfields are only used if this component gets re-registered before being destroyed.
	if (HeightfieldRef)
	{
		LocalHeightfieldGeometryRef = HeightfieldRef->HeightfieldGeometry;
		LocalHeightfieldSimpleGeometryRef = HeightfieldRef->HeightfieldSimpleGeometry;
	}

	// The physics object was destroyed in Super::OnUnregister. However we must
	// extend the lifetime of the collision until the enqueued destroy command
	// if processed on the physics thread, otherwise we may get Destroyed before
	// that happens and the collision geometry will be destroyed with us, leaving 
	// a dangling pointer in physics.
	// NOTE: we don't destroy collision in DestroyPhysicsState because we may
	// change the physics state without generating new collision geometry.
	DeferredDestroyCollision(HeightfieldRef);
	HeightfieldRef = nullptr;
	HeightfieldGuid = FGuid();

	{
		FWriteScopeLock Lock(CachedHeightFieldSamplesLock);
		CachedHeightFieldSamples.Empty();
	}

	if (GetLandscapeProxy())
	{
		// AActor::GetWorld checks for Unreachable and BeginDestroyed
		UWorld* World = GetLandscapeProxy()->GetWorld();

		// Game worlds don't have landscape infos
		if (World)
		{
			ULandscapeInfo* Info = GetLandscapeInfo();
			if (Info)
			{
				Info->UnregisterCollisionComponent(this);
			}
		}
	}
}

ECollisionEnabled::Type ULandscapeHeightfieldCollisionComponent::GetCollisionEnabled() const
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		ALandscapeProxy* Proxy = GetLandscapeProxy();

		return Proxy->BodyInstance.GetCollisionEnabled();
	}
	return ECollisionEnabled::QueryAndPhysics;
}

ECollisionResponse ULandscapeHeightfieldCollisionComponent::GetCollisionResponseToChannel(ECollisionChannel Channel) const
{
	ALandscapeProxy* Proxy = GetLandscapeProxy();

	return Proxy->BodyInstance.GetResponseToChannel(Channel);
}

ECollisionChannel ULandscapeHeightfieldCollisionComponent::GetCollisionObjectType() const
{
	ALandscapeProxy* Proxy = GetLandscapeProxy();

	return Proxy->BodyInstance.GetObjectType();
}

const FCollisionResponseContainer& ULandscapeHeightfieldCollisionComponent::GetCollisionResponseToChannels() const
{
	ALandscapeProxy* Proxy = GetLandscapeProxy();

	return Proxy->BodyInstance.GetResponseToChannels();
}

bool ULandscapeHeightfieldCollisionComponent::AllowsAsyncPhysicsStateCreation() const
{
#if WITH_EDITOR
	return Super::AllowsAsyncPhysicsStateCreation();
#else
	return true;
#endif
}

void ULandscapeHeightfieldCollisionComponent::OnCreatePhysicsState()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeHeightfieldCollisionComponent::OnCreatePhysicsState);
	USceneComponent::OnCreatePhysicsState(); // route OnCreatePhysicsState, skip PrimitiveComponent implementation

	if (!BodyInstance.IsValidBodyInstance())
	{
		CreateCollisionObject();

		// Debug display needs to update its representation, so we invalidate the collision component's render state : 
		MarkRenderStateDirty();

		if (IsValidRef(HeightfieldRef))
		{
			auto CreateShapesAndActors = [this]()
			{
				// Make transform for this landscape component PxActor
				FTransform LandscapeComponentTransform = GetComponentToWorld();
				FMatrix LandscapeComponentMatrix = LandscapeComponentTransform.ToMatrixWithScale();
				FVector LandscapeScale = LandscapeComponentMatrix.ExtractScaling();

				const bool bCreateSimpleCollision = SimpleCollisionSizeQuads > 0;
				const float SimpleCollisionScale = bCreateSimpleCollision ? CollisionScale * CollisionSizeQuads / SimpleCollisionSizeQuads : 0;

				// Create the geometry
				FVector FinalScale(LandscapeScale.X * CollisionScale, LandscapeScale.Y * CollisionScale, LandscapeScale.Z * LANDSCAPE_ZSCALE);

				FActorCreationParams Params;
				Params.InitialTM = LandscapeComponentTransform;
				Params.InitialTM.SetScale3D(FVector(0));
				Params.bQueryOnly = false;
				Params.bStatic = true;
				Params.Scene = GetWorld()->GetPhysicsScene();

#if USE_BODYINSTANCE_DEBUG_NAMES
				const FString DebugName = (GetOwner() != nullptr) ? FString::Printf(TEXT("%s:%s"), *GetOwner()->GetFullName(), *GetName()) : *GetName();
				BodyInstance.CharDebugName = MakeShareable(new TArray<ANSICHAR>(StringToArray<ANSICHAR>(*DebugName, DebugName.Len() + 1)));
				Params.DebugName = BodyInstance.CharDebugName.IsValid() ? BodyInstance.CharDebugName->GetData() : nullptr;
#endif

				FPhysicsActorHandle PhysHandle;
				FPhysicsInterface::CreateActor(Params, PhysHandle);
				Chaos::FRigidBodyHandle_External& Body_External = PhysHandle->GetGameThreadAPI();

				Chaos::FShapesArray ShapeArray;
				TArray<Chaos::FImplicitObjectPtr> Geoms;

				// First add complex geometry
				HeightfieldRef->HeightfieldGeometry->SetScale(FinalScale * LandscapeComponentTransform.GetScale3D().GetSignVector());
				Chaos::FImplicitObjectPtr ImplicitHeightField(HeightfieldRef->HeightfieldGeometry);
				Chaos::FImplicitObjectPtr ChaosHeightFieldFromCooked = MakeImplicitObjectPtr<Chaos::TImplicitObjectTransformed<Chaos::FReal, 3>>(ImplicitHeightField, Chaos::FRigidTransform3(FTransform::Identity));

				TUniquePtr<Chaos::FPerShapeData> NewShape = Chaos::FShapeInstanceProxy::Make(ShapeArray.Num(), ChaosHeightFieldFromCooked);

				// Setup filtering
				FPhysicsFilterBuilder Builder;
				Builder.SetOwnerID(GetOwner()->GetUniqueID());
				Builder.SetComponentID(GetUniqueID());
				Builder.SetCollisionChannelIndex(GetCollisionObjectType());
				Builder.SetResponses(GetCollisionResponseToChannels());
				// If we don't create separate collision for simple, this heightfield is used for both simple and complex.
				Builder.SetFlags(Chaos::EFilterFlags::SimpleCollision, !bCreateSimpleCollision);
				Builder.SetFlags(Chaos::EFilterFlags::ComplexCollision, true);
				Builder.SetFlags(Chaos::EFilterFlags::CCD, true);
				Builder.SetFlags(Chaos::EFilterFlags::StaticShape, true);

				const Chaos::Filter::FShapeFilterData ShapeFilterData = Builder.BuildShapeFilterData();
				const Chaos::Filter::FInstanceData FilterInstanceData = Builder.BuildInstanceData();

				NewShape->SetFilterInstanceData(FilterInstanceData);
				NewShape->SetShapeFilterData(ShapeFilterData);
				NewShape->SetMaterials(HeightfieldRef->UsedChaosMaterials);

				Geoms.Emplace(MoveTemp(ChaosHeightFieldFromCooked));
				ShapeArray.Emplace(MoveTemp(NewShape));

				// Add simple geometry if necessary
				if (bCreateSimpleCollision)
				{
					FVector FinalSimpleCollisionScale(LandscapeScale.X * SimpleCollisionScale, LandscapeScale.Y * SimpleCollisionScale, LandscapeScale.Z * LANDSCAPE_ZSCALE);
					HeightfieldRef->HeightfieldSimpleGeometry->SetScale(FinalSimpleCollisionScale);
					Chaos::FImplicitObjectPtr ImplicitHeightFieldSimple(HeightfieldRef->HeightfieldSimpleGeometry);
					Chaos::FImplicitObjectPtr ChaosSimpleHeightFieldFromCooked = MakeImplicitObjectPtr<Chaos::TImplicitObjectTransformed<Chaos::FReal, 3>>(ImplicitHeightFieldSimple, Chaos::FRigidTransform3(FTransform::Identity));

					TUniquePtr<Chaos::FPerShapeData> NewSimpleShape = Chaos::FShapeInstanceProxy::Make(ShapeArray.Num(), ChaosSimpleHeightFieldFromCooked);

					Builder.SetFlags(Chaos::EFilterFlags::ComplexCollision, false);
					Builder.SetFlags(Chaos::EFilterFlags::SimpleCollision, true);
					const Chaos::Filter::FShapeFilterData ShapeFilterDataSimple = Builder.BuildShapeFilterData();

					NewSimpleShape->SetFilterInstanceData(FilterInstanceData);
					NewSimpleShape->SetShapeFilterData(Builder.BuildShapeFilterData());
					NewSimpleShape->SetMaterials(HeightfieldRef->UsedChaosMaterials);

					Geoms.Emplace(MoveTemp(ChaosSimpleHeightFieldFromCooked));
					ShapeArray.Emplace(MoveTemp(NewSimpleShape));
				}

#if WITH_EDITOR
				// Create a shape for a heightfield which is used only by the landscape editor
				if (!GetWorld()->IsGameWorld() && !GetOutermost()->bIsCookedForEditor)
				{
					HeightfieldRef->EditorHeightfieldGeometry->SetScale(FinalScale * LandscapeComponentTransform.GetScale3D().GetSignVector());
					Chaos::FImplicitObjectPtr ImplicitEditorHeightField(HeightfieldRef->EditorHeightfieldGeometry);
					Chaos::FImplicitObjectPtr ChaosEditorHeightFieldFromCooked = MakeImplicitObjectPtr<Chaos::TImplicitObjectTransformed<Chaos::FReal, 3>>(ImplicitEditorHeightField, Chaos::FRigidTransform3(FTransform::Identity));

					TUniquePtr<Chaos::FPerShapeData> NewEditorShape = Chaos::FShapeInstanceProxy::Make(ShapeArray.Num(), ChaosEditorHeightFieldFromCooked);

					FCollisionResponseContainer CollisionResponse;
					CollisionResponse.SetAllChannels(ECollisionResponse::ECR_Ignore);
					CollisionResponse.SetResponse(ECollisionChannel::ECC_Visibility, ECR_Block);

					FPhysicsFilterBuilder EdBuilder;
					EdBuilder.SetCollisionChannelIndex(ECollisionChannel::ECC_Visibility);
					EdBuilder.SetResponses(CollisionResponse);
					EdBuilder.SetFlags(Chaos::EFilterFlags::SimpleCollision | Chaos::EFilterFlags::ComplexCollision, true);
					EdBuilder.SetFlags(Chaos::EFilterFlags::CCD, true);
					EdBuilder.SetFlags(Chaos::EFilterFlags::StaticShape, true);
					const Chaos::Filter::FShapeFilterData ShapeFilterDataEd = EdBuilder.BuildShapeFilterData();

					NewEditorShape->SetShapeFilterData(ShapeFilterDataEd);
					NewEditorShape->SetFilterInstanceData(FilterInstanceData);
					NewEditorShape->SetMaterials(HeightfieldRef->UsedChaosMaterials);

					Geoms.Emplace(MoveTemp(ChaosEditorHeightFieldFromCooked));
					ShapeArray.Emplace(MoveTemp(NewEditorShape));
				}
#endif // WITH_EDITOR
				// Push the shapes to the actor
				if (Geoms.Num() == 1)
				{
					Body_External.SetGeometry(Geoms[0]);
				}
				else
				{
					Body_External.SetGeometry(MakeImplicitObjectPtr<Chaos::FImplicitObjectUnion>(MoveTemp(Geoms)));
				}

				// Construct Shape Bounds
				for (auto& Shape : ShapeArray)
				{
					Chaos::FRigidTransform3 WorldTransform = Chaos::FRigidTransform3(Body_External.X(), Body_External.R());
					Shape->UpdateShapeBounds(WorldTransform);
				}
				Body_External.MergeShapesArray(MoveTemp(ShapeArray));

				// Set body instance data
				BodyInstance.PhysicsUserData = FPhysicsUserData(&BodyInstance);
				BodyInstance.OwnerComponent = this;
				BodyInstance.SetPhysicsActor(PhysHandle);

				Body_External.SetUserData(&BodyInstance.PhysicsUserData);

				return PhysHandle;
			};

			// Push the actor to the scene
			FPhysScene* PhysScene = GetWorld()->GetPhysicsScene();

			const bool bIsInGameThread = IsInGameThread();
			check(Chaos::CVars::bEnableAsyncInitBody || bIsInGameThread);
			FPhysicsActorHandle PhysHandle = bIsInGameThread ? CreateShapesAndActors() : FPhysicsActorHandle();
			FPhysicsCommand::ExecuteWrite(PhysScene, [&PhysHandle, &CreateShapesAndActors, PhysScene, bIsInGameThread]()
			{
				PhysHandle = !bIsInGameThread ? CreateShapesAndActors() : PhysHandle;
				TArray<FPhysicsActorHandle> Actors = { PhysHandle };
				const bool bImmediateAccelStructureInsertion = true;
				PhysScene->AddActorsToScene_AssumesLocked(Actors, bImmediateAccelStructureInsertion);
			});

			PhysScene->AddToComponentMaps(this, PhysHandle);
			if (BodyInstance.bNotifyRigidBodyCollision)
			{
				PhysScene->RegisterForCollisionEvents(this);
			}
		}
	}
}

void ULandscapeHeightfieldCollisionComponent::OnDestroyPhysicsState()
{
	Super::OnDestroyPhysicsState();
	
	if (FPhysScene_Chaos* PhysScene = GetWorld()->GetPhysicsScene())
	{
		FPhysicsActorHandle ActorHandle = BodyInstance.GetPhysicsActor();
		if (FPhysicsInterface::IsValid(ActorHandle))
		{
			PhysScene->RemoveFromComponentMaps(ActorHandle);
		}
		if (BodyInstance.bNotifyRigidBodyCollision)
		{
			PhysScene->UnRegisterForCollisionEvents(this);
		}
	}
}

void ULandscapeHeightfieldCollisionComponent::ApplyWorldOffset(const FVector& InOffset, bool bWorldShift)
{
	Super::ApplyWorldOffset(InOffset, bWorldShift);

	if (!bWorldShift || !FPhysScene::SupportsOriginShifting())
	{
		RecreatePhysicsState();
	}
}

#if UE_ENABLE_DEBUG_DRAWING
FPrimitiveSceneProxy* ULandscapeHeightfieldCollisionComponent::CreateSceneProxy()
{
	class FLandscapeHeightfieldCollisionComponentSceneProxy final : public FPrimitiveSceneProxy
	{
	public:
		SIZE_T GetTypeHash() const override
		{
			static size_t UniquePointer;
			return reinterpret_cast<size_t>(&UniquePointer);
		}

		// Constructor exists to populate Vertices and Indices arrays which are
		// used to construct the collision mesh inside GetDynamicMeshElements
		FLandscapeHeightfieldCollisionComponentSceneProxy(const ULandscapeHeightfieldCollisionComponent* InComponent, const TArray<Chaos::FMaterialHandle>& InUsedChaosMaterials, const Chaos::FHeightField& InHeightfield, const FLinearColor& InWireframeColor)
			: FPrimitiveSceneProxy(InComponent)
			, VertexFactory(InComponent->GetWorld()->GetFeatureLevel(), "FLandscapeHeightfieldCollisionComponentSceneProxy")
		{
			TArray<FDynamicMeshVertex> Vertices;

			const Chaos::FHeightField::FData<uint16>& GeomData = InHeightfield.GeomData; 
			const int32 NumRows = InHeightfield.GetNumRows();
			const int32 RowBounds = NumRows - 1;
			const int32 NumCols = InHeightfield.GetNumCols();
			const int32 ColBounds = NumCols - 1;
			const int32 NumVerts = NumRows * NumCols;
			const int32 NumTris = RowBounds * ColBounds * 2;
			Vertices.SetNumUninitialized(NumVerts);

			TArray<FColor, TInlineAllocator<16>> MaterialIndexColors;
			MaterialIndexColors.Reserve(InUsedChaosMaterials.Num());
			for (Chaos::FMaterialHandle MaterialHandle : InUsedChaosMaterials)
			{
				Chaos::FChaosPhysicsMaterial* ChaosMaterial = MaterialHandle.Get();
				UPhysicalMaterial* PhysicalMaterial = (ChaosMaterial != nullptr) ? FChaosUserData::Get<UPhysicalMaterial>(ChaosMaterial->UserData) : nullptr;
				MaterialIndexColors.Emplace(PhysicalMaterial ? PhysicalMaterial->DebugColor.ToFColor(/*bSRGB = */false) : FColor::Black);
			}

			for (int32 I = 0; I < NumVerts; I++)
			{
				const Chaos::FVec3 Point = GeomData.GetPointScaled(I);
				const int32 CurrentCol = I % NumCols;
				const int32 CurrentRow = I / NumCols;
				uint8 MaterialIndex = InHeightfield.GetMaterialIndex(CurrentCol, CurrentRow);
				Vertices[I].Position = FVector3f(static_cast<float>(Point.X), static_cast<float>(Point.Y), static_cast<float>(Point.Z));

				// Material indices are not defined for the last row/column in each component since they are per-triangle and not per-vertex.
				// To show something intuitive for the user, we simply extend the previous vertices.
				if (CurrentCol == ColBounds)
				{
					Vertices[I].Color = Vertices[I - 1].Color;
				}
				else if (CurrentRow == ColBounds)
				{
					Vertices[I].Color = Vertices[I - NumRows].Color;
				}
				else
				{
					Vertices[I].Color = (MaterialIndex == 255) ? FColor::Black  : MaterialIndexColors[MaterialIndex];
				}
			}
			IndexBuffer.Indices.SetNumUninitialized(NumTris * 3);

			// Editor heightfields don't have material indices (hence, no holes), in which case InHeightfield.GeomData.MaterialIndices.Num() == 1 : 
			const int32 NumMaterialIndices = InHeightfield.GeomData.MaterialIndices.Num();
			const bool bHasMaterialIndices = (NumMaterialIndices > 1);
			check(!bHasMaterialIndices || (NumMaterialIndices == (RowBounds * ColBounds)));

			int32 TriangleIdx = 0;
			for (int32 Y = 0; Y < RowBounds; Y++)
			{
				for (int32 X = 0; X < ColBounds; X++)
				{
					int32 DataIdx = X + Y * NumCols;
					bool bHole = false;

					if (bHasMaterialIndices)
					{
						// Material indices don't have the final row/column : 
						int32 MaterialIndicesDataIdx = X + Y * ColBounds;
						uint8 LayerIdx = InHeightfield.GeomData.MaterialIndices[MaterialIndicesDataIdx];
						bHole = (LayerIdx == TNumericLimits<uint8>::Max());
					}

					if (bHole)
					{
						IndexBuffer.Indices[TriangleIdx + 0] = (X + 0) + (Y + 0) * NumCols;
						IndexBuffer.Indices[TriangleIdx + 1] = IndexBuffer.Indices[TriangleIdx + 0];
						IndexBuffer.Indices[TriangleIdx + 2] = IndexBuffer.Indices[TriangleIdx + 0];
					}
					else
					{
						IndexBuffer.Indices[TriangleIdx + 0] = (X + 0) + (Y + 0) * NumCols;
						IndexBuffer.Indices[TriangleIdx + 1] = (X + 1) + (Y + 1) * NumCols;
						IndexBuffer.Indices[TriangleIdx + 2] = (X + 1) + (Y + 0) * NumCols;
					}

					TriangleIdx += 3;

					if (bHole)
					{
						IndexBuffer.Indices[TriangleIdx + 0] = (X + 0) + (Y + 0) * NumCols;
						IndexBuffer.Indices[TriangleIdx + 1] = IndexBuffer.Indices[TriangleIdx + 0];
						IndexBuffer.Indices[TriangleIdx + 2] = IndexBuffer.Indices[TriangleIdx + 0];
					}
					else
					{
						IndexBuffer.Indices[TriangleIdx + 0] = (X + 0) + (Y + 0) * NumCols;
						IndexBuffer.Indices[TriangleIdx + 1] = (X + 0) + (Y + 1) * NumCols;
						IndexBuffer.Indices[TriangleIdx + 2] = (X + 1) + (Y + 1) * NumCols;
					}

					TriangleIdx += 3;
				}
			}

			// Allocate the static vertex resources now 
			if (Vertices.Num() > 0)
			{
#if RHI_ENABLE_RESOURCE_INFO
				const FName OwnerName(TEXT("FLandscapeHeightfieldCollisionComponentSceneProxy ") + GetOwnerName().ToString());
#else
				const FName OwnerName = NAME_None;
#endif

				VertexBuffers.InitFromDynamicVertex(&VertexFactory, Vertices);
				BeginInitResource(OwnerName, &VertexBuffers.PositionVertexBuffer);
				BeginInitResource(OwnerName, &VertexBuffers.StaticMeshVertexBuffer);
				BeginInitResource(OwnerName, &VertexBuffers.ColorVertexBuffer);
				BeginInitResource(OwnerName, &IndexBuffer);
				BeginInitResource(OwnerName, &VertexFactory);

				WireframeMaterialInstance.Reset(new FColoredMaterialRenderProxy(
					GEngine->WireframeMaterial ? GEngine->WireframeMaterial->GetRenderProxy() : nullptr,
					InWireframeColor));

				VertexColorMaterialInstance.Reset(new FColoredMaterialRenderProxy(
					GEngine->VertexColorViewModeMaterial_ColorOnly ? GEngine->VertexColorViewModeMaterial_ColorOnly->GetRenderProxy() : nullptr,
					FColor::White));
			}
		}

		virtual ~FLandscapeHeightfieldCollisionComponentSceneProxy()
		{
			VertexBuffers.PositionVertexBuffer.ReleaseResource();
			VertexBuffers.StaticMeshVertexBuffer.ReleaseResource();
			VertexBuffers.ColorVertexBuffer.ReleaseResource();
			IndexBuffer.ReleaseResource();
			VertexFactory.ReleaseResource();
			IndexBuffer.ReleaseResource();
		}

		virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
		{
			FMatrix LocalToWorldNoScale = GetLocalToWorld();
			LocalToWorldNoScale.RemoveScaling();

			const bool bDrawCollision = ViewFamily.EngineShowFlags.Collision && ViewFamily.EngineShowFlags.Landscape && IsCollisionEnabled();

			const bool bShowPhysicalMaterial = CVarLandscapeCollisionMeshShowPhysicalMaterial.GetValueOnRenderThread();
			const float HeightOffset = CVarLandscapeCollisionMeshHeightOffset.GetValueOnRenderThread();
			FVector ZAxis = LocalToWorldNoScale.GetUnitAxis(EAxis::Z);
			LocalToWorldNoScale = LocalToWorldNoScale.ConcatTranslation(FVector(0.0, 0.0, HeightOffset));
			FBoxSphereBounds Bounds = GetBounds();
			Bounds.Origin += ZAxis * HeightOffset;

			const TUniquePtr<FColoredMaterialRenderProxy>& MaterialToUse = bShowPhysicalMaterial ? VertexColorMaterialInstance : WireframeMaterialInstance;

			if (bDrawCollision && AllowDebugViewmodes() && MaterialToUse.IsValid())
			{
				for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
				{
					if (VisibilityMap & (1 << ViewIndex))
					{
						const FSceneView* View = Views[ViewIndex];
						
						FMeshBatch& MeshBatch = Collector.AllocateMesh();
						MeshBatch.MaterialRenderProxy = MaterialToUse.Get();
						MeshBatch.bWireframe = true;
						MeshBatch.VertexFactory = &VertexFactory;
						MeshBatch.ReverseCulling = false;
						MeshBatch.Type = PT_TriangleList;
						MeshBatch.DepthPriorityGroup = SDPG_World;
						MeshBatch.bCanApplyViewModeOverrides = true;

						FMeshBatchElement& BatchElement = MeshBatch.Elements[0];
						BatchElement.IndexBuffer = &IndexBuffer;
						BatchElement.FirstIndex = 0;
						BatchElement.NumPrimitives = IndexBuffer.Indices.Num() / 3;
						check(BatchElement.NumPrimitives != 0);
						BatchElement.MinVertexIndex = 0;
						BatchElement.MaxVertexIndex = VertexBuffers.PositionVertexBuffer.GetNumVertices() - 1;

						FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
						DynamicPrimitiveUniformBuffer.Set(Collector.GetRHICommandList(), LocalToWorldNoScale, LocalToWorldNoScale, Bounds, GetLocalBounds(), /*bReceivesDecals = */false, /*bHasPrecomputedVolumetricLightmap = */false, AlwaysHasVelocity());
						BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;

						Collector.AddMesh(ViewIndex, MeshBatch);
					}
				}
			}
		}

		virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
		{
			// Should we draw this because collision drawing is enabled, and we have collision
			const bool bShowForCollision = View->Family->EngineShowFlags.Collision && IsCollisionEnabled();

			FPrimitiveViewRelevance Result;
			Result.bDrawRelevance = IsShown(View) || bShowForCollision;
			Result.bDynamicRelevance = true;
			Result.bStaticRelevance = false;
			Result.bShadowRelevance = false;
			Result.bEditorPrimitiveRelevance = UseEditorCompositing(View);
			return Result;
		}

		virtual uint32 GetMemoryFootprint(void) const override { return(sizeof(*this) + GetAllocatedSize()); }
		uint32 GetAllocatedSize(void) const { return static_cast<uint32>(FPrimitiveSceneProxy::GetAllocatedSize()); }

	private:
		TUniquePtr<FColoredMaterialRenderProxy> WireframeMaterialInstance = nullptr;
		TUniquePtr<FColoredMaterialRenderProxy> VertexColorMaterialInstance = nullptr;

		FStaticMeshVertexBuffers VertexBuffers;
		FDynamicMeshIndexBuffer32 IndexBuffer;
		FLocalVertexFactory VertexFactory;
	};

	FLandscapeHeightfieldCollisionComponentSceneProxy* Proxy = nullptr;
	if (ULandscapeSubsystem* LandscapeSubsystem = this->GetWorld()->GetSubsystem<ULandscapeSubsystem>(); 
		LandscapeSubsystem && !LandscapeSubsystem->AnyViewShowCollisions())
	{
		return Proxy;
	}
	
	if (HeightfieldRef.IsValid() && IsValidRef(HeightfieldRef))
	{
		const Chaos::FHeightField* LocalHeightfield = nullptr;
		FLinearColor WireframeColor;

		switch (static_cast<EHeightfieldSource>(CVarLandscapeCollisionMeshShow.GetValueOnGameThread()))
		{
		case EHeightfieldSource::None:
			WireframeColor = FColor(0, 0, 0, 0);
			break;
		case EHeightfieldSource::Simple:
			if (HeightfieldRef->HeightfieldSimpleGeometry.IsValid())
			{
				LocalHeightfield = HeightfieldRef->HeightfieldSimpleGeometry.GetReference();
			}
			else if (HeightfieldRef->HeightfieldGeometry.IsValid())
			{
				LocalHeightfield = HeightfieldRef->HeightfieldGeometry.GetReference();
			}

			WireframeColor = FColor(157, 149, 223, 255);
			break;

		case EHeightfieldSource::Complex:
			if (HeightfieldRef->HeightfieldGeometry.IsValid())
			{
				LocalHeightfield = HeightfieldRef->HeightfieldGeometry.GetReference();
			}

			WireframeColor = FColor(0, 255, 255, 255);
			break;

		case EHeightfieldSource::Editor:
#if WITH_EDITOR
			if (HeightfieldRef->EditorHeightfieldGeometry.IsValid())
			{
				LocalHeightfield = HeightfieldRef->EditorHeightfieldGeometry.GetReference();
			}
#endif // WITH_EDITOR
			WireframeColor = FColor(157, 223, 149, 255);
			break;

		default:
			UE_LOGF(LogLandscape, Warning, "Invalid Value for CVar landscape.CollisionMesh.Show");
		}

		if (LocalHeightfield != nullptr)
		{
			Proxy = new FLandscapeHeightfieldCollisionComponentSceneProxy(this, HeightfieldRef->UsedChaosMaterials, *LocalHeightfield, WireframeColor);
		}
	}

	return Proxy;
}
#endif // UE_ENABLE_DEBUG_DRAWING

void ULandscapeHeightfieldCollisionComponent::CreateCollisionObject()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeHeightfieldCollisionComponent::CreateCollisionObject);
	LLM_SCOPE(ELLMTag::ChaosLandscape);

	auto RegisterMaterials = [this](TArray<Chaos::FMaterialHandle>& DestMaterialHandles, const TArray<TObjectPtr<UPhysicalMaterial>>& SrcMaterials)
		{
			for (UPhysicalMaterial* PhysicalMaterial : SrcMaterials)
			{
				if (PhysicalMaterial)
				{
					//todo: total hack until we get landscape fully converted to chaos
					DestMaterialHandles.Add(PhysicalMaterial->GetPhysicsMaterial());
				}
				else
				{
					// If a material fails to load, we have a null value in PhysicalMaterialRenderObjects and end up here.  Substitute the default material.
					ALandscapeProxy* Proxy = GetLandscapeProxy();
					UPhysicalMaterial* DefMaterial = (Proxy && Proxy->DefaultPhysMaterial) ? Proxy->DefaultPhysMaterial : GEngine->DefaultPhysMaterial;
					DestMaterialHandles.Add(DefMaterial->GetPhysicsMaterial());
				}
			}
		};

	// If we have not created a heightfield yet - do it now.
	if (!IsValidRef(HeightfieldRef))
	{
		UE_SCOPELOCK_SHARED_HEIGHTFIELD_REFS();
		UWorld* World = GetWorld();

#if WITH_EDITOR
		const bool bNeedsEditorHeightField = World && !World->IsGameWorld() && !GetOutermost()->bIsCookedForEditor;
#endif // WITH_EDITOR
		FHeightfieldGeometryRef* ExistingHeightfieldRef = nullptr;
		bool bReuseIsValid = true;  // Are pre-existing copies of CookedCollisionData valid to re-use.

		if (!HeightfieldGuid.IsValid())
		{
#if !WITH_EDITORONLY_DATA
			uint32 CollisionHash = 0;
#endif
			HeightfieldGuid = FGuid::NewDeterministicGuid(GetPathName(), CollisionHash);
			bReuseIsValid = false;
		}
		else
		{
			// Look for a heightfield object with the current Guid (this occurs with PIE)
			ExistingHeightfieldRef = GSharedHeightfieldRefs.FindRef(HeightfieldGuid);
		}

#if WITH_EDITOR
		// Use existing heightfield except if it is missing its editor heightfield and the component needs it.
		if (ExistingHeightfieldRef && (!bNeedsEditorHeightField || ExistingHeightfieldRef->EditorHeightfieldGeometry != nullptr))
#else // WITH_EDITOR
		if (ExistingHeightfieldRef)
#endif // !WITH_EDITOR
		{
			HeightfieldRef = ExistingHeightfieldRef;
		}
		else
		{
#if WITH_EDITOR
			// This should only occur if a level prior to VER_UE4_LANDSCAPE_COLLISION_DATA_COOKING 
			// was resaved using a commandlet and not saved in the editor, or if a PhysicalMaterial asset was deleted.
			if (CookedPhysicalMaterials.Num() == 0 || CookedPhysicalMaterials.Contains(nullptr))
			{
				bReuseIsValid = false;
			}

			static FName PhysicsFormatName(FPlatformProperties::GetPhysicsFormat());

			// Use the fast bypass path when the data isn't already available and DDC is disabled for landscape collision.
			bool bBypassCookingStep = GLandscapeCollisionSkipDDC && !GetOutermost()->bIsCookedForEditor &&
				!(bReuseIsValid && CookedCollisionData.Num() > 0);
			if (bBypassCookingStep)
			{
				HeightfieldRef = GSharedHeightfieldRefs.Add(HeightfieldGuid, new FHeightfieldGeometryRef(HeightfieldGuid));

				const bool bGenerateSimpleCollision = SimpleCollisionSizeQuads > 0;
				bool bSuccess = GenerateCollisionObjects(PhysicsFormatName, false, HeightfieldRef->HeightfieldGeometry, bGenerateSimpleCollision, HeightfieldRef->HeightfieldSimpleGeometry, MutableView(CookedPhysicalMaterials));

				if (bNeedsEditorHeightField)
				{
					Chaos::FHeightFieldPtr DummySimpleRef;
					TArray<UPhysicalMaterial*> CookedMaterialsEd;
					bSuccess &= GenerateCollisionObjects(PhysicsFormatName, true, HeightfieldRef->EditorHeightfieldGeometry, false, DummySimpleRef, CookedMaterialsEd);
				}

				if (bSuccess)
				{
					RegisterMaterials(HeightfieldRef->UsedChaosMaterials, CookedPhysicalMaterials);
				}
				else
				{
					// A heightfield with invalid content is as good as an invalid heightfield and ULandscapeHeightfieldCollisionComponent::CreateCollisionObject won't survive it anyway 
					//  so we're better off invalidating it here entirely : 
					HeightfieldRef = nullptr;
				}

				// Return unconditionally even if GenerateCollisionObjects failed.  CookCollisionData succeeds or fails in the same conditions, so trying that
				// as a fallback doesn't gain anything.
				return;
			}

			CookCollisionData(PhysicsFormatName, false, bReuseIsValid, CookedCollisionData, MutableView(CookedPhysicalMaterials));

#endif //WITH_EDITOR

			if (CookedCollisionData.IsEmpty())
			{
				if (LocalHeightfieldGeometryRef.IsValid())
				{
					// create heightfield ref from the local heightfield cached copy
					HeightfieldRef = GSharedHeightfieldRefs.Add(HeightfieldGuid, new FHeightfieldGeometryRef(HeightfieldGuid));

					HeightfieldRef->HeightfieldGeometry = MoveTemp(LocalHeightfieldGeometryRef);
					if (LocalHeightfieldSimpleGeometryRef.IsValid())
					{
						HeightfieldRef->HeightfieldSimpleGeometry = MoveTemp(LocalHeightfieldSimpleGeometryRef);
					}
				}
				else
				{
					if (bCookedCollisionDataWasDeleted)
					{
						// only complain if we actually deleted the data.. otherwise it may have been intentional
						UE_LOGF(LogLandscape, Warning, "Tried to create heightfield collision for component '%ls', but the collision data was deleted!", *GetName());
					}
					return;
				}

				// Fallthrough to the shared register materials code below
			}
			else
			{
				HeightfieldRef = GSharedHeightfieldRefs.Add(HeightfieldGuid, new FHeightfieldGeometryRef(HeightfieldGuid));

				// Create heightfields
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(CreateCollisionObject_ChaosStream);
					FMemoryReader Reader(CookedCollisionData);
					Chaos::FChaosArchive Ar(Reader);
					bool bContainsSimple = false;
					Ar << bContainsSimple;
					Ar << HeightfieldRef->HeightfieldGeometry;

					if (bContainsSimple)
					{
						Ar << HeightfieldRef->HeightfieldSimpleGeometry;
					}
				}
			}

			RegisterMaterials(HeightfieldRef->UsedChaosMaterials, CookedPhysicalMaterials);

			// Release cooked collision data
			// In cooked builds created collision object will never be deleted while component is alive, so we don't need this data anymore
			if (FPlatformProperties::RequiresCookedData() || (World && World->IsGameWorld()))
			{
				CookedCollisionData.Empty();
				bCookedCollisionDataWasDeleted = true;
			}

#if WITH_EDITOR
			// Create heightfield for the landscape editor (no holes in it)
			if (bNeedsEditorHeightField)
			{
				TArray<UPhysicalMaterial*> CookedMaterialsEd;
				if (CookCollisionData(PhysicsFormatName, true, bReuseIsValid, CookedCollisionDataEd, CookedMaterialsEd))
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(CreateCollisionObject_ChaosStream);
					FMemoryReader Reader(CookedCollisionDataEd);
					Chaos::FChaosArchive Ar(Reader);

					// Don't actually care about this but need to strip it out of the data
					bool bContainsSimple = false;
					Ar << bContainsSimple;
					Ar << HeightfieldRef->EditorHeightfieldGeometry;

					CookedCollisionDataEd.Empty();
				}
			}
#endif //WITH_EDITOR
		}
	}
}

void ULandscapeHeightfieldCollisionComponent::CreateCollisionObject(
	bool bUseDefaultMaterialOnly, 
	TArrayView<const uint16> Heights, TArrayView<const uint16> SimpleHeights, 
	TArrayView<const uint8> PhysicalMaterialIds, TArrayView<const uint8> SimplePhysicalMaterialIds, 
	TArrayView<const TObjectPtr<UPhysicalMaterial>> PhysicalMaterialObjects)
{
	TArrayView<const uint8> ComplexMaterialIndicesView;
	TArrayView<const uint8> SimpleMaterialIndicesView;

	bool bGenerateSimpleCollision = SimpleCollisionSizeQuads > 0 && !bUseDefaultMaterialOnly;

	if(!ensureMsgf(!HeightfieldGuid.IsValid(), TEXT("Attempting to create a runtime collision object, but one already exists")))
	{
		return;
	}
	 
	const FCollisionSampleInfo Info = GetCollisionSampleInfo();

	if (!ensure(Heights.Num() == Info.NumSamples))
	{
		return;
	}

	int32 NumQuads = (Info.CollisionSizeVerts - 1) * (Info.CollisionSizeVerts - 1);
	if (!ensure(PhysicalMaterialIds.Num() == NumQuads))
	{
		return;
	}

	if(bGenerateSimpleCollision)
	{
		if (!ensure(SimpleHeights.Num() == Info.NumSimpleSamples))
		{
			return;
		}

		int32 NumSimpleQuads = (Info.SimpleCollisionSizeVerts - 1) * (Info.SimpleCollisionSizeVerts - 1);
		if (!ensure(SimplePhysicalMaterialIds.Num() == NumSimpleQuads))
		{
			return;
		}
	}

	// In non performant builds, validate that the incoming data's indices are all valid
#if !UE_BUILD_TEST && !UE_BUILD_SHIPPING
	for (uint8 Sample : PhysicalMaterialIds)
	{
		if (!ensure(Sample == 0xFF || PhysicalMaterialObjects.IsValidIndex(Sample)))
		{
			return;
		}
	}

	if (bGenerateSimpleCollision)
	{
		for (uint8 Sample : SimplePhysicalMaterialIds)
		{
			if (!ensure(Sample == 0xFF || PhysicalMaterialObjects.IsValidIndex(Sample)))
			{
				return;
			}
		}
	}
#endif

#if !WITH_EDITORONLY_DATA
	uint32 CollisionHash = 0;
#endif
	HeightfieldGuid = FGuid::NewDeterministicGuid(GetPathName(), CollisionHash);

	UE_SCOPELOCK_SHARED_HEIGHTFIELD_REFS();
	HeightfieldRef = GSharedHeightfieldRefs.Add(HeightfieldGuid, new FHeightfieldGeometryRef(HeightfieldGuid));
	HeightfieldRef->HeightfieldGeometry = Chaos::FHeightFieldPtr(new Chaos::FHeightField(Heights, PhysicalMaterialIds, Info.CollisionSizeVerts, Info.CollisionSizeVerts, Chaos::FVec3(1)));
	
#if WITH_EDITOR
	UWorld* World = GetWorld();
	const bool bNeedsEditorHeightField = World && !World->IsGameWorld() && !GetOutermost()->bIsCookedForEditor;
	if (bNeedsEditorHeightField)
	{
		HeightfieldRef->EditorHeightfieldGeometry = Chaos::FHeightFieldPtr(new Chaos::FHeightField(Heights, PhysicalMaterialIds, Info.CollisionSizeVerts, Info.CollisionSizeVerts, Chaos::FVec3(1)));
	}
#endif // WITH_EDITOR

	if (bGenerateSimpleCollision)
	{
		HeightfieldRef->HeightfieldSimpleGeometry = Chaos::FHeightFieldPtr(new Chaos::FHeightField(SimpleHeights, SimplePhysicalMaterialIds, Info.SimpleCollisionSizeVerts, Info.SimpleCollisionSizeVerts, Chaos::FVec3(1)));
	}

	for (UPhysicalMaterial* PhysicalMaterial : PhysicalMaterialObjects)
	{
		HeightfieldRef->UsedChaosMaterials.Add(PhysicalMaterial->GetPhysicsMaterial());
	}
}

#if WITH_EDITOR
void ULandscapeHeightfieldCollisionComponent::SpeculativelyLoadAsyncDDCCollsionData()
{
	if (GetLinkerUEVersion() >= VER_UE4_LANDSCAPE_SERIALIZE_PHYSICS_MATERIALS && !GLandscapeCollisionSkipDDC)
	{
		UWorld* World = GetWorld();
		if (World && HeightfieldGuid.IsValid() && CookedPhysicalMaterials.Num() > 0 && GSharedHeightfieldRefs.FindRef(HeightfieldGuid) == nullptr)
		{
			static FName PhysicsFormatName(FPlatformProperties::GetPhysicsFormat());

			FString Key = GetHFDDCKeyString(PhysicsFormatName, false, HeightfieldGuid, CookedPhysicalMaterials);
			uint32 Handle = GetDerivedDataCacheRef().GetAsynchronous(*Key, GetPathName());
			check(!SpeculativeDDCRequest.IsValid());
			SpeculativeDDCRequest = MakeShareable(new FAsyncPreRegisterDDCRequest(Key, Handle));
			World->AsyncPreRegisterDDCRequests.Add(SpeculativeDDCRequest);
		}
	}
}

ULandscapeHeightfieldCollisionComponent::FWriteRuntimeDataParams ULandscapeHeightfieldCollisionComponent::MakeWriteRuntimeDataParams(bool bUseDefaultMaterialOnly) const
{
	const FCollisionSampleInfo Info = GetCollisionSampleInfo();

	const uint16* Heights = (const uint16*)CollisionHeightData.LockReadOnly();
	checkf(CollisionHeightData.GetElementCount() == Info.NumSamples + Info.NumSimpleSamples, 
		TEXT("Invalid collision height data element count : %lld found while there should be (%d samples + %d simple samples)"), CollisionHeightData.GetElementCount(), Info.NumSamples, Info.NumSimpleSamples);
	const uint16* SimpleHeights = Heights + Info.NumSamples;

	// Physical material data from layer system
	const uint8* DominantLayers = nullptr;
	const uint8* SimpleDominantLayers = nullptr;
	if (DominantLayerData.GetElementCount())
	{
		DominantLayers = (const uint8*)DominantLayerData.LockReadOnly();
		checkf(DominantLayerData.GetElementCount() == Info.NumSamples + Info.NumSimpleSamples,
			TEXT("Invalid dominant layer data element count : %lld found while there should be (%d samples + %d simple samples)"), DominantLayerData.GetElementCount(), Info.NumSamples, Info.NumSimpleSamples);
		SimpleDominantLayers = DominantLayers + Info.NumSamples;
	}

	// Physical material data from render material graph
	const uint8* RenderPhysicalMaterialIds = nullptr;
	const uint8* SimpleRenderPhysicalMaterialIds = nullptr;
	if (PhysicalMaterialRenderData.GetElementCount())
	{
		RenderPhysicalMaterialIds = (const uint8*)PhysicalMaterialRenderData.LockReadOnly();
		checkf(PhysicalMaterialRenderData.GetElementCount() == Info.NumSamples + Info.NumSimpleSamples,
			TEXT("Invalid physical material render data element count : %lld found while there should be (%d samples + %d simple samples)"), PhysicalMaterialRenderData.GetElementCount(), Info.NumSamples, Info.NumSimpleSamples);
		SimpleRenderPhysicalMaterialIds = RenderPhysicalMaterialIds + Info.NumSamples;
	}

	auto MakeSafeArrayView =
		[](auto* Data, int32 DataSamples)
		{
			return Data ? MakeArrayView(Data, DataSamples) : MakeArrayView(Data, 0);
		};

	FWriteRuntimeDataParams WriteParams;
	WriteParams.bUseDefaultMaterialOnly = bUseDefaultMaterialOnly;
	WriteParams.bProcessRenderIndices = true;
	WriteParams.bProcessVisibilityLayer = true;
	WriteParams.Heights = MakeSafeArrayView(Heights, Info.NumSamples);
	WriteParams.SimpleHeights = MakeSafeArrayView(SimpleHeights, Info.NumSimpleSamples);
	WriteParams.DominantLayers = MakeSafeArrayView(DominantLayers, Info.NumSamples);
	WriteParams.SimpleDominantLayers = MakeSafeArrayView(SimpleDominantLayers, Info.NumSimpleSamples);
	WriteParams.RenderPhysicalMaterialIds = MakeSafeArrayView(RenderPhysicalMaterialIds, Info.NumSamples);
	WriteParams.SimpleRenderPhysicalMaterialIds = MakeSafeArrayView(SimpleRenderPhysicalMaterialIds, Info.NumSimpleSamples);
	WriteParams.PhysicalMaterialRenderObjects = MakeSafeArrayView(PhysicalMaterialRenderObjects.GetData(), PhysicalMaterialRenderObjects.Num());
	WriteParams.ComponentLayerInfos = MakeSafeArrayView(ComponentLayerInfos.GetData(), ComponentLayerInfos.Num());
	WriteParams.VisibilityLayerIndex = ComponentLayerInfos.IndexOfByKey(ALandscapeProxy::VisibilityLayer);

	return WriteParams;
}

#endif

ULandscapeHeightfieldCollisionComponent::FCollisionSampleInfo ULandscapeHeightfieldCollisionComponent::GetCollisionSampleInfo() const
{
	FCollisionSampleInfo Result;
	Result.CollisionSizeVerts = CollisionSizeQuads + 1;
	Result.SimpleCollisionSizeVerts = SimpleCollisionSizeQuads > 0 ? SimpleCollisionSizeQuads + 1 : 0;
	Result.NumSamples = FMath::Square(Result.CollisionSizeVerts);
	Result.NumSimpleSamples = FMath::Square(Result.SimpleCollisionSizeVerts);
	return Result;
}


// Generate the heightfield and optional simple heightfield objects
// The dominant materials for the collision object will be added to InOutMaterials if Params.bUseDefaultMaterialOnly is false.
bool ULandscapeHeightfieldCollisionComponent::GenerateCollisionData(const FWriteRuntimeDataParams& Params, Chaos::FHeightFieldPtr& OutHeightField, bool bGenerateSimpleCollision, Chaos::FHeightFieldPtr& OutSimpleHeightField, TArray<UPhysicalMaterial*>& InOutMaterials) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeHeightfieldCollisionComponent::GenerateCollisionData);

	ALandscapeProxy* Proxy = GetLandscapeProxy();
	if (!Proxy || !Proxy->GetRootComponent())
	{
		return false;
	}

	UPhysicalMaterial* DefMaterial = Proxy->DefaultPhysMaterial ? Proxy->DefaultPhysMaterial : GEngine->DefaultPhysMaterial;

	// GetComponentTransform() might not be initialized at this point, so use landscape transform
	const FVector LandscapeScale = Proxy->GetRootComponent()->GetRelativeScale3D();
	const bool bIsMirrored = (LandscapeScale.X * LandscapeScale.Y * LandscapeScale.Z) < 0.f;

	const FCollisionSampleInfo Info = GetCollisionSampleInfo();

	// Generate material indices
	TArray<uint8> MaterialIndices;

	if (!Params.bUseDefaultMaterialOnly)
	{
		// List of materials which is actually used by heightfield
		InOutMaterials.Empty();

		MaterialIndices.Reserve(Info.NumSamples + Info.NumSimpleSamples);

		auto ResolveMaterials = [&MaterialIndices, &bIsMirrored, &Params, &DefMaterial, &InOutMaterials](int32 InCollisionVertExtent, TArrayView<const uint8> InDominantLayers, TArrayView<const uint8> InRenderMaterialIds)
			{
				check(!Params.bUseDefaultMaterialOnly);
				for (int32 RowIndex = 0; RowIndex < InCollisionVertExtent; RowIndex++)
				{
					for (int32 ColIndex = 0; ColIndex < InCollisionVertExtent; ColIndex++)
					{
						const int32 SrcSampleIndex = (RowIndex * InCollisionVertExtent) + (bIsMirrored ? (InCollisionVertExtent - ColIndex - 1) : ColIndex);

						// Materials are not relevant on the last row/column because they are per-triangle and the last row/column don't own any
						if (RowIndex < InCollisionVertExtent - 1 &&
							ColIndex < InCollisionVertExtent - 1)
						{
							int32 MaterialIndex = 0; // Default physical material.
							uint8 DominantLayerIdx = InDominantLayers.IsEmpty() ? -1 : InDominantLayers[SrcSampleIndex];
							ULandscapeLayerInfoObject* Layer = Params.ComponentLayerInfos.IsValidIndex(DominantLayerIdx) ? ToRawPtr(Params.ComponentLayerInfos[DominantLayerIdx]) : nullptr;

							if (Params.bProcessVisibilityLayer && DominantLayerIdx == Params.VisibilityLayerIndex)
							{
								// If it's a hole, use the final index
								MaterialIndex = TNumericLimits<uint8>::Max();
							}
							else if (Params.bProcessRenderIndices && (!Params.PhysicalMaterialRenderObjects.IsEmpty()) && (!InRenderMaterialIds.IsEmpty()))
							{
								uint8 RenderIdx = InRenderMaterialIds[SrcSampleIndex];
								UPhysicalMaterial* DominantMaterial = RenderIdx > 0 ? ToRawPtr(Params.PhysicalMaterialRenderObjects[RenderIdx - 1]) : DefMaterial;
								MaterialIndex = InOutMaterials.AddUnique(DominantMaterial);
							}
							else
							{
								UPhysicalMaterial* DominantMaterial = Layer && Layer->GetPhysicalMaterial() ? ToRawPtr(Layer->GetPhysicalMaterial()) : DefMaterial;
								MaterialIndex = InOutMaterials.AddUnique(DominantMaterial);
							}
							MaterialIndices.Add(IntCastChecked<uint8>(MaterialIndex));
						}
					}
				}
			};

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ResolveMaterials);
			ResolveMaterials(Info.CollisionSizeVerts, Params.DominantLayers, Params.RenderPhysicalMaterialIds);
			ResolveMaterials(Info.SimpleCollisionSizeVerts, Params.SimpleDominantLayers, Params.SimpleRenderPhysicalMaterialIds);
		}
	}
	else
	{
		// for bUseDefaultMaterialOnly==true, much faster to just make the array of zeros directly
		int32 NumMatIndices = FMath::Square(CollisionSizeQuads);
		if (bGenerateSimpleCollision)
		{
			NumMatIndices += FMath::Square(SimpleCollisionSizeQuads);
		}
		MaterialIndices.SetNumZeroed(NumMatIndices);
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CreateHeightField);
		const int32 NumCollisionCells = FMath::Square(CollisionSizeQuads);
		TArrayView<uint8> ComplexMaterialIndicesView(MaterialIndices.GetData(), NumCollisionCells);
		OutHeightField = Chaos::FHeightFieldPtr(new Chaos::FHeightField(Params.Heights, ComplexMaterialIndicesView, Info.CollisionSizeVerts, Info.CollisionSizeVerts, Chaos::FVec3(1)));

		if (bGenerateSimpleCollision)
		{
			const int32 NumSimpleCollisionCells = FMath::Square(SimpleCollisionSizeQuads);
			TArrayView<uint8> SimpleMaterialIndicesView(MaterialIndices.GetData() + NumCollisionCells, NumSimpleCollisionCells);
			OutSimpleHeightField = Chaos::FHeightFieldPtr(new Chaos::FHeightField(Params.SimpleHeights, SimpleMaterialIndicesView, Info.SimpleCollisionSizeVerts, Info.SimpleCollisionSizeVerts, Chaos::FVec3(1)));
		}
	}

	return true;
}

// Generate the heightfield and optional simple heightfield objects and serialize them into a byte array.
bool ULandscapeHeightfieldCollisionComponent::WriteRuntimeData(const FWriteRuntimeDataParams& Params, TArray<uint8>& OutHeightfieldData, TArray<UPhysicalMaterial*>& InOutMaterials) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeHeightfieldCollisionComponent::WriteRuntimeData);

	Chaos::FHeightFieldPtr Heightfield = nullptr;
	Chaos::FHeightFieldPtr HeightfieldSimple = nullptr;
	const bool bGenerateSimpleCollision = SimpleCollisionSizeQuads > 0 && !Params.bUseDefaultMaterialOnly;
	if (GenerateCollisionData(Params, Heightfield, bGenerateSimpleCollision, HeightfieldSimple, InOutMaterials))
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Chaos_stream);
		check(Heightfield);
		check(!bGenerateSimpleCollision || HeightfieldSimple);

		FMemoryWriter Writer(OutHeightfieldData);
		Chaos::FChaosArchive Ar(Writer);

		bool bSerializeGenerateSimpleCollision = bGenerateSimpleCollision;
		Ar << bSerializeGenerateSimpleCollision;

		Ar << Heightfield;
		if (bGenerateSimpleCollision)
		{
			Ar << HeightfieldSimple;
		}
		return true;
	}

	return false;
}

#if WITH_EDITOR

// Create the collision object for the component.  Similar to CookCollisionData, but bypasses the buffer serialization step.
bool ULandscapeHeightfieldCollisionComponent::GenerateCollisionObjects(const FName& Format, bool bUseDefaultMaterialOnly, Chaos::FHeightFieldPtr& OutHeightField, bool bGenerateSimpleCollision, Chaos::FHeightFieldPtr& OutSimpleHeightField, TArray<UPhysicalMaterial*>& InOutMaterials) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeHeightfieldCollisionComponent::GenerateCollisionObjects);

	if (GetOutermost()->bIsCookedForEditor)
	{
		return true;
	}

	FWriteRuntimeDataParams WriteParams = MakeWriteRuntimeDataParams(bUseDefaultMaterialOnly);
	bool bSucceeded = GenerateCollisionData(WriteParams, OutHeightField, bGenerateSimpleCollision, OutSimpleHeightField, InOutMaterials);
	if (CollisionHeightData.IsLocked())
	{
		CollisionHeightData.Unlock();
	}
	if (DominantLayerData.IsLocked())
	{
		DominantLayerData.Unlock();
	}
	if (PhysicalMaterialRenderData.IsLocked())
	{
		PhysicalMaterialRenderData.Unlock();
	}

	return bSucceeded;
}

bool ULandscapeHeightfieldCollisionComponent::CookCollisionData(const FName& Format, bool bUseDefaultMaterialOnly, bool bCheckDDC, TArray<uint8>& OutCookedData, TArray<UPhysicalMaterial*>& InOutMaterials) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeHeightfieldCollisionComponent::CookCollisionData);

	if (GetOutermost()->bIsCookedForEditor)
	{
		return true;
	}

	// Use existing cooked data unless !bCheckDDC in which case the data must be rebuilt.
	if (bCheckDDC && OutCookedData.Num() > 0)
	{
		return true;
	}

	COOK_STAT(auto Timer = LandscapeCollisionCookStats::HeightfieldUsageStats.TimeSyncWork());
	// If we aren't using DDC, only track time spent, so we aren't affecting hit/miss stats
	COOK_STAT(if (GLandscapeCollisionSkipDDC) { Timer.TrackCyclesOnly(); });

	// we have 2 versions of collision objects
	const int32 CookedDataIndex = bUseDefaultMaterialOnly ? 0 : 1;

	if (!GLandscapeCollisionSkipDDC && bCheckDDC && HeightfieldGuid.IsValid())
	{
		// Ensure that content was saved with physical materials before using DDC data
		if (GetLinkerUEVersion() >= VER_UE4_LANDSCAPE_SERIALIZE_PHYSICS_MATERIALS)
		{
			FString DDCKey = GetHFDDCKeyString(Format, bUseDefaultMaterialOnly, HeightfieldGuid, InOutMaterials);

			// Check if the speculatively-loaded data loaded and is what we wanted
			if (SpeculativeDDCRequest.IsValid() && DDCKey == SpeculativeDDCRequest->GetKey())
			{
				// If we have a DDC request in flight, just time the synchronous cycles used.
				COOK_STAT(auto WaitTimer = LandscapeCollisionCookStats::HeightfieldUsageStats.TimeAsyncWait());
				SpeculativeDDCRequest->WaitAsynchronousCompletion();
				bool bSuccess = SpeculativeDDCRequest->GetAsynchronousResults(OutCookedData);
				// World will clean up remaining reference
				SpeculativeDDCRequest.Reset();
				if (bSuccess)
				{
					COOK_STAT(Timer.Cancel());
					COOK_STAT(WaitTimer.AddHit(OutCookedData.Num()));
					bShouldSaveCookedDataToDDC[CookedDataIndex] = false;
					return true;
				}
				else
				{
					// If the DDC request failed, then we waited for nothing and will build the resource anyway. Just ignore the wait timer and treat it all as sync time.
					COOK_STAT(WaitTimer.Cancel());
				}
			}

			if (GetDerivedDataCacheRef().GetSynchronous(*DDCKey, OutCookedData, GetPathName()))
			{
				COOK_STAT(Timer.AddHit(OutCookedData.Num()));
				bShouldSaveCookedDataToDDC[CookedDataIndex] = false;
				return true;
			}
		}
	}

	FWriteRuntimeDataParams WriteParams = MakeWriteRuntimeDataParams(bUseDefaultMaterialOnly);

	TArray<uint8> OutData;
	bool Succeeded = WriteRuntimeData(WriteParams, OutData, InOutMaterials);

	if (CollisionHeightData.IsLocked())
	{
		CollisionHeightData.Unlock();
	}
	if (DominantLayerData.IsLocked())
	{
		DominantLayerData.Unlock();
	}
	if (PhysicalMaterialRenderData.IsLocked())
	{
		PhysicalMaterialRenderData.Unlock();
	}

	if (!Succeeded)
	{
		// We didn't actually build anything, so just track the cycles.
		COOK_STAT(Timer.TrackCyclesOnly());
		return false;
	}

	COOK_STAT(Timer.AddMiss(OutData.Num()));
	OutCookedData.SetNumUninitialized(OutData.Num());
	FMemory::Memcpy(OutCookedData.GetData(), OutData.GetData(), OutData.Num());

	if (!GLandscapeCollisionSkipDDC && bShouldSaveCookedDataToDDC[CookedDataIndex] && HeightfieldGuid.IsValid())
	{
		GetDerivedDataCacheRef().Put(*GetHFDDCKeyString(Format, bUseDefaultMaterialOnly, HeightfieldGuid, InOutMaterials), OutCookedData, GetPathName());
		bShouldSaveCookedDataToDDC[CookedDataIndex] = false;
	}

	return Succeeded;
}

uint32 ULandscapeHeightfieldCollisionComponent::ComputeCollisionHash() const
{
	uint32 Hash = 0;
		
	Hash = HashCombine(GetTypeHash(SimpleCollisionSizeQuads), Hash);
	Hash = HashCombine(GetTypeHash(CollisionSizeQuads), Hash);
	Hash = HashCombine(GetTypeHash(CollisionScale), Hash);

	const FTransform ComponentTransform = GetComponentToWorld();
	Hash = FCrc::MemCrc32(&ComponentTransform, sizeof(ComponentTransform), Hash);

	const void* HeightBuffer = CollisionHeightData.LockReadOnly();
	Hash = FCrc::MemCrc32(HeightBuffer, static_cast<int32>(CollisionHeightData.GetBulkDataSize()), Hash);
	CollisionHeightData.Unlock();

	const void* DominantBuffer = DominantLayerData.LockReadOnly();
	Hash = FCrc::MemCrc32(DominantBuffer, static_cast<int32>(DominantLayerData.GetBulkDataSize()), Hash);
	DominantLayerData.Unlock();

	const void* PhysicalMaterialBuffer = PhysicalMaterialRenderData.LockReadOnly();
	Hash = FCrc::MemCrc32(PhysicalMaterialBuffer, static_cast<int32>(PhysicalMaterialRenderData.GetBulkDataSize()), Hash);
	PhysicalMaterialRenderData.Unlock();

	return Hash;
}

void ULandscapeHeightfieldCollisionComponent::UpdateHeightfieldRegion(int32 ComponentX1, int32 ComponentY1, int32 ComponentX2, int32 ComponentY2)
{
	if (IsValidRef(HeightfieldRef))
	{
		// If we're currently sharing this data with a PIE session, we need to make a new heightfield.
		if (HeightfieldRef->GetRefCount() > 1)
		{
			RecreateCollision();
			return;
		}

		if(!BodyInstance.GetPhysicsActor())
		{
			return;
		}

		// We don't lock the async scene as we only set the geometry in the sync scene's RigidActor.
		// This function is used only during painting for line traces by the painting tools.
		FPhysicsActorHandle PhysActorHandle = BodyInstance.GetPhysicsActor();

		FPhysicsCommand::ExecuteWrite(PhysActorHandle, [&](const FPhysicsActorHandle& Actor)
		{
			int32 CollisionSizeVerts = CollisionSizeQuads + 1;
			int32 SimpleCollisionSizeVerts = SimpleCollisionSizeQuads > 0 ? SimpleCollisionSizeQuads + 1 : 0;
	
			bool bIsMirrored = GetComponentToWorld().GetDeterminant() < 0.f;
	
			uint16* Heights = (uint16*)CollisionHeightData.Lock(LOCK_READ_ONLY);
			checkf(CollisionHeightData.GetElementCount() == FMath::Square(CollisionSizeVerts) + FMath::Square(SimpleCollisionSizeVerts),
				TEXT("Invalid collision height data element count : %lld found while there should be (%d samples + %d simple samples)"), CollisionHeightData.GetElementCount(), FMath::Square(CollisionSizeVerts), FMath::Square(SimpleCollisionSizeVerts));
	
			int32 HeightfieldY1 = ComponentY1;
			int32 HeightfieldX1 = (bIsMirrored ? ComponentX1 : (CollisionSizeVerts - ComponentX2 - 1));
			int32 DstVertsX = ComponentX2 - ComponentX1 + 1;
			int32 DstVertsY = ComponentY2 - ComponentY1 + 1;
			TArray<uint16> Samples;
			Samples.AddZeroed(DstVertsX*DstVertsY);

			for(int32 RowIndex = 0; RowIndex < DstVertsY; RowIndex++)
			{
				for(int32 ColIndex = 0; ColIndex < DstVertsX; ColIndex++)
				{
					int32 SrcX = bIsMirrored ? (ColIndex + ComponentX1) : (ComponentX2 - ColIndex);
					int32 SrcY = RowIndex + ComponentY1;
					int32 SrcSampleIndex = (SrcY * CollisionSizeVerts) + SrcX;
					check(SrcSampleIndex < FMath::Square(CollisionSizeVerts));
					int32 DstSampleIndex = (RowIndex * DstVertsX) + ColIndex;

					Samples[DstSampleIndex] = Heights[SrcSampleIndex];
				}
			}

			CollisionHeightData.Unlock();

			HeightfieldRef->EditorHeightfieldGeometry->EditHeights(Samples, HeightfieldY1, HeightfieldX1, DstVertsY, DstVertsX);

			// Rebuild geometry to update local bounds, and update in acceleration structure.
			const Chaos::FImplicitObjectUnion& Union = PhysActorHandle->GetGameThreadAPI().GetGeometry()->GetObjectChecked<Chaos::FImplicitObjectUnion>();
			TArray<Chaos::FImplicitObjectPtr> NewGeometry;
			for (const Chaos::FImplicitObjectPtr& Object : Union.GetObjects())
			{
				const Chaos::TImplicitObjectTransformed<Chaos::FReal, 3>& TransformedHeightField = Object->GetObjectChecked<Chaos::TImplicitObjectTransformed<Chaos::FReal, 3>>();
				NewGeometry.Emplace(MakeImplicitObjectPtr<Chaos::TImplicitObjectTransformed<Chaos::FReal, 3>>(TransformedHeightField.GetGeometry(), TransformedHeightField.GetTransform()));
			}
			PhysActorHandle->GetGameThreadAPI().SetGeometry(MakeImplicitObjectPtr<Chaos::FImplicitObjectUnion>(MoveTemp(NewGeometry)));

			FPhysScene* PhysScene = GetWorld()->GetPhysicsScene();
			PhysScene->UpdateActorInAccelerationStructure(PhysActorHandle);
		});
	}
}
#endif// WITH_EDITOR

void ULandscapeHeightfieldCollisionComponent::DestroyComponent(bool bPromoteChildren/*= false*/)
{
	ALandscapeProxy* Proxy = GetLandscapeProxy();
	if (Proxy)
	{
		Proxy->CollisionComponents.Remove(this);
	}

	Super::DestroyComponent(bPromoteChildren);
}

FBoxSphereBounds ULandscapeHeightfieldCollisionComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	return CachedLocalBox.TransformBy(LocalToWorld);
}

void ULandscapeHeightfieldCollisionComponent::BeginDestroy()
{
	Super::BeginDestroy();

	// Should have been reset in OnUnregister which is called from Super::BeginDestroy
	if (!ensure(HeightfieldRef == nullptr))
	{
		HeightfieldRef = nullptr;
		HeightfieldGuid = FGuid();

		{
			FWriteScopeLock Lock(CachedHeightFieldSamplesLock);
			CachedHeightFieldSamples.Empty();
		}
	}
}

bool ULandscapeHeightfieldCollisionComponent::RecreateCollision()
{
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
#if WITH_EDITOR
		uint32 NewHash = ComputeCollisionHash();
		if (bPhysicsStateCreated && NewHash == CollisionHash && CollisionHash != 0 && bEnableCollisionHashOptim)
		{
			return false;
		}
		CollisionHash = NewHash;
#endif // WITH_EDITOR

		// Collision geometry must be kept alive as long as we have a particle on the physics
		// that references it. See ExtendCollisionLifetime
		TRefCountPtr<FHeightfieldGeometryRef> HeightfieldRefLifetimeExtender = HeightfieldRef;

		HeightfieldRef = nullptr; // Ensure data will be recreated
		HeightfieldGuid = FGuid();

		{
			FWriteScopeLock Lock(CachedHeightFieldSamplesLock);
			CachedHeightFieldSamples.Empty();
		}

		RecreatePhysicsState();

		// Make sure our collision isn't destroyed while we still have a physics particle active
		// NOTE: Must be after the call to DestroyPhysicsState
		DeferredDestroyCollision(HeightfieldRefLifetimeExtender);

		MarkRenderStateDirty();

	}
	return true;
}

// @todo(chaos): get rid of this when collision shapes are properly ref counted
void ULandscapeHeightfieldCollisionComponent::DeferredDestroyCollision(const TRefCountPtr<FHeightfieldGeometryRef>& HeightfieldRefLifetimeExtender)
{
	// The editor may have a reference to the geometry as well, so we don't destroy it unless we're the last reference
	if (!HeightfieldRefLifetimeExtender.IsValid() || (HeightfieldRefLifetimeExtender->GetRefCount() > 1))
	{
		return;
	}

	UWorld* World = GetWorld();
	if (World == nullptr)
	{
		return;
	}

	FPhysScene* PhysScene = World->GetPhysicsScene();
	if ((PhysScene == nullptr) || (PhysScene->GetSolver() == nullptr))
	{
		return;
	}

	// We could potentially call RecreateCollision multiple times before a physics update happens, especially
	// if we're using the async tick mode for physics. In this case we would have a pending actor in the
	// dirty proxy list on the physics thread with a geometry that has been destructed by the lifetime
	// extender falling out of scope.
	// To avoid this we dispatch an empty callable with the unique geometries which runs after the
	// proxy queue will have been cleared, avoiding a use-after-free. 
	// We also avoid enqueueing any off-thread work until the AutoRTFM transaction has completed.
	// #TODO auto ref counted user objects for Chaos.
	AutoRTFM::OnCommit([this
	                    , PhysScene
					    , ComplexHeightfield = MoveTemp(HeightfieldRefLifetimeExtender->HeightfieldGeometry)
					    , SimpleHeightfield = MoveTemp(HeightfieldRefLifetimeExtender->HeightfieldSimpleGeometry)
#if WITH_EDITORONLY_DATA
					    , EditorHeightfield = MoveTemp(HeightfieldRefLifetimeExtender->EditorHeightfieldGeometry)
#endif
                       ]() mutable
	{
		PhysScene->GetSolver()->EnqueueCommandImmediate([ComplexHeightfield = MoveTemp(ComplexHeightfield)
														 , SimpleHeightfield = MoveTemp(SimpleHeightfield)
#if WITH_EDITORONLY_DATA
														 , EditorHeightfield = MoveTemp(EditorHeightfield)
#endif 
														]() mutable
		{
			ComplexHeightfield = nullptr;
			SimpleHeightfield = nullptr;
#if WITH_EDITORONLY_DATA
			EditorHeightfield = nullptr;
#endif
		});
	});
}

#if WITH_EDITORONLY_DATA
void ULandscapeHeightfieldCollisionComponent::MarkAreaForSnapFoliageInstances(const FBox& InBox)
{
	UE_CVLOG_BOX(CVarVisualLogShowFoliageSnapping->GetBool(), GetOwner(), LogLandscape, Log, InBox, FColor::Blue, TEXT("MarkAreaForSnapFoliageInstances"));

	for (FBox& ExistingBox : DeferredSnapFoliageInstanceBoxes)
	{
		// If the new box is fully included in the other box, then it's not needed at all
		if (InBox.IsInsideOrOn(ExistingBox))
		{
			return;
		}
		// And if the existing box is fully included within the new one, then we can replace the existing one with the new one
		if (ExistingBox.IsInsideOrOn(InBox))
		{
			ExistingBox = InBox;
			return;
		}
	}
	DeferredSnapFoliageInstanceBoxes.Add(InBox);
}

ENUM_CLASS_FLAGS(ULandscapeHeightfieldCollisionComponent::ESnapFoliageInstancesFlags);

// Deprecated
void ULandscapeHeightfieldCollisionComponent::SnapFoliageInstances()
{
	SnapFoliageInstances(ESnapFoliageInstancesFlags::ForceAll);
}

void ULandscapeHeightfieldCollisionComponent::SnapFoliageInstances(ESnapFoliageInstancesFlags InFlags)
{
	check(EnumHasAnyOneFlag(InFlags, ESnapFoliageInstancesFlags::MarkedAreasOnly | ESnapFoliageInstancesFlags::ForceAll));
	if (EnumHasAnyFlags(InFlags, ESnapFoliageInstancesFlags::MarkedAreasOnly))
	{
		for (const FBox& Box : DeferredSnapFoliageInstanceBoxes)
		{
			SnapFoliageInstances(Box);
		}
		DeferredSnapFoliageInstanceBoxes.Empty();
	}
	else if (EnumHasAnyFlags(InFlags, ESnapFoliageInstancesFlags::ForceAll))
	{
		SnapFoliageInstances(FBox(FVector(-WORLD_MAX), FVector(WORLD_MAX)));
		DeferredSnapFoliageInstanceBoxes.Empty();
	}
}

void ULandscapeHeightfieldCollisionComponent::SnapFoliageInstances(const FBox& InInstanceBox)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeHeightfieldCollisionComponent::SnapFoliageInstances);

	bool bAtLeastOneInstanceWithinBox = false;
	
	UWorld* ComponentWorld = GetWorld();
	for (TActorIterator<AInstancedFoliageActor> It(ComponentWorld); It; ++It)
	{
		AInstancedFoliageActor* IFA = *It;
		const auto BaseId = IFA->InstanceBaseCache.GetInstanceBaseId(this);
		if (BaseId == FFoliageInstanceBaseCache::InvalidBaseId)
		{
			continue;
		}
			
		IFA->ForEachFoliageInfo([this, IFA, BaseId, &InInstanceBox, &bAtLeastOneInstanceWithinBox](UFoliageType* Settings, FFoliageInfo& MeshInfo)
		{
			const auto* InstanceSet = MeshInfo.ComponentHash.Find(BaseId);
			if (InstanceSet)
			{
				const FVector ZUnitAxis = GetOwner()->GetRootComponent()->GetComponentTransform().GetUnitAxis(EAxis::Z);
				const float TraceExtentSize = static_cast<float>(Bounds.SphereRadius) * 2.f + 10.f; // extend a little
				const FVector TraceVector = ZUnitAxis * TraceExtentSize;

				TArray<int32> InstancesToRemove;
				TSet<UHierarchicalInstancedStaticMeshComponent*> AffectedFoliageComponents;

				bool bIsMeshInfoDirty = false;
				for (int32 InstanceIndex : *InstanceSet)
				{
					FFoliageInstance& Instance = MeshInfo.Instances[InstanceIndex];

					// Test location should remove any Z offset
					FVector InstanceLocation = FMath::Abs(Instance.ZOffset) > KINDA_SMALL_NUMBER
						? Instance.GetInstanceWorldTransform().TransformPosition(FVector(0, 0, -Instance.ZOffset))
						: Instance.Location;

					if (InInstanceBox.IsInside(InstanceLocation))
					{
						bAtLeastOneInstanceWithinBox = true;

						const double HitDistance = FVector::DotProduct((Bounds.Origin - InstanceLocation), ZUnitAxis);
						const FVector TestLocation = InstanceLocation + ZUnitAxis * HitDistance;
						const FVector Start = TestLocation + TraceVector;
						const FVector End = TestLocation - TraceVector;

						TArray<FHitResult> Results;
						UWorld* World = GetWorld();
						check(World);
						// Editor specific landscape heightfield uses ECC_Visibility collision channel
						World->LineTraceMultiByObjectType(Results, Start, End, FCollisionObjectQueryParams(ECollisionChannel::ECC_Visibility), FCollisionQueryParams(SCENE_QUERY_STAT(FoliageSnapToLandscape), true));

						bool bFoundHit = false;
						FVector HitLocation;
						for (const FHitResult& Hit : Results)
						{
							if (Hit.Component == this)
							{
								bFoundHit = true;
								if ((InstanceLocation - Hit.Location).SizeSquared() > KINDA_SMALL_NUMBER)
								{
									IFA->Modify();

									bIsMeshInfoDirty = true;

									// Remove instance location from the hash. Do not need to update ComponentHash as we re-add below.
									MeshInfo.InstanceHash->RemoveInstance(Instance.Location, InstanceIndex);

									HitLocation = Hit.Location;

									// Update the instance editor data
									Instance.Location = Hit.Location;

									if (Instance.Flags & FOLIAGE_AlignToNormal)
									{
										// Remove previous alignment and align to new normal.
										Instance.Rotation = Instance.PreAlignRotation;
										Instance.AlignToNormal(Hit.Normal, Settings->AlignMaxAngle);
									}

									// Reapply the Z offset in local space
									if (FMath::Abs(Instance.ZOffset) > KINDA_SMALL_NUMBER)
									{
										Instance.Location = Instance.GetInstanceWorldTransform().TransformPosition(FVector(0, 0, Instance.ZOffset));
									}

									// Todo: add do validation with other parameters such as max/min height etc.

									MeshInfo.SetInstanceWorldTransform(InstanceIndex, Instance.GetInstanceWorldTransform(), false);
									// Re-add the new instance location to the hash
									MeshInfo.InstanceHash->InsertInstance(Instance.Location, InstanceIndex);
								}
								break;
							}
						}

						UE_CVLOG_SEGMENT(CVarVisualLogShowFoliageSnapping->GetBool(), GetOwner(), LogLandscape, Log, Start, End, bFoundHit ? FColor::Red : FColor::Green, TEXT("SnapFoliageInstancesRay"));
						if (bFoundHit)
						{
							UE_CVLOG_LOCATION(CVarVisualLogShowFoliageSnapping->GetBool(), GetOwner(), LogLandscape, Log, HitLocation, 5.0f, FColor::Red, TEXT("SnapFoliageInstancesRayHitLocation"));
						}
						else
						{
							// Couldn't find new spot - remove instance
							InstancesToRemove.Add(InstanceIndex);
							bIsMeshInfoDirty = true;
						}

						if (bIsMeshInfoDirty && (MeshInfo.GetComponent() != nullptr))
						{
							AffectedFoliageComponents.Add(MeshInfo.GetComponent());
						}
					}
				}

				// Remove any unused instances
				MeshInfo.RemoveInstances(InstancesToRemove, true);

				for (UHierarchicalInstancedStaticMeshComponent* FoliageComp : AffectedFoliageComponents)
				{
					FoliageComp->InvalidateLightingCache();
				}
			}
			return true; // continue iterating
		});
	}

	UE_CVLOG_BOX(CVarVisualLogShowFoliageSnapping->GetBool(), GetOwner(), LogLandscape, Log, InInstanceBox, bAtLeastOneInstanceWithinBox ? FColor::Red : FColor::Green, TEXT("SnapFoliageInstances"));
}
#endif // WITH_EDITORONLY_DATA

void ULandscapeHeightfieldCollisionComponent::Serialize(FArchive& Ar)
{
	LLM_SCOPE(ELLMTag::Landscape);

#if WITH_EDITOR
	if (Ar.UEVer() >= VER_UE4_LANDSCAPE_COLLISION_DATA_COOKING)
	{
		// Cook data here so CookedPhysicalMaterials is always up to date
		if (Ar.IsCooking() && !HasAnyFlags(RF_ClassDefaultObject))
		{
			FName Format = Ar.CookingTarget()->GetPhysicsFormat(nullptr);
			CookCollisionData(Format, false, true, CookedCollisionData, MutableView(CookedPhysicalMaterials));
		}
	}
#endif// WITH_EDITOR

	// this will also serialize CookedPhysicalMaterials
	Super::Serialize(Ar);

	if (Ar.UEVer() < VER_UE4_LANDSCAPE_COLLISION_DATA_COOKING)
	{
#if WITH_EDITORONLY_DATA
		CollisionHeightData.Serialize(Ar, this);
		DominantLayerData.Serialize(Ar, this);
#endif//WITH_EDITORONLY_DATA
	}
	else
	{
		bool bCooked = Ar.IsCooking() || (FPlatformProperties::RequiresCookedData() && Ar.IsSaving());
		Ar << bCooked;

		if (FPlatformProperties::RequiresCookedData() && !bCooked && Ar.IsLoading())
		{
			UE_LOGF(LogPhysics, Fatal, "This platform requires cooked packages, and physics data was not cooked into %ls.", *GetFullName());
		}

		if (bCooked)
		{
			CookedCollisionData.BulkSerialize(Ar);
		}
		else
		{
#if WITH_EDITORONLY_DATA
			CollisionHeightData.Serialize(Ar, this);
			DominantLayerData.Serialize(Ar, this);

			if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::LandscapePhysicalMaterialRenderData)
			{
				PhysicalMaterialRenderData.Serialize(Ar, this);
			}
#endif//WITH_EDITORONLY_DATA
		}
	}
}

bool ULandscapeHeightfieldCollisionComponent::IsShown(const FEngineShowFlags& ShowFlags) const
{
	return ShowFlags.Landscape;
}

bool ULandscapeHeightfieldCollisionComponent::DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const
{
	check(IsInGameThread());
	if(IsValidRef(HeightfieldRef) && HeightfieldRef->HeightfieldGeometry)
	{
		FTransform HFToW = GetComponentTransform();
		if(HeightfieldRef->HeightfieldSimpleGeometry)
		{
			const float SimpleCollisionScale = CollisionScale * CollisionSizeQuads / SimpleCollisionSizeQuads;
			HFToW.MultiplyScale3D(FVector(SimpleCollisionScale, SimpleCollisionScale, LANDSCAPE_ZSCALE));
			GeomExport.ExportChaosHeightField(HeightfieldRef->HeightfieldSimpleGeometry.GetReference(), HFToW);
		}
		else
		{
			HFToW.MultiplyScale3D(FVector(CollisionScale, CollisionScale, LANDSCAPE_ZSCALE));
			GeomExport.ExportChaosHeightField(HeightfieldRef->HeightfieldGeometry.GetReference(), HFToW);
		}
	}

	return false;
}

void ULandscapeHeightfieldCollisionComponent::GatherGeometrySlice(FNavigableGeometryExport& GeomExport, const FBox& SliceBox) const
{
	FReadScopeLock Lock(CachedHeightFieldSamplesLock);
	
	// note that this function can get called off game thread
	if (CachedHeightFieldSamples.IsEmpty() == false)
	{
		FTransform HFToW = GetComponentTransform();
		HFToW.MultiplyScale3D(FVector(CollisionScale, CollisionScale, LANDSCAPE_ZSCALE));

		GeomExport.ExportChaosHeightFieldSlice(CachedHeightFieldSamples, HeightfieldRowsCount, HeightfieldColumnsCount, HFToW, SliceBox);
	}
}

ENavDataGatheringMode ULandscapeHeightfieldCollisionComponent::GetGeometryGatheringMode() const
{ 
	ALandscapeProxy* Proxy = GetLandscapeProxy();
	return Proxy ? Proxy->NavigationGeometryGatheringMode : ENavDataGatheringMode::Default;
}

void ULandscapeHeightfieldCollisionComponent::PrepareGeometryExportSync()
{
	FWriteScopeLock Lock(CachedHeightFieldSamplesLock);
	
	if(IsValidRef(HeightfieldRef) && HeightfieldRef->HeightfieldGeometry.GetReference() && CachedHeightFieldSamples.IsEmpty())
	{
		const UWorld* World = GetWorld();

		if(World != nullptr)
		{
			HeightfieldRowsCount = HeightfieldRef->HeightfieldGeometry->GetNumRows();
			HeightfieldColumnsCount = HeightfieldRef->HeightfieldGeometry->GetNumCols();
			const int32 HeightsCount = HeightfieldRowsCount * HeightfieldColumnsCount;

			if(CachedHeightFieldSamples.Heights.Num() != HeightsCount)
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_NavMesh_ExportChaosHeightField_saveCells);

				CachedHeightFieldSamples.Heights.SetNumUninitialized(HeightsCount);
				for(int32 Index = 0; Index < HeightsCount; ++Index)
				{
					CachedHeightFieldSamples.Heights[Index] = static_cast<int16>(HeightfieldRef->HeightfieldGeometry->GetHeight(Index));
				}

				const int32 HolesCount = (HeightfieldRowsCount-1) * (HeightfieldColumnsCount-1);
				CachedHeightFieldSamples.Holes.SetNumUninitialized(HolesCount);
				for(int32 Index = 0; Index < HolesCount; ++Index)
				{
					CachedHeightFieldSamples.Holes[Index] = HeightfieldRef->HeightfieldGeometry->IsHole(Index);
				}
			}
		}
	}
}

void ULandscapeHeightfieldCollisionComponent::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	// PostLoad of the landscape can decide to recreate collision, in which case this components checks are irrelevant
	if (!HasAnyFlags(RF_ClassDefaultObject) && IsValidChecked(this))
	{
		bShouldSaveCookedDataToDDC[0] = true;
		bShouldSaveCookedDataToDDC[1] = true;

		ALandscapeProxy* LandscapeProxy = GetLandscapeProxy();
		if (ensure(LandscapeProxy) && GIsEditor)
		{
			// This is to ensure that component relative location is exact section base offset value
			FVector LocalRelativeLocation = GetRelativeLocation();
			float CheckRelativeLocationX = float(SectionBaseX - LandscapeProxy->GetSectionBase().X);
			float CheckRelativeLocationY = float(SectionBaseY - LandscapeProxy->GetSectionBase().Y);
			if (!FMath::IsNearlyEqual(CheckRelativeLocationX, LocalRelativeLocation.X, UE_DOUBLE_KINDA_SMALL_NUMBER) ||
				!FMath::IsNearlyEqual(CheckRelativeLocationY, LocalRelativeLocation.Y, UE_DOUBLE_KINDA_SMALL_NUMBER))
			{
				UE_LOGF(LogLandscape, Warning, "ULandscapeHeightfieldCollisionComponent RelativeLocation disagrees with its section base, attempted automated fix: '%ls', %f,%f vs %f,%f.",
					*GetFullName(), LocalRelativeLocation.X, LocalRelativeLocation.Y, CheckRelativeLocationX, CheckRelativeLocationY);
				LocalRelativeLocation.X = CheckRelativeLocationX;
				LocalRelativeLocation.Y = CheckRelativeLocationY;
				SetRelativeLocation_Direct(LocalRelativeLocation);
			}
		}

		UWorld* World = GetWorld();
		if (World && World->IsGameWorld())
		{
			SpeculativelyLoadAsyncDDCCollsionData();
		}
	}

#if WITH_EDITORONLY_DATA
	// If the RenderComponent is not set yet and we're transferring the property from the lazy object pointer it was previously stored as to the object ptr it is now stored as :
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!RenderComponentRef && RenderComponent_DEPRECATED.IsValid())
	{
		RenderComponentRef = RenderComponent_DEPRECATED.Get();
		RenderComponent_DEPRECATED = nullptr;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // !WITH_EDITORONLY_DATA

#endif // WITH_EDITOR
}

void ULandscapeHeightfieldCollisionComponent::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	if (!ObjectSaveContext.IsProceduralSave())
	{
#if WITH_EDITOR
		ALandscapeProxy* Proxy = GetLandscapeProxy();
		ULandscapeComponent* RenderComponent = GetRenderComponent();

		if (Proxy && Proxy->bBakeMaterialPositionOffsetIntoCollision)
		{
			if (!RenderComponent->GrassData->HasData() || RenderComponent->IsGrassMapOutdated())
			{
				if (!RenderComponent->CanRenderGrassMap())
				{
					RenderComponent->GetMaterialInstance(0, false)->GetMaterialResource(GetFeatureLevelShaderPlatform_Checked(GetWorld()->GetFeatureLevel()))->FinishCompilation();
				}

				ULandscapeSubsystem* LandscapeSubsystem = GetWorld()->GetSubsystem<ULandscapeSubsystem>();
				TArray<TObjectPtr<ULandscapeComponent>> Components = { RenderComponent };
				LandscapeSubsystem->GetGrassMapBuilder()->BuildGrassMapsNowForComponents(Components, /* SlowTask= */ nullptr, /* bMarkDirty= */ false);
			}
		}
#endif// WITH_EDITOR
	}
}

#if WITH_EDITOR
bool ULandscapeHeightfieldCollisionComponent::NeedsLoadForClient() const
{
	if (IsTemplate())
	{
		return true;
	}
	ALandscapeProxy* Proxy = GetLandscapeProxy();
	if (ensure(Proxy))
	{
		bool bStrip = Proxy->bStripPhysicsWhenCookedClient && CVarAllowPhysicsStripping->GetBool();
		return !bStrip;
	}
	return true;
}

bool ULandscapeHeightfieldCollisionComponent::NeedsLoadForServer() const
{
	if (IsTemplate())
	{
		return true;
	}
	ALandscapeProxy* Proxy = GetLandscapeProxy();
	if (ensure(Proxy))
	{
		bool bStrip = Proxy->bStripPhysicsWhenCookedServer && CVarAllowPhysicsStripping->GetBool();
		return !bStrip;
	}
	return true;
}

void ULandscapeInfo::UpdateAllAddCollisions()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeInfo::UpdateAllAddCollisions);
	XYtoAddCollisionMap.Reset();

	// Don't recreate add collisions if the landscape is not registered. This can happen during Undo.
	if (GetLandscapeProxy())
	{
		for (auto It = XYtoComponentMap.CreateIterator(); It; ++It)
		{
			const ULandscapeComponent* const Component = It.Value();
			if (ensure(Component))
			{
				const FIntPoint ComponentBase = Component->GetSectionBase() / ComponentSizeQuads;

				const FIntPoint NeighborsKeys[8] =
				{
					ComponentBase + FIntPoint(-1, -1),
					ComponentBase + FIntPoint(+0, -1),
					ComponentBase + FIntPoint(+1, -1),
					ComponentBase + FIntPoint(-1, +0),
					ComponentBase + FIntPoint(+1, +0),
					ComponentBase + FIntPoint(-1, +1),
					ComponentBase + FIntPoint(+0, +1),
					ComponentBase + FIntPoint(+1, +1)
				};

				// Search for Neighbors...
				for (int32 i = 0; i < 8; ++i)
				{
					ULandscapeComponent* NeighborComponent = XYtoComponentMap.FindRef(NeighborsKeys[i]);

					// UpdateAddCollision() treats a null CollisionComponent as an empty hole
					if (!NeighborComponent || (NeighborComponent->GetCollisionComponent() == nullptr))
					{
						UpdateAddCollision(NeighborsKeys[i]);
					}
				}
			}
		}
	}
}

void ULandscapeInfo::UpdateAddCollision(FIntPoint LandscapeKey)
{
	FLandscapeAddCollision& AddCollision = XYtoAddCollisionMap.FindOrAdd(LandscapeKey);

	// 8 Neighbors...
	// 0 1 2
	// 3   4
	// 5 6 7
	FIntPoint NeighborsKeys[8] =
	{
		LandscapeKey + FIntPoint(-1, -1),
		LandscapeKey + FIntPoint(+0, -1),
		LandscapeKey + FIntPoint(+1, -1),
		LandscapeKey + FIntPoint(-1, +0),
		LandscapeKey + FIntPoint(+1, +0),
		LandscapeKey + FIntPoint(-1, +1),
		LandscapeKey + FIntPoint(+0, +1),
		LandscapeKey + FIntPoint(+1, +1)
	};

	// Todo: Use data accessor not collision

	ULandscapeHeightfieldCollisionComponent* NeighborCollisions[8];
	// Search for Neighbors...
	for (int32 i = 0; i < 8; ++i)
	{
		ULandscapeComponent* Comp = XYtoComponentMap.FindRef(NeighborsKeys[i]);
		if (Comp)
		{
			ULandscapeHeightfieldCollisionComponent* NeighborCollision = Comp->GetCollisionComponent();
			// Skip cooked because CollisionHeightData not saved during cook
			if (NeighborCollision && !NeighborCollision->GetOutermost()->bIsCookedForEditor)
			{
				NeighborCollisions[i] = NeighborCollision;
			}
			else
			{
				NeighborCollisions[i] = nullptr;
			}
		}
		else
		{
			NeighborCollisions[i] = nullptr;
		}
	}

	uint8 CornerSet = 0;
	uint16 HeightCorner[4];

	// Corner Cases...
	if (NeighborCollisions[0])
	{
		if (uint16* Heights = (uint16*)NeighborCollisions[0]->CollisionHeightData.Lock(LOCK_READ_ONLY))
		{
			int32 CollisionSizeVerts = NeighborCollisions[0]->CollisionSizeQuads + 1;
			HeightCorner[0] = Heights[CollisionSizeVerts - 1 + (CollisionSizeVerts - 1)*CollisionSizeVerts];
			CornerSet |= 1;
		}
		NeighborCollisions[0]->CollisionHeightData.Unlock();
	}
	if (NeighborCollisions[2])
	{
		if (uint16* Heights = (uint16*)NeighborCollisions[2]->CollisionHeightData.Lock(LOCK_READ_ONLY))
		{
			int32 CollisionSizeVerts = NeighborCollisions[2]->CollisionSizeQuads + 1;
			HeightCorner[1] = Heights[(CollisionSizeVerts - 1)*CollisionSizeVerts];
			CornerSet |= 1 << 1;
		}
		NeighborCollisions[2]->CollisionHeightData.Unlock();
	}
	if (NeighborCollisions[5])
	{
		if (uint16* Heights = (uint16*)NeighborCollisions[5]->CollisionHeightData.Lock(LOCK_READ_ONLY))
		{
			int32 CollisionSizeVerts = NeighborCollisions[5]->CollisionSizeQuads + 1;
			HeightCorner[2] = Heights[(CollisionSizeVerts - 1)];
			CornerSet |= 1 << 2;
		}
		NeighborCollisions[5]->CollisionHeightData.Unlock();
	}
	if (NeighborCollisions[7])
	{
		if (uint16* Heights = (uint16*)NeighborCollisions[7]->CollisionHeightData.Lock(LOCK_READ_ONLY))
		{
			int32 CollisionSizeVerts = NeighborCollisions[7]->CollisionSizeQuads + 1;
			HeightCorner[3] = Heights[0];
			CornerSet |= 1 << 3;
		}
		NeighborCollisions[7]->CollisionHeightData.Unlock();
	}

	// Other cases...
	if (NeighborCollisions[1])
	{
		if (uint16* Heights = (uint16*)NeighborCollisions[1]->CollisionHeightData.Lock(LOCK_READ_ONLY))
		{
			int32 CollisionSizeVerts = NeighborCollisions[1]->CollisionSizeQuads + 1;
			HeightCorner[0] = Heights[(CollisionSizeVerts - 1)*CollisionSizeVerts];
			CornerSet |= 1;
			HeightCorner[1] = Heights[CollisionSizeVerts - 1 + (CollisionSizeVerts - 1)*CollisionSizeVerts];
			CornerSet |= 1 << 1;
		}
		NeighborCollisions[1]->CollisionHeightData.Unlock();
	}
	if (NeighborCollisions[3])
	{
		if (uint16* Heights = (uint16*)NeighborCollisions[3]->CollisionHeightData.Lock(LOCK_READ_ONLY))
		{
			int32 CollisionSizeVerts = NeighborCollisions[3]->CollisionSizeQuads + 1;
			HeightCorner[0] = Heights[(CollisionSizeVerts - 1)];
			CornerSet |= 1;
			HeightCorner[2] = Heights[CollisionSizeVerts - 1 + (CollisionSizeVerts - 1)*CollisionSizeVerts];
			CornerSet |= 1 << 2;
		}
		NeighborCollisions[3]->CollisionHeightData.Unlock();
	}
	if (NeighborCollisions[4])
	{
		if (uint16* Heights = (uint16*)NeighborCollisions[4]->CollisionHeightData.Lock(LOCK_READ_ONLY))
		{
			int32 CollisionSizeVerts = NeighborCollisions[4]->CollisionSizeQuads + 1;
			HeightCorner[1] = Heights[0];
			CornerSet |= 1 << 1;
			HeightCorner[3] = Heights[(CollisionSizeVerts - 1)*CollisionSizeVerts];
			CornerSet |= 1 << 3;
		}
		NeighborCollisions[4]->CollisionHeightData.Unlock();
	}
	if (NeighborCollisions[6])
	{
		if (uint16* Heights = (uint16*)NeighborCollisions[6]->CollisionHeightData.Lock(LOCK_READ_ONLY))
		{
			int32 CollisionSizeVerts = NeighborCollisions[6]->CollisionSizeQuads + 1;
			HeightCorner[2] = Heights[0];
			CornerSet |= 1 << 2;
			HeightCorner[3] = Heights[(CollisionSizeVerts - 1)];
			CornerSet |= 1 << 3;
		}
		NeighborCollisions[6]->CollisionHeightData.Unlock();
	}

	// Fill unset values
	// First iteration only for valid values distance 1 propagation
	// Second iteration fills left ones...
	FillCornerValues(CornerSet, HeightCorner);
	//check(CornerSet == 15);

	FIntPoint SectionBase = LandscapeKey * ComponentSizeQuads;

	// Transform Height to Vectors...
	FTransform LtoW = GetLandscapeProxy()->LandscapeActorToWorld();
	AddCollision.Corners[0] = LtoW.TransformPosition(FVector(SectionBase.X                     , SectionBase.Y                     , LandscapeDataAccess::GetLocalHeight(HeightCorner[0])));
	AddCollision.Corners[1] = LtoW.TransformPosition(FVector(SectionBase.X + ComponentSizeQuads, SectionBase.Y                     , LandscapeDataAccess::GetLocalHeight(HeightCorner[1])));
	AddCollision.Corners[2] = LtoW.TransformPosition(FVector(SectionBase.X                     , SectionBase.Y + ComponentSizeQuads, LandscapeDataAccess::GetLocalHeight(HeightCorner[2])));
	AddCollision.Corners[3] = LtoW.TransformPosition(FVector(SectionBase.X + ComponentSizeQuads, SectionBase.Y + ComponentSizeQuads, LandscapeDataAccess::GetLocalHeight(HeightCorner[3])));
}

void ULandscapeHeightfieldCollisionComponent::ExportCustomProperties(FOutputDevice& Out, uint32 Indent)
{
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	int32 CollisionSizeVerts = CollisionSizeQuads + 1;
	int32 SimpleCollisionSizeVerts = SimpleCollisionSizeQuads > 0 ? SimpleCollisionSizeQuads + 1 : 0;
	int32 NumHeights = FMath::Square(CollisionSizeVerts) + FMath::Square(SimpleCollisionSizeVerts);
	checkf(CollisionHeightData.GetElementCount() == FMath::Square(CollisionSizeVerts) + FMath::Square(SimpleCollisionSizeVerts),
		TEXT("Invalid collision height data element count : %lld found while there should be (%d samples + %d simple samples)"), CollisionHeightData.GetElementCount(), FMath::Square(CollisionSizeVerts), FMath::Square(SimpleCollisionSizeVerts));

	uint16* Heights = (uint16*)CollisionHeightData.Lock(LOCK_READ_ONLY);

	Out.Logf(TEXT("%sCustomProperties CollisionHeightData "), FCString::Spc(Indent));
	for (int32 i = 0; i < NumHeights; i++)
	{
		Out.Logf(TEXT("%d "), Heights[i]);
	}

	CollisionHeightData.Unlock();
	Out.Logf(TEXT("\r\n"));

	const int32 NumDominantLayerSamples = static_cast<int32>(DominantLayerData.GetElementCount());
	check(NumDominantLayerSamples == 0 || NumDominantLayerSamples == NumHeights);

	if (NumDominantLayerSamples > 0)
	{
		const uint8* DominantLayerSamples = (uint8*)DominantLayerData.Lock(LOCK_READ_ONLY);

		Out.Logf(TEXT("%sCustomProperties DominantLayerData "), FCString::Spc(Indent));
		for (int32 i = 0; i < NumDominantLayerSamples; i++)
		{
			Out.Logf(TEXT("%02x"), DominantLayerSamples[i]);
		}

		DominantLayerData.Unlock();
		Out.Logf(TEXT("\r\n"));
	}
}

void ULandscapeHeightfieldCollisionComponent::ImportCustomProperties(const TCHAR* SourceText, FFeedbackContext* Warn)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeHeightfieldCollisionComponent::ImportCustomProperties);

	if (FParse::Command(&SourceText, TEXT("CollisionHeightData")))
	{
		int32 CollisionSizeVerts = CollisionSizeQuads + 1;
		int32 SimpleCollisionSizeVerts = SimpleCollisionSizeQuads > 0 ? SimpleCollisionSizeQuads + 1 : 0;
		int32 NumHeights = FMath::Square(CollisionSizeVerts) + FMath::Square(SimpleCollisionSizeVerts);

		CollisionHeightData.Lock(LOCK_READ_WRITE);
		uint16* Heights = (uint16*)CollisionHeightData.Realloc(NumHeights);
		FMemory::Memzero(Heights, sizeof(uint16)*NumHeights);

		// Helper to convert digits to uint (assume FChar::IsDigit(InTChar) == true)
		auto TCharToDigit = [](TCHAR InTChar) { return static_cast<uint32>(InTChar - TEXT('0')); };

		FParse::Next(&SourceText);
		uint64 NumTokensParsed = 0;
		int32 Index = 0;
		while (FChar::IsDigit(*SourceText))
		{
			if (Index < NumHeights)
			{
				uint32 Value = 0;
				// Don't use FCString::Atoi as it can be pathologically slow on some platforms because of the StringCast conversion to convert from TCHAR to char string
				while (FChar::IsDigit(*SourceText))
				{
					Value = Value * 10 + TCharToDigit(*SourceText++);
				}
				Heights[Index++] = static_cast<uint16>(Value);
			}
			else
			{
				// If there are more tokens despite the Index having reached NumHeights, then the input is malformed. We still want to advance the parsing in such a case to avoid looping infintely
				while (FChar::IsDigit(*SourceText))
				{
					++SourceText;
				}
			}
			++NumTokensParsed;
			FParse::Next(&SourceText);
		}

		CollisionHeightData.Unlock();

		if (NumTokensParsed != NumHeights)
		{
			Warn->Log(*FText::Format(NSLOCTEXT("Core", "Importing_HeightfieldCollision_SyntaxError", "Syntax Error : num tokens: {0} vs. num height: {1}"), { NumTokensParsed, NumHeights }).ToString());
		}
	}
	else if (FParse::Command(&SourceText, TEXT("DominantLayerData")))
	{
		int32 NumDominantLayerSamples = FMath::Square(CollisionSizeQuads + 1);

		DominantLayerData.Lock(LOCK_READ_WRITE);
		uint8* DominantLayerSamples = (uint8*)DominantLayerData.Realloc(NumDominantLayerSamples);
		FMemory::Memzero(DominantLayerSamples, NumDominantLayerSamples);

		FParse::Next(&SourceText);
		int32 i = 0;
		while (SourceText[0] && SourceText[1])
		{
			if (i < NumDominantLayerSamples)
			{
				DominantLayerSamples[i++] = static_cast<uint8>(FParse::HexDigit(SourceText[0]) * 16 + FParse::HexDigit(SourceText[1]));
			}
			SourceText += 2;
		}

		DominantLayerData.Unlock();

		if (i != NumDominantLayerSamples)
		{
			Warn->Log(*NSLOCTEXT("Core", "SyntaxError", "Syntax Error").ToString());
		}
	}
}

#endif // WITH_EDITOR

ULandscapeInfo* ULandscapeHeightfieldCollisionComponent::GetLandscapeInfo() const
{
	return GetLandscapeProxy()->GetLandscapeInfo();
}

ALandscapeProxy* ULandscapeHeightfieldCollisionComponent::GetLandscapeProxy() const
{
	return CastChecked<ALandscapeProxy>(GetOuter());
}

FIntPoint ULandscapeHeightfieldCollisionComponent::GetSectionBase() const
{
	return FIntPoint(SectionBaseX, SectionBaseY);
}

void ULandscapeHeightfieldCollisionComponent::SetSectionBase(FIntPoint InSectionBase)
{
	SectionBaseX = InSectionBase.X;
	SectionBaseY = InSectionBase.Y;
}

ULandscapeHeightfieldCollisionComponent::ULandscapeHeightfieldCollisionComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetCollisionProfileName(UCollisionProfile::BlockAll_ProfileName);
	SetGenerateOverlapEvents(false);
	CastShadow = false;
	bUseAsOccluder = true;
	bAllowCullDistanceVolume = false;
	Mobility = EComponentMobility::Static;
	bCanEverAffectNavigation = true;
	bHasCustomNavigableGeometry = EHasCustomNavigableGeometry::Yes;

	HeightfieldRowsCount = -1;
	HeightfieldColumnsCount = -1;

	// landscape collision components should be deterministically created and therefor are addressable over the network
	SetNetAddressable();
}

ULandscapeHeightfieldCollisionComponent::ULandscapeHeightfieldCollisionComponent(FVTableHelper& Helper)
	: Super(Helper)
{

}

ULandscapeHeightfieldCollisionComponent::~ULandscapeHeightfieldCollisionComponent() = default;

ULandscapeComponent* ULandscapeHeightfieldCollisionComponent::GetRenderComponent() const
{
	return RenderComponentRef.Get();
}

TOptional<float> ULandscapeHeightfieldCollisionComponent::GetHeight(float X, float Y, EHeightfieldSource HeightFieldSource)
{
	TOptional<float> Height;
	const float ZScale = static_cast<float>(GetComponentTransform().GetScale3D().Z * LANDSCAPE_ZSCALE); // TODO michael.balzer: Is it okay that ZScale is not used in this function?

	if (!IsValidRef(HeightfieldRef))
	{
		return Height;
	}
	
	Chaos::FHeightField* HeightField = nullptr;
	
	switch(HeightFieldSource)
	{
	case EHeightfieldSource::None:
		break;
	case EHeightfieldSource::Simple:
		HeightField = HeightfieldRef->HeightfieldSimpleGeometry.GetReference(); 
		break;
	case EHeightfieldSource::Complex:
		HeightField = HeightfieldRef->HeightfieldGeometry.GetReference(); 
		break;
#if WITH_EDITORONLY_DATA		
	case EHeightfieldSource::Editor:
		HeightField = HeightfieldRef->EditorHeightfieldGeometry.GetReference();
		break;
#endif 
	}
	
	if (HeightField)
	{
		Height = static_cast<float>(HeightField->GetHeightAt({ X, Y }));
	}

	return Height;
}

UPhysicalMaterial* ULandscapeHeightfieldCollisionComponent::GetPhysicalMaterial(float X, float Y, EHeightfieldSource HeightFieldSource)
{
	UPhysicalMaterial* PhysicalMaterial = nullptr;

	if (!IsValidRef(HeightfieldRef))
	{
		return PhysicalMaterial;
	}

	Chaos::FHeightField* HeightField = nullptr;

	switch (HeightFieldSource)
	{
	case EHeightfieldSource::None:
		break;
	case EHeightfieldSource::Simple:
		HeightField = HeightfieldRef->HeightfieldSimpleGeometry.GetReference();
		break;
	case EHeightfieldSource::Complex:
		HeightField = HeightfieldRef->HeightfieldGeometry.GetReference();
		break;
#if WITH_EDITORONLY_DATA		
	case EHeightfieldSource::Editor:
		HeightField = HeightfieldRef->EditorHeightfieldGeometry.GetReference();
		break;
#endif 
	}

	if (HeightField)
	{
		const uint8 MaterialIndex = HeightField->GetMaterialIndexAt({ X, Y });
		if (MaterialIndex != TNumericLimits<uint8>::Max() && HeightfieldRef->UsedChaosMaterials.IsValidIndex(MaterialIndex))
		{
			Chaos::FMaterialHandle MaterialHandle = HeightfieldRef->UsedChaosMaterials[MaterialIndex];
			if (Chaos::FChaosPhysicsMaterial* ChaosMaterial = MaterialHandle.Get())
			{
				PhysicalMaterial = FChaosUserData::Get<UPhysicalMaterial>(ChaosMaterial->UserData);
			}
		}
	}

	return PhysicalMaterial;
}

struct FHeightFieldAccessor
{
	FHeightFieldAccessor(const ULandscapeHeightfieldCollisionComponent::FHeightfieldGeometryRef& InGeometryRef)
	: GeometryRef(InGeometryRef)
	, NumX(InGeometryRef.HeightfieldGeometry.IsValid() ? InGeometryRef.HeightfieldGeometry->GetNumCols() : 0)
	, NumY(InGeometryRef.HeightfieldGeometry.IsValid() ? InGeometryRef.HeightfieldGeometry->GetNumRows() : 0)
	{
	}

	float GetUnscaledHeight(int32 X, int32 Y) const
	{
		return static_cast<float>(GeometryRef.HeightfieldGeometry->GetHeight(X, Y));
	}

	uint8 GetMaterialIndex(int32 X, int32 Y) const
	{
		return GeometryRef.HeightfieldGeometry->GetMaterialIndex(X, Y);
	}

	const ULandscapeHeightfieldCollisionComponent::FHeightfieldGeometryRef& GeometryRef;
	const int32 NumX = 0;
	const int32 NumY = 0;
};

bool ULandscapeHeightfieldCollisionComponent::FillHeightTile(TArrayView<float> Heights, int32 Offset, int32 Stride) const
{
	if (!IsValidRef(HeightfieldRef))
	{
		return false;
	}

	FHeightFieldAccessor Accessor(*HeightfieldRef.GetReference());

	const int32 LastTiledIndex = Offset + FMath::Max(0, Accessor.NumX - 1) + Stride * FMath::Max(0, Accessor.NumY - 1);
	if (!Heights.IsValidIndex(LastTiledIndex))
	{
		return false;
	}

	const FTransform& WorldTransform = GetComponentToWorld();
	const float ZScale = static_cast<float>(WorldTransform.GetScale3D().Z * LANDSCAPE_ZSCALE);

	// Write all values to output array
	ParallelFor(Accessor.NumY, [&Heights, &Accessor, &WorldTransform, Offset, Stride, ZScale](int32 y)
	{
		for (int32 x = 0; x < Accessor.NumX; ++x)
		{
			const float CurrHeight = Accessor.GetUnscaledHeight(x, y);
			const float WorldHeight = static_cast<float>(WorldTransform.TransformPositionNoScale(FVector(0, 0, CurrHeight * ZScale)).Z);

			// write output
			const int32 WriteIndex = Offset + y * Stride + x;
			Heights[WriteIndex] = WorldHeight;
		}
	});

	return true;
}

bool ULandscapeHeightfieldCollisionComponent::FillMaterialIndexTile(TArrayView<uint8> Materials, int32 Offset, int32 Stride) const
{
	if (!IsValidRef(HeightfieldRef))
	{
		return false;
	}

	FHeightFieldAccessor Accessor(*HeightfieldRef.GetReference());

	const int32 LastTiledIndex = Offset + FMath::Max(0, Accessor.NumX - 1) + Stride * FMath::Max(0, Accessor.NumY - 1);
	if (!Materials.IsValidIndex(LastTiledIndex))
	{
		return false;
	}

	// Write all values to output array
	for (int32 y = 0; y < Accessor.NumY; ++y)
	{
		for (int32 x = 0; x < Accessor.NumX; ++x)
		{
			// write output
			const int32 WriteIndex = Offset + y * Stride + x;
			Materials[WriteIndex] = Accessor.GetMaterialIndex(x, y);
		}
	}

	return true;
}

void ULandscapeHeightfieldCollisionComponent::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(sizeof(CookedCollisionData) + CookedCollisionData.GetAllocatedSize() + sizeof(HeightfieldRowsCount) + sizeof(HeightfieldColumnsCount));
	
	if (IsValidRef(HeightfieldRef))
	{
		HeightfieldRef->GetResourceSizeEx(CumulativeResourceSize);
	}

	{
		FReadScopeLock Lock(CachedHeightFieldSamplesLock);
		CachedHeightFieldSamples.GetResourceSizeEx(CumulativeResourceSize);
	}
}

TOptional<float> ALandscapeProxy::GetHeightAtLocation(FVector Location, EHeightfieldSource HeightFieldSource) const
{
	TOptional<float> Height;
	if (ULandscapeInfo* Info = GetLandscapeInfo())
	{
		const FVector ActorSpaceLocation = LandscapeActorToWorld().InverseTransformPosition(Location);
		const FIntPoint Key = FIntPoint(FMath::FloorToInt32(ActorSpaceLocation.X / ComponentSizeQuads), FMath::FloorToInt32(ActorSpaceLocation.Y / ComponentSizeQuads));
		ULandscapeHeightfieldCollisionComponent* Component = Info->XYtoCollisionComponentMap.FindRef(Key);
		if (Component)
		{
			const FVector ComponentSpaceLocation = Component->GetComponentToWorld().InverseTransformPosition(Location);
			const TOptional<float> LocalHeight = Component->GetHeight(static_cast<float>(ComponentSpaceLocation.X), static_cast<float>(ComponentSpaceLocation.Y), HeightFieldSource);
			if (LocalHeight.IsSet())
			{
				Height = static_cast<float>(Component->GetComponentToWorld().TransformPositionNoScale(FVector(0, 0, LocalHeight.GetValue())).Z);
			}
		}
	}
	return Height;
}

UPhysicalMaterial* ALandscapeProxy::GetPhysicalMaterialAtLocation(FVector Location, EHeightfieldSource HeightFieldSource) const
{
	UPhysicalMaterial* PhysicalMaterial = nullptr;
	if (ULandscapeInfo* Info = GetLandscapeInfo())
	{
		const FVector ActorSpaceLocation = LandscapeActorToWorld().InverseTransformPosition(Location);
		const FIntPoint Key = FIntPoint(FMath::FloorToInt32(ActorSpaceLocation.X / ComponentSizeQuads), FMath::FloorToInt32(ActorSpaceLocation.Y / ComponentSizeQuads));
		ULandscapeHeightfieldCollisionComponent* Component = Info->XYtoCollisionComponentMap.FindRef(Key);
		if (Component)
		{
			const FVector ComponentSpaceLocation = Component->GetComponentToWorld().InverseTransformPosition(Location);
			PhysicalMaterial = Component->GetPhysicalMaterial(static_cast<float>(ComponentSpaceLocation.X), static_cast<float>(ComponentSpaceLocation.Y), HeightFieldSource);
		}
	}

	return PhysicalMaterial;
}

void ALandscapeProxy::GetHeightValues(int32& SizeX, int32& SizeY, TArray<float> &ArrayValues) const
{			
	SizeX = 0;
	SizeY = 0;
	ArrayValues.SetNum(0);
	
	// Exit if we have no landscape data
	if (LandscapeComponents.Num() == 0 || CollisionComponents.Num() == 0)
	{
		return;
	}

	// find index coordinate range for landscape
	int32 MinX = MAX_int32;
	int32 MinY = MAX_int32;
	int32 MaxX = -MAX_int32;
	int32 MaxY = -MAX_int32;

	for (ULandscapeComponent* LandscapeComponent : LandscapeComponents)
	{
		// expecting a valid pointer to a landscape component
		if (!LandscapeComponent)
		{
			return;
		}

		// #todo(dmp): should we be using ULandscapeHeightfieldCollisionComponent.CollisionSizeQuads (or HeightFieldData->GetNumCols)
		MinX = FMath::Min(LandscapeComponent->SectionBaseX, MinX);
		MinY = FMath::Min(LandscapeComponent->SectionBaseY, MinY);
		MaxX = FMath::Max(LandscapeComponent->SectionBaseX + LandscapeComponent->ComponentSizeQuads, MaxX);
		MaxY = FMath::Max(LandscapeComponent->SectionBaseY + LandscapeComponent->ComponentSizeQuads, MaxY);		
	}

	if (MinX == MAX_int32)
	{
		return;
	}			
		
	SizeX = (MaxX - MinX + 1);
	SizeY = (MaxY - MinY + 1);
	ArrayValues.SetNumUninitialized(SizeX * SizeY);
	
	for (ULandscapeHeightfieldCollisionComponent *CollisionComponent : CollisionComponents)
	{
		// Make sure we have a valid collision component and a heightfield
		if (!CollisionComponent || !IsValidRef(CollisionComponent->HeightfieldRef))
		{
			SizeX = 0;
			SizeY = 0;
			ArrayValues.SetNum(0);
			return;
		}

		Chaos::FHeightFieldPtr& HeightFieldData = CollisionComponent->HeightfieldRef->HeightfieldGeometry;

		// If we are expecting height data, but it isn't there, clear the return array, and exit
		if (!HeightFieldData.IsValid())
		{
			SizeX = 0;
			SizeY = 0;
			ArrayValues.SetNum(0);
			return;
		}

		const int32 BaseX = CollisionComponent->SectionBaseX - MinX;
		const int32 BaseY = CollisionComponent->SectionBaseY - MinY;

		const int32 NumX = HeightFieldData->GetNumCols();
		const int32 NumY = HeightFieldData->GetNumRows();

		const FTransform& ComponentToWorld = CollisionComponent->GetComponentToWorld();
		const float ZScale = static_cast<float>(ComponentToWorld.GetScale3D().Z * LANDSCAPE_ZSCALE);

		// Write all values to output array
		for (int32 x = 0; x < NumX; ++x)
		{
			for (int32 y = 0; y < NumY; ++y)
			{
				const float CurrHeight = static_cast<float>(HeightFieldData->GetHeight(x, y) * ZScale);				
				const float WorldHeight = static_cast<float>(ComponentToWorld.TransformPositionNoScale(FVector(0, 0, CurrHeight)).Z);
				
				// write output
				const int32 WriteX = BaseX + x;
				const int32 WriteY = BaseY + y;
				const int32 Idx = WriteY * SizeX + WriteX;
				ArrayValues[Idx] = WorldHeight;
			}
		}
	}
}
