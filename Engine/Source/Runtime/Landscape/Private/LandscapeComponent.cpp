// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeComponent.h"
#include "LandscapeLayerInfoObject.h"
#include "Materials/MaterialInstance.h"
#include "LandscapeEdit.h"
#include "LandscapeRender.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapeComponent)

#if WITH_EDITOR

#include "LandscapeHLODBuilder.h"

uint32 ULandscapeComponent::UndoRedoModifiedComponentCount;
TMap<ULandscapeComponent*, uint32> ULandscapeComponent::UndoRedoModifiedComponents;

#endif

FName FWeightmapLayerAllocationInfo::GetLayerName() const
{
	if (LayerInfo)
	{
		return LayerInfo->GetLayerName();
	}
	return NAME_None;
}

uint32 FWeightmapLayerAllocationInfo::GetHash() const
{
	uint32 Hash = PointerHash(LayerInfo);
	Hash = HashCombine(GetTypeHash(WeightmapTextureIndex), Hash);
	Hash = HashCombine(GetTypeHash(WeightmapTextureChannel), Hash);
	return Hash;
}

ELightMapInteractionType ULandscapeComponent::GetStaticLightingType() const 
{ 
	if (GetLightmapType() == ELightmapType::ForceVolumetric)
		return LMIT_GlobalVolume;

	return LMIT_Texture;	
}

#if WITH_EDITOR

void FLandscapeEditToolRenderData::UpdateDebugColorMaterial(const ULandscapeComponent* const Component)
{
	Component->GetLayerDebugColorKey(DebugChannelR, DebugChannelG, DebugChannelB);
}

void FLandscapeEditToolRenderData::UpdateSelectionMaterial(int32 InSelectedType, const ULandscapeComponent* const Component)
{
	// Check selection
	if (SelectedType != InSelectedType && (SelectedType & ST_REGION) && !(InSelectedType & ST_REGION))
	{
		// Clear Select textures...
		if (DataTexture)
		{
			FLandscapeEditDataInterface LandscapeEdit(Component->GetLandscapeInfo());
			LandscapeEdit.ZeroTexture(DataTexture);
		}
	}

	SelectedType = InSelectedType;
}

void ULandscapeComponent::UpdateEditToolRenderData()
{
	FLandscapeComponentSceneProxy* LandscapeSceneProxy = (FLandscapeComponentSceneProxy*)SceneProxy;

	if (LandscapeSceneProxy != nullptr)
	{
		TArray<UMaterialInterface*> UsedMaterialsForVerification;
		const bool bGetDebugMaterials = true;
		GetUsedMaterials(UsedMaterialsForVerification, bGetDebugMaterials);

		// The selection/tool/gizmo materials are displayed in a translucent pass with Disable Depth Test == true so that they always show despite being underneath other objects.
		//  However, for this to work under all circumstances, we must make sure they don't get CPU-culled by primitives whose bounds fully occlude it, so we artificially inflate
		//  their bounds dynamically (without having to invalidate the render state) : 
		const bool bDisableCulling = (EditToolRenderData.SelectedType != FLandscapeEditToolRenderData::ST_NONE)
			|| (EditToolRenderData.ToolMaterial != nullptr)
			|| (EditToolRenderData.GizmoMaterial != nullptr);

		UpdateOcclusionBoundsSlack(bDisableCulling ? TNumericLimits<float>::Max() : 0.0f);

		FLandscapeEditToolRenderData LandscapeEditToolRenderData = EditToolRenderData;
		ENQUEUE_RENDER_COMMAND(UpdateEditToolRenderData)(
			[LandscapeEditToolRenderData, LandscapeSceneProxy, UsedMaterialsForVerification](FRHICommandListImmediate& RHICmdList)
			{
				LandscapeSceneProxy->EditToolRenderData = LandscapeEditToolRenderData;				
				LandscapeSceneProxy->SetUsedMaterialForVerification(UsedMaterialsForVerification);
			});
	}
}

TSubclassOf<UHLODBuilder> ULandscapeComponent::GetCustomHLODBuilderClass() const
{
	return ULandscapeHLODBuilder::StaticClass();
}

#endif
