// Copyright Epic Games, Inc. All Rights Reserved.
#include "LandscapeConfigHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapeConfigHelper)

#if WITH_EDITOR

#include "Engine/World.h"
#include "Misc/ScopeExit.h"
#include "LandscapeInfo.h"
#include "Landscape.h"
#include "LandscapeProxy.h"
#include "LandscapeSettings.h"
#include "LandscapeStreamingProxy.h"
#include "LandscapeSplineActor.h"
#include "LandscapeSplineControlPoint.h"
#include "LandscapeSplinesComponent.h"
#include "LandscapeEdit.h"
#include "LandscapeDataAccess.h"
#include "LandscapeEditLayer.h"
#include "LandscapeSubsystem.h"
#include "ActorPartition/ActorPartitionSubsystem.h"

int32 FLandscapeConfig::NumSectionValues[2] = { 1, 2 };
int32 FLandscapeConfig::SubsectionSizeQuadsValues[6] = { 7, 15, 31, 63, 127, 255 };

FLandscapeConfig::FLandscapeConfig(ULandscapeInfo* InLandscapeInfo)
	: FLandscapeConfig(InLandscapeInfo->ComponentNumSubsections, InLandscapeInfo->SubsectionSizeQuads, InLandscapeInfo->LandscapeActor->GetGridSize() / InLandscapeInfo->ComponentSizeQuads)
{

}

bool FLandscapeConfigChange::Validate() const
{
	bool bValidNumSections = false;
	for (int32 Index = 0; Index < UE_ARRAY_COUNT(NumSectionValues); ++Index)
	{
		if (ComponentNumSubsections == NumSectionValues[Index])
		{
			bValidNumSections = true;
			break;
		}
	}
	bool bValidSectionSizeQuads = false;
	for (int32 Index = 0; Index < UE_ARRAY_COUNT(SubsectionSizeQuadsValues); ++Index)
	{
		if (SubsectionSizeQuads == SubsectionSizeQuadsValues[Index])
		{
			bValidSectionSizeQuads = true;
			break;
		}
	}
	return bValidNumSections && bValidSectionSizeQuads;
}

ALandscapeProxy* FLandscapeConfigHelper::FindOrAddLandscapeStreamingProxy(ULandscapeInfo* InLandscapeInfo, const FIntPoint& InSectionBase)
{
	UWorld* World = InLandscapeInfo->LandscapeActor->GetWorld();
	UActorPartitionSubsystem* ActorPartitionSubsystem = World->GetSubsystem<UActorPartitionSubsystem>();
	const uint32 GridSize = InLandscapeInfo->LandscapeActor->GetGridSize();

	UActorPartitionSubsystem::FCellCoord CellCoord = UActorPartitionSubsystem::FCellCoord::GetCellCoord(InSectionBase, InLandscapeInfo->LandscapeActor->GetLevel(), GridSize);
	return FLandscapeConfigHelper::FindOrAddLandscapeStreamingProxy(ActorPartitionSubsystem, InLandscapeInfo, CellCoord);
}

ALandscapeProxy* FLandscapeConfigHelper::FindOrAddLandscapeStreamingProxy(UActorPartitionSubsystem* InActorPartitionSubsystem, ULandscapeInfo* InLandscapeInfo, const UActorPartitionSubsystem::FCellCoord& InCellCoord)
{
	ALandscape* Landscape = InLandscapeInfo->LandscapeActor.Get();
	check(Landscape);

	auto LandscapeProxyCreated = [InCellCoord, Landscape, InLandscapeInfo](APartitionActor* PartitionActor)
	{
		const FIntPoint CellLocation(static_cast<int32>(InCellCoord.X) * Landscape->GetGridSize(), static_cast<int32>(InCellCoord.Y) * Landscape->GetGridSize());

		ALandscapeProxy* LandscapeProxy = CastChecked<ALandscapeProxy>(PartitionActor);
		// copy shared properties to this new proxy
		LandscapeProxy->SynchronizeSharedProperties(Landscape);
		const FVector ProxyLocation = Landscape->GetActorLocation() + FVector(CellLocation.X * Landscape->GetActorRelativeScale3D().X, CellLocation.Y * Landscape->GetActorRelativeScale3D().Y, 0.0f);

		LandscapeProxy->CreateLandscapeInfo();
		LandscapeProxy->SetActorLocationAndRotation(ProxyLocation, Landscape->GetActorRotation());
		LandscapeProxy->SetSectionBase(FIntPoint(CellLocation.X, CellLocation.Y));
		LandscapeProxy->SetIsSpatiallyLoaded(LandscapeProxy->GetLandscapeInfo()->AreNewLandscapeActorsSpatiallyLoaded());
		// Fixup the transform based on the section base offset
		InLandscapeInfo->FixupProxiesTransform();
	};

	const bool bCreate = true;
	const bool bBoundsSearch = false;

	ALandscapeProxy* LandscapeProxy = Cast<ALandscapeProxy>(InActorPartitionSubsystem->GetActor(ALandscapeStreamingProxy::StaticClass(), InCellCoord, bCreate, InLandscapeInfo->LandscapeGuid, Landscape->GetGridSize(), bBoundsSearch, LandscapeProxyCreated));
	check(!LandscapeProxy || LandscapeProxy->GetGridSize() == Landscape->GetGridSize());
	return LandscapeProxy;
}

bool FLandscapeConfigHelper::ChangeGridSize(ULandscapeInfo* InLandscapeInfo, uint32 InNewGridSizeInComponents, TSet<AActor*>& OutActorsToDelete)
{
	check(InLandscapeInfo);
	check(!InLandscapeInfo->IsTemplate());
	const uint32 GridSize = InLandscapeInfo->GetGridSize(InNewGridSizeInComponents);
	
	InLandscapeInfo->LandscapeActor->Modify();
	InLandscapeInfo->LandscapeActor->SetGridSize(GridSize);

	// This needs to be done before moving components
	InLandscapeInfo->LandscapeActor->InitializeLandscapeLayersWeightmapUsage();

	// Make sure if actor didn't include grid size in name it now does. This will avoid recycling 
	// LandscapeStreamingProxy actors and create new ones with the proper name.
	InLandscapeInfo->LandscapeActor->bIncludeGridSizeInNameForLandscapeActors = true;

	FIntRect Extent;
	InLandscapeInfo->GetLandscapeExtent(Extent.Min.X, Extent.Min.Y, Extent.Max.X, Extent.Max.Y);
	const FBox Bounds(FVector(Extent.Min), FVector(Extent.Max));

	UWorld* World = InLandscapeInfo->LandscapeActor->GetWorld();
	UActorPartitionSubsystem* ActorPartitionSubsystem = World->GetSubsystem<UActorPartitionSubsystem>();

	TArray<ULandscapeComponent*> LandscapeComponents;
	LandscapeComponents.Reserve(InLandscapeInfo->XYtoComponentMap.Num());
	InLandscapeInfo->ForAllLandscapeComponents([&LandscapeComponents](ULandscapeComponent* LandscapeComponent)
	{
		LandscapeComponents.Add(LandscapeComponent);
	});

	TSet<ALandscapeProxy*> ProxiesToDelete;
	
	FActorPartitionGridHelper::ForEachIntersectingCell(ALandscapeStreamingProxy::StaticClass(), Extent, InLandscapeInfo->LandscapeActor->GetLevel(), [ActorPartitionSubsystem, InLandscapeInfo, InNewGridSizeInComponents, &LandscapeComponents, &ProxiesToDelete](const UActorPartitionSubsystem::FCellCoord& CellCoord, const FIntRect& CellBounds)
	{
		TMap<ULandscapeComponent*, UMaterialInterface*> ComponentMaterials;
		TMap<ULandscapeComponent*, UMaterialInterface*> ComponentHoleMaterials;
		TMap <ULandscapeComponent*, TMap<int32, UMaterialInterface*>> ComponentLODMaterials;

		TArray<ULandscapeComponent*> ComponentsToMove;
		const int32 MaxComponents = (int32)(InNewGridSizeInComponents * InNewGridSizeInComponents);
		ComponentsToMove.Reserve(MaxComponents);
		for (int32 i = 0; i < LandscapeComponents.Num();)
		{
			ULandscapeComponent* LandscapeComponent = LandscapeComponents[i];
			if (CellBounds.Contains(LandscapeComponent->GetSectionBase()))
			{
				ComponentMaterials.FindOrAdd(LandscapeComponent, LandscapeComponent->GetLandscapeMaterial());
				ComponentHoleMaterials.FindOrAdd(LandscapeComponent, LandscapeComponent->GetLandscapeHoleMaterial());
				TMap<int32, UMaterialInterface*>& LODMaterials = ComponentLODMaterials.FindOrAdd(LandscapeComponent);
				for (int8 LODIndex = 0; LODIndex <= 8; ++LODIndex)
				{
					LODMaterials.Add(LODIndex, LandscapeComponent->GetLandscapeMaterial(LODIndex));
				}

				ComponentsToMove.Add(LandscapeComponent);
				LandscapeComponents.RemoveAtSwap(i);
				ProxiesToDelete.Add(LandscapeComponent->GetTypedOuter<ALandscapeProxy>());
			}
			else
			{
				i++;
			}
		}

		check(ComponentsToMove.Num() <= MaxComponents);
		if (ComponentsToMove.Num())
		{
			ALandscapeProxy* LandscapeProxy = FLandscapeConfigHelper::FindOrAddLandscapeStreamingProxy(ActorPartitionSubsystem, InLandscapeInfo, CellCoord);
			check(LandscapeProxy);
			InLandscapeInfo->MoveComponentsToProxy(ComponentsToMove, LandscapeProxy);

			// Make sure components retain their Materials if they don't match with their parent proxy
			for (ULandscapeComponent* MovedComponent : ComponentsToMove)
			{
				UMaterialInterface* PreviousLandscapeMaterial = ComponentMaterials.FindChecked(MovedComponent);
				UMaterialInterface* PreviousLandscapeHoleMaterial = ComponentHoleMaterials.FindChecked(MovedComponent);
				TMap<int32, UMaterialInterface*> PreviousLandscapeLODMaterials = ComponentLODMaterials.FindChecked(MovedComponent);

				MovedComponent->OverrideMaterial = nullptr;
				if (PreviousLandscapeMaterial != nullptr && PreviousLandscapeMaterial != MovedComponent->GetLandscapeMaterial())
				{
					// If Proxy doesn't differ from Landscape override material there first
					if(LandscapeProxy->GetLandscapeMaterial() == LandscapeProxy->GetLandscapeActor()->GetLandscapeMaterial())
					{
						LandscapeProxy->LandscapeMaterial = PreviousLandscapeMaterial; 
					}
					else // If it already differs it means that the component differs from it, override on component
					{
						MovedComponent->OverrideMaterial = PreviousLandscapeMaterial;
					}
				}

				MovedComponent->OverrideHoleMaterial = nullptr;
				if (PreviousLandscapeHoleMaterial != nullptr && PreviousLandscapeHoleMaterial != MovedComponent->GetLandscapeHoleMaterial())
				{
					// If Proxy doesn't differ from Landscape override material there first
					if (LandscapeProxy->GetLandscapeHoleMaterial() == LandscapeProxy->GetLandscapeActor()->GetLandscapeHoleMaterial())
					{
						LandscapeProxy->LandscapeHoleMaterial = PreviousLandscapeHoleMaterial;
					}
					else // If it already differs it means that the component differs from it, override on component
					{
						MovedComponent->OverrideHoleMaterial = PreviousLandscapeHoleMaterial;
					}
				}

				TArray<FLandscapePerLODMaterialOverride> PerLODOverrideMaterialsForComponent;
				TArray<FLandscapePerLODMaterialOverride> PerLODOverrideMaterialsForProxy = LandscapeProxy->GetPerLODOverrideMaterials();
				for (int8 LODIndex = 0; LODIndex <= 8; ++LODIndex)
				{
					UMaterialInterface* PreviousLODMaterial = PreviousLandscapeLODMaterials.FindChecked(LODIndex);
					// If Proxy doesn't differ from Landscape override material there first
					if (PreviousLODMaterial != nullptr && PreviousLODMaterial != MovedComponent->GetLandscapeMaterial(LODIndex))
					{
						if (LandscapeProxy->GetLandscapeMaterial(LODIndex) == LandscapeProxy->GetLandscapeActor()->GetLandscapeMaterial(LODIndex))
						{
							PerLODOverrideMaterialsForProxy.Add({ LODIndex, TObjectPtr<UMaterialInterface>(PreviousLODMaterial) });
						}
						else // If it already differs it means that the component differs from it, override on component
						{
							PerLODOverrideMaterialsForComponent.Add({ LODIndex, TObjectPtr<UMaterialInterface>(PreviousLODMaterial) });
						}
					}
				}
				MovedComponent->SetPerLODOverrideMaterials(PerLODOverrideMaterialsForComponent);
				LandscapeProxy->SetPerLODOverrideMaterials(PerLODOverrideMaterialsForProxy);
			}
		}

		return true;
	}, GridSize);
		
	// Only delete Proxies that where not reused
	for (ALandscapeProxy* ProxyToDelete : ProxiesToDelete)
	{
		if (ProxyToDelete->LandscapeComponents.Num() > 0 || ProxyToDelete->IsA<ALandscape>())
		{
			check(ProxyToDelete->GetGridSize() == GridSize);
			continue;
		}

		OutActorsToDelete.Add(ProxyToDelete);
	}

	InLandscapeInfo->ForceLayersFullUpdate();

	return true;
}

bool FLandscapeConfigHelper::PartitionLandscape(UWorld* InWorld, ULandscapeInfo* InLandscapeInfo, uint32 InGridSizeInComponents)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeConfigHelper::PartitionLandscape);

	TSet<AActor*> NewSplineActors;

	// Handle Landscapes with missing LandscapeActor(s)
	if (!InLandscapeInfo->LandscapeActor.Get())
	{
		const TArray<TWeakObjectPtr<ALandscapeStreamingProxy>>& StreamingProxies = InLandscapeInfo->GetSortedStreamingProxies();
		if (!StreamingProxies.IsEmpty())
		{
			// Use the first proxy as the landscape template
			if (ALandscapeProxy* FirstProxy = StreamingProxies[0].Get())
			{
				FActorSpawnParameters SpawnParams;
				FTransform LandscapeTransform = FirstProxy->LandscapeActorToWorld();
				ALandscape* NewLandscape = InWorld->SpawnActor<ALandscape>(ALandscape::StaticClass(), LandscapeTransform, SpawnParams);

				NewLandscape->CopySharedProperties(FirstProxy);

				InLandscapeInfo->RegisterActor(NewLandscape);
			}
		}
	}

	auto MoveControlPointToNewSplineActor = [&NewSplineActors, InLandscapeInfo](ULandscapeSplineControlPoint* ControlPoint)
	{
		AActor* CurrentOwner = ControlPoint->GetTypedOuter<AActor>();
		// Control point as already been moved through its connected segments
		if (NewSplineActors.Contains(CurrentOwner))
		{
			return;
		}

		const FTransform LocalToWorld = ControlPoint->GetOuterULandscapeSplinesComponent()->GetComponentTransform();
		const FVector NewActorLocation = LocalToWorld.TransformPosition(ControlPoint->Location);

		ALandscapeSplineActor* NewSplineActor = InLandscapeInfo->CreateSplineActor(NewActorLocation);

		NewSplineActors.Add(NewSplineActor);
		InLandscapeInfo->MoveSpline(ControlPoint, NewSplineActor);
	};

	// Iterate on copy since we are creating new spline actors
	TArray<TScriptInterface<ILandscapeSplineInterface>> OldSplineActors(InLandscapeInfo->GetSplineActors());
	for (TScriptInterface<ILandscapeSplineInterface> PreviousSplineActor : OldSplineActors)
	{
		if (ULandscapeSplinesComponent* SplineComponent = PreviousSplineActor->GetSplinesComponent())
		{
			SplineComponent->ForEachControlPoint(MoveControlPointToNewSplineActor);
		}
	}

	TSet<AActor*> ActorsToDelete;
	bool bChangedGridSize = FLandscapeConfigHelper::ChangeGridSize(InLandscapeInfo, InGridSizeInComponents, ActorsToDelete);
	for (AActor* ActorToDelete : ActorsToDelete)
	{
		InWorld->DestroyActor(ActorToDelete);
	}

	return bChangedGridSize;
}

void FLandscapeConfigHelper::ExtractLandscapeData(ULandscapeInfo* InLandscapeInfo, const FIntRect& InRegion, const FGuid& InLayerGuid, TArray<uint16>& OutHeightData, TArray<FLandscapeImportLayerInfo>& OutImportMaterialLayerInfos)
{
	FScopedSetLandscapeEditingLayer LayerScope(InLandscapeInfo->LandscapeActor.Get(), InLayerGuid);
	FLandscapeEditDataInterface LandscapeEdit(InLandscapeInfo);
	
	int32 VertsX = InRegion.Width() + 1;
	int32 VertsY = InRegion.Height() + 1;
	OutHeightData.Reset();
	OutHeightData.AddZeroed(VertsX * VertsY);
	
	{
		FIntRect CopyRegion(InRegion);
		LandscapeEdit.GetHeightData(CopyRegion.Min.X, CopyRegion.Min.Y, CopyRegion.Max.X, CopyRegion.Max.Y, OutHeightData.GetData(), 0);
	}

	for (const FLandscapeInfoLayerSettings& LayerSettings : InLandscapeInfo->Layers)
	{
		if (LayerSettings.LayerInfoObj != NULL)
		{
			auto ImportLayerInfo = new(OutImportMaterialLayerInfos) FLandscapeImportLayerInfo(LayerSettings);
			ImportLayerInfo->LayerData.Reset();
			ImportLayerInfo->LayerData.AddZeroed(VertsX * VertsY);
			FIntRect CopyRegion(InRegion);
			LandscapeEdit.GetWeightData(LayerSettings.LayerInfoObj, CopyRegion.Min.X, CopyRegion.Min.Y, CopyRegion.Max.X, CopyRegion.Max.Y, ImportLayerInfo->LayerData.GetData(), 0);
		}
	}
};

void FLandscapeConfigHelper::CopyRegionToComponent(ULandscapeInfo* InLandscapeInfo, const FIntRect& InRegion, bool bInResample, ULandscapeComponent* InComponent)
{
	TArray<uint16> SrcHeightData;
	TArray<FLandscapeImportLayerInfo> SrcWeightData;
	
	ULandscapeInfo* NewLandscapeInfo = InComponent->GetLandscapeInfo();
	
	FIntRect NewRegion = InComponent->GetComponentExtent();
		
	// Create Heightmap Texture
	check(InComponent->GetHeightmap(false) == nullptr);
	int32 HeightmapSize = (InComponent->SubsectionSizeQuads + 1) * InComponent->NumSubsections;
	
	TArray<uint16> NewHeightData;
	TArray<uint8> NewWeightData;

	TArray<FColor> HeightInitData;
	HeightInitData.SetNum(FMath::Square(HeightmapSize));
	const FColor DefaultHeightColor = LandscapeDataAccess::GetDefaultPackedHeightColor();
	for (int32 Index = 0; Index < HeightInitData.Num(); ++Index)
	{
		HeightInitData[Index] = DefaultHeightColor;
	}
	InComponent->InitHeightmapData(HeightInitData, false);

	for (const ULandscapeEditLayerBase* LandscapeEditLayer : InLandscapeInfo->LandscapeActor->GetEditLayersConst())
	{
		check(LandscapeEditLayer != nullptr);
		if (!LandscapeEditLayer->NeedsPersistentTextures())
		{
			continue;
		}

		TMap<UTexture2D*, UTexture2D*> CreatedTextures;
		InComponent->AddDefaultLayerData(LandscapeEditLayer->GetGuid(), { InComponent }, CreatedTextures);

		// No need to copy the persistent textures if they are not manually edited (e.g. fully procedural layers with persistent textures : e.g. the splines layer) :
		if (!LandscapeEditLayer->SupportsEditingTools())
		{
			continue;
		}

		SrcWeightData.Empty();
		ExtractLandscapeData(InLandscapeInfo, InRegion, LandscapeEditLayer->GetGuid(), SrcHeightData, SrcWeightData);
		CopyData(SrcHeightData, NewHeightData, InRegion, NewRegion, bInResample);

		FScopedSetLandscapeEditingLayer LayerScope(NewLandscapeInfo->LandscapeActor.Get(), LandscapeEditLayer->GetGuid());
		{
			FLandscapeEditDataInterface LandscapeEdit(NewLandscapeInfo);
			LandscapeEdit.SetHeightData(NewRegion.Min.X, NewRegion.Min.Y, NewRegion.Max.X, NewRegion.Max.Y, NewHeightData.GetData(), 0, false);
		}

		for (const FLandscapeImportLayerInfo& Layer : SrcWeightData)
		{
			CopyData(Layer.LayerData, NewWeightData, InRegion, NewRegion, bInResample);
			FLandscapeEditDataInterface LandscapeEdit(NewLandscapeInfo);
			LandscapeEdit.SetAlphaData(Layer.LayerInfo, NewRegion.Min.X, NewRegion.Min.Y, NewRegion.Max.X, NewRegion.Max.Y, NewWeightData.GetData(), /*Stride = */0);
		}
	}
}

void FLandscapeConfigHelper::MoveSplinesToLandscape(ULandscapeInfo* InLandscapeInfo, ALandscapeProxy* InLandscape, float InScaleFactor)
{
	if (!InLandscape->GetSplinesComponent())
	{
		InLandscape->CreateSplineComponent();
	}

	InLandscapeInfo->ForEachLandscapeProxy([InLandscapeInfo, InLandscape](ALandscapeProxy* LandscapeProxy)
	{
		if (InLandscape == LandscapeProxy || !LandscapeProxy->GetSplinesComponent())
		{
			return true;
		}

		InLandscapeInfo->MoveSplinesToProxy(LandscapeProxy->GetSplinesComponent(), InLandscape);
		return true;
	});
}

void FLandscapeConfigHelper::MoveFoliageToLandscape(ULandscapeInfo* InLandscapeInfo, ULandscapeInfo* InNewLandscapeInfo)
{
	// Move instances
	for (const TPair<FIntPoint, ULandscapeComponent*>& OldEntry : InLandscapeInfo->XYtoComponentMap)
	{
		ULandscapeHeightfieldCollisionComponent* OldCollisionComponent = OldEntry.Value->GetCollisionComponent();

		if (OldCollisionComponent)
		{
			UWorld* World = OldCollisionComponent->GetWorld();

			for (const TPair<FIntPoint, ULandscapeComponent*>& NewEntry : InNewLandscapeInfo->XYtoComponentMap)
			{
				ULandscapeHeightfieldCollisionComponent* NewCollisionComponent = NewEntry.Value->GetCollisionComponent();

				if (NewCollisionComponent && FBoxSphereBounds::BoxesIntersect(NewCollisionComponent->Bounds, OldCollisionComponent->Bounds))
				{
					// only transfer instances overlapping the new box in x,y
					FBox Box = NewCollisionComponent->Bounds.GetBox();
					FBox OldBox = OldCollisionComponent->Bounds.GetBox();

					// but allow just about any Z (expand old bounds by max extent)
					double Extent = OldBox.GetExtent().GetMax();
					Box.Min.Z = OldBox.Min.Z - Extent;
					Box.Max.Z = OldBox.Max.Z + Extent;

					AInstancedFoliageActor::MoveInstancesToNewComponent(World, OldCollisionComponent, Box, NewCollisionComponent);
				}
			}
		}
	}

	// Snap them to the bounds
	for (const TPair<FIntPoint, ULandscapeComponent*>& NewEntry : InNewLandscapeInfo->XYtoComponentMap)
	{
		ULandscapeHeightfieldCollisionComponent* NewCollisionComponent = NewEntry.Value->GetCollisionComponent();

		if ((NewCollisionComponent != nullptr) && (GetDefault<ULandscapeSettings>()->GetFoliageLandscapeSnappingMode() != EFoliageLandscapeSnappingMode::DontSnap))
		{
			NewCollisionComponent->SnapFoliageInstances(ULandscapeHeightfieldCollisionComponent::ESnapFoliageInstancesFlags::ForceAll);
		}
	}
}

ULandscapeInfo* FLandscapeConfigHelper::ChangeConfiguration(ULandscapeInfo* InLandscapeInfo, const FLandscapeConfigChange& InNewConfig, TSet<AActor*>& OutActorsToDelete, TSet<AActor*>& OutModifiedActors)
{
	// @todo_ow: output a list of ModifiedActors

	if (!InNewConfig.Validate())
	{
		return nullptr;
	}

	check(InLandscapeInfo);
	ALandscape* OldLandscape = InLandscapeInfo->LandscapeActor.Get();
	check(OldLandscape);
	check(!OldLandscape->IsTemplate());
	UWorld* World = OldLandscape->GetWorld();
	UActorPartitionSubsystem* ActorPartitionSubsystem = World->GetSubsystem<UActorPartitionSubsystem>();
	FLandscapeConfig OldConfig(InLandscapeInfo);

	bool bGridSizeOnly = false;
	if (OldConfig.ComponentNumSubsections == InNewConfig.ComponentNumSubsections &&
		OldConfig.SubsectionSizeQuads == InNewConfig.SubsectionSizeQuads)
	{
		if (OldConfig.GridSizeInComponents == InNewConfig.GridSizeInComponents)
		{
			// Nothing to do
			return nullptr;
		}

		// Simple redistribute
		if (!FLandscapeConfigHelper::ChangeGridSize(InLandscapeInfo, InNewConfig.GridSizeInComponents, OutActorsToDelete))
		{
			return nullptr;
		}

		return InLandscapeInfo;
	}
		
	int32 LSMinX, LSMinY, LSMaxX, LSMaxY;
	if (!InLandscapeInfo->GetLandscapeExtent(LSMinX, LSMinY, LSMaxX, LSMaxY))
	{
		return nullptr;
	}
		
	const int32 NewComponentSizeQuads = InNewConfig.GetComponentSizeQuads();
	const int32 OldComponentSizeQuads = OldConfig.GetComponentSizeQuads();
	
	const bool bResample = InNewConfig.ResizeMode == ELandscapeResizeMode::Resample;
	const float SizeFactor = (float)OldComponentSizeQuads / NewComponentSizeQuads;
	const float NewScaleFactor = bResample ? SizeFactor : 1.0f;
	const FVector OldScale = OldLandscape->GetActorScale();
	const FVector NewScale = OldScale * NewScaleFactor;

	const FIntPoint OldSize(LSMaxX - LSMinX, LSMaxY - LSMinY);
	FIntPoint NewLSMin((LSMinX / OldComponentSizeQuads) * NewComponentSizeQuads, (LSMinY / OldComponentSizeQuads) * NewComponentSizeQuads);

	FVector ActorOffset(0, 0, 0);
	if (InNewConfig.bZeroBased)
	{
		ActorOffset = FVector(NewLSMin) * OldScale * SizeFactor;
		NewLSMin = FIntPoint(0, 0);
	}

	FIntPoint NewLSMax(NewLSMin.X + (OldSize.X / OldComponentSizeQuads) * NewComponentSizeQuads, NewLSMin.Y + (OldSize.Y / OldComponentSizeQuads) * NewComponentSizeQuads);
		
	if (InNewConfig.ResizeMode == ELandscapeResizeMode::Expand)
	{
		NewLSMax.X = NewLSMin.X + FMath::DivideAndRoundUp(OldSize.X, NewComponentSizeQuads) * NewComponentSizeQuads;
		NewLSMax.Y = NewLSMin.Y + FMath::DivideAndRoundUp(OldSize.Y, NewComponentSizeQuads) * NewComponentSizeQuads;
	}
	else if (InNewConfig.ResizeMode == ELandscapeResizeMode::Clip)
	{
		NewLSMax.X = NewLSMin.X + FMath::Max(1, OldSize.X / NewComponentSizeQuads) * NewComponentSizeQuads;
		NewLSMax.Y = NewLSMin.Y + FMath::Max(1, OldSize.Y / NewComponentSizeQuads) * NewComponentSizeQuads;
	}
			
	FActorSpawnParameters SpawnParams;
	ALandscape* NewLandscape = World->SpawnActor<ALandscape>(OldLandscape->GetActorLocation() + ActorOffset, OldLandscape->GetActorRotation(), SpawnParams);
	
	NewLandscape->CopySharedProperties(OldLandscape);
	NewLandscape->SetLandscapeGuid(FGuid::NewGuid());
	NewLandscape->SetGridSize(InNewConfig.GetGridSizeQuads());
	NewLandscape->ComponentSizeQuads = InNewConfig.GetComponentSizeQuads();
	NewLandscape->NumSubsections = InNewConfig.ComponentNumSubsections;
	NewLandscape->SubsectionSizeQuads = InNewConfig.SubsectionSizeQuads;	
	NewLandscape->SetActorRelativeScale3D(NewScale);

	// Copy Layer Data
	for (const FLandscapeLayer& OldLayer : OldLandscape->GetLayersConst())
	{
		const FLandscapeLayer* NewLayer = NewLandscape->DuplicateLayerAndMoveBrushes(OldLayer);
		check((NewLayer == nullptr) || (NewLayer->EditLayer != nullptr)); // it's possible DuplicateLayerAndMoveBrushes fails (e.g. max number of layers reached), but if not, we should always have an EditLayer
	}
		
	ULandscapeInfo* NewLandscapeInfo = NewLandscape->CreateLandscapeInfo();
		
	for (int32 X = NewLSMin.X; X < NewLSMax.X; X+=NewComponentSizeQuads)
	{
		for (int32 Y = NewLSMin.Y; Y < NewLSMax.Y; Y+=NewComponentSizeQuads)
		{
			FIntPoint ComponentKey(X/NewComponentSizeQuads, Y/NewComponentSizeQuads);
			FIntPoint NewSectionBase(X, Y);
						
			// Means we don't have more data to copy...
			if (!bResample && ((NewSectionBase.X-NewLSMin.X) > OldSize.X || (NewSectionBase.Y-NewLSMin.Y) > OldSize.Y))
			{
				continue;
			}

			UActorPartitionSubsystem::FCellCoord CellCoord = UActorPartitionSubsystem::FCellCoord::GetCellCoord(NewSectionBase, NewLandscape->GetLevel(), NewLandscape->GetGridSize());
			ALandscapeProxy* LandscapeProxy = FLandscapeConfigHelper::FindOrAddLandscapeStreamingProxy(ActorPartitionSubsystem, NewLandscapeInfo, CellCoord);
			check(LandscapeProxy);

			ULandscapeComponent* NewComponent = NewObject<ULandscapeComponent>(LandscapeProxy, NAME_None, RF_Transactional);
			NewComponent->Init(NewSectionBase.X, NewSectionBase.Y, NewComponentSizeQuads, InNewConfig.ComponentNumSubsections, InNewConfig.SubsectionSizeQuads);

			// Add to map before resample so that SetHeightData/SetWeightData can find the component. XYtoComponentMap is usually updated in RegisterComponent
			NewLandscapeInfo->XYtoComponentMap.Add(ComponentKey, NewComponent);

			FIntRect SrcRegion(0, 0, 0, 0);
			if (bResample)
			{
				FIntPoint OldSectionBase(ComponentKey.X* OldComponentSizeQuads, ComponentKey.Y* OldComponentSizeQuads);
				SrcRegion.Min = OldSectionBase;
				SrcRegion.Max = OldSectionBase + OldComponentSizeQuads;
				if (InNewConfig.bZeroBased)
				{
					SrcRegion += FIntPoint(LSMinX, LSMinY);
				}
			}
			else
			{
				FIntPoint RegionOffset(NewLSMin.X - LSMinX, NewLSMin.Y - LSMinY);
				SrcRegion.Min = NewSectionBase - RegionOffset;
				SrcRegion.Max.X = FMath::Min(LSMaxX, SrcRegion.Min.X + NewComponentSizeQuads);
				SrcRegion.Max.Y = FMath::Min(LSMaxY, SrcRegion.Min.Y + NewComponentSizeQuads);
			}

			CopyRegionToComponent(InLandscapeInfo, SrcRegion, bResample, NewComponent);

			NewComponent->UpdateMaterialInstances();
			NewComponent->UpdateCachedBounds();
			NewComponent->UpdateBounds();

			NewComponent->RegisterComponent();
		}
	}

	NewLandscapeInfo->ForceLayersFullUpdate();
	
	// Do as last step so that components have a collision component
	MoveFoliageToLandscape(InLandscapeInfo, NewLandscapeInfo);

	return NewLandscapeInfo;
}

#endif
