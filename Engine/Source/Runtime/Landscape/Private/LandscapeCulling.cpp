// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeCulling.h"
#include "LandscapeRender.h"
#include "SceneView.h"
#include "ShaderParameterStruct.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "ShaderPlatformCachedIniValue.h"
#include "MeshMaterialShader.h"
#include "DataDrivenShaderPlatformInfo.h"

static TAutoConsoleVariable<int32> CVarLandscapeSupportCulling(
	TEXT("landscape.SupportGPUCulling"),
	1,
	TEXT("Whether to support landscape GPU culling"),
	ECVF_ReadOnly | ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarLandscapeEnableGPUCulling(
	TEXT("landscape.EnableGPUCulling"),
	1,
	TEXT("Whether to use landscape GPU culling when it's supported. Allows to toggle culling at runtime"),
	ECVF_RenderThreadSafe);

static TAutoConsoleVariable<int32> CVarLandscapeEnableGPUCullingShadows(
	TEXT("landscape.EnableGPUCullingShadows"),
	1,
	TEXT("Whether to use landscape GPU culling for a shadow views when it's supported. Allows to toggle shadow views culling at runtime"),
	ECVF_RenderThreadSafe);


/** Vertex factory for a tiled landscape rendering  */
class FLandscapeTileVertexFactory : public FLandscapeVertexFactory
{
	DECLARE_VERTEX_FACTORY_TYPE(FLandscapeTileVertexFactory);

public:
	FLandscapeTileVertexFactory(ERHIFeatureLevel::Type InFeatureLevel)
		: FLandscapeVertexFactory(InFeatureLevel)
	{
	}

	struct FDataType : public FLandscapeVertexFactory::FDataType
	{
		FVertexStreamComponent TileDataComponent;
	};

	void SetData(const FDataType& InData)
	{
		FLandscapeVertexFactory::Data = InData;
		Data = InData;
		UpdateRHI(FRHICommandListImmediate::Get());
	}

	/**
	* Should we cache the material's shadertype on this platform with this vertex factory?
	*/
	static bool ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters);

	/**
	* Can be overridden by FVertexFactory subclasses to modify their compile environment just before compilation occurs.
	*/
	static void ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	/**
	* Get vertex elements used when during PSO precaching materials using this vertex factory type
	*/
	static void GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements);

	void Copy(const FLandscapeTileVertexFactory& Other);
	// FRenderResource interface.
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

	FDataType Data;
};

//
// FLandscapeTileVertexFactory
//
void FLandscapeTileVertexFactory::InitRHI(FRHICommandListBase& RHICmdList)
{
	// list of declaration items
	FVertexDeclarationElementList Elements;

	// position decls
	Elements.Add(AccessStreamComponent(Data.PositionComponent, 0));
	// Tile per-instance data
	Elements.Add(AccessStreamComponent(Data.TileDataComponent, 1));

	// create the actual device decls
	InitDeclaration(Elements);
}

/**
* Copy the data from another vertex factory
* @param Other - factory to copy from
*/
void FLandscapeTileVertexFactory::Copy(const FLandscapeTileVertexFactory& Other)
{
	FLandscapeTileVertexFactory* VertexFactory = this;
	const FDataType* DataCopy = &Other.Data;
	ENQUEUE_RENDER_COMMAND(FLandscapeTileVertexFactoryCopyData)(
		[VertexFactory, DataCopy](FRHICommandListImmediate& RHICmdList)
		{
			VertexFactory->Data = *DataCopy;
		});
	BeginUpdateResourceRHI(this);
}

void FLandscapeTileVertexFactory::GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements)
{
	Elements.Add(FVertexElement(0, 0, VET_UByte4, 0, sizeof(FLandscapeVertex), false)); // Position
	Elements.Add(FVertexElement(1, 0, VET_UByte4, 1, 4u, true)); // Tile per-instance data
}

bool FLandscapeTileVertexFactory::ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters)
{
	static FShaderPlatformCachedIniValue<int32> LandscapeSupportCullingIniValue(TEXT("landscape.SupportGPUCulling"));
	return FLandscapeVertexFactory::ShouldCompilePermutation(Parameters) && LandscapeSupportCullingIniValue.Get(Parameters.Platform) != 0;
}

void FLandscapeTileVertexFactory::ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FLandscapeVertexFactory::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("LANDSCAPE_TILE"), TEXT("1"));
}

IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FLandscapeTileVertexFactory, SF_Vertex, FLandscapeVertexFactoryVertexShaderParameters);
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FLandscapeTileVertexFactory, SF_Pixel, FLandscapeVertexFactoryPixelShaderParameters);

IMPLEMENT_VERTEX_FACTORY_TYPE(FLandscapeTileVertexFactory, "/Engine/Private/LandscapeVertexFactory.ush",
	  EVertexFactoryFlags::UsedWithMaterials
	| EVertexFactoryFlags::SupportsStaticLighting
	| EVertexFactoryFlags::SupportsDynamicLighting
	| EVertexFactoryFlags::SupportsLightmapBaking
	| EVertexFactoryFlags::SupportsPSOPrecaching
	| EVertexFactoryFlags::SupportsLandscape
);



namespace UE::Landscape::Culling
{

constexpr static uint32 LANDSCAPE_TILE_QUADS = 4u;
constexpr static uint32 MIN_SECTION_SIZE_QUADS = 31u;
constexpr static uint32 INDIRECT_ARGS_NUM_WORDS = 5u;
constexpr static uint32 TILE_DATA_ENTRY_NUM_BYTES = 4u;

struct FLumenCVarsState
{
	FLumenCVarsState()
	{
		IConsoleVariable* SupportedCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Lumen.Supported"));
		bLumenSupported = (SupportedCVar && SupportedCVar->GetInt() != 0);
		
		// No need to register to changes : all primitive components are invalidated when this CVar dynamically changes so render states of landscape components get appropriately re-created
		IConsoleVariable* DiffuseIndirectCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Lumen.DiffuseIndirect.Allow"));
		DiffuseIndirectInt = DiffuseIndirectCVar->AsVariableInt();
	}

	bool IsActive() const
	{
		return bLumenSupported && (DiffuseIndirectInt && DiffuseIndirectInt->GetValueOnAnyThread() != 0);
	}

	TConsoleVariableData<int32>* DiffuseIndirectInt;
	bool bLumenSupported = false;
};

bool UseCulling(EShaderPlatform Platform)
{
	static FLumenCVarsState LumenCVarsState;
	static FShaderPlatformCachedIniValue<int32> LandscapeSupportCullingIniValue(TEXT("landscape.SupportGPUCulling"));
	
	return 
		LandscapeSupportCullingIniValue.Get(Platform) != 0 &&
		// These features require VF PrimitiveID support which is not possible with culling VF atm
		// Note that VSM and Lumen test includes runtime logic, so this function can't be used in cook time decisions
		!UseVirtualShadowMaps(Platform, GMaxRHIFeatureLevel) &&
		!(FDataDrivenShaderPlatformInfo::GetSupportsLumenGI(Platform) && LumenCVarsState.IsActive());
}

FVertexFactoryType* GetTileVertexFactoryType()
{
	return &FLandscapeTileVertexFactory::StaticType;
}

class FLandscapeTileMesh final : public FRenderResource
{
public:
	FLandscapeTileMesh(FRHICommandListBase& RHICmdList)
	{
		InitResource(RHICmdList);
	}

	/** Destructor. */
	virtual ~FLandscapeTileMesh()
	{
		ReleaseResource();
	}

	virtual void InitResource(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseResource() override;
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;

public:
	FVertexBuffer VertexBuffer;
	FIndexBuffer IndexBuffer;
};


void FLandscapeTileMesh::InitResource(FRHICommandListBase& RHICmdList)
{
	FRenderResource::InitResource(RHICmdList);
	VertexBuffer.InitResource(RHICmdList);
	IndexBuffer.InitResource(RHICmdList);
}

void FLandscapeTileMesh::ReleaseResource()
{
	FRenderResource::ReleaseResource();
	VertexBuffer.ReleaseResource();
	IndexBuffer.ReleaseResource();
}

void FLandscapeTileMesh::InitRHI(FRHICommandListBase& RHICmdList)
{
	const uint32 TileSizeVertx = LANDSCAPE_TILE_QUADS + 1u;

	// create a static vertex buffer
	{
		const uint32 NumVertx = FMath::Square(TileSizeVertx);
		const uint32 Stride = sizeof(FLandscapeVertex);

		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::Create(TEXT("FLandscapeTileMeshVertexBuffer"), NumVertx * Stride, Stride, EBufferUsageFlags::Static | EBufferUsageFlags::VertexBuffer)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer)
			.SetInitActionInitializer();

		TRHIBufferInitializer<FLandscapeVertex> Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);

		uint32 VertexIndex = 0;
		for (uint32 y = 0; y < TileSizeVertx; y++)
		{
			for (uint32 x = 0; x < TileSizeVertx; x++)
			{
				FLandscapeVertex Vertex{};
				Vertex.VertexX = static_cast<uint8>(x);
				Vertex.VertexY = static_cast<uint8>(y);
				Vertex.SubX = 0;
				Vertex.SubY = 0;

				Initializer.WriteValueAtIndex(VertexIndex, Vertex);
				VertexIndex++;
			}
		}

		VertexBuffer.VertexBufferRHI = Initializer.Finalize();
	}

	// 
	{
		const int32 NumIndices = FMath::Square(LANDSCAPE_TILE_QUADS) * 6u;
		const uint32 Stride = sizeof(uint16);

		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::Create(TEXT("FLandscapeTileMeshIndexBuffer"), NumIndices * Stride, Stride, EBufferUsageFlags::Static | EBufferUsageFlags::IndexBuffer)
			.SetInitialState(ERHIAccess::VertexOrIndexBuffer)
			.SetInitActionInitializer();

		TRHIBufferInitializer<uint16> Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);

		uint32 IndexOffset = 0;
		for (uint32 y = 0; y < LANDSCAPE_TILE_QUADS; y++)
		{
			for (uint32 x = 0; x < LANDSCAPE_TILE_QUADS; x++)
			{
				uint16 i00 = static_cast<uint16>((x + 0) + (y + 0) * (TileSizeVertx));
				uint16 i10 = static_cast<uint16>((x + 1) + (y + 0) * (TileSizeVertx));
				uint16 i11 = static_cast<uint16>((x + 1) + (y + 1) * (TileSizeVertx));
				uint16 i01 = static_cast<uint16>((x + 0) + (y + 1) * (TileSizeVertx));

				Initializer[IndexOffset + 0] = i00;
				Initializer[IndexOffset + 1] = i11;
				Initializer[IndexOffset + 2] = i10;

				Initializer[IndexOffset + 3] = i00;
				Initializer[IndexOffset + 4] = i01;
				Initializer[IndexOffset + 5] = i11;

				IndexOffset += 6;
			}
		}

		IndexBuffer.IndexBufferRHI = Initializer.Finalize();
	}
}


class FLandscapeTileDataBuffer final : public FVertexBuffer
{
	uint32 NumSubsections;
	uint32 SubsectionSizeQuads;
public:

	FLandscapeTileDataBuffer(FRHICommandListBase& RHICmdList, uint32 InNumSubsections, uint32 InSubsectionSizeQuads)
		: NumSubsections(InNumSubsections)
		, SubsectionSizeQuads(InSubsectionSizeQuads)
	{
		InitResource(RHICmdList);
	}

	/** Destructor. */
	virtual ~FLandscapeTileDataBuffer()
	{
		ReleaseResource();
	}

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
};

void FLandscapeTileDataBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	const uint32 SubsectionTilesRow = FMath::DivideAndRoundUp(SubsectionSizeQuads, LANDSCAPE_TILE_QUADS);
	const uint32 SubsectionTiles = SubsectionTilesRow * SubsectionTilesRow;
	const uint32 ComponentTiles = SubsectionTiles * NumSubsections * NumSubsections;
	const uint32 Stride = TILE_DATA_ENTRY_NUM_BYTES;

	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::Create(TEXT("FLandscapeTileVertexBuffer"), ComponentTiles * Stride, Stride, EBufferUsageFlags::Static | EBufferUsageFlags::VertexBuffer)
		.SetInitialState(ERHIAccess::VertexOrIndexBuffer)
		.SetInitActionInitializer();

	TRHIBufferInitializer<uint8> Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);

	uint32 TileOffset = 0;
	for (uint32 SubY = 0; SubY < NumSubsections; SubY++)
	{
		for (uint32 SubX = 0; SubX < NumSubsections; SubX++)
		{
			for (uint32 y = 0; y < SubsectionTilesRow; y++)
			{
				for (uint32 x = 0; x < SubsectionTilesRow; x++)
				{
					Initializer[TileOffset + 0] = static_cast<uint8>(x * LANDSCAPE_TILE_QUADS);
					Initializer[TileOffset + 1] = static_cast<uint8>(y * LANDSCAPE_TILE_QUADS);
					Initializer[TileOffset + 2] = static_cast<uint8>(SubX);
					Initializer[TileOffset + 3] = static_cast<uint8>(SubY);
					TileOffset += Stride;
				}
			}
		}
	}

	VertexBufferRHI = Initializer.Finalize();
}

class FBuildLandscapeTileDataCS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FBuildLandscapeTileDataCS);
	SHADER_USE_PARAMETER_STRUCT(FBuildLandscapeTileDataCS, FGlobalShader)

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		static FShaderPlatformCachedIniValue<int32> LandscapeSupportCullingIniValue(TEXT("landscape.SupportGPUCulling"));
		return LandscapeSupportCullingIniValue.Get(Parameters.Platform) != 0;
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("INDIRECT_ARGS_NUM_WORDS"), INDIRECT_ARGS_NUM_WORDS);
		OutEnvironment.SetDefine(TEXT("LANDSCAPE_TILE_QUADS"), LANDSCAPE_TILE_QUADS);
	}

	struct FLandscapeView
	{
		FMatrix44f ViewToClip;
		FMatrix44f TranslatedWorldToClip;
		FVector3f RelativePreViewTranslation;
		float _Pad0;
		FVector3f ViewTilePosition;
		float _Pad1;
	};

	struct FLandscapeSection
	{
		FMatrix44f LocalToRelativeWorld;
		FVector3f TilePosition;
		float LocalZ;
		float HalfHeight;
		float NeighborLODExtent;
		float _Pad[2];
	};

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, IndirectArgsBufferOut)
		SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, TileDataOut)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FLandscapeView>, LandscapeViews)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<FLandscapeSection>, LandscapeSections)
		SHADER_PARAMETER(uint32, NumLandscapeSections)
		SHADER_PARAMETER(uint32, NumLandscapeViews)
		SHADER_PARAMETER(uint32, SubsectionSizeTiles)
		SHADER_PARAMETER(uint32, NumSubsections)
		SHADER_PARAMETER(uint32, NearClip)
	END_SHADER_PARAMETER_STRUCT()
};
IMPLEMENT_GLOBAL_SHADER(FBuildLandscapeTileDataCS, "/Engine/Private/Landscape/LandscapeCulling.usf", "BuildLandscapeTileDataCS", SF_Compute);

struct FViewStateIntermediateData
{
	FRDGBufferRef SectionsBufferRDG;
	TArray<FIntPoint, TInlineAllocator<8>> SectionRenderCoords;
};

struct FDispatchIntermediates
{
	TRefCountPtr<FRDGPooledBuffer> IndirectArgsBuffer;
	TRefCountPtr<FRDGPooledBuffer> TileDataBuffer;
};

struct FArgumentsKey
{
	const void* ViewPtr;
	FIntPoint RenderCoord;
	uint32 ViewStateKey;

	bool operator==(const FArgumentsKey& Other) const
	{
		return ViewPtr == Other.ViewPtr && RenderCoord == Other.RenderCoord && ViewStateKey == Other.ViewStateKey;
	}
};

uint32 GetTypeHash(const FArgumentsKey& Key)
{
	uint32 Hash = PointerHash(Key.ViewPtr);
	Hash = HashCombine(Hash, GetTypeHash(Key.RenderCoord));
	Hash = HashCombine(Hash, ::GetTypeHash(Key.ViewStateKey));
	return Hash;
}

struct FCullingEntry
{
	uint32 LandscapeKey;
	int32 ReferenceCount;
	uint32 NumSubsections;
	uint32 SubsectionSizeQuads;

	TArray<FViewStateIntermediateData, TInlineAllocator<2>> IntermediateData;
	TArray<FDispatchIntermediates, TInlineAllocator<2>> DispatchIntermediateData;
	TMap<FArgumentsKey, FArguments> CullingArguments;
};

struct FCullingSystem
{
	TArray<FCullingEntry> Landscapes; 
	TArray<const FSceneView*, TInlineAllocator<2>> Views;
};

FCullingSystem GCullingSystem;

static bool EnableGPUCullingForSection(int32 SubsectionSizeVerts, int32 NumSubsections)
{
	uint32 SectionSizeQuads = (SubsectionSizeVerts - 1) * NumSubsections;
	return (SectionSizeQuads >= MIN_SECTION_SIZE_QUADS);
}

static void ResetIntermediateData()
{
	for (int32 LandscapeIdx = 0; LandscapeIdx < GCullingSystem.Landscapes.Num(); ++LandscapeIdx)
	{
		FCullingEntry& CullingEntry = GCullingSystem.Landscapes[LandscapeIdx];

		CullingEntry.IntermediateData.Reset();
		CullingEntry.DispatchIntermediateData.Reset();
		CullingEntry.CullingArguments.Reset();
	}

	GCullingSystem.Views.Reset();
}

void PreRenderViewFamily(FSceneViewFamily& InViewFamily)
{
	ResetIntermediateData();
}

void InitSharedBuffers(FRHICommandListBase& RHICmdList, FLandscapeSharedBuffers& SharedBuffers, const ERHIFeatureLevel::Type InFeatureLevel)
{
	if (SharedBuffers.TileVertexFactory == nullptr)
	{
		uint32 SubsectionSizeQuads = (SharedBuffers.SubsectionSizeVerts - 1);

		FLandscapeTileMesh* TileMesh = new FLandscapeTileMesh(RHICmdList);
		FLandscapeTileDataBuffer* TileDataBuffer = new FLandscapeTileDataBuffer(RHICmdList, SharedBuffers.NumSubsections, SubsectionSizeQuads);
		FLandscapeTileVertexFactory* TileVF = new FLandscapeTileVertexFactory(InFeatureLevel);

		uint32 TileDataStride = TILE_DATA_ENTRY_NUM_BYTES;

		TileVF->Data.PositionComponent = FVertexStreamComponent(&TileMesh->VertexBuffer, 0, sizeof(FLandscapeVertex), VET_UByte4);
		TileVF->Data.TileDataComponent = FVertexStreamComponent(TileDataBuffer, 0, TileDataStride, VET_UByte4, EVertexStreamUsage::Instancing);
		TileVF->InitResource(RHICmdList);

		SharedBuffers.TileMesh = TileMesh;
		SharedBuffers.TileVertexFactory = TileVF;
		SharedBuffers.TileDataBuffer = TileDataBuffer;
	}
}

void SetupMeshBatch(const FLandscapeSharedBuffers& SharedBuffers, FMeshBatch& MeshBatch)
{
	if (!EnableGPUCullingForSection(SharedBuffers.SubsectionSizeVerts, SharedBuffers.NumSubsections))
	{
		return;
	}

	// TileVertexFactory can be temporarily null when we switch conditions on landscape culling. In this
	// case, we just fallback to the non landscape culling path.
	if (MeshBatch.LODIndex == 0 && SharedBuffers.TileVertexFactory != nullptr)
	{
		uint32 SubsectionSizeQuads = (SharedBuffers.SubsectionSizeVerts - 1);


		MeshBatch.bViewDependentArguments = true;
		MeshBatch.VertexFactory = SharedBuffers.TileVertexFactory;

		FMeshBatchElement& BatchElement = MeshBatch.Elements[0];
		BatchElement.IndexBuffer = &static_cast<FLandscapeTileMesh*>(SharedBuffers.TileMesh)->IndexBuffer;
		BatchElement.NumPrimitives = FMath::Square(LANDSCAPE_TILE_QUADS) * 2u;
		BatchElement.NumInstances = FMath::Square(FMath::DivideAndRoundUp(SubsectionSizeQuads, LANDSCAPE_TILE_QUADS)) * FMath::Square(SharedBuffers.NumSubsections);
		BatchElement.FirstIndex = 0;
		BatchElement.MinVertexIndex = 0;
		BatchElement.MaxVertexIndex = FMath::Square(LANDSCAPE_TILE_QUADS + 1u) - 1u;
	}
}

void RegisterLandscape(FRHICommandListBase& RHICmdList, FLandscapeSharedBuffers& SharedBuffers, ERHIFeatureLevel::Type FeatureLevel, uint32 LandscapeKey, int32 SubsectionSizeVerts, int32 NumSubsections)
{
	if (!EnableGPUCullingForSection(SubsectionSizeVerts, NumSubsections))
	{
		return;
	}

	InitSharedBuffers(RHICmdList, SharedBuffers, FeatureLevel);

	FCullingEntry* CullingEntryPtr = GCullingSystem.Landscapes.FindByPredicate([LandscapeKey](const FCullingEntry& Other) {
		return Other.LandscapeKey == LandscapeKey;
	});

	if (!CullingEntryPtr)
	{
		FCullingEntry& CullingEntry = GCullingSystem.Landscapes.AddDefaulted_GetRef();

		CullingEntry.LandscapeKey = LandscapeKey;
		CullingEntry.NumSubsections = NumSubsections;
		CullingEntry.SubsectionSizeQuads = (SubsectionSizeVerts - 1);
		CullingEntry.ReferenceCount = 1;
	}
	else
	{
		CullingEntryPtr->ReferenceCount++;
	}
}

void UnregisterLandscape(uint32 LandscapeKey)
{
	for (int32 i = GCullingSystem.Landscapes.Num() - 1; i >= 0; --i)
	{
		FCullingEntry& CullingEntry = GCullingSystem.Landscapes[i];
		if (CullingEntry.LandscapeKey == LandscapeKey)
		{
			CullingEntry.ReferenceCount--;
			if (CullingEntry.ReferenceCount <= 0)
			{
				GCullingSystem.Landscapes.RemoveAtSwap(i);
			}
			break;
		}
	}
}

static float ComputeNeighborsMaxLOD(const FLandscapeRenderSystem& RenderSystem, const TResourceArray<float>& SectionLODValues, FIntPoint RenderCoord)
{
	static const FIntPoint NeightborBaseOffsets[4]
	{
		{1, 0}, {-1, 0}, {0, 1}, {0, -1}
	};

	float NeighborsMaxLOD = 0.f;
	for (int32 i = 0; i < UE_ARRAY_COUNT(NeightborBaseOffsets); ++i)
	{
		if (RenderSystem.IsValidCoord(RenderCoord + NeightborBaseOffsets[i]))
		{
			int32 SectionIdx = RenderSystem.GetSectionLinearIndex(RenderCoord + NeightborBaseOffsets[i]);
			if (SectionLODValues.IsValidIndex(SectionIdx))
			{
				NeighborsMaxLOD = FMath::Max(SectionLODValues[SectionIdx], NeighborsMaxLOD);
			}
		}
	}

	return NeighborsMaxLOD;
}

static void ComputeSectionIntermediateData(FRDGBuilder& GraphBuilder, TArrayView<const FSceneView*> Views)
{
	TArray<FBuildLandscapeTileDataCS::FLandscapeSection, TInlineAllocator<16>> SectionsData;

	for (int32 ViewIdx = 0; ViewIdx < Views.Num(); ++ViewIdx)
	{
		const FSceneView& View = *Views[ViewIdx];
		GCullingSystem.Views.Add(&View);
		
		// Collect all LOD0 sections for each ViewState+Landscape and upload to GPU
		for (int32 LandscapeIdx = 0; LandscapeIdx < GCullingSystem.Landscapes.Num(); ++LandscapeIdx)
		{
			FCullingEntry& CullingEntry = GCullingSystem.Landscapes[LandscapeIdx];

			FViewStateIntermediateData& ViewStateIntermediates = CullingEntry.IntermediateData.AddDefaulted_GetRef();
			ViewStateIntermediates.SectionsBufferRDG = nullptr;

			const FLandscapeRenderSystem* RenderSystem = FLandscapeSceneViewExtension::GetLandscapeRenderSystem(View.Family->Scene, CullingEntry.LandscapeKey);
			// This landscape render system might not correspond to the scene we're rendering, so we might end up with nothing here : 
			if (RenderSystem != nullptr)
			{
				const TResourceArray<float>& SectionLODValues = RenderSystem->GetCachedSectionLODValues(View);

				for (int32 SectionIdx = 0; SectionIdx < SectionLODValues.Num(); ++SectionIdx)
				{
					const int32 LODValue = static_cast<int32>(SectionLODValues[SectionIdx]);
					FLandscapeSectionInfo* SectionInfo = RenderSystem->SectionInfos[SectionIdx];

					if (LODValue == 0 && SectionInfo != nullptr)
					{
						FBuildLandscapeTileDataCS::FLandscapeSection& Section = SectionsData.AddDefaulted_GetRef();

						FBoxSphereBounds SectionLocalBounds;
						FMatrix SectionLocalToWorld;
						SectionInfo->GetSectionBoundsAndLocalToWorld(SectionLocalBounds, SectionLocalToWorld);
						const FLargeWorldRenderPosition SectionAbsoluteOrigin(SectionLocalToWorld.GetOrigin());
						const int32 NeighborsMaxLOD = static_cast<int32>(FMath::RoundFromZero(ComputeNeighborsMaxLOD(*RenderSystem, SectionLODValues, SectionInfo->RenderCoord)));

						Section.LocalToRelativeWorld = FLargeWorldRenderScalar::MakeToRelativeWorldMatrix(SectionAbsoluteOrigin.GetTileOffset(), SectionLocalToWorld);
						Section.TilePosition = SectionAbsoluteOrigin.GetTile();
						Section.LocalZ = static_cast<float>(SectionLocalBounds.Origin.Z);
						Section.HalfHeight = static_cast<float>(SectionLocalBounds.BoxExtent.Z);
						// How many quads to add to each tile extent to compensate for a neighbors LOD
						Section.NeighborLODExtent = FMath::Max(static_cast<float>((1 << NeighborsMaxLOD) - 1), 1.f);

						ViewStateIntermediates.SectionRenderCoords.Add(SectionInfo->RenderCoord);
					}
				}

				if (SectionsData.Num() != 0)
				{
					ViewStateIntermediates.SectionsBufferRDG = CreateStructuredBuffer<FBuildLandscapeTileDataCS::FLandscapeSection>(GraphBuilder, TEXT("LandscapeCulling.SectionsData"), SectionsData);
					SectionsData.Reset();
				}
			}
		}
	}
}

static void DispatchCulling(FRDGBuilder& GraphBuilder, TArrayView<const FSceneView*> CullingViews, TArrayView<FViewMatrices> CullingViewMatrices, bool bNearClip)
{
	// A separate dispatch for each Landscape and ViewState
	for (int32 ViewStateIdx = 0; ViewStateIdx < GCullingSystem.Views.Num(); ++ViewStateIdx)
	{
		const FSceneView* View = GCullingSystem.Views[ViewStateIdx];
		uint32 ViewStateKey = View->GetViewKey();

		TArray<FBuildLandscapeTileDataCS::FLandscapeView, TInlineAllocator<8>> CullingViewsData;
		CullingViewsData.Reserve(CullingViews.Num());
		TArray<const FSceneView*, TInlineAllocator<8>> FilteredViews;
		FilteredViews.Reserve(CullingViews.Num());

		// Filter out views for a current ViewState
		for (int32 CullingViewIdx = 0; CullingViewIdx < CullingViews.Num(); ++CullingViewIdx)
		{
			const FSceneView& CullingView = *CullingViews[CullingViewIdx];
			if (&CullingView != View)
			{
				continue;
			}
			
			FilteredViews.Add(&CullingView);
			FBuildLandscapeTileDataCS::FLandscapeView& LandscapeView = CullingViewsData.AddDefaulted_GetRef();

			const FViewMatrices& ViewMatrices = CullingViewMatrices[CullingViewIdx];
			const FLargeWorldRenderPosition AbsoluteViewOrigin(ViewMatrices.GetViewOrigin());
			const FVector ViewTileOffset = AbsoluteViewOrigin.GetTileOffset();
			const FVector3f ViewTilePosition = AbsoluteViewOrigin.GetTile();
						
			LandscapeView.ViewToClip = FMatrix44f(ViewMatrices.GetViewToClip());
			LandscapeView.TranslatedWorldToClip = FMatrix44f(ViewMatrices.GetTranslatedWorldToClip());
			LandscapeView.RelativePreViewTranslation = FVector3f(ViewMatrices.GetPreViewTranslation() + ViewTileOffset);
			LandscapeView.ViewTilePosition = ViewTilePosition;
		}

		// Share view data between all landscapes
		FRDGBufferRef CullingViewsBufferRDG = nullptr;

		for (int32 LandscapeIdx = 0; LandscapeIdx < GCullingSystem.Landscapes.Num(); ++LandscapeIdx)
		{
			FCullingEntry& CullingEntry = GCullingSystem.Landscapes[LandscapeIdx];
			FViewStateIntermediateData& ViewStateIntermediates = CullingEntry.IntermediateData[ViewStateIdx];

			uint32 NumCullingViews = CullingViewsData.Num();
			uint32 NumCullingSections = ViewStateIntermediates.SectionRenderCoords.Num();
			uint32 NumCullingItems = NumCullingSections * NumCullingViews;
			if (NumCullingItems == 0)
			{
				continue;
			}

			TArray<FRHIDrawIndexedIndirectParameters> IndirectArgsParameters;
			IndirectArgsParameters.Init(FRHIDrawIndexedIndirectParameters{ FMath::Square(LANDSCAPE_TILE_QUADS) * 6u, 0u, 0u, 0u, 0u }, NumCullingItems);

			const uint32 IndirectArgsBufferElements = INDIRECT_ARGS_NUM_WORDS * IndirectArgsParameters.Num();
			FRDGBufferRef IndirectArgsRDG = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateIndirectDesc(IndirectArgsBufferElements), TEXT("LandscapeCulling.DrawIndirectArgsBuffer"));

			const uint32 NumTilesSectionRow = FMath::DivideAndRoundUp(CullingEntry.SubsectionSizeQuads, LANDSCAPE_TILE_QUADS) * CullingEntry.NumSubsections;
			const uint32 NumTilesSection = FMath::Square(NumTilesSectionRow);

			const uint32 TileDataBufferElements = NumTilesSection * NumCullingItems;
			FRDGBufferDesc TileDataBufferDesc = FRDGBufferDesc::CreateByteAddressDesc(TileDataBufferElements * TILE_DATA_ENTRY_NUM_BYTES);
			TileDataBufferDesc.Usage |= BUF_VertexBuffer;
			FRDGBufferRef TileDataRDG = GraphBuilder.CreateBuffer(TileDataBufferDesc, TEXT("LandscapeCulling.CulledTileDataBuffer"));

			// Create buffer for indirect args and upload draw arg data, also clears the instance to zero
			GraphBuilder.QueueBufferUpload(IndirectArgsRDG, IndirectArgsParameters.GetData(), IndirectArgsParameters.Num() * sizeof(FRHIDrawIndexedIndirectParameters));

			if (CullingViewsBufferRDG == nullptr)
			{
				CullingViewsBufferRDG = CreateStructuredBuffer<FBuildLandscapeTileDataCS::FLandscapeView>(GraphBuilder, TEXT("LandscapeCulling.ViewsData"), CullingViewsData);
			}

			{
				ERHIFeatureLevel::Type FeatureLevel = CullingViews[0]->GetFeatureLevel();
				FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(FeatureLevel);
				FIntVector SizeInTiles{ (int32)NumTilesSectionRow, (int32)NumTilesSectionRow, (int32)NumCullingItems };
				FIntVector GroupSize{ 8, 8, 1 };

				auto PassParameters = GraphBuilder.AllocParameters<FBuildLandscapeTileDataCS::FParameters>();
				PassParameters->IndirectArgsBufferOut = GraphBuilder.CreateUAV(IndirectArgsRDG);
				PassParameters->TileDataOut = GraphBuilder.CreateUAV(TileDataRDG);
				PassParameters->LandscapeViews = GraphBuilder.CreateSRV(CullingViewsBufferRDG);
				PassParameters->LandscapeSections = GraphBuilder.CreateSRV(ViewStateIntermediates.SectionsBufferRDG);
				PassParameters->SubsectionSizeTiles = FMath::DivideAndRoundUp(CullingEntry.SubsectionSizeQuads, LANDSCAPE_TILE_QUADS);
				PassParameters->NumSubsections = CullingEntry.NumSubsections;
				PassParameters->NumLandscapeViews = NumCullingViews;
				PassParameters->NumLandscapeSections = NumCullingSections;
				PassParameters->NearClip = bNearClip ? 1 : 0;

				auto ComputeShader = ShaderMap->GetShader<FBuildLandscapeTileDataCS>();

				FComputeShaderUtils::AddPass(
					GraphBuilder,
					RDG_EVENT_NAME("LandscapeCulling, SubsectionSizeQuads %d, Subsections %d, Views:%d", CullingEntry.SubsectionSizeQuads, CullingEntry.NumSubsections, NumCullingViews),
					ComputeShader,
					PassParameters,
					FComputeShaderUtils::GetGroupCount(SizeInTiles, GroupSize)
				);
			}
			FDispatchIntermediates& DispatchIntermediates = CullingEntry.DispatchIntermediateData.AddDefaulted_GetRef();
			DispatchIntermediates.IndirectArgsBuffer = ConvertToExternalAccessBuffer(GraphBuilder, IndirectArgsRDG, ERHIAccess::IndirectArgs);
			DispatchIntermediates.TileDataBuffer = ConvertToExternalAccessBuffer(GraphBuilder, TileDataRDG, ERHIAccess::VertexOrIndexBuffer);

			FRHIBuffer* IndirectArgsBufferRHI = DispatchIntermediates.IndirectArgsBuffer->GetRHI();
			FRHIBuffer* TileDataBufferRHI = DispatchIntermediates.TileDataBuffer->GetRHI();

			// Create a map for each view+section combo to its draw args
			for (int32 ViewIdx = 0; ViewIdx < FilteredViews.Num(); ++ViewIdx)
			{
				const FSceneView* ViewPtr = FilteredViews[ViewIdx];
				for (int32 SectionIdx = 0; SectionIdx < ViewStateIntermediates.SectionRenderCoords.Num(); ++SectionIdx)
				{
					FIntPoint RenderCoord = ViewStateIntermediates.SectionRenderCoords[SectionIdx];
					check(RenderCoord.X > INT32_MIN);
					FArguments& Args = CullingEntry.CullingArguments.Add(FArgumentsKey{ ViewPtr, RenderCoord, ViewStateKey });
					Args.IndirectArgsBuffer = IndirectArgsBufferRHI;
					Args.TileDataVertexBuffer = TileDataBufferRHI;
					// offsets are in bytes
					Args.IndirectArgsOffset = (4u * INDIRECT_ARGS_NUM_WORDS) * ((ViewIdx * NumCullingSections) + SectionIdx);
					Args.TileDataOffset = TILE_DATA_ENTRY_NUM_BYTES * NumTilesSection * ((ViewIdx * NumCullingSections) + SectionIdx);
				}
			}
		}
	}
}

void InitMainViews(FRDGBuilder& GraphBuilder, TArrayView<const FSceneView*> Views)
{
	if (GCullingSystem.Landscapes.Num() == 0 ||
		CVarLandscapeEnableGPUCulling.GetValueOnRenderThread() == 0)
	{
		return;
	}

	QUICK_SCOPE_CYCLE_COUNTER(STAT_LandscapeCulling_InitMainViews);

	check(GCullingSystem.Views.Num() == 0);

	ComputeSectionIntermediateData(GraphBuilder, Views);

	TArray<FViewMatrices, TInlineAllocator<4>> CullingViewMatrices;
	for (const FSceneView* View : Views)
	{
		CullingViewMatrices.Add(View->ViewMatrices);
	}
	DispatchCulling(GraphBuilder, Views, CullingViewMatrices, true);
}

void InitShadowViews(FRDGBuilder& GraphBuilder, TArrayView<const FSceneView*> ShadowDepthViews, TArrayView<FViewMatrices> ShadowViewMatrices)
{
	if (GCullingSystem.Landscapes.Num() == 0 ||
		CVarLandscapeEnableGPUCulling.GetValueOnRenderThread() == 0 || 
		CVarLandscapeEnableGPUCullingShadows.GetValueOnRenderThread() == 0)
	{
		return;
	}

	QUICK_SCOPE_CYCLE_COUNTER(STAT_LandscapeCulling_InitShadowViews);

	DispatchCulling(GraphBuilder, ShadowDepthViews, ShadowViewMatrices, false);
}

bool GetViewArguments(const FSceneView& View, uint32 LandscapeKey, FIntPoint RenderCoord, int32 LODIndex, FArguments& Args)
{
	if (LODIndex != 0 || 
		GCullingSystem.Landscapes.Num() == 0 ||
		CVarLandscapeEnableGPUCulling.GetValueOnRenderThread() == 0)
	{
		return false;
	}

	const FCullingEntry* CullingEntryPtr = GCullingSystem.Landscapes.FindByPredicate([LandscapeKey](const FCullingEntry& Other) {
		return Other.LandscapeKey == LandscapeKey;
	});

	if (CullingEntryPtr == nullptr)
	{
		return false;
	}

	check(RenderCoord.X > INT32_MIN);
	FArgumentsKey ArgumentsKey{ &View, RenderCoord, View.GetViewKey() };
	const FArguments* ArgsPtr = CullingEntryPtr->CullingArguments.Find(ArgumentsKey);
	if (ArgsPtr)
	{
		Args = *ArgsPtr;
		return true;
	}
	return false;
}

}//namespace UE::Landscape::Culling
