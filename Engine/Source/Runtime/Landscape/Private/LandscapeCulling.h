// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FRHIBuffer;
class FVertexFactoryType;
enum EShaderPlatform : uint16;
namespace ERHIFeatureLevel { enum Type : int; }
class FLandscapeSharedBuffers;
struct FMeshBatch;
class FSceneViewFamily;
class FSceneView;
class FRDGBuilder;
struct FViewMatrices;
class FRHICommandListBase;

/**
* Landscape GPU culling implementation
* Each landscape section is split to a smaller tiles (4x4 quads)
* Tiles are frustum culled in compute
* Culling is done for each view, including shadow views
* Only active for LOD0 atm
*/
namespace UE {
	namespace Landscape {
		namespace Culling {

			struct FArguments
			{
				FRHIBuffer* IndirectArgsBuffer;
				FRHIBuffer* TileDataVertexBuffer;
				uint32 IndirectArgsOffset;
				uint32 TileDataOffset;
			};

			bool UseCulling(EShaderPlatform ShaderPlatform);
			
			FVertexFactoryType* GetTileVertexFactoryType();

			void SetupMeshBatch(const FLandscapeSharedBuffers& SharedBuffers, FMeshBatch& MeshBatch);
			void RegisterLandscape(FRHICommandListBase& RHICmdList, FLandscapeSharedBuffers& SharedBuffers, ERHIFeatureLevel::Type FeatureLevel, uint32 LandscapeKey, int32 SubsectionSizeVerts, int32 NumSubsections);
			void UnregisterLandscape(uint32 LandscapeKey);

			void PreRenderViewFamily(FSceneViewFamily& InViewFamily);
			void InitMainViews(FRDGBuilder& GraphBuilder, TArrayView<const FSceneView*> Views);
			void InitShadowViews(FRDGBuilder& GraphBuilder, TArrayView<const FSceneView*> ShadowDepthViews, TArrayView<FViewMatrices> ShadowViewMatrices);
			bool GetViewArguments(const FSceneView& View, uint32 LandscapeKey, FIntPoint SectionBase, int32 LODIndex, FArguments& Args);
		}
	}
};
