// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeDataAccess.h"
#include "LandscapeComponent.h"

#if WITH_EDITOR


FLandscapeComponentDataInterfaceBase::FLandscapeComponentDataInterfaceBase(ULandscapeComponent* InComponent, int32 InMipLevel, bool bInWorkOnEditingLayer)
	: MipLevel(InMipLevel)
{
	check(IsInParallelGameThread() || IsInGameThread());

	UTexture2D* HeightMapTexture = InComponent->GetHeightmap(bInWorkOnEditingLayer);
	HeightmapStride = HeightMapTexture->Source.GetSizeX() >> InMipLevel;
	HeightmapComponentOffsetX = FMath::RoundToInt((HeightMapTexture->Source.GetSizeX() >> InMipLevel) * InComponent->HeightmapScaleBias.Z);
	HeightmapComponentOffsetY = FMath::RoundToInt((HeightMapTexture->Source.GetSizeY() >> InMipLevel) * InComponent->HeightmapScaleBias.W);
	HeightmapSubsectionOffset = (InComponent->SubsectionSizeQuads + 1) >> InMipLevel;

	ComponentSizeQuads = InComponent->ComponentSizeQuads;
	ComponentSizeVerts = (InComponent->ComponentSizeQuads + 1) >> InMipLevel;
	SubsectionSizeVerts = (InComponent->SubsectionSizeQuads + 1) >> InMipLevel;
	ComponentNumSubsections = InComponent->NumSubsections;
}

FLandscapeComponentDataInterface::FLandscapeComponentDataInterface(ULandscapeComponent* InComponent, int32 InMipLevel, bool bInWorkOnEditingLayer)
	: FLandscapeComponentDataInterfaceBase(InComponent, InMipLevel, bInWorkOnEditingLayer)
	, Component(InComponent)
	, bWorkOnEditingLayer(bInWorkOnEditingLayer)
	, HeightMipData(nullptr)
{
	check(IsInParallelGameThread() || IsInGameThread());

	UTexture2D* HeightMapTexture = InComponent->GetHeightmap(bWorkOnEditingLayer);
	if (MipLevel < HeightMapTexture->Source.GetNumMips())
	{
		HeightMipData = (FColor*)DataInterface.LockMip(HeightMapTexture, MipLevel);
	}
}

FLandscapeComponentDataInterface::~FLandscapeComponentDataInterface()
{
	check(IsInParallelGameThread() || IsInGameThread());

	if (HeightMipData)
	{
		UTexture2D* HeightMapTexture = Component->GetHeightmap(bWorkOnEditingLayer);
		DataInterface.UnlockMip(HeightMapTexture, MipLevel);
	}
}

void FLandscapeComponentDataInterface::GetHeightmapTextureData(TArray<FColor>& OutData, bool bOkToFail)
{
	check(IsInParallelGameThread() || IsInGameThread());
	if (bOkToFail && !HeightMipData)
	{
		OutData.Empty();
		return;
	}
#if LANDSCAPE_VALIDATE_DATA_ACCESS
	check(HeightMipData);
#endif
	int32 HeightmapSize = ((Component->SubsectionSizeQuads + 1) * Component->NumSubsections) >> MipLevel;
	OutData.Empty(FMath::Square(HeightmapSize));
	OutData.AddUninitialized(FMath::Square(HeightmapSize));

	for (int32 SubY = 0; SubY < HeightmapSize; SubY++)
	{
		// X/Y of the vertex we're looking at in component's coordinates.
		int32 CompY = SubY;

		// UV coordinates of the data offset into the texture
		int32 TexV = SubY + HeightmapComponentOffsetY;

		// Copy the data
		FMemory::Memcpy(&OutData[CompY * HeightmapSize], &HeightMipData[HeightmapComponentOffsetX + TexV * HeightmapStride], HeightmapSize * sizeof(FColor));
	}
}

bool FLandscapeComponentDataInterface::GetWeightmapTextureData(ULandscapeLayerInfoObject* InLayerInfo, TArray<uint8>& OutData, bool bInUseEditingWeightmap, bool bInRemoveSubsectionDuplicates)
{
	check(IsInParallelGameThread() || IsInGameThread());
	int32 LayerIdx = INDEX_NONE;
	const TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = Component->GetWeightmapLayerAllocations(bInUseEditingWeightmap);
	const TArray<UTexture2D*>& ComponentWeightmapTextures = Component->GetWeightmapTextures(bInUseEditingWeightmap);

	for (int32 Idx = 0; Idx < ComponentWeightmapLayerAllocations.Num(); Idx++)
	{
		if (ComponentWeightmapLayerAllocations[Idx].LayerInfo == InLayerInfo)
		{
			LayerIdx = Idx;
			break;
		}
	}
	if (LayerIdx < 0)
	{
		return false;
	}
	if (ComponentWeightmapLayerAllocations[LayerIdx].WeightmapTextureIndex >= ComponentWeightmapTextures.Num())
	{
		return false;
	}
	if (ComponentWeightmapLayerAllocations[LayerIdx].WeightmapTextureChannel >= 4)
	{
		return false;
	}

	// If requested to skip the duplicate row/col of texture data
	int32 WeightmapSize = bInRemoveSubsectionDuplicates ?
		((Component->SubsectionSizeQuads * Component->NumSubsections) + 1) >> MipLevel :
		((Component->SubsectionSizeQuads + 1) * Component->NumSubsections) >> MipLevel;
	
	OutData.Empty(FMath::Square(WeightmapSize));
	OutData.AddUninitialized(FMath::Square(WeightmapSize));

	// DataInterface Lock is a LockMipReadOnly on the texture
	const FColor* WeightMipData = (const FColor*)DataInterface.LockMip(ComponentWeightmapTextures[ComponentWeightmapLayerAllocations[LayerIdx].WeightmapTextureIndex], MipLevel);

	// Channel remapping
	int32 ChannelOffsets[4] = { (int32)STRUCT_OFFSET(FColor, R), (int32)STRUCT_OFFSET(FColor, G), (int32)STRUCT_OFFSET(FColor, B), (int32)STRUCT_OFFSET(FColor, A) };

	const uint8* SrcTextureData = (const uint8*)WeightMipData + ChannelOffsets[ComponentWeightmapLayerAllocations[LayerIdx].WeightmapTextureChannel];

	for (int32 i = 0; i < FMath::Square(WeightmapSize); i++)
	{
		// If removing subsection duplicates, convert vertex to texel index
		OutData[i] = bInRemoveSubsectionDuplicates ? SrcTextureData[VertexIndexToTexel(i) * sizeof(FColor)] : SrcTextureData[i * sizeof(FColor)];
	}

	DataInterface.UnlockMip(ComponentWeightmapTextures[ComponentWeightmapLayerAllocations[LayerIdx].WeightmapTextureIndex], MipLevel);
	return true;
}

// Deprecated
FColor* FLandscapeComponentDataInterface::GetXYOffsetData(int32 LocalX, int32 LocalY) const
{
	return nullptr;
}

FVector FLandscapeComponentDataInterface::GetLocalVertex(int32 LocalX, int32 LocalY) const
{
	const float ScaleFactor = (float)ComponentSizeQuads / (float)(ComponentSizeVerts - 1);
	return FVector(LocalX * ScaleFactor, LocalY * ScaleFactor, LandscapeDataAccess::GetLocalHeight(GetHeight(LocalX, LocalY)));
}

FVector FLandscapeComponentDataInterface::GetWorldVertex(int32 LocalX, int32 LocalY) const
{
	return Component->GetComponentTransform().TransformPosition(GetLocalVertex(LocalX, LocalY));
}

void FLandscapeComponentDataInterface::GetWorldTangentVectors(int32 LocalX, int32 LocalY, FVector& WorldTangentX, FVector& WorldTangentY, FVector& WorldTangentZ) const
{
	FColor* Data = GetHeightData(LocalX, LocalY);
	WorldTangentZ = LandscapeDataAccess::UnpackNormal(*Data);
	WorldTangentX = FVector(-WorldTangentZ.Z, 0.f, WorldTangentZ.X);
	WorldTangentY = FVector(0.f, WorldTangentZ.Z, -WorldTangentZ.Y);

	WorldTangentX = Component->GetComponentTransform().TransformVectorNoScale(WorldTangentX);
	WorldTangentY = Component->GetComponentTransform().TransformVectorNoScale(WorldTangentY);
	WorldTangentZ = Component->GetComponentTransform().TransformVectorNoScale(WorldTangentZ);
}

void FLandscapeComponentDataInterface::GetWorldPositionTangents(int32 LocalX, int32 LocalY, FVector& WorldPos, FVector& WorldTangentX, FVector& WorldTangentY, FVector& WorldTangentZ) const
{
	FColor* Data = GetHeightData(LocalX, LocalY);
	WorldTangentZ = LandscapeDataAccess::UnpackNormal(*Data);
	WorldTangentX = FVector(WorldTangentZ.Z, 0.f, -WorldTangentZ.X);
	WorldTangentY = WorldTangentZ ^ WorldTangentX;

	float Height = LandscapeDataAccess::UnpackHeight(*Data);

	const float ScaleFactor = (float)ComponentSizeQuads / (float)(ComponentSizeVerts - 1);
	WorldPos = Component->GetComponentTransform().TransformPosition(FVector(LocalX * ScaleFactor, LocalY * ScaleFactor, Height));
	WorldTangentX = Component->GetComponentTransform().TransformVectorNoScale(WorldTangentX);
	WorldTangentY = Component->GetComponentTransform().TransformVectorNoScale(WorldTangentY);
	WorldTangentZ = Component->GetComponentTransform().TransformVectorNoScale(WorldTangentZ);
}

int32 FLandscapeComponentDataInterface::GetHeightmapSizeX(int32 MipIndex) const
{
	check(IsInParallelGameThread() || IsInGameThread());
	return Component->GetHeightmap(bWorkOnEditingLayer)->Source.GetSizeX() >> MipIndex;
}

int32 FLandscapeComponentDataInterface::GetHeightmapSizeY(int32 MipIndex) const
{
	check(IsInParallelGameThread() || IsInGameThread());
	return Component->GetHeightmap(bWorkOnEditingLayer)->Source.GetSizeY() >> MipIndex;
}

#endif // WITH_EDITOR
