// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeEdgeFixup.h"
#include "Containers/StridedView.h"
#include "Landscape.h"
#include "LandscapeComponent.h"
#include "LandscapePrivate.h"
#include "LandscapeGroup.h"
#include "RenderingThread.h"
#include "TextureResource.h"
#include "Hash/xxhash.h"
#include "Streaming/TextureMipDataProvider.h"
#include "LandscapeTextureHash.h"


// enables verbose debug spew
//#define ENABLE_LANDSCAPE_EDGE_FIXUP_DEBUG_SPEW 1

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapeEdgeFixup)
#ifdef ENABLE_LANDSCAPE_EDGE_FIXUP_DEBUG_SPEW
#define FIXUP_DEBUG_LOG(...) UE_LOG(LogLandscape, Warning, __VA_ARGS__)
#define FIXUP_DEBUG_LOG_PATCH(...) UE_LOG(LogLandscape, Warning, __VA_ARGS__)
#define FIXUP_DEBUG_LOG_RENDER(...) UE_LOG(LogLandscape, Warning, __VA_ARGS__)
#define FIXUP_DEBUG_LOG_DETAIL(...) UE_LOG(LogLandscape, Warning, __VA_ARGS__)
#else
#define FIXUP_DEBUG_LOG(...) UE_LOG(LogLandscape, Verbose, __VA_ARGS__)
#define FIXUP_DEBUG_LOG_PATCH(...) do {} while(0)
#define FIXUP_DEBUG_LOG_RENDER(...) do {} while(0)
#define FIXUP_DEBUG_LOG_DETAIL(...) do {} while(0)
#endif // ENABLE_LANDSCAPE_EDGE_FIXUP_DEBUG_SPEW

namespace UE::Landscape
{
	EDirectionFlags ToFlag(EDirectionIndex Index)
	{
		return static_cast<EDirectionFlags>(0x01 << static_cast<uint32>(Index));
	}

	EDirectionIndex GetOppositeIndex(EDirectionIndex Index)
	{
		uint32 OppositeIndex = (static_cast<uint32>(Index) + 4) & 0x07;	// add 4, mod 8
		return static_cast<EDirectionIndex>(OppositeIndex);
	}

	FIntPoint GetNeighborRelativePosition(ENeighborIndex DirectionIndex)
	{
		static FIntPoint RelativePositions[static_cast<int32>(ENeighborIndex::Count)] =
		{
			FIntPoint(0, -1),		// Bottom
			FIntPoint(1, -1),		// Bottom Right
			FIntPoint(1, 0),		// Right
			FIntPoint(1, 1),		// Top Right
			FIntPoint(0, 1),		// Top
			FIntPoint(-1, 1),		// Top Left
			FIntPoint(-1, 0),		// Left
			FIntPoint(-1, -1),		// Bottom Left
		};
		if ((DirectionIndex >= ENeighborIndex::First) && (DirectionIndex <= ENeighborIndex::Last))
		{
			return RelativePositions[static_cast<int32>(DirectionIndex)];
		}
		return FIntPoint(0, 0);
	}

	const FString& GetDirectionString(EDirectionIndex Index)
	{
		static const FString DirectionStrings[static_cast<int32>(EDirectionIndex::Count)] =
		{
			FString("B"),			// Bottom
			FString("BR"),			// Bottom Right
			FString("R"),			// Right
			FString("TR"),			// Top Right
			FString("T"),			// Top
			FString("TL"),			// Top Left
			FString("L"),			// Left
			FString("BL"),			// Bottom Left
		};
		if ((Index >= EDirectionIndex::First) && (Index <= EDirectionIndex::Last))
		{
			return DirectionStrings[static_cast<int32>(Index)];
		}
		static const FString ErrorString("Error");
		return ErrorString;
	}

	// returns true if the given direction is diagonal
	bool IsDiagonalCorner(EDirectionIndex DirectionIndex)
	{
		// cardinal directions have first bit zero, diagonals one
		return (static_cast<uint32>(DirectionIndex) & 0x01) == 1;
	}

	bool IsTopOrBottomEdge(EEdgeIndex EdgeIndex)
	{
		// top and bottom are (index mod 4 == 0)
		return (static_cast<uint32>(EdgeIndex) & 0x03) == 0;
	}

	bool IsDoubleTriangleCorner(EEdgeIndex EdgeIndex)
	{
		// double corner triangles are top-right (3) and bottom-left (7)
		return (static_cast<uint32>(EdgeIndex) & 0x02) == 0;
	}

	// this converts a set of edge flags into a set of neighbors that are blended with those edges
	ENeighborFlags EdgesToAffectedNeighbors(EEdgeFlags EdgeFlags)
	{
		// changed edges only affect the corresponding neighbor
		// changed corners affect the corner neighbor AND the adjacent edge neighbors as well
		uint32 RawFlags = static_cast<uint32>(EdgeFlags);
		uint32 CornerFlags = RawFlags & static_cast<uint32>(ENeighborFlags::AllCorners);
		uint32 AdjacentA = (CornerFlags << 1) | (CornerFlags >> 7); // rotate left one (mod 8, applied below)
		uint32 AdjacentB = (CornerFlags << 7) | (CornerFlags >> 1); // rotate right one (mod 8, applied below)
		return static_cast<ENeighborFlags>((RawFlags | AdjacentA | AdjacentB) & static_cast<uint32>(ENeighborFlags::All));
	}

	// this converts a set of neighbors into the local edges that they blend with
	// (i.e. one of the neighbor's corners/edges is blended with the local edge)
	EEdgeFlags NeighborsToBlendedEdges(ENeighborFlags NeighborFlags)
	{
		// corner neighbors only blend with the corresponding corner 
		// edge neighbors blend the corresponding edge AND also blend with the adjacent corners
		uint32 RawFlags = static_cast<uint32>(NeighborFlags);
		uint32 EdgeFlags = RawFlags & static_cast<uint32>(EEdgeFlags::AllEdges);
		uint32 AdjacentA = (EdgeFlags << 1) | (EdgeFlags >> 7); // rotate left one (mod 8, applied below)
		uint32 AdjacentB = (EdgeFlags << 7) | (EdgeFlags >> 1);	// rotate right one (mod 8, applied below)
		return static_cast<EEdgeFlags>((RawFlags | AdjacentA | AdjacentB) & static_cast<uint32>(EEdgeFlags::All));
	}

	// select the direction rotated clockwise (45 degrees * ClockwiseRotationOffset)
	// this works for positive or negative offsets
	EDirectionIndex RotateDirection(EDirectionIndex NeighborIndex, int32 ClockwiseRotationOffset)
	{
		int32 Index = static_cast<int32>(NeighborIndex);
		int32 AdjacentIndex = (Index - ClockwiseRotationOffset) & (static_cast<int32>(EDirectionIndex::Count) - 1);
		return static_cast<EDirectionIndex>(AdjacentIndex);
	}

	EDirectionFlags RotateFlags(EDirectionFlags DirFlags, int32 ClockwiseRotationOffset)
	{
		uint32 FlagBits = (uint32) DirFlags;
		FlagBits = FlagBits | (FlagBits << 8);
		FlagBits = (FlagBits >> (ClockwiseRotationOffset & 0x07)) & 0xFF;
		return static_cast<EDirectionFlags>(FlagBits);
	}

	void GetSourceTextureEdgeStartCoord(EEdgeIndex EdgeIndex, int32 EdgeLength, int32& OutStartX, int32& OutStartY)
	{
		//						StartX	StartY	Stride		Count			Order
		// 	Bottom = 0,			0		0		1			EdgeLength		left to right
		// 	BottomRight = 1,	EL-1	0		0			1
		// 	Right = 2,			EL-1	0		EL			EdgeLength		bottom to top
		// 	TopRight = 3,		EL-1	EL-1	0			1
		// 	Top = 4,			0		EL-1	1			EdgeLength		left to right
		// 	TopLeft = 5,		0		EL-1	0			1
		// 	Left = 6,			0		0		EL			EdgeLength		bottom to top
		// 	BottomLeft = 7,		0		0		0			1

		check((EdgeIndex >= EEdgeIndex::First) && (EdgeIndex <= EEdgeIndex::Last));
		int32 Index = static_cast<int32>(EdgeIndex);
		OutStartX = (((Index + 7) & 0x07) < 3) ? (EdgeLength-1) : 0;
		OutStartY = (((Index + 5) & 0x07) < 3) ? (EdgeLength-1) : 0;
	}

	int32 GetSourceTextureEdgeStartStrideCount(EEdgeIndex EdgeIndex, int32 EdgeLength, int32& OutStride, int32& OutCount)
	{
		int32 StartX, StartY;
		GetSourceTextureEdgeStartCoord(EdgeIndex, EdgeLength, StartX, StartY);
		int32 Offset = StartY * EdgeLength + StartX;

		if (!IsDiagonalCorner(EdgeIndex))
		{
			int32 Count = EdgeLength;
			int32 Stride = IsTopOrBottomEdge(EdgeIndex) ? 1 : EdgeLength;

			// exclude corners from the edge
			Offset += Stride;
			Count -= 2;

			OutStride = Stride;
			OutCount = Count;
			return Offset;
		}
		else
		{
			OutStride = 0;
			OutCount = 1;
			return Offset;
		}
	}

	const TArrayView<const ENeighborIndex> AllEdgeNeighbors()
	{
		static const ENeighborIndex EdgeNeighbors[4] =
		{
			ENeighborIndex::Bottom,
			ENeighborIndex::Right,
			ENeighborIndex::Top,
			ENeighborIndex::Left,
		};
		return TArrayView<const ENeighborIndex>(&EdgeNeighbors[0], 4);
	}

	const TArrayView<const ENeighborIndex> AllCornerNeighbors()
	{
		static const ENeighborIndex CornerNeighbors[4] =
		{
			ENeighborIndex::BottomRight,
			ENeighborIndex::TopRight,
			ENeighborIndex::TopLeft,
			ENeighborIndex::BottomLeft,
		};
		return TArrayView<const ENeighborIndex>(&CornerNeighbors[0], 4);
	}

	// size 8x8 texture (with mips)
	// 
	// C T T T T T T C
	// L             R
	// L             R   C T T C
	// L             R   L     R   C C
	// L             R   L     R   C C
	// L             R   C B B C
	// L             R
	// C B B B B B B C
	//      mip 0         mip 1   mip 2
	//
	// data is stored in this order:  edges are stored left to right and bottom to top (for left and right edges)
	//
	// EdgeSnapshot memory layout:
	// B B				<< mip 1 (4x4) 2 pixel edges,	left to right
	// B B B B B B		<< mip 0 (8x8) 6 pixel edges,	left to right
	// R R				<< mip 1						bottom to top
	// R R R R R R		<< mip 0						bottom to top
	// T T												left to right
	// T T T T T T										left to right
	// L L												bottom to top
	// L L L L L L										bottom to top
	// 
	// C C C C			bottom-right, top-right, top-left, bottom-left (no mips needed)

	int32 GetEdgeArrayMipOffset(int32 MipEdgeLength)
	{
		// mip size		edge size	end		start(offset)
		// 2x2			0			0		0
		// 4x4			2			2		0
		// 8x8			6			8		2
		// 16x16		14			22		8
		// 32x32		30			52		22
		// 64x64		62			114		52
		// 128x128		126			240		114
		// 256x256		254			494		240

		if (MipEdgeLength < 4)
		{
			return 0;
		}
		check((MipEdgeLength & (MipEdgeLength - 1)) == 0); // is power of two
		int32 Log2 = FMath::FloorLog2(MipEdgeLength);
		return MipEdgeLength - Log2 * 2;
	}

	int32 GetEdgeDataSize(int32 EdgeLength)
	{
		int32 DirectionSize = GetEdgeArrayMipOffset(EdgeLength * 2);
		return 4 * DirectionSize;
	}
}

TArrayView<UE::Landscape::FHeightmapTexel> FHeightmapTextureEdgeSnapshot::GetEdgeData(UE::Landscape::EEdgeIndex InEdgeIndex, int32 InMipIndex)
{
	using namespace UE::Landscape;
	check((InEdgeIndex >= EEdgeIndex::First) && (InEdgeIndex <= EEdgeIndex::Last));
	check(!IsDiagonalCorner(InEdgeIndex));
	{
		const int32 DirectionSize = EdgeData.Num() / 4;
		const int32 MipEdgeLength = EdgeLength >> InMipIndex;
		const int32 MipOffset = GetEdgeArrayMipOffset(MipEdgeLength);
		const int32 DirOffset = DirectionSize * ((int32)InEdgeIndex / 2);
		const int32 EdgeCount = MipEdgeLength - 2;
		check(DirOffset + MipOffset + EdgeCount <= EdgeData.Num());	// check data exists in the buffer
		return TArrayView<FHeightmapTexel>((FHeightmapTexel*) &EdgeData[DirOffset + MipOffset], EdgeCount);
	}
}

UE::Landscape::FHeightmapTexel FHeightmapTextureEdgeSnapshot::GetCornerData(UE::Landscape::EEdgeIndex InCornerIndex)
{
	using namespace UE::Landscape;
	check((InCornerIndex >= EEdgeIndex::First) && (InCornerIndex <= EEdgeIndex::Last));
	check(IsDiagonalCorner(InCornerIndex));

	// The snapshot corner hashes ARE the corner data (since they are the same size)
	FHeightmapTexel Texel;
	Texel.Data32 = SnapshotEdgeHashes[(int32)InCornerIndex];
	return Texel;
}

#if WITH_EDITOR
TSharedPtr<FHeightmapTextureEdgeSnapshot> FHeightmapTextureEdgeSnapshot::CreateEdgeSnapshotFromHeightmapSource(UTexture2D* InHeightmap, const FVector& LandscapeGridScale)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeHeightmapTextureEdgeSnapshot::CreateEdgeSnapshotFromHeightmapSource);	
	using namespace UE::Landscape;

	FTextureSource& Source = InHeightmap->Source;
	check(Source.IsValid());
	check(Source.GetSizeX() == Source.GetSizeY());

	TArray64<uint8> MipData;
	if (!Source.GetMipData(MipData, 0))
	{
		return nullptr;
	}
	const FGuid TextureSourceID = ULandscapeTextureHash::GetHash(InHeightmap);

	FIXUP_DEBUG_LOG_DETAIL(TEXT("    CaptureEdgeDataFromHeightmapSource_Internal %p (%s) - ID %s"), InHeightmap, *LandscapeGridScale.ToString(), *TextureSourceID.ToString());

	TArrayView64<FHeightmapTexel> TexelData((FHeightmapTexel*)MipData.GetData(), MipData.Num() / sizeof(FHeightmapTexel));
	TSharedPtr<FHeightmapTextureEdgeSnapshot> NewEdgeSnapshot = MakeShared<FHeightmapTextureEdgeSnapshot>();
	NewEdgeSnapshot->CaptureEdgeDataFromTextureData_Internal(TexelData, Source.GetSizeY(), LandscapeGridScale, TextureSourceID);
	return NewEdgeSnapshot;
}
#endif // WITH_EDITOR


UE::Landscape::EEdgeFlags FHeightmapTextureEdgeSnapshot::CompareEdges(const FHeightmapTextureEdgeSnapshot& Other) const
{
	using namespace UE::Landscape;

	EEdgeFlags Changed = EEdgeFlags::None;
	for (EEdgeIndex EdgeIndex : TEnumRange<EEdgeIndex>())
	{
		EEdgeFlags EdgeFlag = ToFlag(EdgeIndex);

		if (SnapshotEdgeHashes[(int32)EdgeIndex] != Other.SnapshotEdgeHashes[(int32)EdgeIndex] ||
			InitialEdgeHashes[(int32)EdgeIndex] != Other.InitialEdgeHashes[(int32)EdgeIndex])
		{
			Changed |= EdgeFlag;
		}
	}

	return Changed;
}

void FHeightmapTextureEdgeSnapshot::ResizeForEdgeLength(const int32 InEdgeLength)
{
	if (EdgeLength != InEdgeLength)
	{
		EdgeLength = InEdgeLength;
		const int32 EdgeDataSize = UE::Landscape::GetEdgeDataSize(EdgeLength);
		EdgeData.SetNumZeroed(EdgeDataSize);
	}
}


void FHeightmapTextureEdgeSnapshot::CaptureSingleEdgeDataAndComputeNormalsAndHashes(
	const UE::Landscape::FHeightmapTexel* TextureData, const UE::Landscape::EEdgeIndex EdgeOrCorner, const FVector& LandscapeGridScale)
{
	using namespace UE::Landscape;

	// compute texture source stats
	int32 SourceStride = 0;
	int32 SourceCount = 0;
	const int32 SourceOffset = GetSourceTextureEdgeStartStrideCount(EdgeOrCorner, EdgeLength, SourceStride, SourceCount);
	const FHeightmapTexel* Src = &TextureData[SourceOffset]; // first pixel of the specified edge or corner in the source data
	int32 SrcLineStrideTexels = EdgeLength;

	// The triangle topology for each quad is:
	// 
	// 00 ------ 10
	// | \       |
	// |  \      |
	// |   \     |
	// | NB \ NT |
	// |     \   |
	// |      \  |
	// |       \ |
	// 01 ------ 11
	// 
	// We compute VertexNormals by considering every quad that borders the desired edge or corner, and accumulating the NT/NB normals into the neighboring vertices

	auto CalculateNormalsForQuadAt = [SrcLineStrideTexels, LandscapeGridScale](const FHeightmapTexel* QuadTopLeftSrcPixel, FVector& OutNT, FVector& OutNB)
	{
		auto GetHeight = [SrcLineStrideTexels](const FHeightmapTexel* SrcPixel, int32 DX, int32 DY)
		{
			return SrcPixel[DY * SrcLineStrideTexels + DX].GetHeight16();
		};

		uint16 Height00 = GetHeight(QuadTopLeftSrcPixel, 0, 0);
		uint16 Height01 = GetHeight(QuadTopLeftSrcPixel, 0, 1);
		uint16 Height10 = GetHeight(QuadTopLeftSrcPixel, 1, 0);
		uint16 Height11 = GetHeight(QuadTopLeftSrcPixel, 1, 1);

		const FVector Vert00 = FVector(0.0f, 0.0f, LandscapeDataAccess::GetLocalHeight(Height00)) * LandscapeGridScale;
		const FVector Vert01 = FVector(0.0f, 1.0f, LandscapeDataAccess::GetLocalHeight(Height01)) * LandscapeGridScale;
		const FVector Vert10 = FVector(1.0f, 0.0f, LandscapeDataAccess::GetLocalHeight(Height10)) * LandscapeGridScale;
		const FVector Vert11 = FVector(1.0f, 1.0f, LandscapeDataAccess::GetLocalHeight(Height11)) * LandscapeGridScale;

		// top and bottom triangle normals
		OutNT = ((Vert00 - Vert10) ^ (Vert10 - Vert11)).GetSafeNormal();
		OutNB = ((Vert11 - Vert01) ^ (Vert01 - Vert00)).GetSafeNormal();
	};

	//						StartX	StartY	Stride		Count			Order
	// 	Bottom = 0,			0		0		1			EdgeLength		left to right
	// 	BottomRight = 1,	EL-1	0		0			1
	// 	Right = 2,			EL-1	0		EL			EdgeLength		bottom to top
	// 	TopRight = 3,		EL-1	EL-1	0			1
	// 	Top = 4,			0		EL-1	1			EdgeLength		left to right
	// 	TopLeft = 5,		0		EL-1	0			1
	// 	Left = 6,			0		0		EL			EdgeLength		bottom to top
	// 	BottomLeft = 7,		0		0		0			1

	//the edges do not include the corner vertices, but the edge normals still need to consider the corner quads:
	// 
	//    BL    B0  B1   BR
	//      v - v - v - v
	//      | \ | \ | \ |
	//   L0 v - v - v - v R0
	//      | \ | \ | \ |
	//   L1 v - v - v - v R1
	//      | \ | \ | \ |
	//      v - v - v - v
	//    TL    T0  T1   TR

	uint32 InitialHash;
	uint32 SnapshotHash;
	if (IsDiagonalCorner(EdgeOrCorner))
	{
		check(SourceCount == 1);
		const FHeightmapTexel* QuadTopLeftSrcPixel = Src;
		FVector NT, NB, VertexNormal;
		switch (EdgeOrCorner)
		{
			case EEdgeIndex::BottomRight:
				QuadTopLeftSrcPixel -= 1;
				CalculateNormalsForQuadAt(QuadTopLeftSrcPixel, NT, NB);
				VertexNormal = NT;
				break;
			case EEdgeIndex::TopLeft:
				QuadTopLeftSrcPixel -= SrcLineStrideTexels;
				CalculateNormalsForQuadAt(QuadTopLeftSrcPixel, NT, NB);
				VertexNormal = NB;
				break;
			case EEdgeIndex::TopRight:
				QuadTopLeftSrcPixel -= (SrcLineStrideTexels + 1);
				// fallthrough
			case EEdgeIndex::BottomLeft:
				CalculateNormalsForQuadAt(QuadTopLeftSrcPixel, NT, NB);
				VertexNormal = NT + NB;
				break;
		}

		// Note: Snapshot corner data is only stored in the SnapshotHash (since they are the same)
		FHeightmapTexel Dest;

		// setup normal data
		Dest.SetNormal(VertexNormal);

		// copy height data
		Dest.HeightL = Src->HeightL;
		Dest.HeightH = Src->HeightH;

		// hashes for corners are just a copy of the data (we take advantage of this in the hash comparison)
		InitialHash = Src->Data32;
		SnapshotHash = Dest.Data32;
	}
	else
	{
		const int32 MipZeroIndex = 0;
		TArrayView<FHeightmapTexel> LocalEdgeData = GetEdgeData(EdgeOrCorner, MipZeroIndex);
		FHeightmapTexel* Dst = LocalEdgeData.GetData();
		check(SourceCount == LocalEdgeData.Num());

		const FHeightmapTexel* QuadTopLeftSrcPixel = Src;
		FVector NT, NB;
		TArray<FVector, TFixedAllocator<512>> VertexNormals;	// if this ~12288 byte allocation blows the stack, we can convert it to a static array
		VertexNormals.SetNumZeroed(SourceCount, EAllowShrinking::No);
		switch (EdgeOrCorner)
		{
			case EEdgeIndex::Top:
				QuadTopLeftSrcPixel -= (SrcLineStrideTexels + 1);
				CalculateNormalsForQuadAt(QuadTopLeftSrcPixel, NT, NB);
				VertexNormals[0] += NT + NB;
				for (int32 i = 0; i < SourceCount - 1; ++i)
				{
					QuadTopLeftSrcPixel += 1;
					CalculateNormalsForQuadAt(QuadTopLeftSrcPixel, NT, NB);
					VertexNormals[i] += NB;
					VertexNormals[i + 1] += NT + NB;
				}
				QuadTopLeftSrcPixel += 1;
				CalculateNormalsForQuadAt(QuadTopLeftSrcPixel, NT, NB);
				VertexNormals[SourceCount - 1] += NB;
				break;
			case EEdgeIndex::Left:
				QuadTopLeftSrcPixel -= SrcLineStrideTexels;
				CalculateNormalsForQuadAt(QuadTopLeftSrcPixel, NT, NB);
				VertexNormals[0] += NB;
				for (int32 i = 0; i < SourceCount - 1; ++i)
				{
					QuadTopLeftSrcPixel += SrcLineStrideTexels;
					CalculateNormalsForQuadAt(QuadTopLeftSrcPixel, NT, NB);
					VertexNormals[i] += NT + NB;
					VertexNormals[i + 1] += NB;
				}
				QuadTopLeftSrcPixel += SrcLineStrideTexels;
				CalculateNormalsForQuadAt(QuadTopLeftSrcPixel, NT, NB);
				VertexNormals[SourceCount - 1] += NT + NB;
				break;
			case EEdgeIndex::Bottom:
				QuadTopLeftSrcPixel -= 1;
				CalculateNormalsForQuadAt(QuadTopLeftSrcPixel, NT, NB);
				VertexNormals[0] += NT;
				for (int32 i = 0; i < SourceCount - 1; ++i)
				{
					QuadTopLeftSrcPixel += 1;
					CalculateNormalsForQuadAt(QuadTopLeftSrcPixel, NT, NB);
					VertexNormals[i] += NT + NB;
					VertexNormals[i + 1] += NT;
				}
				QuadTopLeftSrcPixel += 1;
				CalculateNormalsForQuadAt(QuadTopLeftSrcPixel, NT, NB);
				VertexNormals[SourceCount - 1] += NT + NB;
				break;
			case EEdgeIndex::Right:
				QuadTopLeftSrcPixel -= (SrcLineStrideTexels + 1);
				CalculateNormalsForQuadAt(QuadTopLeftSrcPixel, NT, NB);
				VertexNormals[0] += NT + NB;
				for (int32 i = 0; i < SourceCount - 1; ++i)
				{
					QuadTopLeftSrcPixel += SrcLineStrideTexels;
					CalculateNormalsForQuadAt(QuadTopLeftSrcPixel, NT, NB);
					VertexNormals[i] += NT;
					VertexNormals[i + 1] += NT + NB;
				}
				QuadTopLeftSrcPixel += SrcLineStrideTexels;
				CalculateNormalsForQuadAt(QuadTopLeftSrcPixel, NT, NB);
				VertexNormals[SourceCount - 1] += NT;
				break;
		}

		int32 SrcReadStrideTexels = IsTopOrBottomEdge(EdgeOrCorner) ? 1 : SrcLineStrideTexels;

		// edge hashes are the hash of each pixel in the edge
		FXxHash64Builder InitHashBuilder;
		FXxHash64Builder SnapHashBuilder;
		for (int32 i = 0; i < SourceCount; ++i)
		{
			// setup normal data
			Dst->SetNormal(VertexNormals[i]);

			// copy height data
			Dst->HeightH = Src->HeightH;
			Dst->HeightL = Src->HeightL;

			// update hashes
			InitHashBuilder.Update(Src, sizeof(FHeightmapTexel));
			SnapHashBuilder.Update(Dst, sizeof(FHeightmapTexel));

			Dst++;
			Src += SrcReadStrideTexels;
		}

		InitialHash = GetTypeHash(InitHashBuilder.Finalize());
		SnapshotHash = GetTypeHash(SnapHashBuilder.Finalize());

		// downsample to fill out the mip edge data
		{
			int32 MipCount = FMath::CeilLogTwo(EdgeLength) - 1;	// we don't need to fill out the 1x1 or 2x2 mips as those have zero edge data..
			TArrayView<FHeightmapTexel> PrevMipEdge = LocalEdgeData;
			for (int32 MipIndex = 1; MipIndex < MipCount; MipIndex++)
			{
				TArrayView<FHeightmapTexel> MipEdgeData = GetEdgeData(EdgeOrCorner, MipIndex);

				// the downsample pattern between mips: first and last elements get dropped, the rest have a 2 --> 1 averaging downsample applied
				// NOTE that edge downsampling is specifically designed to not pull in non-edge data (does not average with the "middle" of the heightmap)
				int32 MipEdgeLength = (PrevMipEdge.Num() - 2) / 2;
				check(MipEdgeLength == MipEdgeData.Num());

				FHeightmapTexel* MDst = MipEdgeData.GetData();
				FHeightmapTexel* MSrc = PrevMipEdge.GetData() + 1;	// drop the first element
				for (int32 x = 0; x < MipEdgeLength; x++)
				{
					// blend normal
					MDst->NormalX = (MSrc[0].NormalX + (int32) MSrc[1].NormalX) / 2;
					MDst->NormalY = (MSrc[0].NormalY + (int32) MSrc[1].NormalY) / 2;

					// blend heights
					uint32 Height0 = MSrc[0].GetHeight16();
					uint32 Height1 = MSrc[1].GetHeight16();
					uint32 HeightBlended = (Height0 + Height1) / 2;
					MDst->SetHeight16(HeightBlended);

					MDst += 1;
					MSrc += 2;
				}

				PrevMipEdge = MipEdgeData;
			}
		}
	}

	uint32 OldInitialHash = InitialEdgeHashes[static_cast<int32>(EdgeOrCorner)];
	uint32 OldSnapshotHash = SnapshotEdgeHashes[static_cast<int32>(EdgeOrCorner)];
	FIXUP_DEBUG_LOG_DETAIL(TEXT("    - %s Hash i:%x s:%x --> i:%x s:%x"), *GetDirectionString(EdgeOrCorner), OldInitialHash, OldSnapshotHash, InitialHash, SnapshotHash);

	InitialEdgeHashes[static_cast<int32>(EdgeOrCorner)] = InitialHash;
	SnapshotEdgeHashes[static_cast<int32>(EdgeOrCorner)] = SnapshotHash;
}

void FHeightmapTextureEdgeSnapshot::CaptureEdgeDataFromTextureData_Internal(const TArrayView64<UE::Landscape::FHeightmapTexel>& InHeightmapTextureData, int32 InEdgeLength, const FVector& LandscapeGridScale, const FGuid& InTextureSourceID)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FHeightmapTextureEdgeSnapshot::CaptureEdgeDataFromTextureData_Internal);
	using namespace UE::Landscape;

#if WITH_EDITOR
	TextureSourceID = InTextureSourceID;
#endif // WITH_EDITOR

	check(InHeightmapTextureData.Num() == InEdgeLength * InEdgeLength);
	ResizeForEdgeLength(InEdgeLength);

	const FHeightmapTexel* TextureData = InHeightmapTextureData.GetData();
	for (EEdgeIndex EdgeIndex : TEnumRange<EEdgeIndex>())
	{
		CaptureSingleEdgeDataAndComputeNormalsAndHashes(
			TextureData,
			EdgeIndex,
			LandscapeGridScale);
	}

	FIXUP_DEBUG_LOG_DETAIL(TEXT("     CaptureEdgeDataFromTextureData_Internal --[%x %x %x %x %x %x %x %x | %x %x %x %x %x %x %x %x]--"),
		SnapshotEdgeHashes[0], SnapshotEdgeHashes[1], SnapshotEdgeHashes[2], SnapshotEdgeHashes[3],
		SnapshotEdgeHashes[4], SnapshotEdgeHashes[5], SnapshotEdgeHashes[6], SnapshotEdgeHashes[7],
		InitialEdgeHashes[0], InitialEdgeHashes[1], InitialEdgeHashes[2], InitialEdgeHashes[3],
		InitialEdgeHashes[4], InitialEdgeHashes[5], InitialEdgeHashes[6], InitialEdgeHashes[7]);

	return;
}

struct FHeightmapTextureEdgeSnapshotCustomVersion
{
	enum Type
	{
		BeforeCustomVersionWasAdded = 0,
		BeforeInitialHashWasAdded = 1,
		BeforeCornerDataWasRemoved = 2,
		BeforeChangedCornerHash = 3,
		BeforeChangedCookedFormat = 4,
		LatestVersion = 5
	};

	static const FGuid GUID;
};

const FGuid FHeightmapTextureEdgeSnapshotCustomVersion::GUID(0x12345678, 0x12345678, 0x12345678, 0x12345678);
FCustomVersionRegistration GRegisterFHeightmapTextureEdgeSnapshotCustomVersion(
	FHeightmapTextureEdgeSnapshotCustomVersion::GUID,
	FHeightmapTextureEdgeSnapshotCustomVersion::LatestVersion,
	TEXT("FHeightmapTextureEdgeSnapshotCustomVersion"));

FArchive& operator<<(FArchive& Ar, FHeightmapTextureEdgeSnapshot& EdgeSnapshot)
{
	Ar.UsingCustomVersion(FHeightmapTextureEdgeSnapshotCustomVersion::GUID);

	int32 CustomVersion = Ar.CustomVer(FHeightmapTextureEdgeSnapshotCustomVersion::GUID);

	Ar << EdgeSnapshot.EdgeLength;
	Ar << EdgeSnapshot.EdgeData;
	if (CustomVersion <= FHeightmapTextureEdgeSnapshotCustomVersion::BeforeCornerDataWasRemoved)
	{
		TStaticArray<uint32, 4> CornerData = {};
		Ar << CornerData;
	}
	Ar << EdgeSnapshot.SnapshotEdgeHashes;

	if (CustomVersion > FHeightmapTextureEdgeSnapshotCustomVersion::BeforeCustomVersionWasAdded)
	{
#if WITH_EDITOR
		if (!Ar.IsCooking())
		{
			EdgeSnapshot.TextureSourceID.Serialize(Ar);
		}
#endif // WITH_EDITOR
	}

	if (CustomVersion > FHeightmapTextureEdgeSnapshotCustomVersion::BeforeInitialHashWasAdded)
	{
		Ar << EdgeSnapshot.InitialEdgeHashes;
	}

#if WITH_EDITOR
	if (Ar.IsLoading() && CustomVersion < FHeightmapTextureEdgeSnapshotCustomVersion::LatestVersion)
	{
		// invalidate the guid so that we trigger re-capture of this snapshot with the latest snapshot code
		EdgeSnapshot.TextureSourceID.Invalidate();
	}
#endif // WITH_EDITOR

	return Ar;
}


ULandscapeHeightmapTextureEdgeFixup::ULandscapeHeightmapTextureEdgeFixup(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, EdgeSnapshot(MakeShared<FHeightmapTextureEdgeSnapshot>())
{
}

void ULandscapeHeightmapTextureEdgeFixup::SetHeightmapTexture(UTexture2D* InHeightmapTexture)
{
	if (this->HeightmapTexture == nullptr)
	{
		// first time setup
		this->HeightmapTexture = InHeightmapTexture;

		// we assume the texture is in a pristine state the very first time
		// so we initialize the GPU edge hash to our initial edge hashes
		this->GPUEdgeHashes = EdgeSnapshot->InitialEdgeHashes;
		this->GPUEdgeModifiedFlags = UE::Landscape::EEdgeFlags::None;

		FIXUP_DEBUG_LOG_DETAIL(TEXT("    Set Initial GPU Edge Hashes: [B:%x BR:%x R:%x TR:%x T:%x TL:%x L:%x BL:%x]"),
			GPUEdgeHashes[0], GPUEdgeHashes[1], GPUEdgeHashes[2], GPUEdgeHashes[3], GPUEdgeHashes[4], GPUEdgeHashes[5], GPUEdgeHashes[6], GPUEdgeHashes[7]);
	}
	else
	{
		// not the first time, tracker may have existing state
		check(this->HeightmapTexture == InHeightmapTexture);

		// do not modify GPUEdgeHashes here, it records the existing state of the GPU texture (it may have already been modified)

		FIXUP_DEBUG_LOG_DETAIL(TEXT("    Repeat EdgeFixup Setup : no change GPU: [B:%x BR:%x R:%x TR:%x T:%x TL:%x L:%x BL:%x] CPU: [B:%x BR:%x R:%x TR:%x T:%x TL:%x L:%x BL:%x]"),
			GPUEdgeHashes[0], GPUEdgeHashes[1], GPUEdgeHashes[2], GPUEdgeHashes[3], GPUEdgeHashes[4], GPUEdgeHashes[5], GPUEdgeHashes[6], GPUEdgeHashes[7],
			EdgeSnapshot->SnapshotEdgeHashes[0], EdgeSnapshot->SnapshotEdgeHashes[1], EdgeSnapshot->SnapshotEdgeHashes[2], EdgeSnapshot->SnapshotEdgeHashes[3],
			EdgeSnapshot->SnapshotEdgeHashes[4], EdgeSnapshot->SnapshotEdgeHashes[5], EdgeSnapshot->SnapshotEdgeHashes[6], EdgeSnapshot->SnapshotEdgeHashes[7]);
	}
}

// set the active landscape component
void ULandscapeHeightmapTextureEdgeFixup::SetActiveComponent(ULandscapeComponent* InComponent, FLandscapeGroup* InGroup, const bool bDisableCurrentActive)
{
	check(HeightmapTexture != nullptr);

	if (ActiveComponent != nullptr)
	{
		check(ActiveGroup != nullptr);

		if (bDisableCurrentActive)
		{
			// deactivate the active component/group, add it to the disabled list
			ActiveGroup->DisableAndUnmap(this);

			FIXUP_DEBUG_LOG_DETAIL(TEXT("    DisableAndUnmap %p (%d,%d) <Group %u Key %u> edgefixup:%p MAPPED:%d TOTAL:%d"),
				ActiveComponent.Get(), GroupCoord.X, GroupCoord.Y, ActiveComponent->GetLandscapeProxy()->LODGroupKey, ActiveGroup->LandscapeGroupKey, this,
				ActiveGroup->XYToEdgeFixupMap.Num(), ActiveGroup->AllRegisteredFixups.Num());
		}
		else
		{
			// unmap, don't flag as disabled
			ActiveGroup->Unmap(this);

			FIXUP_DEBUG_LOG_DETAIL(TEXT("    Unmap %p (%d,%d) <Group %u Key %u> edgefixup:%p MAPPED:%d TOTAL:%d"),
				ActiveComponent.Get(), GroupCoord.X, GroupCoord.Y, ActiveComponent->GetLandscapeProxy()->LODGroupKey, ActiveGroup->LandscapeGroupKey, this,
				ActiveGroup->XYToEdgeFixupMap.Num(), ActiveGroup->AllRegisteredFixups.Num());
		}
	}
	else
	{
		check(ActiveGroup == nullptr);
	}
	
	ActiveComponent = InComponent;
	ActiveGroup = InGroup;

	if (InComponent != nullptr)
	{
		InGroup->Map(this, InComponent);

		FIXUP_DEBUG_LOG_DETAIL(TEXT("    Map %p (%d,%d) <Group %u Key %u> edgefixup:%p MAPPED:%d TOTAL:%d "),
			InComponent, GroupCoord.X, GroupCoord.Y, InComponent->GetLandscapeProxy()->LODGroupKey, InGroup->LandscapeGroupKey, this,
			InGroup->XYToEdgeFixupMap.Num(), InGroup->AllRegisteredFixups.Num());
	}
}

void ULandscapeHeightmapTextureEdgeFixup::RequestEdgeTexturePatchingForNeighbors(UE::Landscape::ENeighborFlags NeighborsNeedingPatching)
{
	using namespace UE::Landscape;
	check(bMapped);
	for (ENeighborIndex NeighborIndex : TEnumRange<ENeighborIndex>())
	{
		if (EnumHasAnyFlags(NeighborsNeedingPatching, ToFlag(NeighborIndex)))
		{
			// a change to an edge or corner always affects the neighbor in the given direction
			if (ULandscapeHeightmapTextureEdgeFixup* Neighbor = ActiveGroup->GetNeighborEdgeFixup(GroupCoord, NeighborIndex))
			{
				FIXUP_DEBUG_LOG_DETAIL(TEXT("  %s Neighbor %p (%d,%d) flagged for edge texture patching"), 
					*GetDirectionString(NeighborIndex), Neighbor->ActiveComponent.Get(), Neighbor->GroupCoord.X, Neighbor->GroupCoord.Y);
				check(ActiveGroup->AllRegisteredFixups.Contains(Neighbor));
				check(Neighbor->bMapped);
				ActiveGroup->HeightmapsNeedingEdgeTexturePatching.Add(Neighbor);
			}
		}
	}
}

void ULandscapeHeightmapTextureEdgeFixup::BlendCornerData(UE::Landscape::FHeightmapTexel& OutTexel, UE::Landscape::EEdgeIndex CornerIndex, const UE::Landscape::FNeighborSnapshots& NeighborSnapshots)
{
	using namespace UE::Landscape;

	check(IsDiagonalCorner(CornerIndex));

	uint32 NormalX = 0;
	uint32 NormalY = 0;
	uint32 Height = 0;
	uint32 SampleCount = 0;

	auto Accumulate = [&NormalX, &NormalY, &Height, &SampleCount](FHeightmapTextureEdgeSnapshot* Snapshot, EEdgeIndex EdgeIndex)
	{
		if (Snapshot != nullptr)
		{
			FHeightmapTexel CornerData = Snapshot->GetCornerData(EdgeIndex);

			// to match corner blend behavior, TL and BR are weighted 1, TR and BL are weighted 2
			// because they have twice as many triangles incidient on the corner.
			if (IsDoubleTriangleCorner(EdgeIndex))
			{
				NormalX += CornerData.NormalX;
				NormalY += CornerData.NormalY;
				Height += CornerData.GetHeight16();
				SampleCount++;
			}
			else
			{
				NormalX += CornerData.NormalX * 2;
				NormalY += CornerData.NormalY * 2;
				Height += CornerData.GetHeight16() * 2;
				SampleCount += 2;
			}
		}
	};

	FHeightmapTextureEdgeSnapshot* NeighborSnapshot = NeighborSnapshots.NeighborSnapshots[(int32)CornerIndex];
	FHeightmapTextureEdgeSnapshot* NeighborEdgeASnapshot = NeighborSnapshots.NeighborSnapshots[(int32)RotateDirection(CornerIndex, 1)];
	FHeightmapTextureEdgeSnapshot* NeighborEdgeBSnapshot = NeighborSnapshots.NeighborSnapshots[(int32)RotateDirection(CornerIndex, -1)];

	Accumulate(NeighborSnapshots.LocalSnapshot, CornerIndex);
	Accumulate(NeighborSnapshot, GetOppositeIndex(CornerIndex));
	Accumulate(NeighborEdgeASnapshot, RotateDirection(CornerIndex, -2));
	Accumulate(NeighborEdgeBSnapshot, RotateDirection(CornerIndex, 2));
	check(SampleCount >= 2);	// at least we should have the local and one neighbor before callign this function

	OutTexel.NormalX = NormalX / SampleCount;
	OutTexel.NormalY = NormalY / SampleCount;
	OutTexel.SetHeight16(Height / SampleCount);
}

void ULandscapeHeightmapTextureEdgeFixup::BlendEdgeData(FHeightmapTextureEdgeSnapshot& EdgeSnapshot, UE::Landscape::EEdgeIndex EdgeIndex, int32 MipIndex, FHeightmapTextureEdgeSnapshot& NeighborEdgeSnapshot, TStridedView<UE::Landscape::FHeightmapTexel>& OutDestView)
{
	using namespace UE::Landscape;
	check(!IsDiagonalCorner(EdgeIndex));

	TArrayView<const FHeightmapTexel> ThisEdgeData = EdgeSnapshot.GetEdgeData(EdgeIndex, MipIndex);
	TArrayView<const FHeightmapTexel> NeighborData = NeighborEdgeSnapshot.GetEdgeData(GetOppositeIndex(EdgeIndex), MipIndex);
	int32 Count = ThisEdgeData.Num();
	check(Count == NeighborData.Num());

	// allocate the buffer if necessary
	if (OutDestView.Num() == 0)
	{
		FHeightmapTexel* Buffer = new FHeightmapTexel[Count];
		OutDestView = TStridedView<FHeightmapTexel>(sizeof(FHeightmapTexel), Buffer, Count);
	}
	check(Count == OutDestView.Num());

	// blend data between the edge and the neighboring edge
	const FHeightmapTexel* SrcE = &ThisEdgeData[0];
	const FHeightmapTexel* SrcN = &NeighborData[0];
	FHeightmapTexel* Dst = &OutDestView[0];
	int32 DstStrideInTexels = OutDestView.GetStride() / sizeof(FHeightmapTexel);

	for (int32 p = 0; p < Count; p++)
	{
		Dst->NormalX = (SrcE->NormalX + (int32) SrcN->NormalX) / 2;
		Dst->NormalY = (SrcE->NormalY + (int32) SrcN->NormalY) / 2;

		// blend heights
		uint32 HeightE = SrcE->GetHeight16();
		uint32 HeightN = SrcN->GetHeight16();
		uint32 HeightBlended = (HeightE + HeightN) / 2;
		Dst->SetHeight16(HeightBlended);

		SrcE += 1;
		SrcN += 1;
		Dst += DstStrideInTexels;
	}
}

void ULandscapeHeightmapTextureEdgeFixup::GetNeighborSnapshots(UE::Landscape::FNeighborSnapshots &OutSnapshots)
{
	using namespace UE::Landscape;

	check(ActiveGroup != nullptr); // should be caught before calling this function
	check(bMapped);

	ENeighborFlags ExistingNeighbors = ENeighborFlags::None;
	ENeighborFlags AnyModified = GPUEdgeModifiedFlags;

	for (ENeighborIndex NeighborIndex : TEnumRange<ENeighborIndex>())
	{
		if (ULandscapeHeightmapTextureEdgeFixup* Neighbor = ActiveGroup->GetNeighborEdgeFixup(GroupCoord, NeighborIndex))
		{
			ENeighborFlags NeighborFlags = ToFlag(NeighborIndex);
			ExistingNeighbors |= NeighborFlags;
			OutSnapshots.NeighborSnapshots[(int32)NeighborIndex] = &Neighbor->EdgeSnapshot.Get();

			EEdgeFlags NeighborEdgeModifiedFlags = Neighbor->GPUEdgeModifiedFlags;
			if (EnumHasAnyFlags(NeighborEdgeModifiedFlags, ToFlag(GetOppositeIndex(NeighborIndex))))
			{
				EnumAddFlags(AnyModified, NeighborFlags);
			}

			if (!IsDiagonalCorner(NeighborIndex))
			{
				// this neighbor edge blends into our adjacent local corners
				EDirectionIndex RC = RotateDirection(NeighborIndex, 1);
				EDirectionIndex LC = RotateDirection(NeighborIndex, -1);

				if (EnumHasAnyFlags(NeighborEdgeModifiedFlags, ToFlag(GetOppositeIndex(RC))))
				{
					EnumAddFlags(AnyModified, ToFlag(LC));
				}

				if (EnumHasAnyFlags(NeighborEdgeModifiedFlags, ToFlag(GetOppositeIndex(LC))))
				{
					EnumAddFlags(AnyModified, ToFlag(RC));
				}
			}
		}
		else
		{
			OutSnapshots.NeighborSnapshots[(int32)NeighborIndex] = nullptr;
		}
	}

	OutSnapshots.ExistingNeighbors = ExistingNeighbors;
	OutSnapshots.EdgesWithAnyModifiedNeighbor = AnyModified;
	OutSnapshots.LocalSnapshot = &EdgeSnapshot.Get();
	OutSnapshots.GPUEdgeHashes = GPUEdgeHashes;
}

int32 ULandscapeHeightmapTextureEdgeFixup::CheckAndPatchTextureEdgesFromEdgeSnapshots()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CheckAndPatchTextureEdgesFromEdgeSnapshots);

	using namespace UE::Landscape;

	FNeighborSnapshots NeighborSnapshots;
	GetNeighborSnapshots(NeighborSnapshots);

	int32 PatchedEdgeCount = 0;

	// check any edges that may need to be patched, and patch them if necessary
	if (EnumHasAnyFlags(NeighborSnapshots.ExistingNeighbors, EEdgeFlags::AllEdges))
	{
		for (EEdgeIndex EdgeIndex : AllEdgeNeighbors())
		{
			FHeightmapTextureEdgeSnapshot* NeighborSnapshot = NeighborSnapshots.NeighborSnapshots[(int32)EdgeIndex];
			if (NeighborSnapshot == nullptr)
			{
				FIXUP_DEBUG_LOG_PATCH(TEXT("  nopatch  %s Edge - no neighbor"), *GetDirectionString(EdgeIndex));
				continue;
			}

			const EEdgeFlags EdgeFlag = ToFlag(EdgeIndex);

			const bool bModified = EnumHasAnyFlags(NeighborSnapshots.EdgesWithAnyModifiedNeighbor, EdgeFlag);
			if (!bModified)
			{
				// with no modified edges affecting this edge, then we can compare initial states first, as it will likely be a match
				const uint32 LocalInitialHash = NeighborSnapshots.LocalSnapshot->InitialEdgeHashes[(int32)EdgeIndex];
				const uint32 NeighborInitialHash = NeighborSnapshot->InitialEdgeHashes[(int32)GetOppositeIndex(EdgeIndex)];
				check(LocalInitialHash == NeighborSnapshots.GPUEdgeHashes[(int32)EdgeIndex]);
				const bool bInitialStateMatch = (LocalInitialHash == NeighborInitialHash);

				if (bInitialStateMatch)
				{
					FIXUP_DEBUG_LOG_PATCH(TEXT("  nopatch  %s Edge - initial state match"), *GetDirectionString(EdgeIndex));
					continue;
				}
			}

			// lastly, check if this edge has already been patched to the desired state, by comparing hashes
			const uint32 LocalEdgeHash = NeighborSnapshots.LocalSnapshot->SnapshotEdgeHashes[(int32)EdgeIndex];
			const uint32 NeighborEdgeHash = NeighborSnapshot->SnapshotEdgeHashes[(int32)GetOppositeIndex(EdgeIndex)];
			const uint32 GPUEdgeHash = NeighborSnapshots.GPUEdgeHashes[(int32)EdgeIndex];
			uint32 CombinedEdgeHash = HashCombineFast(LocalEdgeHash, NeighborEdgeHash);

			if (CombinedEdgeHash == GPUEdgeHash)
			{
				FIXUP_DEBUG_LOG_PATCH(TEXT("  nopatch  %s Edge - modified match (%x,%x) (== old %x)"), *GetDirectionString(EdgeIndex), LocalEdgeHash, NeighborEdgeHash, GPUEdgeHash);
				continue;
			}

			FIXUP_DEBUG_LOG_PATCH(TEXT("  PATCHING %s Edge (hash %x | %x, new %x != old %x)"),
				*GetDirectionString(EdgeIndex), LocalEdgeHash, NeighborEdgeHash, CombinedEdgeHash, GPUEdgeHash);
			PatchTextureEdge_Internal(EdgeIndex);
			PatchedEdgeCount++;
			GPUEdgeHashes[(int32)EdgeIndex] = CombinedEdgeHash;
			EnumAddFlags(GPUEdgeModifiedFlags, EdgeFlag);
		}
	}

	// check any corners that may need to be patched, and patch them if necessary
	const EEdgeFlags CornersWithOneOrMoreNeighbors = NeighborsToBlendedEdges(NeighborSnapshots.ExistingNeighbors);
	if (EnumHasAnyFlags(CornersWithOneOrMoreNeighbors, EEdgeFlags::AllCorners))
	{
		for (EEdgeIndex CornerIndex : AllCornerNeighbors())
		{
			const EEdgeFlags CornerFlag = ToFlag(CornerIndex);

			// don't patch a corner unless at least one neighbor exists
			if (!EnumHasAnyFlags(CornersWithOneOrMoreNeighbors, CornerFlag))
			{
				FIXUP_DEBUG_LOG_PATCH(TEXT("  nopatch  %s Corner - no neighbors"), *GetDirectionString(CornerIndex));
				continue;
			}

			const bool bModified = EnumHasAnyFlags(NeighborSnapshots.EdgesWithAnyModifiedNeighbor, CornerFlag);
			if (!bModified)
			{
				// if each edge's initial hash matches, no need to patch
				// (this is slightly different than the blend matching, as the blend may only include a subset of the edges, and produce a different result,
				// whereas the initial state may include data from edges that are not yet loaded)
				const uint32 LocalInitialHash = NeighborSnapshots.LocalSnapshot->InitialEdgeHashes[(int32)CornerIndex];

				auto NeighborHasSameInitialHash = [LocalInitialHash, CornerIndex, NeighborSnapshots](int32 NeighborRotation, int32 NeighborEdgeRotation) -> bool
				{
					if (FHeightmapTextureEdgeSnapshot* Snapshot = NeighborSnapshots.NeighborSnapshots[(int32)RotateDirection(CornerIndex, NeighborRotation)])
					{
						return Snapshot->InitialEdgeHashes[(int32)RotateDirection(CornerIndex, NeighborEdgeRotation)] == LocalInitialHash;
					}
					return true;
				};

				const bool bInitialStateMatch =
					NeighborHasSameInitialHash(1, -2) &&
					NeighborHasSameInitialHash(-1, 2) &&
					NeighborHasSameInitialHash(0, 4);

				if (bInitialStateMatch)
				{
					FIXUP_DEBUG_LOG_PATCH(TEXT("  nopatch  %s Corner - initial state match"), *GetDirectionString(CornerIndex));
					continue;
				}
			}

			// Because the corner hashes are just a copy of the corner data, we can directly compare them with the expected result
			FHeightmapTexel Blended, GPUExisting = {};
			BlendCornerData(Blended, CornerIndex, NeighborSnapshots);

			GPUExisting.Data32 = GPUEdgeHashes[(int32)CornerIndex];

			if (Blended.Data32 == GPUExisting.Data32)
			{
				FIXUP_DEBUG_LOG_PATCH(TEXT("  nopatch  %s Corner - exact match (%x == old %x)"), *GetDirectionString(CornerIndex), Blended.Data32, GPUExisting.Data32);
				continue;
			}

			// Check if we're only off by one in the normal, and don't bother patching these cases.
			// It might be noticeable with a very shiny material, but not in most landscape cases.
			// These cases generally happen due to floating point and round off error and inexactness between GPU and CPU implementations.
			if (Blended.IsSameHeight(GPUExisting) &&
				(abs(Blended.NormalX - (int32)GPUExisting.NormalX) <= 1) &&
				(abs(Blended.NormalY - (int32)GPUExisting.NormalY) <= 1))
			{
				FIXUP_DEBUG_LOG_PATCH(TEXT("  nopatch  %s Corner - off-by-one match (%x ~ old %x)"), *GetDirectionString(CornerIndex), Blended.Data32, GPUExisting.Data32);
				continue;
			}

			FIXUP_DEBUG_LOG_PATCH(TEXT("  PATCHING %s Corner (new %x != old %x)"), *GetDirectionString(CornerIndex), Blended.Data32, GPUExisting.Data32);
			PatchTextureCorner_Internal(CornerIndex, Blended);
			PatchedEdgeCount++;
			GPUEdgeHashes[(int32)CornerIndex] = Blended.Data32;
			EnumAddFlags(GPUEdgeModifiedFlags, CornerFlag);
		}
	}

	return PatchedEdgeCount;
}

void ULandscapeHeightmapTextureEdgeFixup::PatchTextureEdge_Internal(UE::Landscape::EEdgeIndex EdgeIndex)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PatchTextureEdge_Internal);

	using namespace UE::Landscape;

	FTextureResource* HeightTextureResource = HeightmapTexture->GetResource();
	if (!HeightTextureResource)
	{
		return;
	}

#if WITH_EDITOR
	// this should be caught earlier -- we can't patch default textures
	check(!HeightmapTexture->IsDefaultTexture());
#endif // WITH_EDITOR
	check(!IsDiagonalCorner(EdgeIndex));

	const int32 NumMips = HeightmapTexture->GetNumMips();
	const int32 ResidentMips = HeightmapTexture->GetNumResidentMips();

	ULandscapeHeightmapTextureEdgeFixup* EdgeNeighbor = ActiveGroup->GetNeighborEdgeFixup(GroupCoord, EdgeIndex);
		
	// the first (and largest) resident mip. For example, if all mips are resident, then this is mip 0.
	const int32 FirstResidentMipIndex = NumMips - ResidentMips;

	// this is the size of the first mip that has edges (2x2 and 1x1 mips don't have "edges", just "corners")
	const int32 MinMipSizeToFix = 4;

	// update this edge using data from neighbor and us		
	int32 MipSize = EdgeSnapshot->EdgeLength >> FirstResidentMipIndex;
	for (int32 MipIndex = FirstResidentMipIndex; MipIndex < NumMips && MipSize >= MinMipSizeToFix; MipIndex++, MipSize /= 2)
	{
		// passing an empty TStridedView to BlendEdgeData will make it allocate a buffer to hold the blended data
		// (which we then delete in the render command below)
		TStridedView<UE::Landscape::FHeightmapTexel> DestView;
		BlendEdgeData(EdgeSnapshot.Get(), EdgeIndex, MipIndex, EdgeNeighbor->EdgeSnapshot.Get(), DestView);

		ENQUEUE_RENDER_COMMAND(UpdateLandscapeHeightmapEdge)(
			[HeightTextureResource, DestView, EdgeIndex, MipSize, MipIndex, FirstResidentMipIndex, ResidentMips](FRHICommandListImmediate& RHICmdList)
			{
				const uint8* Buffer = DestView[0].Data;

				FTextureRHIRef RHIHeightmapTexture = HeightTextureResource->GetTextureRHI();

				const FRHITextureDesc& Desc = RHIHeightmapTexture->GetDesc();

				// Resource doesn't know about the unstreamed mips, so it's zero Mip starts at the first resident mip
				check(ResidentMips == Desc.NumMips);
				int32 ResourceMipIndex = MipIndex - FirstResidentMipIndex;

				if (ResourceMipIndex >= (int32) RHIHeightmapTexture->GetNumMips())
				{
					UE_LOGF(LogLandscape, Error, "   Can't Update Edge %d Mip %d because the texture resource does not have that mip", EnumToUnderlyingType(EdgeIndex), ResourceMipIndex);
				}
				else
				{
					// determine the min/max range to update on the edge, based on whether we are updating the corners or not				
					int32 DstStartX, DstStartY;
					int32 SrcStartX, SrcStartY, SrcWidth, SrcHeight;
					SrcStartX = 0;
					SrcStartY = 0;
					if (IsTopOrBottomEdge(EdgeIndex))
					{
						DstStartX = 1;
						DstStartY = (EdgeIndex == EEdgeIndex::Bottom) ? 0 : (MipSize - 1);
						SrcWidth = MipSize - 2;
						SrcHeight = 1;
					}
					else
					{
						DstStartX = (EdgeIndex == EEdgeIndex::Left) ? 0 : (MipSize - 1);
						DstStartY = 1;
						SrcWidth = 1;
						SrcHeight = MipSize - 2;
					}

					{
						const uint32 SourcePitch = sizeof(UE::Landscape::FHeightmapTexel) * SrcWidth;
						FUpdateTextureRegion2D UpdateRegion(DstStartX, DstStartY, SrcStartX, SrcStartY, SrcWidth, SrcHeight);

						FIXUP_DEBUG_LOG_RENDER(TEXT("    PatchTextureEdge_Internal [UpdateTexture2D] res:%p e:%d mip:%d reg:[%d %d %d %d %d %d]"), HeightTextureResource, EdgeIndex, MipIndex,
							UpdateRegion.DestX, UpdateRegion.DestY, UpdateRegion.SrcX, UpdateRegion.SrcY, UpdateRegion.Width, UpdateRegion.Height);

						RHICmdList.UpdateTexture2D(RHIHeightmapTexture, ResourceMipIndex, UpdateRegion, SourcePitch, Buffer);
					}
				}

				// free the data buffer (in our specific case, DestView has ownership of the memory)
				Buffer = nullptr;
				UE::Landscape::FHeightmapTexel* DestViewData = &DestView[0];
				delete[] DestViewData;
			}
		);
	}
}

void ULandscapeHeightmapTextureEdgeFixup::PatchTextureCorner_Internal(UE::Landscape::EEdgeIndex CornerIndex, UE::Landscape::FHeightmapTexel Texel)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PatchTextureCorner_Internal);

	using namespace UE::Landscape;

	FTextureResource* HeightTextureResource = HeightmapTexture->GetResource();
	if (!HeightTextureResource)
	{
		return;
	}

#if WITH_EDITOR
	// this should be caught earlier -- we can't patch default textures
	check(!HeightmapTexture->IsDefaultTexture());
#endif // WITH_EDITOR
	check(IsDiagonalCorner(CornerIndex));

	const int32 NumMips = HeightmapTexture->GetNumMips();
	const int32 ResidentMips = HeightmapTexture->GetNumResidentMips();

	// the first (and largest) resident mip. For example, if all mips are resident, then this is mip 0.
	const int32 FirstResidentMipIndex = NumMips - ResidentMips;

	int32 FirstResidentMipSize = (EdgeSnapshot->EdgeLength >> FirstResidentMipIndex);

	ENQUEUE_RENDER_COMMAND(UpdateLandscapeHeightmapCorner)(
		[HeightTextureResource, Texel, CornerIndex, FirstResidentMipSize](FRHICommandListImmediate& RHICmdList)
		{
			FTextureRHIRef RHIHeightmapTexture = HeightTextureResource->GetTextureRHI();

			const FRHITextureDesc& Desc = RHIHeightmapTexture->GetDesc();
			const int32 NumMips = Desc.NumMips;

			// this is the size of the smallest mip that has 4 corners (1x1 mip only has one texel)
			const int32 MinMipSizeToFix = 2;

			int32 MipSize = FirstResidentMipSize;
			check(FirstResidentMipSize == Desc.GetSize().X);

			for (int32 ResourceMipIndex = 0; ResourceMipIndex < NumMips && MipSize >= MinMipSizeToFix; ResourceMipIndex++, MipSize /= 2)
			{
				// grab the corner coordinates in the texture
				int32 DstStartX, DstStartY;
				GetSourceTextureEdgeStartCoord(CornerIndex, MipSize, DstStartX, DstStartY);

				// update one pixel
				uint32 SourcePitch = 4;
				const int32 SrcStartX = 0;
				const int32 SrcStartY = 0;
				const int32 SrcWidth = 1;
				const int32 SrcHeight = 1;

				FUpdateTextureRegion2D UpdateRegion(DstStartX, DstStartY, SrcStartX, SrcStartY, SrcWidth, SrcHeight);

				FIXUP_DEBUG_LOG_RENDER(TEXT("    PatchTextureCorner_Internal [UpdateTexture2D] res:%p e:%d mip:%d reg:[%d %d %d %d %d %d]"), HeightTextureResource, CornerIndex, ResourceMipIndex,
					UpdateRegion.DestX, UpdateRegion.DestY, UpdateRegion.SrcX, UpdateRegion.SrcY, UpdateRegion.Width, UpdateRegion.Height);

				RHICmdList.UpdateTexture2D(RHIHeightmapTexture, ResourceMipIndex, UpdateRegion, SourcePitch, Texel.Data);
			}
		}
	);
}

int32 ULandscapeHeightmapTextureEdgeFixup::PatchTextureEdgesForStreamingMips(int32 FirstMipIndexInclusive, int32 LastMipIndexExclusive, FTextureMipInfoArray& DestMipInfos, const UE::Landscape::FNeighborSnapshots& NeighborSnapshots)
{
	int32 PatchedEdges = 0;

	// TODO [chris.tchou] : it would be more efficient to iterate the mips internally to each edge, more similar to the non-streaming patch case
	for (int MipIndex = FirstMipIndexInclusive; MipIndex < LastMipIndexExclusive; MipIndex++)
	{
		FTextureMipInfo& DestMipInfo = DestMipInfos[MipIndex];
		PatchedEdges += PatchTextureEdgesForSingleMip(MipIndex, DestMipInfo, NeighborSnapshots);
	}
	return PatchedEdges;
}

int32 ULandscapeHeightmapTextureEdgeFixup::PatchTextureEdgesForSingleMip(int32 MipIndex, FTextureMipInfo& DestMipInfo, const UE::Landscape::FNeighborSnapshots& NeighborSnapshots)
{
	using namespace UE::Landscape;

	check(DestMipInfo.Format == PF_B8G8R8A8); // format used by FHeightmapTexel

	FHeightmapTexel* Dest = (FHeightmapTexel*) DestMipInfo.DestData;

	// some mip allocators do not provide RowPitch -- in that case assume SizeX texels is the pitch
	uint32 DstPitchInTexels;
	if (DestMipInfo.RowPitch > 0)
	{
		DstPitchInTexels = DestMipInfo.RowPitch / sizeof(FHeightmapTexel);
	}
	else
	{
		DstPitchInTexels = DestMipInfo.SizeX;
	}

	int32 MipEdgeLength = NeighborSnapshots.LocalSnapshot->EdgeLength >> MipIndex;
	check(DestMipInfo.SizeX >= (uint32) MipEdgeLength);
	check(DestMipInfo.SizeY >= (uint32) MipEdgeLength);

	int32 PatchedEdges = 0;

	if (EnumHasAnyFlags(NeighborSnapshots.EdgesWithAnyModifiedNeighbor, EEdgeFlags::AllEdges))
	{
		for (EEdgeIndex EdgeIndex : AllEdgeNeighbors())
		{
			// only patch edges that have been modified (to match the existing state of the other mips)
			if (!EnumHasAnyFlags(NeighborSnapshots.EdgesWithAnyModifiedNeighbor, ToFlag(EdgeIndex)))
			{
				continue;
			}

			// only patch edges that have an existing neighbor
			if (FHeightmapTextureEdgeSnapshot* NeighborSnapshot = NeighborSnapshots.NeighborSnapshots[(int32)EdgeIndex])
			{
				// update this edge using data from neighbor and us
				TStridedView<UE::Landscape::FHeightmapTexel> DestView;
				int32 CopyCount = MipEdgeLength - 2;

				if (IsTopOrBottomEdge(EdgeIndex))
				{
					// horizontal
					int32 DstStartX = 1;
					int32 DstStartY = (EdgeIndex == EEdgeIndex::Bottom) ? 0 : (MipEdgeLength - 1);
					int32 DstOffset = DstStartX + DstStartY * DstPitchInTexels;
					DestView = TStridedView<FHeightmapTexel>((int32)sizeof(FHeightmapTexel), &Dest[DstOffset], CopyCount);
				}
				else
				{
					// vertical
					int32 DstStartX = (EdgeIndex == EEdgeIndex::Left) ? 0 : (MipEdgeLength - 1);
					int32 DstStartY = 1;
					int32 DstOffset = DstStartX + DstStartY * DstPitchInTexels;
					DestView = TStridedView<FHeightmapTexel>(DstPitchInTexels * sizeof(FHeightmapTexel), &Dest[DstOffset], CopyCount);
				}
				BlendEdgeData(*NeighborSnapshots.LocalSnapshot, EdgeIndex, MipIndex, *NeighborSnapshot, DestView);
				PatchedEdges++;

				FIXUP_DEBUG_LOG_RENDER(TEXT("    Patch Streamed Mip Edge:%s mip:%d len:%d"), *GetDirectionString(EdgeIndex), MipIndex, MipEdgeLength);
			}
		}
	}

	const EEdgeFlags CornersWithOneOrMoreNeighbors = NeighborsToBlendedEdges(NeighborSnapshots.ExistingNeighbors);
	if (EnumHasAnyFlags(CornersWithOneOrMoreNeighbors, EEdgeFlags::AllCorners))
	{
		for (EEdgeIndex CornerIndex : AllCornerNeighbors())
		{
			// only patch corners that have been modified (to match the existing state of the other mips)
			if (!EnumHasAnyFlags(NeighborSnapshots.EdgesWithAnyModifiedNeighbor, ToFlag(CornerIndex)))
			{
				continue;
			}

			// and corners that have at least one existing neighbor
			if (!EnumHasAnyFlags(CornersWithOneOrMoreNeighbors, ToFlag(CornerIndex)))
			{
				continue;
			}

			int32 DestStride = 0;
			int32 DestCount = 0;
			const int32 DestOffset = GetSourceTextureEdgeStartStrideCount(CornerIndex, MipEdgeLength, DestStride, DestCount);

			FHeightmapTexel &Texel = Dest[DestOffset];
			BlendCornerData(Texel, CornerIndex, NeighborSnapshots);
			PatchedEdges++;

			FIXUP_DEBUG_LOG_RENDER(TEXT("    Patch Streamed Mip Corner:%s mip:%d len:%d"), *GetDirectionString(CornerIndex), MipIndex, MipEdgeLength);
		}
	}

	return PatchedEdges;
}

#if WITH_EDITOR
void ULandscapeHeightmapTextureEdgeFixup::RequestEdgeSnapshotUpdateFromHeightmapSource(bool bInUpdateGPUEdgeHashes)
{
	if (bInUpdateGPUEdgeHashes)
	{
		bUpdateGPUEdgeHashes = true;
	}
	check(ActiveGroup->AllRegisteredFixups.Contains(this));
	ActiveGroup->HeightmapsNeedingEdgeSnapshotCapture.Add(this);
}

UE::Landscape::ENeighborFlags ULandscapeHeightmapTextureEdgeFixup::UpdateEdgeSnapshotFromHeightmapSource(const FVector& LandscapeGridScale, bool bForceUpdate, bool* bOutSuccess)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UpdateEdgeSnapshotFromHeightmapSource);

	using namespace UE::Landscape;

	ENeighborFlags ChangedEdges = ENeighborFlags::None;

	if (bForceUpdateSnapshot)
	{
		bForceUpdate = true;
	}

	// no need to update if source ID matches (we assume texture has not actually changed)		
	FGuid HeightmapSourceID = ULandscapeTextureHash::GetHash(HeightmapTexture);
	if (bForceUpdate || (HeightmapSourceID != EdgeSnapshot->TextureSourceID))
	{
		FIXUP_DEBUG_LOG_DETAIL(TEXT("  UpdateEdgeSnapshotFromHeightmapSource %p HeightMap: %p -- UPDATED (ID: %s --> %s) force:%d"), ActiveComponent.Get(), HeightmapTexture.Get(),
			*EdgeSnapshot->TextureSourceID.ToString(), *HeightmapSourceID.ToString(), bForceUpdate);
			
		// create a new edge data
		TSharedPtr<FHeightmapTextureEdgeSnapshot> NewEdgeSnapshot = FHeightmapTextureEdgeSnapshot::CreateEdgeSnapshotFromHeightmapSource(HeightmapTexture, LandscapeGridScale);
		if (!NewEdgeSnapshot.IsValid())
		{
			// failed to capture an edge snapshot (for example HeightmapTexture could have corrupt data)
			if (bOutSuccess)
			{
				UE_LOGF(LogLandscape, Warning, "Failed to capture edge snapshot from landscape heightmap, landscape edge patching may not work correctly.");
				*bOutSuccess = false;
			}
			return ENeighborFlags::None;
		}
		else
		{
			// compare against previous edge data to see what edges have changed (and might cause neighbors to need to patch)
			ChangedEdges = NewEdgeSnapshot->CompareEdges(EdgeSnapshot.Get());

			// assign the new edge data
			EdgeSnapshot = NewEdgeSnapshot.ToSharedRef();

			if (bUpdateGPUEdgeHashes)
			{
				GPUEdgeHashes = EdgeSnapshot->InitialEdgeHashes;
				GPUEdgeModifiedFlags = EEdgeFlags::None;

				FIXUP_DEBUG_LOG_DETAIL(TEXT("    Reset GPU Edge Hashes: [B:%x BR:%x R:%x TR:%x T:%x TL:%x L:%x BL:%x]"),
					GPUEdgeHashes[0], GPUEdgeHashes[1], GPUEdgeHashes[2], GPUEdgeHashes[3], GPUEdgeHashes[4], GPUEdgeHashes[5], GPUEdgeHashes[6], GPUEdgeHashes[7]);
			}
			if (bOutSuccess)
			{
				*bOutSuccess = true;
			}
		}
	}
	else
	{
		FIXUP_DEBUG_LOG_DETAIL(TEXT("  UpdateEdgeSnapshotFromHeightmapSource %p HeightMap: %p -- SourceID matches (%s)"), ActiveComponent.Get(), HeightmapTexture.Get(), *EdgeSnapshot->TextureSourceID.ToString());
		if (bOutSuccess)
		{
			*bOutSuccess = true;
		}
	}

	bForceUpdateSnapshot = false;
	bUpdateGPUEdgeHashes = false;
	bDoNotPatchUntilGPUEdgeHashesUpdated = false;

	return ChangedEdges;
}
#endif // WITH_EDITOR

void ULandscapeHeightmapTextureEdgeFixup::Serialize(FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		// To ensure thread-safe CopyOnWrite behavior when loading, we need to allocate a new chunk of memory to serialize into, then replace the existing reference with that
		TSharedRef<FHeightmapTextureEdgeSnapshot> NewEdgeSnapshot = MakeShared<FHeightmapTextureEdgeSnapshot>();
		Ar << NewEdgeSnapshot.Get();
		EdgeSnapshot = NewEdgeSnapshot;

		FIXUP_DEBUG_LOG_DETAIL(TEXT("Loaded EdgeFixup - Snapshot [B:%x BR:%x R:%x TR:%x T:%x TL:%x L:%x BL:%x] (ID: %s)"),
			EdgeSnapshot->SnapshotEdgeHashes[0], EdgeSnapshot->SnapshotEdgeHashes[1], EdgeSnapshot->SnapshotEdgeHashes[2], EdgeSnapshot->SnapshotEdgeHashes[3],
			EdgeSnapshot->SnapshotEdgeHashes[4], EdgeSnapshot->SnapshotEdgeHashes[5], EdgeSnapshot->SnapshotEdgeHashes[6], EdgeSnapshot->SnapshotEdgeHashes[7],
			*EdgeSnapshot->GetTextureSourceIDAsString());

		FIXUP_DEBUG_LOG_DETAIL(TEXT("                 - Initial [B:%x BR:%x R:%x TR:%x T:%x TL:%x L:%x BL:%x]"),
			EdgeSnapshot->InitialEdgeHashes[0], EdgeSnapshot->InitialEdgeHashes[1], EdgeSnapshot->InitialEdgeHashes[2], EdgeSnapshot->InitialEdgeHashes[3],
			EdgeSnapshot->InitialEdgeHashes[4], EdgeSnapshot->InitialEdgeHashes[5], EdgeSnapshot->InitialEdgeHashes[6], EdgeSnapshot->InitialEdgeHashes[7]);

		FIXUP_DEBUG_LOG_DETAIL(TEXT("                 - GPU [B:%x BR:%x R:%x TR:%x T:%x TL:%x L:%x BL:%x]"),
			GPUEdgeHashes[0], GPUEdgeHashes[1], GPUEdgeHashes[2], GPUEdgeHashes[3],
			GPUEdgeHashes[4], GPUEdgeHashes[5], GPUEdgeHashes[6], GPUEdgeHashes[7]);
	}
	else
	{
		Ar << EdgeSnapshot.Get();

		FIXUP_DEBUG_LOG_DETAIL(TEXT("  Saved EdgeFixup - Snapshot [B:%x BR:%x R:%x TR:%x T:%x TL:%x L:%x BL:%x] (ID: %s)"),
			EdgeSnapshot->SnapshotEdgeHashes[0], EdgeSnapshot->SnapshotEdgeHashes[1], EdgeSnapshot->SnapshotEdgeHashes[2], EdgeSnapshot->SnapshotEdgeHashes[3],
			EdgeSnapshot->SnapshotEdgeHashes[4], EdgeSnapshot->SnapshotEdgeHashes[5], EdgeSnapshot->SnapshotEdgeHashes[6], EdgeSnapshot->SnapshotEdgeHashes[7],
			*EdgeSnapshot->GetTextureSourceIDAsString());

		FIXUP_DEBUG_LOG_DETAIL(TEXT("                  - Initial [B:%x BR:%x R:%x TR:%x T:%x TL:%x L:%x BL:%x]"),
			EdgeSnapshot->InitialEdgeHashes[0], EdgeSnapshot->InitialEdgeHashes[1], EdgeSnapshot->InitialEdgeHashes[2], EdgeSnapshot->InitialEdgeHashes[3],
			EdgeSnapshot->InitialEdgeHashes[4], EdgeSnapshot->InitialEdgeHashes[5], EdgeSnapshot->InitialEdgeHashes[6], EdgeSnapshot->InitialEdgeHashes[7]);

		FIXUP_DEBUG_LOG_DETAIL(TEXT("                  - GPU [B:%x BR:%x R:%x TR:%x T:%x TL:%x L:%x BL:%x]"),
			GPUEdgeHashes[0], GPUEdgeHashes[1], GPUEdgeHashes[2], GPUEdgeHashes[3],
			GPUEdgeHashes[4], GPUEdgeHashes[5], GPUEdgeHashes[6], GPUEdgeHashes[7]);
	}
}

void ULandscapeHeightmapTextureEdgeFixup::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);
	ULandscapeHeightmapTextureEdgeFixup* const TypedThis = Cast<ULandscapeHeightmapTextureEdgeFixup>(InThis);
	Collector.AddReferencedObject(TypedThis->HeightmapTexture);
	Collector.AddReferencedObject(TypedThis->ActiveComponent);
}

ULandscapeHeightmapTextureEdgeFixup* ULandscapeHeightmapTextureEdgeFixup::FindOrCreateFor(UTexture2D* InHeightmapTexture)
{
	check(InHeightmapTexture);

	// try to get an existing factory
	ULandscapeHeightmapTextureEdgeFixup* Fixup = InHeightmapTexture->GetAssetUserData<ULandscapeHeightmapTextureEdgeFixup>();

#if WITH_EDITORONLY_DATA
	if (Fixup == nullptr)
	{
		check(InHeightmapTexture->Source.IsValid())
		check(InHeightmapTexture->Source.GetFormat() == TSF_BGRA8);

		// create a new Fixup, and attach it to the texture via User Data (and as outer)
		Fixup = NewObject<ULandscapeHeightmapTextureEdgeFixup>(InHeightmapTexture);	
		InHeightmapTexture->AddAssetUserData(Fixup);
		FIXUP_DEBUG_LOG_DETAIL(TEXT("  FindOrCreateFor texture %p -- CREATE FIXUP"), InHeightmapTexture);
	}
	else if (Fixup)
	{
		FIXUP_DEBUG_LOG_DETAIL(TEXT("  FindOrCreateFor texture %p -- FOUND FIXUP"), InHeightmapTexture);
	}
	else
	{
		FIXUP_DEBUG_LOG_DETAIL(TEXT("  FindOrCreateFor texture %p -- FAILED"), InHeightmapTexture);
	}
#endif // WITH_EDITORONLY_DATA
	
	if (Fixup)
	{
		Fixup->SetHeightmapTexture(InHeightmapTexture);
	}

	return Fixup;
}

#undef FIXUP_DEBUG_LOG
#undef FIXUP_DEBUG_LOG_PATCH
#undef FIXUP_DEBUG_LOG_RENDER
#undef FIXUP_DEBUG_LOG_DETAIL
