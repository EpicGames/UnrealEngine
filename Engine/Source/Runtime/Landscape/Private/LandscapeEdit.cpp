// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
LandscapeEdit.cpp: Landscape editing
=============================================================================*/

#include "LandscapeEdit.h"
#include "LandscapePrivate.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Misc/FeedbackContext.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectIterator.h"
#include "UObject/Package.h"
#include "Misc/PackageName.h"
#include "Landscape.h"
#include "LandscapeEditLayer.h"
#include "LandscapeEditReadback.h"
#include "LandscapeStreamingProxy.h"
#include "LandscapeInfo.h"
#include "LandscapeComponent.h"
#include "LandscapeModule.h"
#include "LandscapeLayerInfoObject.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialExpressionLandscapeVisibilityMask.h"
#include "Materials/MaterialExpressionLandscapeLayerWeight.h"
#include "Materials/MaterialExpressionLandscapeLayerSample.h"
#include "Materials/MaterialExpressionLandscapeLayerBlend.h"
#include "Materials/MaterialExpressionLandscapeLayerSwitch.h"
#include "Materials/MaterialExpressionLandscapePhysicalMaterialOutput.h"
#include "LandscapeDataAccess.h"
#include "LandscapeRender.h"
#include "LandscapePrivate.h"
#include "Materials/MaterialInstanceConstant.h"
#include "LandscapeMaterialInstanceConstant.h"
#include "LandscapeHeightfieldCollisionComponent.h"
#include "LandscapeMeshCollisionComponent.h"
#include "LandscapeGizmoActiveActor.h"
#include "InstancedFoliageActor.h"
#include "LevelUtils.h"
#include "Logging/TokenizedMessage.h"
#include "Logging/MessageLog.h"
#include "Misc/MapErrors.h"
#include "LandscapeSplinesComponent.h"
#include "Serialization/MemoryWriter.h"
#include "MaterialCachedData.h"
#include "Math/UnrealMathUtility.h"
#include "ImageUtils.h"

#if WITH_EDITOR
#include "Engine/World.h"
#include "LandscapeSubsystem.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshDescription.h"
#include "StaticMeshOperations.h"
#include "MeshUtilitiesCommon.h"
#include "OverlappingCorners.h"
#include "MeshBuild.h"
#include "StaticMeshBuilder.h"
#include "NaniteBuilder.h"
#include "Rendering/NaniteResources.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/UObjectToken.h"
#include "EngineModule.h"
#include "EngineUtils.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "LandscapeEditorModule.h"
#include "LandscapeFileFormatInterface.h"
#include "ComponentRecreateRenderStateContext.h"
#include "ComponentReregisterContext.h"
#include "Interfaces/ITargetPlatform.h"
#include "Engine/TextureRenderTarget2D.h"
#include "ScopedTransaction.h"
#include "Editor.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHandle.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/Landscape/LandscapeActorDesc.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "ActorPartition/ActorPartitionSubsystem.h"
#include "LandscapeUtils.h"
#include "LandscapeUtilsPrivate.h"
#include "LandscapeSplineActor.h"
#include "Materials/MaterialExpressionLandscapeGrassOutput.h"
#include "ShaderPlatformCachedIniValue.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Misc/TransactionObjectEvent.h"
#include "LandscapeTextureStorageProvider.h"
#endif
#include "Algo/Compare.h"
#include "LandscapeProxy.h"
#include "Algo/Count.h"
#include "Algo/Transform.h"
#include "Algo/ForEach.h"
#include "Algo/AllOf.h"
#include "Serialization/MemoryWriter.h"
#include "Engine/Canvas.h"
#include "Spatial/PointHashGrid3.h"
#include "Engine/Texture2DArray.h"
#include "Async/ParallelFor.h"
#include "Templates/Greater.h"

DEFINE_LOG_CATEGORY(LogLandscape);
DEFINE_LOG_CATEGORY(LogLandscapeBP);

#define LOCTEXT_NAMESPACE "Landscape"

static TAutoConsoleVariable<int32> CVarLandscapeApplyPhysicalMaterialChangesImmediately(
    TEXT("landscape.ApplyPhysicalMaterialChangesImmediately"),
	1,
    TEXT("Applies physical material task changes immediately rather than during the next cook/PIE."));

extern FAutoConsoleVariable CVarStripLayerTextureMipsOnLoad;
extern FAutoConsoleVariable CVarLandscapeSilenceInvalidPropertyOverridesMapCheckWarnings;

#if WITH_EDITOR

// Used to temporarily disable material instance updates (typically used for cases where multiple updates are called on sample component)
// Instead, one call per component is done at the end
LANDSCAPE_API bool GDisableUpdateLandscapeMaterialInstances = false;

LANDSCAPE_API FName ALandscape::AffectsLandscapeActorDescProperty(TEXT("AffectsLandscape"));

// Channel remapping
extern const size_t ChannelOffsets[4];

extern float LandscapeNaniteBuildLag;

ULandscapeLayerInfoObject* ALandscapeProxy::VisibilityLayer = nullptr;

FOnLandscapeProxyFixupSharedDataDelegate ALandscapeProxy::OnLandscapeProxyFixupSharedDataDelegate;
#endif //WITH_EDITOR

void ULandscapeComponent::Init(int32 InBaseX, int32 InBaseY, int32 InComponentSizeQuads, int32 InNumSubsections, int32 InSubsectionSizeQuads)
{
	ALandscapeProxy* LandscapeProxy = GetLandscapeProxy();
	check(LandscapeProxy && !LandscapeProxy->LandscapeComponents.Contains(this));
	LandscapeProxy->LandscapeComponents.Add(this);

	SetSectionBase(FIntPoint(InBaseX, InBaseY));
	SetRelativeLocation(FVector(GetSectionBase() - GetLandscapeProxy()->GetSectionBase()));
	ComponentSizeQuads = InComponentSizeQuads;
	NumSubsections = InNumSubsections;
	SubsectionSizeQuads = InSubsectionSizeQuads;
	check(NumSubsections * SubsectionSizeQuads == ComponentSizeQuads);

	AttachToComponent(LandscapeProxy->GetRootComponent(), FAttachmentTransformRules::KeepRelativeTransform);
	const int32 ComponentVerts = (SubsectionSizeQuads + 1) * NumSubsections;
	
	WeightmapScaleBias = FVector4(1.0f / (float)ComponentVerts, 1.0f / (float)ComponentVerts, 0.5f / (float)ComponentVerts, 0.5f / (float)ComponentVerts);
	WeightmapSubsectionOffset = (float)(SubsectionSizeQuads + 1) / (float)ComponentVerts;
		
	UpdatedSharedPropertiesFromActor();

	// Update CachedLocalBox with approximate bounds to avoid group misalignment warnings during registration
	CachedLocalBox = ComputeApproximateBounds();
}

FBox ULandscapeComponent::ComputeApproximateBounds()
{
	FVector MinBox(0, 0, LandscapeDataAccess::GetLocalHeight(0));
	FVector MaxBox(ComponentSizeQuads + 1, ComponentSizeQuads + 1, LandscapeDataAccess::GetLocalHeight(UINT16_MAX));
	return FBox(MinBox, MaxBox);
}

#if WITH_EDITOR
namespace UE::Landscape::Private
{
	/** 
	* Struct to hold, for a given 4 neighboring pixels of a given mip of the heightmap, information about the difference between those and the resulting pixel in the next mip. 
	*  This allows landscape components to evaluate the error (height delta) between any 2 mip levels
	*/
	struct FQuadHeightInfo
	{
		FQuadHeightInfo() = default;
		FQuadHeightInfo(double InQuadVertices[4])
		{
			Min = FMath::Min(InQuadVertices[0], FMath::Min3(InQuadVertices[1], InQuadVertices[2], InQuadVertices[3]));
			Max = FMath::Max(InQuadVertices[0], FMath::Max3(InQuadVertices[1], InQuadVertices[2], InQuadVertices[3]));
			Average = (InQuadVertices[0] + InQuadVertices[1] + InQuadVertices[2] + InQuadVertices[3]) / 4.0;
		}

		double Min = DBL_MAX;
		double Max = -DBL_MAX;
		double Average = 0.0f;
	};

	int32 ComputeQuadInfosOffset(int32 InMipIndex, int32 InTextureSize)
	{
		int32 NumQuadsForMip = FMath::Square(InTextureSize / 2);
		int32 Offset = 0;
		for (int32 X = 0; X < InMipIndex; ++X)
		{
			Offset += NumQuadsForMip;
			NumQuadsForMip /= 4;
		}
		return Offset;
	}

	int32 ComputeQuadInfosCount(int32 InNumRelevantMips, int32 InTextureSize)
	{
		int32 Count = 0; 
		int32 NumQuadsForMip = FMath::Square(InTextureSize / 2);
		for (int32 MipIndex = 0; MipIndex < InNumRelevantMips; ++MipIndex)
		{
			Count += NumQuadsForMip;
			NumQuadsForMip /= 4;
		}
		return Count;
	}

	TArrayView<FQuadHeightInfo> GetMipQuadInfosForMip(const TArrayView<FQuadHeightInfo>& InQuadInfos, int32 InMipIndex, int32 InTextureSize)
	{
		int32 MipOffset = ComputeQuadInfosOffset(InMipIndex, InTextureSize);
		int32 NextMipOffset = ComputeQuadInfosOffset(InMipIndex + 1, InTextureSize);
		return MakeArrayView(InQuadInfos.GetData() + MipOffset, NextMipOffset - MipOffset);
	}

	TArrayView<const FQuadHeightInfo> GetMipQuadInfosForMipConst(const TArrayView<const FQuadHeightInfo>& InQuadInfos, int32 InMipIndex, int32 InTextureSize)
	{
		TArrayView<FQuadHeightInfo> Result = GetMipQuadInfosForMip(MakeArrayView(const_cast<FQuadHeightInfo*>(InQuadInfos.GetData()), InQuadInfos.Num()), InMipIndex, InTextureSize);
		return MakeArrayView<const FQuadHeightInfo>(Result.GetData(), Result.Num());
	}

	void ComputeMaxDelta(const FQuadHeightInfo& InSourceQuadInfo, const FQuadHeightInfo& InDestinationQuadInfo, double& InOutMaxDelta)
	{
		InOutMaxDelta = FMath::Max3(
			InOutMaxDelta,
			FMath::Abs(InSourceQuadInfo.Min - InDestinationQuadInfo.Average), 
			FMath::Abs(InSourceQuadInfo.Max - InDestinationQuadInfo.Average));
	}

	TArray<double> ComputeMipToMipMaxDeltas(const TArrayView<const FQuadHeightInfo>& InAllMipsQuadInfos, int32 InNumTextureMips, int32 InNumRelevantMips)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ComputeMipToMipMaxDeltas);
		const int32 TextureSize = 1 << (InNumTextureMips - 1);

		bool bForceSingleThread = !FApp::ShouldUseThreadingForPerformance();
		const EParallelForFlags ParallelFlags = bForceSingleThread ? EParallelForFlags::ForceSingleThread : EParallelForFlags::None;

		constexpr int32 MaxBatchSize = 512; // Tested faster than 1024, about the same as 256.  Take 512 in favor of smaller task context allocation size.
		int32 MaxTaskContexts = 1;  // Keep the arena size small if not using threads.
		if (!bForceSingleThread)
		{
			const TArrayView<const FQuadHeightInfo> LargestMipInfo = GetMipQuadInfosForMipConst(InAllMipsQuadInfos, 0, TextureSize);
			int32 LargestJobSize = LargestMipInfo.Num();
			MaxTaskContexts = (LargestJobSize / MaxBatchSize) + 1;
			MaxTaskContexts = FMath::Min(MaxTaskContexts, FTaskGraphInterface::Get().GetNumWorkerThreads() + 1);  // Similar logic to ParallelForImpl::GetNumberOfThreadTasks, but doesn't need to exactly match.
		}
	
		constexpr int32 ByteAlignment = PLATFORM_CACHE_LINE_SIZE;
		constexpr int32 WordAlignment = ByteAlignment / sizeof(double);

		// Task contexts will be blocks of ResultSize doubles, with the block size rounded up to be a multiple of the cache line size, to prevent false-sharing.
		int32 ResultSize = ComputeMipToMipMaxDeltasCount(InNumRelevantMips);
		int32 ResultSizeAlignedWords = ((ResultSize + WordAlignment - 1) / WordAlignment) * WordAlignment;
		TArray<double> TaskContextArena;
		TaskContextArena.AddZeroed(ResultSizeAlignedWords * MaxTaskContexts);

		using TaskContext = TArrayView<double>;
		TArray<TaskContext, TInlineAllocator<65>> TaskContexts;  // Task contexts the threads use are TArrayViews into the area allocation.
		TaskContexts.AddUninitialized(MaxTaskContexts);
		for (int32 Idx = 0; Idx < MaxTaskContexts; ++Idx)
		{
			TaskContexts[Idx] = MakeArrayView(&TaskContextArena[Idx * ResultSizeAlignedWords], ResultSizeAlignedWords);
		};

		for (int32 SourceMipIndex = 0; SourceMipIndex < InNumRelevantMips - 1; ++SourceMipIndex)
		{
			const TArrayView<const FQuadHeightInfo> SourceMipQuadInfos = GetMipQuadInfosForMipConst(InAllMipsQuadInfos, SourceMipIndex, TextureSize);
			const int32 NumQuadsForSourceMip = SourceMipQuadInfos.Num();
			const int32 SourceMipQuadsStride = (TextureSize >> SourceMipIndex) / 2;
			check(SourceMipQuadsStride * SourceMipQuadsStride == NumQuadsForSourceMip);
			const int32 NumMaxDeltasForSourceMip = ComputeMaxDeltasCountForMip(SourceMipIndex, InNumRelevantMips);
			const int32 SourceMipMaxDeltasOffset = ComputeMaxDeltasOffsetForMip(SourceMipIndex, InNumRelevantMips);
			check(SourceMipMaxDeltasOffset + NumMaxDeltasForSourceMip <= ResultSize);

			ParallelForWithExistingTaskContext(MakeArrayView(TaskContexts), NumQuadsForSourceMip, MaxBatchSize,
				[SourceMipQuadsStride, &SourceMipQuadInfos, SourceMipMaxDeltasOffset, NumMaxDeltasForSourceMip, InNumRelevantMips, SourceMipIndex, &InAllMipsQuadInfos, TextureSize]
				(TaskContext& Context, int32 SourceMipQuadIndex)
				{
					// Accumulate results in the Context of the worker thread
					TArrayView<double> SourceMipToDestinationMipMaxDeltas = MakeArrayView(Context.GetData() + SourceMipMaxDeltasOffset, NumMaxDeltasForSourceMip);

					const FIntPoint SourceMipQuadCoords(SourceMipQuadIndex % SourceMipQuadsStride, SourceMipQuadIndex / SourceMipQuadsStride);
					const FQuadHeightInfo& SourceMipQuadInfo = SourceMipQuadInfos[SourceMipQuadIndex];

					// Special case for N to N+1 because all the info is located within the row of InAllMipsQuadInfos already so we don't need to fetch from another mip: FQuadHeightInfo's Average is the next mip's value:
					int32 DestinationMipQuadsStride = SourceMipQuadsStride / 2;
					FIntPoint DestinationMipQuadCoords = SourceMipQuadCoords / 2;
					ComputeMaxDelta(SourceMipQuadInfo, SourceMipQuadInfo, SourceMipToDestinationMipMaxDeltas[0]);

					for (int32 DestinationMipIndex = SourceMipIndex + 2; DestinationMipIndex < InNumRelevantMips; ++DestinationMipIndex)
					{
						DestinationMipQuadsStride /= 2;
						DestinationMipQuadCoords /= 2;
						const int32 DestinationMipQuadIndex = DestinationMipQuadCoords.X + DestinationMipQuadCoords.Y * DestinationMipQuadsStride;

						TArrayView<const FQuadHeightInfo> DestinationMipQuadInfos = GetMipQuadInfosForMipConst(InAllMipsQuadInfos, DestinationMipIndex, TextureSize);
						checkSlow(DestinationMipQuadInfos.Num() == DestinationMipQuadsStride * DestinationMipQuadsStride);

						const FQuadHeightInfo& DestinationMipQuadInfo = DestinationMipQuadInfos[DestinationMipQuadIndex];
						int32 DestinationMipRelativeIndex = DestinationMipIndex - SourceMipIndex - 1;
						ComputeMaxDelta(SourceMipQuadInfo, DestinationMipQuadInfo, SourceMipToDestinationMipMaxDeltas[DestinationMipRelativeIndex]);
					}
				},
				ParallelFlags);
		}

		// Collapse completed task contexts to the final result.
		TArray<double> MipToMipMaxDeltas;
		MipToMipMaxDeltas.AddZeroed(ResultSize);  //ComputeMaxDelta uses absolute values, so starting with 0 is ok.
		for (int32 ContextIdx = 0; ContextIdx < MaxTaskContexts; ++ContextIdx)
		{
			for (int32 ValueIdx = 0; ValueIdx < ResultSize; ++ValueIdx)
			{
				MipToMipMaxDeltas[ValueIdx] = FMath::Max(MipToMipMaxDeltas[ValueIdx], TaskContexts[ContextIdx][ValueIdx]);
			}
		}

		return MipToMipMaxDeltas;
	}
}

bool ULandscapeComponent::UpdateCachedBounds(bool bInApproximateBounds)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeComponent::UpdateCachedBounds);
	check(IsInParallelGameThread() || IsInGameThread());

	if ((SubsectionSizeQuads == 0) || (NumSubsections == 0) || (ComponentSizeQuads == 0))
	{
		// the landscape component is in an uninitialized/default state, we cannot calculate meaningful bounds
		return false;
	}

	bool bChanged = false;
	FBox NewLocalBox;

	// Update local-space bounding box
	NewLocalBox.Init();
	if (bInApproximateBounds)
	{
		NewLocalBox = ComputeApproximateBounds();
		check(NewLocalBox.GetExtent().Z > 0.0);
	}
	else
	{
		// We purposefully don't reset MipToMipMaxDeltas when bInApproximateBounds is true because we can totally live with a MipToMipMaxDeltas that is not up-to-date
		//  and we want to minimize the render state changes so we track if there was any change to that data here : 
		TArray<double> PreviousMipToMipMaxDeltas;
		Swap(MipToMipMaxDeltas, PreviousMipToMipMaxDeltas); 

		using namespace UE::Landscape::Private;
		TArray<FQuadHeightInfo> AllMipsQuadInfos;
		const int32 TextureSize = (SubsectionSizeQuads + 1) * NumSubsections;
		const int32 NumTextureMips = FMath::FloorLog2(TextureSize) + 1;
		// We actually only don't need to process the last texture mip, since a 1 vertex landscape is meaningless. When using 2x2 subsections, we can even drop an additional mip 
		//  as the 4 texels of the penultimate mip will be identical (i.e. 4 sub-sections of 1 vertex are equally meaningless) :
		const int32 NumRelevantMips = GetNumRelevantMips();
		const int32 FinalMipIndex = NumRelevantMips - 1;
		check(FinalMipIndex > 0);

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FetchMipQuads);

			AllMipsQuadInfos.AddZeroed(ComputeQuadInfosCount(NumRelevantMips, TextureSize));

			struct alignas(PLATFORM_CACHE_LINE_SIZE) FMinMaxType  //pad to cache line size to avoid false-sharing
			{
				double Min = TNumericLimits<double>::Max();
				double Max = TNumericLimits<double>::Lowest();
			};
			TArray<FMinMaxType, TInlineAllocator<17>> ContextValues;
			// 1024 batch size limit is a little arbitrary.  Each task isn't very complicated and the job size isn't very large so one expects a low ceiling on effective parallelism
			// due to memory bandwidth and overhead.  Quick test shows this is significantly faster than 4096 or 128 on a workstation with more cores than needed.
			// With TextureSize=256, MaxBatchSize=1024, MaxTaskContexts is 17.
			const int32 MaxBatchSize = 1024;

			TArrayView<FQuadHeightInfo> LargestMipInfo = GetMipQuadInfosForMip(AllMipsQuadInfos, 0, TextureSize);
			int32 LargestJobSize = LargestMipInfo.Num();
			int32 MaxTaskContexts = (LargestJobSize / MaxBatchSize) + 1;
			MaxTaskContexts = FMath::Min(MaxTaskContexts, FTaskGraphInterface::Get().GetNumWorkerThreads() + 1);  // Similar logic to ParallelForImpl::GetNumberOfThreadTasks, but doesn't need to exactly match.
			ContextValues.SetNum(MaxTaskContexts);

			const EParallelForFlags ParallelFlags = FApp::ShouldUseThreadingForPerformance() ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread;

			// Outer loop over mip levels.
			for (int32 MipIndex = 0; MipIndex < NumRelevantMips; ++MipIndex)
			{
				FLandscapeComponentDataInterface CDI(this, MipIndex, /*bWorkOnEditingLayer = */false);
				TArrayView<FQuadHeightInfo> MipQuadInfos = GetMipQuadInfosForMip(AllMipsQuadInfos, MipIndex, TextureSize);
				const int32 NumQuadsForMip = MipQuadInfos.Num();
				const int32 MipTextureSize = TextureSize >> MipIndex;
				const int32 MipTextureSubSectionSize = MipTextureSize / NumSubsections;
				const int32 MipQuadsStride = MipTextureSize / 2;

				// Inner loop, over quads in the mip
				ParallelForWithExistingTaskContext(MakeArrayView(ContextValues), NumQuadsForMip, MaxBatchSize,
					[&MipQuadInfos, &CDI, MipQuadsStride, MipTextureSubSectionSize, MipIndex](FMinMaxType& Context, int32 MipQuadIndex)
					{
						FTaskTagScope TaskTag(ETaskTag::EParallelGameThread);

						int32 QuadY = MipQuadIndex / MipQuadsStride;
						int32 QuadX = MipQuadIndex - QuadY * MipQuadsStride;
						int32 X = QuadX * 2;
						int32 Y = QuadY * 2;
						// 2x2 Subsections have a duplicate pixel in the middle so we have to subtract one for the pixels in those subsections, since FLandscapeComponentDataInterface provides a view into 
						//  pixels as if there was no subsection (hence the pixel indices for a 128 heightmap is in the range [0, 127] when using a single (1x1) subsection, but [0, 126] when using 2x2 subsections) :
						X -= (X >= MipTextureSubSectionSize) ? 1 : 0;
						Y -= (Y >= MipTextureSubSectionSize) ? 1 : 0;
						double QuadVertices[4] = {
							CDI.GetLocalHeight(X + 0, Y + 0),
							CDI.GetLocalHeight(X + 1, Y + 0),
							CDI.GetLocalHeight(X + 0, Y + 1),
							CDI.GetLocalHeight(X + 1, Y + 1) };
						FQuadHeightInfo QuadInfo(QuadVertices);
						MipQuadInfos[MipQuadIndex] = QuadInfo;

						if (MipIndex == 0)
						{
							Context.Min = FMath::Min(Context.Min, QuadInfo.Min);
							Context.Max = FMath::Max(Context.Max, QuadInfo.Max);
						}
					},
					ParallelFlags);
			}

			FMinMaxType TotalMinMax;
			for (const FMinMaxType& Context : ContextValues)
			{
				TotalMinMax.Min = FMath::Min(Context.Min, TotalMinMax.Min);
				TotalMinMax.Max = FMath::Max(Context.Max, TotalMinMax.Max);
			}
			check(TotalMinMax.Max >= TotalMinMax.Min);

			NewLocalBox = FBox(FVector(0.0, 0.0, TotalMinMax.Min), FVector(ComponentSizeQuads, ComponentSizeQuads, TotalMinMax.Max));
		}

		MipToMipMaxDeltas = ComputeMipToMipMaxDeltas(MakeArrayView<const FQuadHeightInfo>(AllMipsQuadInfos.GetData(), AllMipsQuadInfos.Num()), NumTextureMips, NumRelevantMips);

		if (!Algo::Compare(MipToMipMaxDeltas, PreviousMipToMipMaxDeltas, [](double InLHS, double InRHS) { return FMath::IsNearlyEqual(InLHS, InRHS); }))
		{
			// MipToMipMaxDeltas is used on the render thread, we need to reflect the change there : 
			MarkRenderStateDirty();
		}
	}
	if (NewLocalBox.GetExtent().Z == 0)
	{
		// expand bounds to avoid flickering issues with zero-size bounds
		NewLocalBox = NewLocalBox.ExpandBy(FVector(0, 0, 1));
	}

	if (NewLocalBox != CachedLocalBox)
	{
		CachedLocalBox = NewLocalBox;
		bChanged = true;
	}

	// Update collision component bounds
	ULandscapeHeightfieldCollisionComponent* HFCollisionComponent = GetCollisionComponent();
	if (HFCollisionComponent)
	{
		HFCollisionComponent->CachedLocalBox = CachedLocalBox;
		HFCollisionComponent->UpdateComponentToWorld();
	}

	return bChanged;
}

#endif //WITH_EDITOR

void ULandscapeComponent::UpdateNavigationRelevance()
{
	ALandscapeProxy* Proxy = GetLandscapeProxy();
	ULandscapeHeightfieldCollisionComponent* CollisionComponent = GetCollisionComponent();

	if (CollisionComponent && Proxy)
	{
		CollisionComponent->SetCanEverAffectNavigation(Proxy->bUsedForNavigation);
	}
}

void ULandscapeComponent::UpdateRejectNavmeshUnderneath()
{
	ALandscapeProxy* Proxy = GetLandscapeProxy();
	ULandscapeHeightfieldCollisionComponent* CollisionComponent = GetCollisionComponent();

	if (CollisionComponent && Proxy)
	{
		CollisionComponent->bFillCollisionUnderneathForNavmesh = Proxy->bFillCollisionUnderLandscapeForNavmesh;
	}
}

#if WITH_EDITOR
bool ULandscapeComponent::ValidateCombinationMaterial(UMaterialInstanceConstant* InCombinationMaterial) const
{
	if (InCombinationMaterial == nullptr)
	{
		return false;
	}

	const TArray<FStaticTerrainLayerWeightParameter>& TerrainLayerWeightParameters = InCombinationMaterial->GetEditorOnlyStaticParameters().TerrainLayerWeightParameters;
	const TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapAllocations = GetWeightmapLayerAllocations();

	if (TerrainLayerWeightParameters.Num() != ComponentWeightmapAllocations.Num())
	{
		UE_LOGF(LogLandscape, Display, "Material instance %ls in landscape component %ls doesn't match the expected shader combination: different number of allocations (expected: %i, found: %i)",
			*InCombinationMaterial->GetName(), *GetPathName(), ComponentWeightmapAllocations.Num(), TerrainLayerWeightParameters.Num());

		return false;
	}

	for (const FWeightmapLayerAllocationInfo& Allocation : ComponentWeightmapAllocations)
	{
		if (Allocation.LayerInfo == nullptr)
		{
			UE_LOGF(LogLandscape, Display, "Material instance %ls in landscape component %ls doesn't match the expected shader combination: invalid layer info",
				*InCombinationMaterial->GetName(), *GetPathName());

			return false;
		}

		// Each weightmap allocation must have its equivalent in the material's TerrainLayerWeightParameters : 
		if (!TerrainLayerWeightParameters.ContainsByPredicate(([&Allocation](const FStaticTerrainLayerWeightParameter& TerrainLayerWeightParameter)
			{
				return ((TerrainLayerWeightParameter.LayerName == Allocation.LayerInfo->GetLayerName())
					&& (TerrainLayerWeightParameter.WeightmapIndex == Allocation.WeightmapTextureIndex));
			})))
		{
			UE_LOGF(LogLandscape, Display, "Material instance %ls in landscape component %ls doesn't match the expected shader combination: missing layer %ls",
				*InCombinationMaterial->GetName(), *GetPathName(), *Allocation.LayerInfo->GetLayerName().ToString());

			return false;
		}
	}

	return true;
}

/**
* Generate a key for this component's layer allocations to use with MaterialInstanceConstantMap.
*/
FString ULandscapeComponent::GetLayerAllocationKey(const TArray<FWeightmapLayerAllocationInfo>& Allocations, UMaterialInterface* LandscapeMaterial, bool bMobile /*= false*/)
{
	if (!LandscapeMaterial)
	{
		return FString();
	}

	FString Result = LandscapeMaterial->GetPathName();

	// Generate a string to describe each allocation
	TArray<FString> LayerStrings;
	for (int32 LayerIdx = 0; LayerIdx < Allocations.Num(); LayerIdx++)
	{
		if (Allocations[LayerIdx].GetLayerName() != NAME_None) // ignore layers that have no name
		{
			LayerStrings.Add(FString::Printf(TEXT("_%s%d"), *Allocations[LayerIdx].GetLayerName().ToString(), Allocations[LayerIdx].WeightmapTextureIndex));
		}
	}
	// Sort them alphabetically so we can share across components even if the order is different
	LayerStrings.Sort(TGreater<FString>());

	for (int32 LayerIdx = 0; LayerIdx < LayerStrings.Num(); LayerIdx++)
	{
		Result += LayerStrings[LayerIdx];
	}

	if (bMobile)
	{
		Result += TEXT("M");
	}

	return Result;
}

UMaterialInstanceConstant* ULandscapeComponent::GetCombinationMaterial(FMaterialUpdateContext* InMaterialUpdateContext, const TArray<FWeightmapLayerAllocationInfo>& Allocations, int8 InLODIndex, bool bMobile /*= false*/) const
{
	const bool bComponentHasHoles = ComponentHasVisibilityPainted();
	UMaterialInterface* const LandscapeMaterial = GetLandscapeMaterial(InLODIndex);
	UMaterialInterface* const HoleMaterial = bComponentHasHoles ? GetLandscapeHoleMaterial() : nullptr;
	UMaterialInterface* const MaterialToUse = bComponentHasHoles && HoleMaterial ? HoleMaterial : LandscapeMaterial;
	bool bOverrideBlendMode = bComponentHasHoles && !HoleMaterial && IsOpaqueBlendMode(*LandscapeMaterial);

	if (bOverrideBlendMode)
	{
		UMaterial* Material = LandscapeMaterial->GetMaterial();
		if (Material && Material->bUsedAsSpecialEngineMaterial)
		{
			bOverrideBlendMode = false;

			static TWeakPtr<SNotificationItem> ExistingNotification;
			if (!ExistingNotification.IsValid())
			{
				// let the user know why they are not seeing holes
				FNotificationInfo Info(LOCTEXT("AssignLandscapeMaterial", "You must assign a regular, non-engine material to your landscape in order to see holes created with the visibility tool."));
				Info.ExpireDuration = 5.0f;
				Info.bUseSuccessFailIcons = true;
				ExistingNotification = TWeakPtr<SNotificationItem>(FSlateNotificationManager::Get().AddNotification(Info));
			}
			return nullptr;
		}
	}

	if (ensure(MaterialToUse != nullptr))
	{
		ALandscapeProxy* Proxy = GetLandscapeProxy();
		FString LayerKey = GetLayerAllocationKey(Allocations, MaterialToUse, bMobile);

		// Find or set a matching MIC in the Landscape's map.
		UMaterialInstanceConstant* CombinationMaterialInstance = Proxy->MaterialInstanceConstantMap.FindRef(*LayerKey);
		if (CombinationMaterialInstance == nullptr || CombinationMaterialInstance->Parent != MaterialToUse || GetOuter() != CombinationMaterialInstance->GetOuter())
		{
			FlushRenderingCommands();

			ULandscapeMaterialInstanceConstant* LandscapeCombinationMaterialInstance = NewObject<ULandscapeMaterialInstanceConstant>(GetOuter());
			LandscapeCombinationMaterialInstance->bMobile = bMobile;
			CombinationMaterialInstance = LandscapeCombinationMaterialInstance;
			UE_LOGF(LogLandscape, Verbose, "Looking for key %ls, making new combination %ls", *LayerKey, *CombinationMaterialInstance->GetName());
			Proxy->MaterialInstanceConstantMap.Add(*LayerKey, CombinationMaterialInstance);
			CombinationMaterialInstance->SetParentEditorOnly(MaterialToUse, false);

			CombinationMaterialInstance->BasePropertyOverrides.bOverride_BlendMode = bOverrideBlendMode;
			if (bOverrideBlendMode)
			{
				CombinationMaterialInstance->BasePropertyOverrides.BlendMode = bComponentHasHoles ? BLEND_Masked : BLEND_Opaque;
			}

			FStaticParameterSet StaticParameters;
			CombinationMaterialInstance->GetStaticParameterValues(StaticParameters);

			for (const FWeightmapLayerAllocationInfo& Allocation : Allocations)
			{
				if (Allocation.LayerInfo)
				{
					FName LayerName = Allocation.GetLayerName();
					if (LayerName != NAME_None) // ignore allocations that have no name
					{
						StaticParameters.EditorOnly.TerrainLayerWeightParameters.Add(FStaticTerrainLayerWeightParameter(LayerName, Allocation.WeightmapTextureIndex));
					}
				}
			}

			CombinationMaterialInstance->UpdateStaticPermutation(StaticParameters, InMaterialUpdateContext);
			CombinationMaterialInstance->PostEditChange();
		}

		return CombinationMaterialInstance;
	}
	return nullptr;
}

void ULandscapeComponent::UpdateMaterialInstances_Internal(FMaterialUpdateContext& Context)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeComponent::UpdateMaterialInstances_Internal);

	const int8 MaxLOD = IntCastChecked<int8>(FMath::CeilLogTwo(SubsectionSizeQuads + 1) - 1);
	decltype(MaterialPerLOD) NewMaterialPerLOD;
	LODIndexToMaterialIndex.SetNumUninitialized(MaxLOD+1);
	int8 LastLODIndex = INDEX_NONE;

	UMaterialInterface* BaseMaterial = GetLandscapeMaterial();
	UMaterialInterface* LOD0Material = GetLandscapeMaterial(0);

	for (int8 LODIndex = 0; LODIndex <= MaxLOD; ++LODIndex)
	{
		UMaterialInterface* CurrentMaterial = GetLandscapeMaterial(LODIndex);

		// if we have a LOD0 override, do not let the base material override it, it should override everything!
		if (CurrentMaterial == BaseMaterial && BaseMaterial != LOD0Material)
		{
			CurrentMaterial = LOD0Material;
		}

		const int8* MaterialLOD = NewMaterialPerLOD.Find(CurrentMaterial);

		if (MaterialLOD != nullptr)
		{
			LODIndexToMaterialIndex[LODIndex] = *MaterialLOD > LastLODIndex ? *MaterialLOD : LastLODIndex;
		}
		else
		{
			const int8 AddedIndex = IntCastChecked<int8>(NewMaterialPerLOD.Num());
			NewMaterialPerLOD.Add(CurrentMaterial, LODIndex);
			LODIndexToMaterialIndex[LODIndex] = AddedIndex;
			LastLODIndex = AddedIndex;
		}
	}

	MaterialPerLOD = NewMaterialPerLOD;

	MaterialInstances.SetNumZeroed(MaterialPerLOD.Num());
	int8 MaterialIndex = 0;

	const TArray<FWeightmapLayerAllocationInfo>& WeightmapBaseLayerAllocation = GetWeightmapLayerAllocations();
	const TArray<UTexture2D*>& WeightmapBaseTexture = GetWeightmapTextures();
	UTexture2D* BaseHeightmap = GetHeightmap();

	for (auto It = MaterialPerLOD.CreateConstIterator(); It; ++It)
	{
		const int8 MaterialLOD = It.Value();

		// Find or set a matching MIC in the Landscape's map.
		UMaterialInstanceConstant* CombinationMaterialInstance = GetCombinationMaterial(&Context, WeightmapBaseLayerAllocation, MaterialLOD, false);

		if (CombinationMaterialInstance != nullptr)
		{
			// Create the instance for this component, that will use the layer combination instance.
			UMaterialInstanceConstant* MaterialInstance = NewObject<ULandscapeMaterialInstanceConstant>(GetOuter());
			MaterialInstances[MaterialIndex] = MaterialInstance;

			// Material Instances don't support Undo/Redo (the shader map goes out of sync and crashes happen)
			// so we call UpdateMaterialInstances() from ULandscapeComponent::PostEditUndo instead
			//MaterialInstance->SetFlags(RF_Transactional);
			//MaterialInstance->Modify();

			MaterialInstance->SetParentEditorOnly(CombinationMaterialInstance);
			MaterialInstance->ClearParameterValuesEditorOnly();
			Context.AddMaterialInstance(MaterialInstance); // must be done after SetParent

			FLinearColor Masks[4] = { FLinearColor(1.0f, 0.0f, 0.0f, 0.0f), FLinearColor(0.0f, 1.0f, 0.0f, 0.0f), FLinearColor(0.0f, 0.0f, 1.0f, 0.0f), FLinearColor(0.0f, 0.0f, 0.0f, 1.0f) };

			// Set the layer mask
			for (int32 AllocIdx = 0; AllocIdx < WeightmapBaseLayerAllocation.Num(); AllocIdx++)
			{
				const FWeightmapLayerAllocationInfo& Allocation = WeightmapBaseLayerAllocation[AllocIdx];

				MaterialInstance->SetVectorParameterValueEditorOnly(FName(*FString::Printf(TEXT("LayerMask_%s"), *Allocation.GetLayerName().ToString())), Masks[Allocation.WeightmapTextureChannel]);
			}

			// Set the weightmaps
			for (int32 i = 0; i < WeightmapBaseTexture.Num(); i++)
			{
				MaterialInstance->SetTextureParameterValueEditorOnly(FName(*FString::Printf(TEXT("Weightmap%d"), i)), WeightmapBaseTexture[i]);
			}

			// Set the heightmap, if needed.
			if (BaseHeightmap)
			{
				MaterialInstance->SetTextureParameterValueEditorOnly(FName("Heightmap"), BaseHeightmap);
			}
			MaterialInstance->PostEditChange();
		}

		++MaterialIndex;
	}

	MaterialInstances.Remove(nullptr);
	MaterialInstances.Shrink();

	if (MaterialPerLOD.Num() == 0)
	{
		MaterialInstances.Empty(1);
		MaterialInstances.Add(nullptr);
		LODIndexToMaterialIndex.Empty(1);
		LODIndexToMaterialIndex.Add(0);
	}

	// Update mobile combination material
	{
		GenerateMobileWeightmapLayerAllocations();

		MobileCombinationMaterialInstances.SetNumZeroed(MaterialPerLOD.Num());
		int8 MobileMaterialIndex = 0;

		for (auto It = MaterialPerLOD.CreateConstIterator(); It; ++It)
		{
			const int8 MaterialLOD = It.Value();

			UMaterialInstanceConstant* MobileCombinationMaterialInstance = GetCombinationMaterial(&Context, MobileWeightmapLayerAllocations, MaterialLOD, true);
			MobileCombinationMaterialInstances[MobileMaterialIndex] = MobileCombinationMaterialInstance;

			if (MobileCombinationMaterialInstance != nullptr)
			{
				Context.AddMaterialInstance(MobileCombinationMaterialInstance);
			}
						
			++MobileMaterialIndex;
		}
	}
}

void ULandscapeComponent::UpdateMaterialInstances()
{
	if (GDisableUpdateLandscapeMaterialInstances)
		return;

	// we're not having the material update context recreate the render state because we will manually do it for only this component
	TOptional<FComponentRecreateRenderStateContext> RecreateRenderStateContext;
	RecreateRenderStateContext.Emplace(this);
	TOptional<FMaterialUpdateContext> MaterialUpdateContext;
	MaterialUpdateContext.Emplace(FMaterialUpdateContext::EOptions::Default & ~FMaterialUpdateContext::EOptions::RecreateRenderStates);

	UpdateMaterialInstances_Internal(MaterialUpdateContext.GetValue());

	// End material update
	MaterialUpdateContext.Reset();

	// Recreate the render state for this component, needed to update the static drawlist which has cached the MaterialRenderProxies
	// Must be after the FMaterialUpdateContext is destroyed
	RecreateRenderStateContext.Reset();
}

void ULandscapeComponent::UpdateMaterialInstances(FMaterialUpdateContext& InOutMaterialContext, TArray<FComponentRecreateRenderStateContext>& InOutRecreateRenderStateContext)
{
	InOutRecreateRenderStateContext.Add(this);
	UpdateMaterialInstances_Internal(InOutMaterialContext);
}

void ALandscapeProxy::UpdateAllComponentMaterialInstances(FMaterialUpdateContext& InOutMaterialContext, TArray<FComponentRecreateRenderStateContext>& InOutRecreateRenderStateContext, bool bInInvalidateCombinationMaterials)
{
	if (bInInvalidateCombinationMaterials)
	{
		MaterialInstanceConstantMap.Reset();
	}

	for (ULandscapeComponent* Component : LandscapeComponents)
	{
		Component->UpdateMaterialInstances(InOutMaterialContext, InOutRecreateRenderStateContext);
	}

	UpdateNaniteMaterials();
}

void ALandscapeProxy::UpdateNaniteMaterials()
{
	if (!GetLandscapeActor() || !GetLandscapeActor()->IsNaniteEnabled() )
	{
		return;
	}
		
	// Update the Nanite component Materials from the LandscapeComponents
	for(ULandscapeNaniteComponent* NaniteComponent : NaniteComponents)
	{
		NaniteComponent->UpdateMaterials();
	}
}

void ALandscapeProxy::UpdateAllComponentMaterialInstances(bool bInInvalidateCombinationMaterials)
{
	if (bInInvalidateCombinationMaterials)
{
		MaterialInstanceConstantMap.Reset();
	}

	// we're not having the material update context recreate render states because we will manually do it for only our components
	TArray<FComponentRecreateRenderStateContext> RecreateRenderStateContexts;
	RecreateRenderStateContexts.Reserve(LandscapeComponents.Num());

	for (ULandscapeComponent* Component : LandscapeComponents)
	{
		RecreateRenderStateContexts.Emplace(Component);
	}
	TOptional<FMaterialUpdateContext> MaterialUpdateContext;
	MaterialUpdateContext.Emplace(FMaterialUpdateContext::EOptions::Default & ~FMaterialUpdateContext::EOptions::RecreateRenderStates);

	for (ULandscapeComponent* Component : LandscapeComponents)
	{
		Component->UpdateMaterialInstances_Internal(MaterialUpdateContext.GetValue());
	}

	// End material update
	MaterialUpdateContext.Reset();

	// Recreate the render state for our components, needed to update the static drawlist which has cached the MaterialRenderProxies
	// Must be after the FMaterialUpdateContext is destroyed
	RecreateRenderStateContexts.Empty();

	UpdateNaniteMaterials();
}

int32 ULandscapeComponent::GetNumMaterials() const
{
	return 1;
}

bool ULandscapeComponent::GetMaterialPropertyPath(int32 ElementIndex, UObject*& OutOwner, FString& OutPropertyPath, FProperty*& OutProperty)
{
	if (ElementIndex == 0)
	{
		if (OverrideMaterial)
		{
			OutOwner = this;
			OutPropertyPath = GET_MEMBER_NAME_STRING_CHECKED(ULandscapeComponent, OverrideMaterial);
			OutProperty = ULandscapeComponent::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ULandscapeComponent, OverrideMaterial));
	
			return true;
		}
		if (ALandscapeProxy* Proxy = GetLandscapeProxy())
		{
			OutOwner = Proxy;
			OutPropertyPath = GET_MEMBER_NAME_STRING_CHECKED(ALandscapeProxy, LandscapeHoleMaterial);
			OutProperty = ALandscapeProxy::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ALandscapeProxy, LandscapeHoleMaterial));
			return true;
		}
	}

	return false;
}

class UMaterialInterface* ULandscapeComponent::GetMaterial(int32 ElementIndex) const
{
	if (ensure(ElementIndex == 0))
	{
		return GetLandscapeMaterial(IntCastChecked<int8>(ElementIndex));
	}

	return nullptr;
}

void ULandscapeComponent::SetMaterial(int32 ElementIndex, class UMaterialInterface* Material)
{
	if (ensure(ElementIndex == 0))
	{
		GetLandscapeProxy()->LandscapeMaterial = Material;
	}
}

void ULandscapeComponent::PreFeatureLevelChange(ERHIFeatureLevel::Type PendingFeatureLevel)
{
	Super::PreFeatureLevelChange(PendingFeatureLevel);

	if (PendingFeatureLevel == ERHIFeatureLevel::ES3_1)
	{
		// See if we need to cook platform data for mobile preview in editor
		CheckGenerateMobilePlatformData(/*bIsCooking = */ false, /*TargetPlatform = */ nullptr);
	}
}

void ULandscapeComponent::PreEditUndo()
{
	// We bump the proxy's version numbers here in PRE-EDIT:
	// This means all of the PostEditUndo() operations (across all Components on the Proxy) 
	// see a consistent number and ensure we only do WeightmapFixup once.
	ALandscapeProxy* Proxy = GetLandscapeProxy();
	Proxy->CurrentVersion++;
	
	// Keep track of components involved in the undo/redo and store their weightmap hashes before they get changed.
	// Note the hash covers allocations and pointers, but not weightmap contents.
	UndoRedoModifiedComponents.Add(this, ComputeWeightmapsHash());
}

void ULandscapeComponent::PostEditUndo()
{
	ALandscapeProxy* Proxy = GetLandscapeProxy();	
	if (IsValid(Proxy)) // this post edit can be deleting the landscape
	{
		// PreEditUndo of the LandscapeStreamingProxy resets it's pointer to the top level landscape so this ensures it's fixed up prior
		// to requesting HeightMap or WeightMap updates
		ULandscapeInfo* LandscapeInfo = Proxy->GetLandscapeInfo();
		if (LandscapeInfo && !LandscapeInfo->IsRegistered(Proxy))
		{
			LandscapeInfo->RegisterActor(Proxy, true);
		}
	}

	// On undo, request a recompute weightmap usages, as they are not transacted (they combine information between multiple components)
	Proxy->RequestProxyLayersWeightmapUsageUpdate();

	Super::PostEditUndo();

	if (IsValidChecked(this))
	{
		EditToolRenderData.UpdateSelectionMaterial(EditToolRenderData.SelectedType, this);
	}
	
	// Updating the bounds after undo will re-sync CachedLocalBox with the heightmap data. This is necessary because ULandscapeComponent might be part of 
	//  the transaction buffer, but the final heightmaps aren't, so undoing might desynchronize them. If the heightmap update that is requested right after 
	//  ends up modifying heightmap data, then CachedLocalBox will get updated properly again so this is a 'safe' way to fix the issue
	if (IsValidChecked(this))
	{
		UpdateCachedBounds();
	}

	const bool bUpdateAll = true;
	RequestHeightmapUpdate(bUpdateAll);
	RequestWeightmapUpdate(bUpdateAll);
	
	// Nanite data is non-transactional so we simply want to regenerate it on undo (depending on whether auto-update is enabled, it will either trigger its 
	//  rebuild or will simply invalidate it) : 
	Proxy->InvalidateOrUpdateNaniteRepresentation(/* bInCheckContentId = */true, /*InTargetPlatform = */nullptr);

	// Only Fixup Weightmaps and Update Material Instances when the LAST modified component has been processed by PostEditUndo()
	UndoRedoModifiedComponentCount++;
	if (UndoRedoModifiedComponents.Num() == UndoRedoModifiedComponentCount)
	{
		// process all the proxies across all of the components (except proxies that are pending kill)
		TSet<ALandscapeProxy*> Proxies;
		for (auto& Iter : UndoRedoModifiedComponents)
		{
			ULandscapeComponent* Component = Iter.Key;
			ALandscapeProxy* ComponentProxy = Component->GetLandscapeProxy();
			if (IsValid(ComponentProxy))
			{
				Proxies.Add(ComponentProxy);
			}
		}

		for (ALandscapeProxy* ComponentProxy : Proxies)
		{
			// Here we create and register with the LandscapeInfo early (before PostRegisterAllComponents)
			ULandscapeInfo* LandscapeInfo = ComponentProxy->GetLandscapeInfo();
			if (!IsValid(LandscapeInfo))
			{
				LandscapeInfo = ComponentProxy->CreateLandscapeInfo(true);
				check(LandscapeInfo->IsRegistered(ComponentProxy));
			}
			else if (!LandscapeInfo->IsRegistered(ComponentProxy))
			{
				LandscapeInfo->RegisterActor(ComponentProxy, true);
			}

			// So that we can ensure that weightmaps are fixed up (which depend on the Landscape Info registration)
			ComponentProxy->FixupWeightmaps();
		}

		// Update Material Instances on each modified component, in case the material property has changed (uses the weightmap allocations fixed by FixupWeightmaps)
		{
			// We're not having the material update context recreate the render state because we will manually do it for only these components
			TArray<FComponentRecreateRenderStateContext> RecreateRenderStateContexts;
			{
				FMaterialUpdateContext MaterialUpdateContext(FMaterialUpdateContext::EOptions::Default & ~FMaterialUpdateContext::EOptions::RecreateRenderStates);
				for (auto& Iter : UndoRedoModifiedComponents)
				{
					ULandscapeComponent* Component = Iter.Key;
					if (IsValid(Component)) // Components can be pending kill, if they were removed by the undo operation
					{
						ALandscapeProxy* ComponentProxy = Component->GetLandscapeProxy();
						if (IsValid(ComponentProxy))
						{
							Component->UpdateMaterialInstances(MaterialUpdateContext, RecreateRenderStateContexts);

							uint32 PreHash = Iter.Value;
							uint32 PostHash = Component->ComputeWeightmapsHash();
							if (PreHash != PostHash)
							{
								// Component's weightmap allocations changed by Undo/Redo.  Keep track of this so that appropriate invalidations can be
								// done during merge.
								Component->bUndoChangedWeightmapAllocs = true;
							}
						}
					}
				}
			}
		}

		// After updating the component materials, update the nanite materials on each proxy
		for (ALandscapeProxy* ComponentProxy : Proxies)
		{
			ComponentProxy->UpdateNaniteMaterials();
		}

		// Clear out the temporarily recorded undo info
		UndoRedoModifiedComponentCount = 0;
		UndoRedoModifiedComponents.Empty();
	}
}

TUniquePtr<FWorldPartitionActorDesc> ALandscapeProxy::CreateClassActorDesc() const
{
	return TUniquePtr<FWorldPartitionActorDesc>(new FLandscapeActorDesc());
}

bool ALandscapeProxy::GetReferencedContentObjects(TArray<UObject*>& Objects) const
{
	Super::GetReferencedContentObjects(Objects);

	TSet<UMaterialInterface*> LandscapeMaterials;
	RetrieveAllLandscapeMaterials(LandscapeMaterials);
	for (UMaterialInterface* Material : LandscapeMaterials)
	{
		Objects.AddUnique(Material);
	}

	return true;
}

void ALandscapeProxy::FixupWeightmaps()
{
	WeightmapUsageMap.Empty();

	// We've just reinitialized the weightmap usages map and FixupWeightmaps will reconstruct it component by component, but we might in the process delete invalid layers (e.g. those whose landscape info has been deleted),
	//  in which case ValidateProxyLayersWeightmapUsage() on the entire proxy will be called and, because of WeightmapUsageMap's cleanup above, might report missing weightmap usages. 
	//  To avoid triggering asserts, we simply disable validation until the fixup operation is done : 
	bTemporarilyDisableWeightmapUsagesValidation = true;
	ON_SCOPE_EXIT
	{
		WeightmapFixupVersion = CurrentVersion;
		bTemporarilyDisableWeightmapUsagesValidation = false;

		// Rebuild weightmap usages, now that the allocations are fixed
		InitializeProxyLayersWeightmapUsage();

		// Now that the job is done, weightmap usages should be valid again
		ValidateProxyLayersWeightmapUsage();
	};

	for (ULandscapeComponent* Component : LandscapeComponents)
	{
		if (Component != nullptr)
		{
			Component->FixupWeightmaps();
		}
	}
}

void ALandscapeProxy::RepairInvalidTextures()
{
	TArray<UTexture*> InvalidTextures;
	for (ULandscapeComponent* Component : LandscapeComponents)
	{
		if (Component != nullptr)
		{
			InvalidTextures.Append(Component->RepairInvalidTextures());
		}
	}

	if (!InvalidTextures.IsEmpty())
	{
		FMessageLog("MapCheck").Error()
			->AddToken(FUObjectToken::Create(this, FText::FromString(GetActorNameOrLabel())))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT("InvalidTexturesMessage", "Invalid data detected on landscape {0} {0}|plural(one = texture, other = textures). The data has been cleared to avoid fatal error."), InvalidTextures.Num())))
			->AddToken(FMapErrorToken::Create(FMapErrors::FixedUpDeletedLayerWeightmap));
	}
}

bool IsValidLandscapeTextureSourceData(const UTexture& InTexture)
{
	FIntPoint SourceDataSize = InTexture.Source.GetLogicalSize();
	return ((SourceDataSize.X * SourceDataSize.Y) > 0) == InTexture.Source.HasPayloadData();
}

TArray<UTexture*> ULandscapeComponent::RepairInvalidTextures()
{
	TArray<UTexture*> AllTextures = GetGeneratedTextures();

	TArray<UTexture*> InvalidTextures;
	for (UTexture* Texture : AllTextures)
	{
		Texture->ConditionalPostLoad();
		if (!IsValidLandscapeTextureSourceData(*Texture))
		{
			UE_LOGF(LogLandscape, Error, "Invalid data found in texture %ls from landscape component %ls : clearing data.", *Texture->GetName(), *GetPathName());
			const bool bClear = true;
			CreateEmptyTextureMips(CastChecked<UTexture2D>(Texture), ELandscapeTextureUsage::Unknown, ELandscapeTextureType::Unknown, bClear);
			Texture->PostEditChange();
			InvalidTextures.Add(Texture);
		}
	}

	return InvalidTextures;
}

void ULandscapeComponent::FixupWeightmaps()
{
	// Fixup weightmaps in the base/render layer
	FixupWeightmaps(FGuid());

	// Also fixup all edit layers weightmaps
	ForEachLayer([&](const FGuid& LayerGuid, FLandscapeLayerComponentData& LayerData)
	{
		FixupWeightmaps(LayerGuid);
	});
}

void ULandscapeComponent::FixupWeightmaps(const FGuid& InEditLayerGuid)
{
	if (GIsEditor && !HasAnyFlags(RF_ClassDefaultObject))
	{
		ULandscapeInfo* Info = GetLandscapeInfo();
		ALandscapeProxy* Proxy = GetLandscapeProxy();
		ALandscape* Landscape = Proxy->GetLandscapeActor();

		TArray<TObjectPtr<UTexture2D>>& LocalWeightmapTextures = GetWeightmapTextures(InEditLayerGuid);
		TArray<TObjectPtr<ULandscapeWeightmapUsage>>& LocalWeightmapTextureUsages = GetWeightmapTexturesUsage(InEditLayerGuid);
		TArray<FWeightmapLayerAllocationInfo>& LocalWeightmapLayerAllocations = GetWeightmapLayerAllocations(InEditLayerGuid);

		if (Info)
		{
			// It's very important that the Proxies be registered with LandscapeInfo before calling FixupWeightmaps
			// Otherwise the code below may think ALL layers have been removed, and delete all of the corresponding weightmaps... which would be bad.
			check(Info->IsRegistered(Proxy));

			// We're going to re-build the texture usage array for this layer on this component
			LocalWeightmapTextureUsages.Empty();
			LocalWeightmapTextureUsages.AddDefaulted(LocalWeightmapTextures.Num());

			// There might be target layers that are referenced in the landscape component that are not present anymore in the parent landscape: we will delete them (with a log, to let the user know):
			TArray<ULandscapeLayerInfoObject*> LayersToDelete;
			// There might be target layers that are referenced in the landscape component but the parent landscape has another LayerInfo associated with that name... 
			//  those, we can salvage too, we simply need to point the component's layer infos to the landscape's:
			TMap<ULandscapeLayerInfoObject*, ULandscapeLayerInfoObject*> LayersToRemap; // key == old layer info, value == new layer info

			// make sure the weightmap textures are fully loaded or deleting layers from them will crash! :)
			for (UTexture* WeightmapTexture : LocalWeightmapTextures)
			{
				WeightmapTexture->ConditionalPostLoad();
			}

			auto MarkProxyDirty = [this, Info, Landscape]()
				{
					// Force resave the proxy through the modified landscape system, so that the user can then use the Build > Save Modified Landscapes (or Build > Build Landscape) button and therefore manually trigger the re-save of all modified proxies.
					if (Landscape != nullptr)
					{
						Info->MarkObjectDirty(/*InObject = */this, /*bInForceResave = */true, Landscape);
					}
				};

			// LayerInfo Validation check...
			for (const FWeightmapLayerAllocationInfo& Allocation : LocalWeightmapLayerAllocations)
			{
				if (!Allocation.LayerInfo)
				{
					LayersToDelete.Add(Allocation.LayerInfo);
				}
				else if(Allocation.LayerInfo != ALandscapeProxy::VisibilityLayer && Landscape && !Landscape->HasTargetLayer(Allocation.LayerInfo))
				{
					if (Landscape->HasTargetLayer(Allocation.LayerInfo->GetLayerName()))
					{
						if (ULandscapeLayerInfoObject* NewLayerInfo = Landscape->GetTargetLayers()[Allocation.LayerInfo->GetLayerName()].LayerInfoObj)
						{
							// Point the old layer info to the new one : 
							LayersToRemap.Add(Allocation.LayerInfo, NewLayerInfo);
						}
						else
						{
							// No way to retrieve a layer info by that name, consider it a deleted layer : 
							LayersToDelete.Add(Allocation.LayerInfo);
						}
					}
					else
					{
						// No way to retrieve a layer info by that name, consider it a deleted layer : 
						LayersToDelete.Add(Allocation.LayerInfo);
					}
				}
			}
			
			auto GetLayersText = [](int32 NumLayers) { return FText::Format(LOCTEXT("Layers", "{0}|plural(one = layer, other = layers)"), NumLayers); };
			auto GetForEditLayerText = [&InEditLayerGuid, this]() { return InEditLayerGuid.IsValid() ? FText::Format(LOCTEXT("ForEditLayer", " for edit layer {0}"), FText::FromName(LayersData[InEditLayerGuid].DebugName)) : FText::GetEmpty(); };

			if (!LayersToDelete.IsEmpty())
			{
				TArray<FString> LayerStrings;
				Algo::Transform(LayersToDelete, LayerStrings, [](ULandscapeLayerInfoObject* LayerInfo) { return LayerInfo ? LayerInfo->GetLayerName().ToString() : FString("null"); });
				FString LayersToDeleteString = *FString::Join(LayerStrings, TEXT(", "));

				// Report the deletion info to MapCheck (no need for a warning as the fix is automatic and trivial) :
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("Layers"), GetLayersText(LayersToDelete.Num()));
				Arguments.Add(TEXT("TargetLayerNames"), FText::FromString(LayersToDeleteString));
				Arguments.Add(TEXT("ForEditLayer"), GetForEditLayerText());
				FMessageLog("MapCheck").Info()
					->AddToken(FUObjectToken::Create(Proxy, FText::FromString(Proxy->GetActorNameOrLabel())))
					->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_FixedRemoveUnusedLayerWeightmap", "Removed unused layer weightmap{ForEditLayer} : "
					"Target {Layers}: \"{TargetLayerNames}\""), Arguments)))
					->AddToken(FMapErrorToken::Create(FMapErrors::FixedUpDeletedLayerWeightmap));

				// Perform the fixup proper : 
				MarkProxyDirty();

				// Delete material layer in the base/render layer
				{
					FGuid BaseLayerGuid;
					FLandscapeEditDataInterface LandscapeEdit(Info);
					for (int32 Idx = 0; Idx < LayersToDelete.Num(); ++Idx)
					{
						DeleteLayerInternal(LayersToDelete[Idx], LandscapeEdit, BaseLayerGuid);
					}
				}

				// For each edit layer that exists on the local component, delete material layer
				ForEachLayer([&](const FGuid& LayerGuid, FLandscapeLayerComponentData& LayerData)
				{
					FLandscapeEditDataInterface LandscapeEdit(Info);
					for (int32 Idx = 0; Idx < LayersToDelete.Num(); ++Idx)
					{
						DeleteLayerInternal(LayersToDelete[Idx], LandscapeEdit, LayerGuid);
					}
				});
								
			}

			if (!LayersToRemap.IsEmpty())
			{
				TArray<FString> LayerStrings;
				Algo::Transform(LayersToRemap, LayerStrings, [](const TPair<ULandscapeLayerInfoObject*, ULandscapeLayerInfoObject*> &LayerToRemap)	
					{ 
						ULandscapeLayerInfoObject* OldLayerInfo = LayerToRemap.Key;
						ULandscapeLayerInfoObject* NewLayerInfo = LayerToRemap.Value;
						return FString::Format(TEXT("{0} ({1} -> {2})"), {OldLayerInfo->GetLayerName().ToString(), OldLayerInfo->GetFullName(), NewLayerInfo->GetFullName()});
					});
				FString LayersToRemapString = *FString::Join(LayerStrings, TEXT(", "));

				// Report the remapping info to MapCheck (no need for a warning as the fix is automatic and trivial) :
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("Layers"), GetLayersText(LayersToRemap.Num()));
				Arguments.Add(TEXT("TargetLayerNames"), FText::FromString(LayersToRemapString));
				Arguments.Add(TEXT("ForEditLayer"), GetForEditLayerText());
				FMessageLog("MapCheck").Info()
					->AddToken(FUObjectToken::Create(Proxy, FText::FromString(Proxy->GetActorNameOrLabel())))
					->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_FixedUpDeletedLayerWeightmap", "Fixed up layer info{ForEditLayer} : "
						"Target {Layers}: \"{TargetLayerNames}\""), Arguments)))
					->AddToken(FMapErrorToken::Create(FMapErrors::FixedUpNonMatchingTargetLayer));

				// Perform the fixup proper : 
				MarkProxyDirty();
				for (FWeightmapLayerAllocationInfo& Allocation : LocalWeightmapLayerAllocations)
				{
					if (ULandscapeLayerInfoObject** NewLayerInfo = LayersToRemap.Find(Allocation.LayerInfo))
					{
						check(Allocation.GetLayerName() == (*NewLayerInfo)->GetLayerName());
						Allocation.LayerInfo = *NewLayerInfo;
					}
				}
			}

			bool bFixedWeightmapTextureIndex = false;

			// Store the weightmap allocations in WeightmapUsageMap
			for (int32 LayerIdx = 0; LayerIdx < LocalWeightmapLayerAllocations.Num();)
			{
				FWeightmapLayerAllocationInfo& Allocation = LocalWeightmapLayerAllocations[LayerIdx];
				if (!Allocation.IsAllocated())
				{
					MarkProxyDirty();
					LocalWeightmapLayerAllocations.RemoveAt(LayerIdx);
					continue;
				}

				// Fix up any problems caused by the layer deletion bug.
				if (Allocation.WeightmapTextureIndex >= LocalWeightmapTextures.Num())
				{
					MarkProxyDirty();
					Allocation.WeightmapTextureIndex = static_cast<uint8>(LocalWeightmapTextures.Num() - 1);
					if (!bFixedWeightmapTextureIndex)
					{
						FMessageLog("MapCheck").Warning()
							->AddToken(FUObjectToken::Create(Proxy, FText::FromString(Proxy->GetActorNameOrLabel())))
							->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_FixedUpIncorrectLayerWeightmap", "Fixed up incorrect layer weightmap texture index")))
							->AddToken(FMapErrorToken::Create(FMapErrors::FixedUpIncorrectLayerWeightmap));
					}
					bFixedWeightmapTextureIndex = true;
				}

				UTexture2D* WeightmapTexture = LocalWeightmapTextures[Allocation.WeightmapTextureIndex];

				TObjectPtr<ULandscapeWeightmapUsage>* TempUsage = Proxy->WeightmapUsageMap.Find(WeightmapTexture);

				if (TempUsage == nullptr)
				{
					TempUsage = &Proxy->WeightmapUsageMap.Add(WeightmapTexture, GetLandscapeProxy()->CreateWeightmapUsage());
					(*TempUsage)->LayerGuid = InEditLayerGuid;
				}

				ULandscapeWeightmapUsage* Usage = *TempUsage;
				check(Usage->LayerGuid == InEditLayerGuid); // A single weightmap must belong to exactly one layer

				LocalWeightmapTextureUsages[Allocation.WeightmapTextureIndex] = Usage; // Keep a ref to it for faster access

				// Detect a shared layer allocation, caused by a previous undo or layer deletion bugs
				if (Usage->ChannelUsage[Allocation.WeightmapTextureChannel] != nullptr &&
					Usage->ChannelUsage[Allocation.WeightmapTextureChannel] != this)
				{
					FFormatNamedArguments Arguments;
					Arguments.Add(TEXT("LayerName"), FText::FromString(Allocation.GetLayerName().ToString()));
					Arguments.Add(TEXT("ChannelName"), FText::FromString(Usage->ChannelUsage[Allocation.WeightmapTextureChannel]->GetName()));
					FMessageLog("MapCheck").Warning()
						->AddToken(FUObjectToken::Create(Proxy, FText::FromString(Proxy->GetActorNameOrLabel())))
						->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_FixedUpSharedLayerWeightmap", "Fixed up shared weightmap texture for layer {LayerName} in landscape component (shares with '{ChannelName}')"), Arguments)))
						->AddToken(FMapErrorToken::Create(FMapErrors::FixedUpSharedLayerWeightmap));
					LocalWeightmapLayerAllocations.RemoveAt(LayerIdx);
					continue;
				}
				else
				{
					Usage->ChannelUsage[Allocation.WeightmapTextureChannel] = this;
				}
				++LayerIdx;
			}

			RemoveInvalidWeightmaps(InEditLayerGuid);
		}
	}
}

void ULandscapeComponent::UpdateLayerAllowListFromPaintedLayers()
{
	const TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = GetWeightmapLayerAllocations();

	for (const FWeightmapLayerAllocationInfo& Allocation : ComponentWeightmapLayerAllocations)
	{
		LayerAllowList.AddUnique(Allocation.LayerInfo);
	}
}

//
// LandscapeComponentAlphaInfo
//
struct FLandscapeComponentAlphaInfo
{
	int32 LayerIndex;
	TArray<uint8> AlphaValues;

	// tor
	FLandscapeComponentAlphaInfo(ULandscapeComponent* InOwner, int32 InLayerIndex)
		: LayerIndex(InLayerIndex)
	{
		AlphaValues.Empty(FMath::Square(InOwner->ComponentSizeQuads + 1));
		AlphaValues.AddZeroed(FMath::Square(InOwner->ComponentSizeQuads + 1));
	}

	bool IsLayerAllZero() const
	{
		for (int32 Index = 0; Index < AlphaValues.Num(); Index++)
		{
			if (AlphaValues[Index] != 0)
			{
				return false;
			}
		}
		return true;
	}
};

struct FCollisionSize
{
public:
	static FCollisionSize CreateSimple(bool bUseSimpleCollision, int32 InNumSubSections, int32 InSubsectionSizeQuads, int32 InMipLevel)
	{
		return bUseSimpleCollision ? Create(InNumSubSections, InSubsectionSizeQuads, InMipLevel) : FCollisionSize();
	}

	static FCollisionSize Create(int32 InNumSubSections, int32 InSubsectionSizeQuads, int32 InMipLevel)
	{
		return FCollisionSize(InNumSubSections, InSubsectionSizeQuads, InMipLevel);
	}

	FCollisionSize(FCollisionSize&& Other) = default;
	FCollisionSize& operator=(FCollisionSize&& Other) = default;
private:
	FCollisionSize(int32 InNumSubsections, int32 InSubsectionSizeQuads, int32 InMipLevel)
	{
		SubsectionSizeVerts = ((InSubsectionSizeQuads + 1) >> InMipLevel);
		SubsectionSizeQuads = SubsectionSizeVerts - 1;
		SizeVerts = InNumSubsections * SubsectionSizeQuads + 1;
		SizeVertsSquare = FMath::Square(SizeVerts);
	}

	FCollisionSize()
	{
	}

public:
	int32 SubsectionSizeVerts = 0;
	int32 SubsectionSizeQuads = 0;
	int32 SizeVerts = 0;
	int32 SizeVertsSquare = 0;
};

void ULandscapeComponent::UpdateDirtyCollisionHeightData(FIntRect Region)
{
	// Take first value as is
	if (LayerDirtyCollisionHeightData.IsEmpty())
	{
		LayerDirtyCollisionHeightData = Region;
	}
	else
	{
		// Merge min/max region
		LayerDirtyCollisionHeightData.Include(Region.Min);
		LayerDirtyCollisionHeightData.Include(Region.Max);
	}
}

void ULandscapeComponent::ClearDirtyCollisionHeightData()
{
	LayerDirtyCollisionHeightData = FIntRect();
}

void ULandscapeComponent::UpdateCollisionHeightData(const FColor* const HeightmapTextureMipData, const FColor* const SimpleCollisionHeightmapTextureData, int32 ComponentX1/*=0*/, int32 ComponentY1/*=0*/, int32 ComponentX2/*=MAX_int32*/, int32 ComponentY2/*=MAX_int32*/, bool bUpdateBounds/*=false*/, bool bInUpdateHeightfieldRegion/*=true*/)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeComponent::UpdateCollisionHeightData);
	
	ALandscapeProxy* Proxy = GetLandscapeProxy();
	FIntPoint ComponentKey = GetSectionBase() / ComponentSizeQuads;
	ULandscapeHeightfieldCollisionComponent* CollisionComp = GetCollisionComponent();
	ULandscapeHeightfieldCollisionComponent* OldCollisionComponent = CollisionComp;

	const bool bUsingSimpleCollision = (SimpleCollisionMipLevel > CollisionMipLevel && SimpleCollisionHeightmapTextureData);
	
	FCollisionSize CollisionSize = FCollisionSize::Create(NumSubsections, SubsectionSizeQuads, CollisionMipLevel);
	FCollisionSize SimpleCollisionSize = FCollisionSize::CreateSimple(bUsingSimpleCollision, NumSubsections, SubsectionSizeQuads, SimpleCollisionMipLevel);
		
	const int32 TotalCollisionSize = CollisionSize.SizeVertsSquare + SimpleCollisionSize.SizeVertsSquare;

	uint16* CollisionHeightData = nullptr;
	bool CreatedNew = false;
	bool ChangeType = false;

	// In Edit Layers the Collision Component gets regenerated after the heightmap changes are undone and doesn't need to be transacted, only update dirtied collision height data
	if (bInUpdateHeightfieldRegion && ComponentX1 == 0 && ComponentY1 == 0 && ComponentX2 == MAX_int32 && ComponentY2 == MAX_int32 && !LayerDirtyCollisionHeightData.IsEmpty())
	{
		ComponentX1 = LayerDirtyCollisionHeightData.Min.X;
		ComponentY1 = LayerDirtyCollisionHeightData.Min.Y;
		ComponentX2 = LayerDirtyCollisionHeightData.Max.X;
		ComponentY2 = LayerDirtyCollisionHeightData.Max.Y;
	}
	ClearDirtyCollisionHeightData();

	if (CollisionComp)
	{
		ComponentX1 = FMath::Clamp(ComponentX1, 0, ComponentSizeQuads);
		ComponentY1 = FMath::Clamp(ComponentY1, 0, ComponentSizeQuads);
		ComponentX2 = FMath::Clamp(ComponentX2, 0, ComponentSizeQuads);
		ComponentY2 = FMath::Clamp(ComponentY2, 0, ComponentSizeQuads);

		if (ComponentX2 < ComponentX1 || ComponentY2 < ComponentY1)
		{
			// nothing to do
			return;
		}

		if (bUpdateBounds)
		{
			CollisionComp->CachedLocalBox = CachedLocalBox;
			CollisionComp->UpdateComponentToWorld();
		}
	}
	else
	{
		CreatedNew = true;
		ChangeType = CollisionComp != nullptr;
		ComponentX1 = 0;
		ComponentY1 = 0;
		ComponentX2 = ComponentSizeQuads;
		ComponentY2 = ComponentSizeQuads;

		RecreateCollisionComponent(bUsingSimpleCollision);
		CollisionComp = GetCollisionComponent();
	}

	CollisionHeightData = (uint16*)CollisionComp->CollisionHeightData.Lock(LOCK_READ_WRITE);

	const int32 HeightmapSizeU = GetHeightmap()->Source.GetSizeX();
	const int32 HeightmapSizeV = GetHeightmap()->Source.GetSizeY();

	// Handle Material WPO baked into heightfield collision
	const bool bUsingGrassMapHeights = Proxy->bBakeMaterialPositionOffsetIntoCollision && GrassData->HasData() && !IsGrassMapOutdated();
	uint16* GrassHeights = nullptr;
	if (bUsingGrassMapHeights)
	{
		if (CollisionMipLevel == 0)
		{
			GrassHeights = GrassData->GetHeightData().GetData();
		}
		else
		{
			if (GrassData->HeightMipData.Contains(CollisionMipLevel))
			{
				GrassHeights = GrassData->HeightMipData[CollisionMipLevel].GetData();
			}
		}
	}

	UpdateCollisionHeightBuffer(ComponentX1, ComponentY1, ComponentX2, ComponentY2, CollisionMipLevel, HeightmapSizeU, HeightmapSizeV, HeightmapTextureMipData, CollisionHeightData, GrassHeights);
		
	if (bUsingSimpleCollision)
	{
		uint16* SimpleCollisionGrassHeights = bUsingGrassMapHeights && GrassData->HeightMipData.Contains(SimpleCollisionMipLevel) ? GrassData->HeightMipData[SimpleCollisionMipLevel].GetData() : nullptr;
		uint16* const SimpleCollisionHeightData = CollisionHeightData + CollisionSize.SizeVertsSquare;
		UpdateCollisionHeightBuffer(ComponentX1, ComponentY1, ComponentX2, ComponentY2, SimpleCollisionMipLevel, HeightmapSizeU, HeightmapSizeV, SimpleCollisionHeightmapTextureData, SimpleCollisionHeightData, SimpleCollisionGrassHeights);
	}

	CollisionComp->CollisionHeightData.Unlock();

	// If we updated an existing component, we need to update the phys x heightfield edit data
	if (!CreatedNew && bInUpdateHeightfieldRegion)
	{
		if (CollisionMipLevel == 0)
		{
			CollisionComp->UpdateHeightfieldRegion(ComponentX1, ComponentY1, ComponentX2, ComponentY2);
		}
		else
		{
			// Ratio to convert update region coordinate to collision mip coordinates
			const float CollisionQuadRatio = (float)CollisionSize.SubsectionSizeQuads / (float)SubsectionSizeQuads;
			const int32 CollisionCompX1 = FMath::FloorToInt((float)ComponentX1 * CollisionQuadRatio);
			const int32 CollisionCompY1 = FMath::FloorToInt((float)ComponentY1 * CollisionQuadRatio);
			const int32 CollisionCompX2 = FMath::CeilToInt( (float)ComponentX2 * CollisionQuadRatio);
			const int32 CollisionCompY2 = FMath::CeilToInt( (float)ComponentY2 * CollisionQuadRatio);
			CollisionComp->UpdateHeightfieldRegion(CollisionCompX1, CollisionCompY1, CollisionCompX2, CollisionCompY2);
		}
	}

	{
		// set relevancy for navigation system
		ALandscapeProxy* LandscapeProxy = CollisionComp->GetLandscapeProxy();
		CollisionComp->SetCanEverAffectNavigation(LandscapeProxy ? LandscapeProxy->bUsedForNavigation : false);
	}

	// Move any foliage instances if we created a new collision component.
	if (OldCollisionComponent && OldCollisionComponent != CollisionComp)
	{
		AInstancedFoliageActor::MoveInstancesToNewComponent(Proxy->GetWorld(), OldCollisionComponent, CollisionComp);
	}
	
	if (CreatedNew && !ChangeType)
	{
		UpdateCollisionLayerData();
	}

	if (CreatedNew && Proxy->GetRootComponent()->IsRegistered())
	{
		CollisionComp->RegisterComponent();
	}

	// Debug display needs to update its representation, so we invalidate the collision component's render state : 
	CollisionComp->MarkRenderStateDirty();

	// Invalidate rendered physical materials
	// These are updated in UpdatePhysicalMaterialTasks()
	InvalidatePhysicalMaterial();
}

void ULandscapeComponent::DestroyCollisionData()
{
	ULandscapeHeightfieldCollisionComponent* CollisionComp = GetCollisionComponent();
	
	if (CollisionComp)
	{
		CollisionComp->DestroyComponent();
		SetCollisionComponent(nullptr);
	}
}

void ULandscapeComponent::UpdateCollisionData(bool bInUpdateHeightfieldRegion)
{
	TArray64<uint8> CollisionMipData;
	TArray64<uint8> SimpleCollisionMipData;

	check( GetHeightmap()->Source.IsValid() );

	verify( GetHeightmap()->Source.GetMipData(CollisionMipData, CollisionMipLevel) );
	if (SimpleCollisionMipLevel > CollisionMipLevel)
	{
		verify( GetHeightmap()->Source.GetMipData(SimpleCollisionMipData, SimpleCollisionMipLevel) );
	}

	UpdateCollisionHeightData(
		(FColor*)CollisionMipData.GetData(),
		SimpleCollisionMipLevel > CollisionMipLevel ? (FColor*)SimpleCollisionMipData.GetData() : nullptr,
		0, 0, MAX_int32, MAX_int32, true, bInUpdateHeightfieldRegion);
}

void ULandscapeComponent::RecreateCollisionComponent(bool bUseSimpleCollision)
{
	ULandscapeHeightfieldCollisionComponent* CollisionComp = GetCollisionComponent();
	TArray<uint8> DominantLayerData;
	TArray<ULandscapeLayerInfoObject*> LayerInfos;
	ALandscapeProxy* Proxy = GetLandscapeProxy();
	const FCollisionSize CollisionSize = FCollisionSize::Create(NumSubsections, SubsectionSizeQuads, CollisionMipLevel);
	const FCollisionSize SimpleCollisionSize = FCollisionSize::CreateSimple(bUseSimpleCollision, NumSubsections, SubsectionSizeQuads, SimpleCollisionMipLevel);
	const int32 TotalCollisionSize = CollisionSize.SizeVertsSquare + SimpleCollisionSize.SizeVertsSquare;

	if (CollisionComp) // remove old component before changing to other type collision...
	{
		if (CollisionComp->DominantLayerData.GetElementCount())
		{
			check(CollisionComp->DominantLayerData.GetElementCount() >= TotalCollisionSize);
			DominantLayerData.AddUninitialized(TotalCollisionSize);

			const uint8* SrcDominantLayerData = (uint8*)CollisionComp->DominantLayerData.Lock(LOCK_READ_ONLY);
			FMemory::Memcpy(DominantLayerData.GetData(), SrcDominantLayerData, TotalCollisionSize * CollisionComp->DominantLayerData.GetElementSize());
			CollisionComp->DominantLayerData.Unlock();
		}

		if (CollisionComp->ComponentLayerInfos.Num())
		{
			LayerInfos = CollisionComp->ComponentLayerInfos;
		}

		Proxy->Modify();
		CollisionComp->DestroyComponent();
		CollisionComp = nullptr;
	}

	CollisionComp = NewObject<ULandscapeHeightfieldCollisionComponent>(Proxy, NAME_None, RF_Transactional);

	CollisionComp->SetRelativeLocation(GetRelativeLocation());
	CollisionComp->SetupAttachment(Proxy->GetRootComponent(), NAME_None);
	Proxy->CollisionComponents.Add(CollisionComp);

	CollisionComp->SetRenderComponent(this);
	CollisionComp->SetSectionBase(GetSectionBase());
	CollisionComp->CollisionSizeQuads = CollisionSize.SubsectionSizeQuads * NumSubsections;
	CollisionComp->CollisionScale = (float)(ComponentSizeQuads) / (float)(CollisionComp->CollisionSizeQuads);
	CollisionComp->SimpleCollisionSizeQuads = SimpleCollisionSize.SubsectionSizeQuads * NumSubsections;
	CollisionComp->CachedLocalBox = CachedLocalBox;
	CollisionComp->SetGenerateOverlapEvents(Proxy->bGenerateOverlapEvents);

	// Reallocate raw collision data
	CollisionComp->CollisionHeightData.Lock(LOCK_READ_WRITE);
	uint16* CollisionHeightData = (uint16*)CollisionComp->CollisionHeightData.Realloc(TotalCollisionSize);
	FMemory::Memzero(CollisionHeightData, TotalCollisionSize * CollisionComp->CollisionHeightData.GetElementSize());
	CollisionComp->CollisionHeightData.Unlock();

	if (DominantLayerData.Num())
	{
		CollisionComp->DominantLayerData.Lock(LOCK_READ_WRITE);
		uint8* DestDominantLayerData = (uint8*)CollisionComp->DominantLayerData.Realloc(TotalCollisionSize);
		FMemory::Memcpy(DestDominantLayerData, DominantLayerData.GetData(), TotalCollisionSize * CollisionComp->DominantLayerData.GetElementSize());
		CollisionComp->DominantLayerData.Unlock();
	}

	if (LayerInfos.Num())
	{
		CollisionComp->ComponentLayerInfos = MoveTemp(LayerInfos);
	}
	SetCollisionComponent(CollisionComp);
}

void ULandscapeComponent::UpdateCollisionHeightBuffer(	int32 InComponentX1, int32 InComponentY1, int32 InComponentX2, int32 InComponentY2, int32 InCollisionMipLevel, int32 InHeightmapSizeU, int32 InHeightmapSizeV,
														const FColor* const InHeightmapTextureMipData, uint16* OutCollisionHeightData, uint16* InGrassHeightData)
{
	FCollisionSize CollisionSize = FCollisionSize::Create(NumSubsections, SubsectionSizeQuads, InCollisionMipLevel);

	// Ratio to convert update region coordinate to collision mip coordinates
	const float CollisionQuadRatio = (float)CollisionSize.SubsectionSizeQuads / (float)SubsectionSizeQuads;

	const int32 SubSectionX1 = FMath::Max(0, FMath::DivideAndRoundDown(InComponentX1 - 1, SubsectionSizeQuads));
	const int32 SubSectionY1 = FMath::Max(0, FMath::DivideAndRoundDown(InComponentY1 - 1, SubsectionSizeQuads));
	const int32 SubSectionX2 = FMath::Min(FMath::DivideAndRoundUp(InComponentX2 + 1, SubsectionSizeQuads), NumSubsections);
	const int32 SubSectionY2 = FMath::Min(FMath::DivideAndRoundUp(InComponentY2 + 1, SubsectionSizeQuads), NumSubsections);

	const int32 MipSizeU = InHeightmapSizeU >> InCollisionMipLevel;
	const int32 MipSizeV = InHeightmapSizeV >> InCollisionMipLevel;

	const int32 HeightmapOffsetX = FMath::RoundToInt32(HeightmapScaleBias.Z * (float)InHeightmapSizeU) >> InCollisionMipLevel;
	const int32 HeightmapOffsetY = FMath::RoundToInt32(HeightmapScaleBias.W * (float)InHeightmapSizeV) >> InCollisionMipLevel;

	for (int32 SubsectionY = SubSectionY1; SubsectionY < SubSectionY2; ++SubsectionY)
	{
		for (int32 SubsectionX = SubSectionX1; SubsectionX < SubSectionX2; ++SubsectionX)
		{
			// Area to update in subsection coordinates
			const int32 SubX1 = InComponentX1 - SubsectionSizeQuads * SubsectionX;
			const int32 SubY1 = InComponentY1 - SubsectionSizeQuads * SubsectionY;
			const int32 SubX2 = InComponentX2 - SubsectionSizeQuads * SubsectionX;
			const int32 SubY2 = InComponentY2 - SubsectionSizeQuads * SubsectionY;

			// Area to update in collision mip level coords
			const int32 CollisionSubX1 = FMath::FloorToInt((float)SubX1 * CollisionQuadRatio);
			const int32 CollisionSubY1 = FMath::FloorToInt((float)SubY1 * CollisionQuadRatio);
			const int32 CollisionSubX2 = FMath::CeilToInt((float)SubX2 * CollisionQuadRatio);
			const int32 CollisionSubY2 = FMath::CeilToInt((float)SubY2 * CollisionQuadRatio);

			// Clamp area to update
			const int32 VertX1 = FMath::Clamp<int32>(CollisionSubX1, 0, CollisionSize.SubsectionSizeQuads);
			const int32 VertY1 = FMath::Clamp<int32>(CollisionSubY1, 0, CollisionSize.SubsectionSizeQuads);
			const int32 VertX2 = FMath::Clamp<int32>(CollisionSubX2, 0, CollisionSize.SubsectionSizeQuads);
			const int32 VertY2 = FMath::Clamp<int32>(CollisionSubY2, 0, CollisionSize.SubsectionSizeQuads);

			for (int32 VertY = VertY1; VertY <= VertY2; VertY++)
			{
				for (int32 VertX = VertX1; VertX <= VertX2; VertX++)
				{
					// this uses Quads as we don't want the duplicated vertices
					const int32 CompVertX = CollisionSize.SubsectionSizeQuads * SubsectionX + VertX;
					const int32 CompVertY = CollisionSize.SubsectionSizeQuads * SubsectionY + VertY;

					if (InGrassHeightData)
					{
						uint16& CollisionHeight = OutCollisionHeightData[CompVertX + CompVertY * CollisionSize.SizeVerts];
						const uint16& NewHeight = InGrassHeightData[CompVertX + CompVertY * CollisionSize.SizeVerts];
						CollisionHeight = NewHeight;
					}
					else
					{
						// X/Y of the vertex we're looking indexed into the texture data
						const int32 TexX = HeightmapOffsetX + CollisionSize.SubsectionSizeVerts * SubsectionX + VertX;
						const int32 TexY = HeightmapOffsetY + CollisionSize.SubsectionSizeVerts * SubsectionY + VertY;
						const FColor& TexData = InHeightmapTextureMipData[TexX + TexY * MipSizeU];

						// Copy collision data
						uint16& CollisionHeight = OutCollisionHeightData[CompVertX + CompVertY * CollisionSize.SizeVerts];
						const uint16 NewHeight = TexData.R << 8 | TexData.G;

						CollisionHeight = NewHeight;
					}
				}
			}
		}
	}
}

void ULandscapeComponent::UpdateDominantLayerBuffer(int32 InComponentX1, int32 InComponentY1, int32 InComponentX2, int32 InComponentY2, int32 InCollisionMipLevel, int32 InWeightmapSizeU, int32 InVisibilityLayerIdx, const TArray<uint8*>& InCollisionDataPtrs, const TArray<ULandscapeLayerInfoObject*>& InLayerInfos, uint8* OutDominantLayerData)
{
	const int32 MipSizeU = InWeightmapSizeU >> InCollisionMipLevel;

	FCollisionSize CollisionSize = FCollisionSize::Create(NumSubsections, SubsectionSizeQuads, InCollisionMipLevel);
	
	// Ratio to convert update region coordinate to collision mip coordinates
	const float CollisionQuadRatio = (float)CollisionSize.SubsectionSizeQuads / (float)SubsectionSizeQuads;

	const int32 SubSectionX1 = FMath::Max(0, FMath::DivideAndRoundDown(InComponentX1 - 1, SubsectionSizeQuads));
	const int32 SubSectionY1 = FMath::Max(0, FMath::DivideAndRoundDown(InComponentY1 - 1, SubsectionSizeQuads));
	const int32 SubSectionX2 = FMath::Min(FMath::DivideAndRoundUp(InComponentX2 + 1, SubsectionSizeQuads), NumSubsections);
	const int32 SubSectionY2 = FMath::Min(FMath::DivideAndRoundUp(InComponentY2 + 1, SubsectionSizeQuads), NumSubsections);
	for (int32 SubsectionY = SubSectionY1; SubsectionY < SubSectionY2; ++SubsectionY)
	{
		for (int32 SubsectionX = SubSectionX1; SubsectionX < SubSectionX2; ++SubsectionX)
		{
			// Area to update in subsection coordinates
			const int32 SubX1 = InComponentX1 - SubsectionSizeQuads * SubsectionX;
			const int32 SubY1 = InComponentY1 - SubsectionSizeQuads * SubsectionY;
			const int32 SubX2 = InComponentX2 - SubsectionSizeQuads * SubsectionX;
			const int32 SubY2 = InComponentY2 - SubsectionSizeQuads * SubsectionY;

			// Area to update in collision mip level coords
			const int32 CollisionSubX1 = FMath::FloorToInt((float)SubX1 * CollisionQuadRatio);
			const int32 CollisionSubY1 = FMath::FloorToInt((float)SubY1 * CollisionQuadRatio);
			const int32 CollisionSubX2 = FMath::CeilToInt((float)SubX2 * CollisionQuadRatio);
			const int32 CollisionSubY2 = FMath::CeilToInt((float)SubY2 * CollisionQuadRatio);

			// Clamp area to update
			const int32 VertX1 = FMath::Clamp<int32>(CollisionSubX1, 0, CollisionSize.SubsectionSizeQuads);
			const int32 VertY1 = FMath::Clamp<int32>(CollisionSubY1, 0, CollisionSize.SubsectionSizeQuads);
			const int32 VertX2 = FMath::Clamp<int32>(CollisionSubX2, 0, CollisionSize.SubsectionSizeQuads);
			const int32 VertY2 = FMath::Clamp<int32>(CollisionSubY2, 0, CollisionSize.SubsectionSizeQuads);

			for (int32 VertY = VertY1; VertY <= VertY2; VertY++)
			{
				for (int32 VertX = VertX1; VertX <= VertX2; VertX++)
				{
					// X/Y of the vertex we're looking indexed into the texture data
					const int32 TexX = CollisionSize.SubsectionSizeVerts * SubsectionX + VertX;
					const int32 TexY = CollisionSize.SubsectionSizeVerts * SubsectionY + VertY;
					const int32 DataOffset = (TexX + TexY * MipSizeU) * sizeof(FColor);

					uint8 DominantLayer = 255; // 255 as invalid value
					int32 DominantWeight = 0;
					const uint8 NumLayers = IntCastChecked<uint8>(InCollisionDataPtrs.Num());
					for (uint8 LayerIdx = 0; LayerIdx < NumLayers; LayerIdx++)
					{
						const uint8 LayerWeight = InCollisionDataPtrs[LayerIdx][DataOffset];
						const uint8 LayerMinimumWeight = InLayerInfos[LayerIdx] ? (uint8)(InLayerInfos[LayerIdx]->GetMinimumCollisionRelevanceWeight() * 255) :  0;

						if (LayerIdx == InVisibilityLayerIdx) // Override value for visibility layer holes
						{
							if (LayerWeight > 170) // 255 * 0.66...
							{
								DominantLayer = LayerIdx;
								DominantWeight = INT_MAX;
							}
						}
						else if (LayerWeight > DominantWeight && LayerWeight >= LayerMinimumWeight)
						{
							DominantLayer = LayerIdx;
							DominantWeight = LayerWeight;
						}
					}

					// this uses Quads as we don't want the duplicated vertices
					const int32 CompVertX = CollisionSize.SubsectionSizeQuads * SubsectionX + VertX;
					const int32 CompVertY = CollisionSize.SubsectionSizeQuads * SubsectionY + VertY;

					// Set collision data
					OutDominantLayerData[CompVertX + CompVertY * CollisionSize.SizeVerts] = DominantLayer;
				}
			}
		}
	}
}

void ULandscapeComponent::UpdateCollisionLayerData(const FColor* const* const WeightmapTextureMipData, const FColor* const* const SimpleCollisionWeightmapTextureMipData, int32 ComponentX1, int32 ComponentY1, int32 ComponentX2, int32 ComponentY2)
{
	ALandscapeProxy* Proxy = GetLandscapeProxy();
	FIntPoint ComponentKey = GetSectionBase() / ComponentSizeQuads;

	ULandscapeHeightfieldCollisionComponent* CollisionComp = GetCollisionComponent();

	if (CollisionComp)
	{
		const bool bUsingSimpleCollision = (SimpleCollisionMipLevel > CollisionMipLevel && SimpleCollisionWeightmapTextureMipData);

		TArray<ULandscapeLayerInfoObject*> CandidateLayers;
		TArray<uint8*> CandidateDataPtrs;
		TArray<uint8*> SimpleCollisionDataPtrs;

		bool bExistingLayerMismatch = false;
		int32 VisibilityLayerIdx = INDEX_NONE;

		const TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = GetWeightmapLayerAllocations(false);
		const TArray<UTexture2D*>& ComponentWeightmapsTexture = GetWeightmapTextures(false);

		// Find the layers we're interested in
		for (int32 AllocIdx = 0; AllocIdx < ComponentWeightmapLayerAllocations.Num(); AllocIdx++)
		{
			const FWeightmapLayerAllocationInfo& AllocInfo = ComponentWeightmapLayerAllocations[AllocIdx];
			ULandscapeLayerInfoObject* LayerInfo = AllocInfo.LayerInfo;
			if (LayerInfo != nullptr && AllocInfo.IsAllocated())
			{
				int32 Idx = CandidateLayers.Add(LayerInfo);
				CandidateDataPtrs.Add(((uint8*)WeightmapTextureMipData[AllocInfo.WeightmapTextureIndex]) + ChannelOffsets[AllocInfo.WeightmapTextureChannel]);

				if (bUsingSimpleCollision)
				{
					SimpleCollisionDataPtrs.Add(((uint8*)SimpleCollisionWeightmapTextureMipData[AllocInfo.WeightmapTextureIndex]) + ChannelOffsets[AllocInfo.WeightmapTextureChannel]);
				}

				// Check if we still match the collision component.
				if (!CollisionComp->ComponentLayerInfos.IsValidIndex(Idx) || CollisionComp->ComponentLayerInfos[Idx] != LayerInfo)
				{
					bExistingLayerMismatch = true;
				}

				if (LayerInfo == ALandscapeProxy::VisibilityLayer)
				{
					VisibilityLayerIdx = Idx;
					bExistingLayerMismatch = true; // always rebuild whole component for hole
				}
			}
		}

		if (CandidateLayers.Num() == 0)
		{
			// No layers, so don't update any weights
			CollisionComp->DominantLayerData.RemoveBulkData();
			CollisionComp->ComponentLayerInfos.Empty();
		}
		else
		{
			uint8* DominantLayerData = (uint8*)CollisionComp->DominantLayerData.Lock(LOCK_READ_WRITE);
			FCollisionSize CollisionSize = FCollisionSize::Create(NumSubsections, SubsectionSizeQuads, CollisionMipLevel);
			FCollisionSize SimpleCollisionSize = FCollisionSize::CreateSimple(bUsingSimpleCollision, NumSubsections, SubsectionSizeQuads, SimpleCollisionMipLevel);
		
				
			// If there's no existing data, or the layer allocations have changed, we need to update the data for the whole component.
			if (bExistingLayerMismatch || CollisionComp->DominantLayerData.GetElementCount() == 0)
			{
				ComponentX1 = 0;
				ComponentY1 = 0;
				ComponentX2 = ComponentSizeQuads;
				ComponentY2 = ComponentSizeQuads;
											
				const int32 TotalCollisionSize = CollisionSize.SizeVertsSquare + SimpleCollisionSize.SizeVertsSquare;

				
				DominantLayerData = (uint8*)CollisionComp->DominantLayerData.Realloc(TotalCollisionSize);
				FMemory::Memzero(DominantLayerData, TotalCollisionSize);
				CollisionComp->ComponentLayerInfos = CandidateLayers;
			}
			else
			{
				ComponentX1 = FMath::Clamp(ComponentX1, 0, ComponentSizeQuads);
				ComponentY1 = FMath::Clamp(ComponentY1, 0, ComponentSizeQuads);
				ComponentX2 = FMath::Clamp(ComponentX2, 0, ComponentSizeQuads);
				ComponentY2 = FMath::Clamp(ComponentY2, 0, ComponentSizeQuads);
			}

			const int32 WeightmapSizeU = ComponentWeightmapsTexture[0]->Source.GetSizeX();
						
			// gmartin: WeightmapScaleBias not handled?			
			UpdateDominantLayerBuffer(ComponentX1, ComponentY1, ComponentX2, ComponentY2, CollisionMipLevel, WeightmapSizeU, VisibilityLayerIdx, CandidateDataPtrs, CollisionComp->ComponentLayerInfos, DominantLayerData);

			if (bUsingSimpleCollision)
			{
				uint8* const SimpleCollisionHeightData = DominantLayerData + CollisionSize.SizeVertsSquare;
				UpdateDominantLayerBuffer(ComponentX1, ComponentY1, ComponentX2, ComponentY2, SimpleCollisionMipLevel, WeightmapSizeU, VisibilityLayerIdx, SimpleCollisionDataPtrs, CollisionComp->ComponentLayerInfos, SimpleCollisionHeightData);
			}

			CollisionComp->DominantLayerData.Unlock();
		}

		// Invalidate rendered physical materials
		// These are updated in UpdatePhysicalMaterialTasks()
		InvalidatePhysicalMaterial();

		// We do not force an update of the physics data here. We don't need the layer information in the editor and it
		// causes problems if we update it multiple times in a single frame.

		// Debug display needs to update its representation, so we invalidate the collision component's render state : 
		CollisionComp->MarkRenderStateDirty();
	}
}


void ULandscapeComponent::UpdateCollisionLayerData()
{
	const TArray<UTexture2D*>& ComponentWeightmapsTexture = GetWeightmapTextures();

	// Generate the dominant layer data
	TArray<TArray64<uint8>> WeightmapTextureMipData;
	TArray<FColor*> WeightmapTextureMipDataParam;
	WeightmapTextureMipData.Reserve(ComponentWeightmapsTexture.Num());
	WeightmapTextureMipDataParam.Reserve(ComponentWeightmapsTexture.Num());
	for (int32 WeightmapIdx = 0; WeightmapIdx < ComponentWeightmapsTexture.Num(); ++WeightmapIdx)
	{
		TArray64<uint8>& MipData = WeightmapTextureMipData.AddDefaulted_GetRef();
		verify( ComponentWeightmapsTexture[WeightmapIdx]->Source.GetMipData(MipData, CollisionMipLevel) );
		WeightmapTextureMipDataParam.Add((FColor*)MipData.GetData());
	}

	TArray<TArray64<uint8>> SimpleCollisionWeightmapMipData;
	TArray<FColor*> SimpleCollisionWeightmapMipDataParam;
	if (SimpleCollisionMipLevel > CollisionMipLevel)
	{
		SimpleCollisionWeightmapMipData.Reserve(ComponentWeightmapsTexture.Num());
		SimpleCollisionWeightmapMipDataParam.Reserve(ComponentWeightmapsTexture.Num());
		for (int32 WeightmapIdx = 0; WeightmapIdx < ComponentWeightmapsTexture.Num(); ++WeightmapIdx)
		{
			TArray64<uint8>& MipData = SimpleCollisionWeightmapMipData.AddDefaulted_GetRef();
			verify( ComponentWeightmapsTexture[WeightmapIdx]->Source.GetMipData(MipData, SimpleCollisionMipLevel) );
			SimpleCollisionWeightmapMipDataParam.Add((FColor*)MipData.GetData());
		}
	}

	UpdateCollisionLayerData(WeightmapTextureMipDataParam.GetData(), SimpleCollisionWeightmapMipDataParam.GetData());
}

uint32 ULandscapeComponent::CalculatePhysicalMaterialTaskHash() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeComponent::CalculatePhysicalMaterialTaskHash);
	uint32 Hash = 0;
	
	// Take into account any material changes.
	UMaterialInterface* Material = GetLandscapeMaterial();
	uint32 MaterialAllStateCRC = Material->ComputeAllStateCRC();

	// also take into account the collision mip level
	Hash = FCrc::TypeCrc32(CollisionMipLevel, Hash);

	int32 SimpleCollisionMipLevelForHash = (SimpleCollisionMipLevel > CollisionMipLevel) ? SimpleCollisionMipLevel : 0;
	Hash = FCrc::TypeCrc32(SimpleCollisionMipLevelForHash, Hash);

	// We could take into account heightmap and weightmap changes here by adding to the hash.
	// Instead we are resetting the stored hash in UpdateCollisionHeightData() and UpdateCollisionLayerData().
	// (this is not ideal, as if we forget to reset the stored hash somewhere it will not recalculate collision...)

	return MaterialAllStateCRC;
}

bool ULandscapeComponent::GetRenderPhysicalMaterials(TArray<UPhysicalMaterial*>& OutPhysicalMaterials) const 
{
	bool bReturnValue = false;
	OutPhysicalMaterials.Reset();

	if (UMaterialInterface* Material = GetLandscapeMaterial())
	{
		ERHIFeatureLevel::Type FeatureLevel = GMaxRHIFeatureLevel;
		{
			TArray<const UMaterialExpressionLandscapePhysicalMaterialOutput*> Expressions;
			Material->GetMaterial()->GetAllExpressionsOfType<UMaterialExpressionLandscapePhysicalMaterialOutput>(Expressions);
			if (Expressions.Num() > 0)
			{
				// Assume only one valid physical material output material node
				for (const FPhysicalMaterialInput& Input : Expressions[0]->Inputs)
				{
					OutPhysicalMaterials.Add(Input.PhysicalMaterial);
					bReturnValue |= (Input.PhysicalMaterial != nullptr);
				}
			}
		}
	}

	return bReturnValue;
}

void ULandscapeComponent::InvalidatePhysicalMaterial()
{
	PhysicalMaterialHash = 0;
	bPendingPhysicalMaterialInvalidation = true;
}

bool ULandscapeComponent::CanUpdatePhysicalMaterial()
{
	ERHIFeatureLevel::Type FeatureLevel = GetWorld() ? (ERHIFeatureLevel::Type)GetWorld()->GetFeatureLevel() : GMaxRHIFeatureLevel;
	if (FeatureLevel <= ERHIFeatureLevel::ES3_1)
	{
		// physical material update is not supported on ES3_1 level hardware
		return false;
	}

	if (SceneProxy == nullptr)
	{
		return false;
	}

	return GetCollisionComponent() != nullptr;
}

void ULandscapeComponent::FinalizePhysicalMaterial(bool bInImmediatePhysicsRebuild)
{
	if (!PhysicalMaterialTask.IsValid())
	{
		return;
	}

	if (!PhysicalMaterialTask.IsComplete())
	{
		return;
	}

	UpdateCollisionPhysicalMaterialData(PhysicalMaterialTask.GetResultMaterials(), PhysicalMaterialTask.GetResultIds());

	// bPendingPhysicalMaterialInvalidation is reset to false when the task was started.  If the physical material was
	// re-invalidated since then, don't set PhysicalMaterialHash.  PhysicalMaterialTask will be released and PhysicalMaterialHash
	// will remain mismatched (typically 0), so on the next tick a new task will be started.  It's important that we don't lose
	// track of these invalidations while the task is running.  We might only get notified of a change once.
	if (!bPendingPhysicalMaterialInvalidation)
	{
		PhysicalMaterialHash = PhysicalMaterialTask.GetHash();
	}

	PhysicalMaterialTask.Release();

	if (bInImmediatePhysicsRebuild)
	{
		ULandscapeHeightfieldCollisionComponent* CollisionComponent = GetCollisionComponent();
		check(CollisionComponent != nullptr);

		CollisionComponent->RecreateCollision();
	}
}

void ULandscapeComponent::UpdatePhysicalMaterialTasks()
{
	if (!CanUpdatePhysicalMaterial())
	{
		// Cancel any existing tasks we have.
		if (PhysicalMaterialTask.IsValid())
		{
			PhysicalMaterialTask.Release();
		}
		return;
	}

	// Check if we need to launch a new task to update the physical material.
	uint32 Hash = CalculatePhysicalMaterialTaskHash();
	ULandscapeHeightfieldCollisionComponent* CollisionComponent = GetCollisionComponent();
	check(CollisionComponent != nullptr);

	if (Hash == PhysicalMaterialHash)
	{
		if (PhysicalMaterialTask.IsValid())
		{
			PhysicalMaterialTask.Release();
		}
		return;
	}

	if (PhysicalMaterialTask.GetHash() != Hash)
	{
		// ULandscapeSubsystem can start ticking before the weightmaps are loaded.  They are "streamed in", but are still just the default placeholder texture.
		// Rendering the physical material in this state will silently produce the wrong result.
		bool bWeightMapsReady = false;

		// WP levels may have a deleted ALandscape actor resulting in a non-editable landscape (Invalid LandscapeInfo). UpdateLayersContent will not run leaving bForceMiplevelsToBeResident unset for the remaining landscape components
		const ULandscapeInfo* LandscapeInfo = GetLandscapeInfo();
		if (LandscapeInfo && LandscapeInfo->LandscapeActor.IsValid())
		{
			bWeightMapsReady = Algo::AllOf(WeightmapTextures, [](UTexture2D* Weightmap)
				{
					// We expect FLandscapeTextureStreamingManager to have already started managing the data lifetime of the weightmaps, via UpdateLayersContent.
					check(!Weightmap || Weightmap->bForceMiplevelsToBeResident);

					return Weightmap && !Weightmap->IsDefaultTexture() && !Weightmap->HasPendingInitOrStreaming() && Weightmap->IsFullyStreamedIn();
				});
		}

		TArray<UPhysicalMaterial*> PhysicalMaterials;
		if (!bWeightMapsReady)
		{
			//Do nothing.  Don't initialize the task or set PhysicalMaterialHash.  Wait for a future tick when the weightmaps are ready.
		}
		else if (GetRenderPhysicalMaterials(PhysicalMaterials))
		{
			bool bSuccess = PhysicalMaterialTask.Init(this, Hash);
			check(bSuccess && PhysicalMaterialTask.IsValid());

			bPendingPhysicalMaterialInvalidation = false;  //new task is starting, so consume the pending invalidation
		}
		else
		{
			PhysicalMaterialHash = Hash;
			// Clear the renderable physical material properties as we don't need them :
			CollisionComponent->PhysicalMaterialRenderObjects.Reset();
			CollisionComponent->PhysicalMaterialRenderData.RemoveBulkData();
			PhysicalMaterialTask.Release();
		}
	
	}

	// If we have a current task, update it
	if (PhysicalMaterialTask.IsValid())
	{
		if (PhysicalMaterialTask.IsComplete())
		{
			// Potentially, we do not force an update of the physics data here (behind a CVar, as we don't necessarily need the 
			//  information immediately in the editor and update will happen on cook or PIE) :
			FinalizePhysicalMaterial(CVarLandscapeApplyPhysicalMaterialChangesImmediately.GetValueOnGameThread() != 0);
		}
		else
		{
			PhysicalMaterialTask.Tick();
		}
	}
}

void ULandscapeComponent::UpdateCollisionPhysicalMaterialData(TArray<UPhysicalMaterial*> const& InPhysicalMaterials, TArray<uint8> const& InMaterialIds)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeComponent::UpdateCollisionPhysicalMaterialData);
	ULandscapeHeightfieldCollisionComponent* CollisionComponent = GetCollisionComponent();
	check(CollisionComponent != nullptr);

	// Copy the physical material array
	CollisionComponent->PhysicalMaterialRenderObjects = InPhysicalMaterials;

	// Copy the physical material IDs for both the full and (optional) simple collision.
	const int32 CollisionVerts = CollisionComponent->CollisionSizeQuads + 1;

	// the material ids passed in should already be at the desired collision resolution
	check(InMaterialIds.Num() == CollisionVerts * CollisionVerts);

	// if simple collision is used, this is how many verts it needs
	const int32 SimpleCollisionSizeVerts = CollisionComponent->SimpleCollisionSizeQuads > 0 ? CollisionComponent->SimpleCollisionSizeQuads + 1 : 0;
	const int32 BulkDataSize = CollisionVerts * CollisionVerts + SimpleCollisionSizeVerts * SimpleCollisionSizeVerts;

	void* Data = CollisionComponent->PhysicalMaterialRenderData.Lock(LOCK_READ_WRITE);
	Data = CollisionComponent->PhysicalMaterialRenderData.Realloc(BulkDataSize);
	uint8* WritePtr = (uint8*)Data;

	const int32 CollisionSizes[2] = { CollisionVerts, SimpleCollisionSizeVerts };
	for (int32 i = 0; i < 2; ++i)
	{
		const int32 CollisionSizeVerts = CollisionSizes[i];
		if (CollisionSizeVerts == CollisionVerts)
		{
			FMemory::Memcpy(WritePtr, InMaterialIds.GetData(), CollisionVerts * CollisionVerts);
			WritePtr += CollisionVerts * CollisionVerts;
		}
		else if (CollisionSizeVerts > 0)
		{
			const float ScaleFactor = (CollisionVerts - 1.0f) / (CollisionSizeVerts - 1.0f);

			// For simple collision, we just do a point-sampled resample to the desired resolution
			// This may result in off-by-half-a-pixel misalignments, but we assume the simple collision would be fine with that
			// (the alternative would be to directly render the MaterialIDs at both regular and simple resolutions)
			for (int32 y = 0; y < CollisionSizeVerts; y++)					// [0 .. CollisionSizeVerts-1]
			{
				int32 sampleY = FMath::RoundToInt32(y * ScaleFactor);		// [0 .. CollisionVerts-1]
				for (int32 x = 0; x < CollisionSizeVerts; x++)				// [0 .. CollisionSizeVerts-1]
				{
					int32 sampleX = FMath::RoundToInt32(x * ScaleFactor);	// [0 .. CollisionVerts-1]
					*WritePtr++ = InMaterialIds[sampleY * CollisionVerts + sampleX];
				}
			}
		}
	}

	check(WritePtr - (uint8*)Data == BulkDataSize);
	CollisionComponent->PhysicalMaterialRenderData.Unlock();
}

void ULandscapeComponent::GenerateHeightmapMips(TArray<FColor*>& HeightmapTextureMipData, int32 ComponentX1/*=0*/, int32 ComponentY1/*=0*/, int32 ComponentX2/*=MAX_int32*/, int32 ComponentY2/*=MAX_int32*/, FLandscapeTextureDataInfo* TextureDataInfo/*=nullptr*/)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeComponent::GenerateHeightmapMips);
	
	bool EndX = false;
	bool EndY = false;

	if (ComponentX1 == MAX_int32)
	{
		EndX = true;
		ComponentX1 = 0;
	}

	if (ComponentY1 == MAX_int32)
	{
		EndY = true;
		ComponentY1 = 0;
	}

	if (ComponentX2 == MAX_int32)
	{
		ComponentX2 = ComponentSizeQuads;
	}
	if (ComponentY2 == MAX_int32)
	{
		ComponentY2 = ComponentSizeQuads;
	}

	int32 HeightmapSizeU = GetHeightmap()->Source.GetSizeX();
	int32 HeightmapSizeV = GetHeightmap()->Source.GetSizeY();

	const int32 HeightmapOffsetX = FMath::RoundToInt32(HeightmapScaleBias.Z * (float)HeightmapSizeU);
	const int32 HeightmapOffsetY = FMath::RoundToInt32(HeightmapScaleBias.W * (float)HeightmapSizeV);

	for (int32 SubsectionY = 0; SubsectionY < NumSubsections; SubsectionY++)
	{
		// Check if subsection is fully above or below the area we are interested in
		if ((ComponentY2 < SubsectionSizeQuads*SubsectionY) ||		// above
			(ComponentY1 > SubsectionSizeQuads*(SubsectionY + 1)))	// below
		{
			continue;
		}

		for (int32 SubsectionX = 0; SubsectionX < NumSubsections; SubsectionX++)
		{
			// Check if subsection is fully to the left or right of the area we are interested in
			if ((ComponentX2 < SubsectionSizeQuads*SubsectionX) ||		// left
				(ComponentX1 > SubsectionSizeQuads*(SubsectionX + 1)))	// right
			{
				continue;
			}

			// Area to update in previous mip level coords
			int32 PrevMipSubX1 = ComponentX1 - SubsectionSizeQuads*SubsectionX;
			int32 PrevMipSubY1 = ComponentY1 - SubsectionSizeQuads*SubsectionY;
			int32 PrevMipSubX2 = ComponentX2 - SubsectionSizeQuads*SubsectionX;
			int32 PrevMipSubY2 = ComponentY2 - SubsectionSizeQuads*SubsectionY;

			int32 PrevMipSubsectionSizeQuads = SubsectionSizeQuads;
			float InvPrevMipSubsectionSizeQuads = 1.0f / (float)SubsectionSizeQuads;

			int32 PrevMipSizeU = HeightmapSizeU;
			int32 PrevMipSizeV = HeightmapSizeV;

			int32 PrevMipHeightmapOffsetX = HeightmapOffsetX;
			int32 PrevMipHeightmapOffsetY = HeightmapOffsetY;

			for (int32 Mip = 1; Mip < HeightmapTextureMipData.Num(); Mip++)
			{
				int32 MipSizeU = HeightmapSizeU >> Mip;
				int32 MipSizeV = HeightmapSizeV >> Mip;

				int32 MipSubsectionSizeQuads = ((SubsectionSizeQuads + 1) >> Mip) - 1;
				float InvMipSubsectionSizeQuads = 1.0f / (float)MipSubsectionSizeQuads;

				int32 MipHeightmapOffsetX = HeightmapOffsetX >> Mip;
				int32 MipHeightmapOffsetY = HeightmapOffsetY >> Mip;

				// Area to update in current mip level coords
				int32 MipSubX1 = FMath::FloorToInt((float)MipSubsectionSizeQuads * (float)PrevMipSubX1 * InvPrevMipSubsectionSizeQuads);
				int32 MipSubY1 = FMath::FloorToInt((float)MipSubsectionSizeQuads * (float)PrevMipSubY1 * InvPrevMipSubsectionSizeQuads);
				int32 MipSubX2 = FMath::CeilToInt((float)MipSubsectionSizeQuads * (float)PrevMipSubX2 * InvPrevMipSubsectionSizeQuads);
				int32 MipSubY2 = FMath::CeilToInt((float)MipSubsectionSizeQuads * (float)PrevMipSubY2 * InvPrevMipSubsectionSizeQuads);

				// Clamp area to update
				int32 VertX1 = FMath::Clamp<int32>(MipSubX1, 0, MipSubsectionSizeQuads);
				int32 VertY1 = FMath::Clamp<int32>(MipSubY1, 0, MipSubsectionSizeQuads);
				int32 VertX2 = FMath::Clamp<int32>(MipSubX2, 0, MipSubsectionSizeQuads);
				int32 VertY2 = FMath::Clamp<int32>(MipSubY2, 0, MipSubsectionSizeQuads);

				for (int32 VertY = VertY1; VertY <= VertY2; VertY++)
				{
					for (int32 VertX = VertX1; VertX <= VertX2; VertX++)
					{
						// Convert VertX/Y into previous mip's coords
						float PrevMipVertX = (float)PrevMipSubsectionSizeQuads * (float)VertX * InvMipSubsectionSizeQuads;
						float PrevMipVertY = (float)PrevMipSubsectionSizeQuads * (float)VertY * InvMipSubsectionSizeQuads;

#if 0
						// Validate that the vertex we skip wouldn't use the updated data in the parent mip.
						// Note this validation is doesn't do anything unless you change the VertY/VertX loops 
						// above to process all verts from 0 .. MipSubsectionSizeQuads.
						if (VertX < VertX1 || VertX > VertX2)
						{
							check(FMath::CeilToInt(PrevMipVertX) < PrevMipSubX1 || FMath::FloorToInt(PrevMipVertX) > PrevMipSubX2);
							continue;
						}

						if (VertY < VertY1 || VertY > VertY2)
						{
							check(FMath::CeilToInt(PrevMipVertY) < PrevMipSubY1 || FMath::FloorToInt(PrevMipVertY) > PrevMipSubY2);
							continue;
						}
#endif

						// X/Y of the vertex we're looking indexed into the texture data
						int32 TexX = (MipHeightmapOffsetX)+(MipSubsectionSizeQuads + 1) * SubsectionX + VertX;
						int32 TexY = (MipHeightmapOffsetY)+(MipSubsectionSizeQuads + 1) * SubsectionY + VertY;

						float fPrevMipTexX = (float)(PrevMipHeightmapOffsetX)+(float)((PrevMipSubsectionSizeQuads + 1) * SubsectionX) + PrevMipVertX;
						float fPrevMipTexY = (float)(PrevMipHeightmapOffsetY)+(float)((PrevMipSubsectionSizeQuads + 1) * SubsectionY) + PrevMipVertY;

						int32 PrevMipTexX = FMath::FloorToInt(fPrevMipTexX);
						float fPrevMipTexFracX = FMath::Fractional(fPrevMipTexX);
						int32 PrevMipTexY = FMath::FloorToInt(fPrevMipTexY);
						float fPrevMipTexFracY = FMath::Fractional(fPrevMipTexY);

						checkSlow(TexX >= 0 && TexX < MipSizeU);
						checkSlow(TexY >= 0 && TexY < MipSizeV);
						checkSlow(PrevMipTexX >= 0 && PrevMipTexX < PrevMipSizeU);
						checkSlow(PrevMipTexY >= 0 && PrevMipTexY < PrevMipSizeV);

						int32 PrevMipTexX1 = FMath::Min<int32>(PrevMipTexX + 1, PrevMipSizeU - 1);
						int32 PrevMipTexY1 = FMath::Min<int32>(PrevMipTexY + 1, PrevMipSizeV - 1);

						// Padding for missing data for MIP 0
						if (Mip == 1)
						{
							if (EndX && SubsectionX == NumSubsections - 1 && VertX == VertX2)
							{
								for (int32 PaddingIdx = PrevMipTexX + PrevMipTexY * PrevMipSizeU; PaddingIdx + 1 < PrevMipTexY1 * PrevMipSizeU; ++PaddingIdx)
								{
									HeightmapTextureMipData[Mip - 1][PaddingIdx + 1] = HeightmapTextureMipData[Mip - 1][PaddingIdx];
								}
							}

							if (EndY && SubsectionX == NumSubsections - 1 && SubsectionY == NumSubsections - 1 && VertY == VertY2 && VertX == VertX2)
							{
								for (int32 PaddingYIdx = PrevMipTexY; PaddingYIdx + 1 < PrevMipSizeV; ++PaddingYIdx)
								{
									for (int32 PaddingXIdx = 0; PaddingXIdx < PrevMipSizeU; ++PaddingXIdx)
									{
										HeightmapTextureMipData[Mip - 1][PaddingXIdx + (PaddingYIdx + 1) * PrevMipSizeU] = HeightmapTextureMipData[Mip - 1][PaddingXIdx + PaddingYIdx * PrevMipSizeU];
									}
								}
							}
						}

						FColor* TexData = &(HeightmapTextureMipData[Mip])[TexX + TexY * MipSizeU];
						FColor *PreMipTexData00 = &(HeightmapTextureMipData[Mip - 1])[PrevMipTexX + PrevMipTexY * PrevMipSizeU];
						FColor *PreMipTexData01 = &(HeightmapTextureMipData[Mip - 1])[PrevMipTexX + PrevMipTexY1 * PrevMipSizeU];
						FColor *PreMipTexData10 = &(HeightmapTextureMipData[Mip - 1])[PrevMipTexX1 + PrevMipTexY * PrevMipSizeU];
						FColor *PreMipTexData11 = &(HeightmapTextureMipData[Mip - 1])[PrevMipTexX1 + PrevMipTexY1 * PrevMipSizeU];

						// Lerp height values
						uint16 PrevMipHeightValue00 = PreMipTexData00->R << 8 | PreMipTexData00->G;
						uint16 PrevMipHeightValue01 = PreMipTexData01->R << 8 | PreMipTexData01->G;
						uint16 PrevMipHeightValue10 = PreMipTexData10->R << 8 | PreMipTexData10->G;
						uint16 PrevMipHeightValue11 = PreMipTexData11->R << 8 | PreMipTexData11->G;
						uint16 HeightValue = static_cast<uint16>(FMath::RoundToInt(
							FMath::Lerp(
							FMath::Lerp((float)PrevMipHeightValue00, (float)PrevMipHeightValue10, fPrevMipTexFracX),
							FMath::Lerp((float)PrevMipHeightValue01, (float)PrevMipHeightValue11, fPrevMipTexFracX),
							fPrevMipTexFracY)));

						TexData->R = HeightValue >> 8;
						TexData->G = HeightValue & 255;

						// Lerp tangents
						TexData->B = static_cast<uint8>(FMath::RoundToInt(
							FMath::Lerp(
							FMath::Lerp((float)PreMipTexData00->B, (float)PreMipTexData10->B, fPrevMipTexFracX),
							FMath::Lerp((float)PreMipTexData01->B, (float)PreMipTexData11->B, fPrevMipTexFracX),
							fPrevMipTexFracY)));

						TexData->A = static_cast<uint8>(FMath::RoundToInt(
							FMath::Lerp(
							FMath::Lerp((float)PreMipTexData00->A, (float)PreMipTexData10->A, fPrevMipTexFracX),
							FMath::Lerp((float)PreMipTexData01->A, (float)PreMipTexData11->A, fPrevMipTexFracX),
							fPrevMipTexFracY)));

						// Padding for missing data
						if (EndX && SubsectionX == NumSubsections - 1 && VertX == VertX2)
						{
							for (int32 PaddingIdx = TexX + TexY * MipSizeU; PaddingIdx + 1 < (TexY + 1) * MipSizeU; ++PaddingIdx)
							{
								HeightmapTextureMipData[Mip][PaddingIdx + 1] = HeightmapTextureMipData[Mip][PaddingIdx];
							}
						}

						if (EndY && SubsectionX == NumSubsections - 1 && SubsectionY == NumSubsections - 1 && VertY == VertY2 && VertX == VertX2)
						{
							for (int32 PaddingYIdx = TexY; PaddingYIdx + 1 < MipSizeV; ++PaddingYIdx)
							{
								for (int32 PaddingXIdx = 0; PaddingXIdx < MipSizeU; ++PaddingXIdx)
								{
									HeightmapTextureMipData[Mip][PaddingXIdx + (PaddingYIdx + 1) * MipSizeU] = HeightmapTextureMipData[Mip][PaddingXIdx + PaddingYIdx * MipSizeU];
								}
							}
						}

					}
				}

				// Record the areas we updated
				if (TextureDataInfo)
				{
					int32 TexX1 = (MipHeightmapOffsetX)+(MipSubsectionSizeQuads + 1) * SubsectionX + VertX1;
					int32 TexY1 = (MipHeightmapOffsetY)+(MipSubsectionSizeQuads + 1) * SubsectionY + VertY1;
					int32 TexX2 = (MipHeightmapOffsetX)+(MipSubsectionSizeQuads + 1) * SubsectionX + VertX2;
					int32 TexY2 = (MipHeightmapOffsetY)+(MipSubsectionSizeQuads + 1) * SubsectionY + VertY2;
					TextureDataInfo->AddMipUpdateRegion(Mip, TexX1, TexY1, TexX2, TexY2);
				}

				// Copy current mip values to prev as we move to the next mip.
				PrevMipSubsectionSizeQuads = MipSubsectionSizeQuads;
				InvPrevMipSubsectionSizeQuads = InvMipSubsectionSizeQuads;

				PrevMipSizeU = MipSizeU;
				PrevMipSizeV = MipSizeV;

				PrevMipHeightmapOffsetX = MipHeightmapOffsetX;
				PrevMipHeightmapOffsetY = MipHeightmapOffsetY;

				// Use this mip's area as we move to the next mip
				PrevMipSubX1 = MipSubX1;
				PrevMipSubY1 = MipSubY1;
				PrevMipSubX2 = MipSubX2;
				PrevMipSubY2 = MipSubY2;
			}
		}
	}
}

void ULandscapeComponent::CreateEmptyTextureMips(UTexture2D* Texture, ELandscapeTextureUsage TextureUsage, ELandscapeTextureType TextureType, bool bClear /*= false*/)
{
	check( Texture->Source.IsValid() );
	ETextureSourceFormat Format = Texture->Source.GetFormat();
	int32 SizeU = Texture->Source.GetSizeX();
	int32 SizeV = Texture->Source.GetSizeY();

	if (bClear)
	{
		Texture->Source.Init2DWithMipChain(SizeU, SizeV, Format);
		int32 NumMips = Texture->Source.GetNumMips();
		if (NumMips > 0)
		{
			Texture->Source.LockMip(0); // taking an outer lock ensures we don't do expensive re-hashing until all mip modification is complete
			for (int32 MipIndex = 0; MipIndex < NumMips; ++MipIndex)
			{
				uint8* MipData = Texture->Source.LockMip(MipIndex);
				check( MipData );
				FMemory::Memzero(MipData, Texture->Source.CalcMipSize(MipIndex));
				Texture->Source.UnlockMip(MipIndex);
			}
			Texture->Source.UnlockMip(0); // release outer lock
			ULandscapeTextureHash::UpdateHash(Texture, TextureUsage, TextureType);
		}
	}
	else
	{
		TArray64<uint8> TopMipData;
		verify( Texture->Source.GetMipData(TopMipData, 0) );
		Texture->Source.Init2DWithMipChain(SizeU, SizeV, Format);
		int32 NumMips = Texture->Source.GetNumMips();
		uint8* MipData = Texture->Source.LockMip(0);
		check( MipData );
		FMemory::Memcpy(MipData, TopMipData.GetData(), TopMipData.Num());
		Texture->Source.UnlockMip(0);
		ULandscapeTextureHash::CheckHashIsUpToDate(Texture);	
	}
}

template<typename DataType>
void ULandscapeComponent::GenerateMipsTempl(int32 InNumSubsections, int32 InSubsectionSizeQuads, UTexture2D* Texture, DataType* BaseMipData)
{
	// Stores pointers to the locked mip data
	TArray<DataType*> MipData;
	MipData.Add(BaseMipData);
	for (int32 MipIndex = 1; MipIndex < Texture->Source.GetNumMips(); ++MipIndex)
	{
		MipData.Add((DataType*)Texture->Source.LockMip(MipIndex));
	}

	// Update the newly created mips
	UpdateMipsTempl<DataType>(InNumSubsections, InSubsectionSizeQuads, Texture, MipData);

	// Unlock all the new mips, but not the base mip's data
	for (int32 i = 1; i < MipData.Num(); i++)
	{
		Texture->Source.UnlockMip(i);
	}

	// Note that we do not need to update the heightmap hash here (even if it was a heightmap) because the hash is only based on mip 0, which we don't change in this function
}

void ULandscapeComponent::GenerateWeightmapMips(int32 InNumSubsections, int32 InSubsectionSizeQuads, UTexture2D* WeightmapTexture, FColor* BaseMipData)
{
	GenerateMipsTempl<FColor>(InNumSubsections, InSubsectionSizeQuads, WeightmapTexture, BaseMipData);
}

namespace
{
	template<typename DataType>
	void BiLerpTextureData(DataType* Output, const DataType* Data00, const DataType* Data10, const DataType* Data01, const DataType* Data11, float FracX, float FracY)
	{
		*Output = static_cast<DataType>(FMath::RoundToInt(
			FMath::Lerp(
			FMath::Lerp((float)*Data00, (float)*Data10, FracX),
			FMath::Lerp((float)*Data01, (float)*Data11, FracX),
			FracY)));
	}

	template<>
	void BiLerpTextureData(FColor* Output, const FColor* Data00, const FColor* Data10, const FColor* Data01, const FColor* Data11, float FracX, float FracY)
	{
		Output->R = static_cast<uint8>(FMath::RoundToInt(
			FMath::Lerp(
			FMath::Lerp((float)Data00->R, (float)Data10->R, FracX),
			FMath::Lerp((float)Data01->R, (float)Data11->R, FracX),
			FracY)));
		Output->G = static_cast<uint8>(FMath::RoundToInt(
			FMath::Lerp(
			FMath::Lerp((float)Data00->G, (float)Data10->G, FracX),
			FMath::Lerp((float)Data01->G, (float)Data11->G, FracX),
			FracY)));
		Output->B = static_cast<uint8>(FMath::RoundToInt(
			FMath::Lerp(
			FMath::Lerp((float)Data00->B, (float)Data10->B, FracX),
			FMath::Lerp((float)Data01->B, (float)Data11->B, FracX),
			FracY)));
		Output->A = static_cast<uint8>(FMath::RoundToInt(
			FMath::Lerp(
			FMath::Lerp((float)Data00->A, (float)Data10->A, FracX),
			FMath::Lerp((float)Data01->A, (float)Data11->A, FracX),
			FracY)));
	}

	template<typename DataType>
	void AverageTexData(DataType* Output, const DataType* Data00, const DataType* Data10, const DataType* Data01, const DataType* Data11)
	{
		*Output = (((int32)(*Data00) + (int32)(*Data10) + (int32)(*Data01) + (int32)(*Data11)) >> 2);
	}

	template<>
	void AverageTexData(FColor* Output, const FColor* Data00, const FColor* Data10, const FColor* Data01, const FColor* Data11)
	{
		Output->R = (((int32)Data00->R + (int32)Data10->R + (int32)Data01->R + (int32)Data11->R) >> 2);
		Output->G = (((int32)Data00->G + (int32)Data10->G + (int32)Data01->G + (int32)Data11->G) >> 2);
		Output->B = (((int32)Data00->B + (int32)Data10->B + (int32)Data01->B + (int32)Data11->B) >> 2);
		Output->A = (((int32)Data00->A + (int32)Data10->A + (int32)Data01->A + (int32)Data11->A) >> 2);
	}

};

template<typename DataType>
void ULandscapeComponent::UpdateMipsTempl(int32 InNumSubsections, int32 InSubsectionSizeQuads, UTexture2D* Texture, TArray<DataType*>& TextureMipData, int32 ComponentX1/*=0*/, int32 ComponentY1/*=0*/, int32 ComponentX2/*=MAX_int32*/, int32 ComponentY2/*=MAX_int32*/, struct FLandscapeTextureDataInfo* TextureDataInfo/*=nullptr*/)
{
	int32 WeightmapSizeU = Texture->Source.GetSizeX();
	int32 WeightmapSizeV = Texture->Source.GetSizeY();

	// Find the maximum mip where each texel's data comes from just one subsection.
	int32 MaxWholeSubsectionMip = FMath::FloorLog2(InSubsectionSizeQuads + 1) - 1;

	// clamp to actual number of mips
	MaxWholeSubsectionMip = FMath::Min(MaxWholeSubsectionMip, TextureMipData.Num() - 1);

	// Update the mip where each texel's data comes from just one subsection.
	for (int32 SubsectionY = 0; SubsectionY < InNumSubsections; SubsectionY++)
	{
		// Check if subsection is fully above or below the area we are interested in
		if ((ComponentY2 < InSubsectionSizeQuads*SubsectionY) ||	// above
			(ComponentY1 > InSubsectionSizeQuads*(SubsectionY + 1)))	// below
		{
			continue;
		}

		for (int32 SubsectionX = 0; SubsectionX < InNumSubsections; SubsectionX++)
		{
			// Check if subsection is fully to the left or right of the area we are interested in
			if ((ComponentX2 < InSubsectionSizeQuads*SubsectionX) ||	// left
				(ComponentX1 > InSubsectionSizeQuads*(SubsectionX + 1)))	// right
			{
				continue;
			}

			// Area to update in previous mip level coords
			int32 PrevMipSubX1 = ComponentX1 - InSubsectionSizeQuads*SubsectionX;
			int32 PrevMipSubY1 = ComponentY1 - InSubsectionSizeQuads*SubsectionY;
			int32 PrevMipSubX2 = ComponentX2 - InSubsectionSizeQuads*SubsectionX;
			int32 PrevMipSubY2 = ComponentY2 - InSubsectionSizeQuads*SubsectionY;

			int32 PrevMipSubsectionSizeQuads = InSubsectionSizeQuads;
			float InvPrevMipSubsectionSizeQuads = 1.0f / (float)InSubsectionSizeQuads;

			int32 PrevMipSizeU = WeightmapSizeU;
			int32 PrevMipSizeV = WeightmapSizeV;

			for (int32 Mip = 1; Mip <= MaxWholeSubsectionMip; Mip++)
			{
				int32 MipSizeU = WeightmapSizeU >> Mip;
				int32 MipSizeV = WeightmapSizeV >> Mip;

				int32 MipSubsectionSizeQuads = ((InSubsectionSizeQuads + 1) >> Mip) - 1;
				float InvMipSubsectionSizeQuads = 1.0f / (float)MipSubsectionSizeQuads;

				// Area to update in current mip level coords
				int32 MipSubX1 = FMath::FloorToInt((float)MipSubsectionSizeQuads * (float)PrevMipSubX1 * InvPrevMipSubsectionSizeQuads);
				int32 MipSubY1 = FMath::FloorToInt((float)MipSubsectionSizeQuads * (float)PrevMipSubY1 * InvPrevMipSubsectionSizeQuads);
				int32 MipSubX2 = FMath::CeilToInt((float)MipSubsectionSizeQuads * (float)PrevMipSubX2 * InvPrevMipSubsectionSizeQuads);
				int32 MipSubY2 = FMath::CeilToInt((float)MipSubsectionSizeQuads * (float)PrevMipSubY2 * InvPrevMipSubsectionSizeQuads);

				// Clamp area to update
				int32 VertX1 = FMath::Clamp<int32>(MipSubX1, 0, MipSubsectionSizeQuads);
				int32 VertY1 = FMath::Clamp<int32>(MipSubY1, 0, MipSubsectionSizeQuads);
				int32 VertX2 = FMath::Clamp<int32>(MipSubX2, 0, MipSubsectionSizeQuads);
				int32 VertY2 = FMath::Clamp<int32>(MipSubY2, 0, MipSubsectionSizeQuads);

				for (int32 VertY = VertY1; VertY <= VertY2; VertY++)
				{
					for (int32 VertX = VertX1; VertX <= VertX2; VertX++)
					{
						// Convert VertX/Y into previous mip's coords
						float PrevMipVertX = (float)PrevMipSubsectionSizeQuads * (float)VertX * InvMipSubsectionSizeQuads;
						float PrevMipVertY = (float)PrevMipSubsectionSizeQuads * (float)VertY * InvMipSubsectionSizeQuads;

						// X/Y of the vertex we're looking indexed into the texture data
						int32 TexX = (MipSubsectionSizeQuads + 1) * SubsectionX + VertX;
						int32 TexY = (MipSubsectionSizeQuads + 1) * SubsectionY + VertY;

						float fPrevMipTexX = (float)((PrevMipSubsectionSizeQuads + 1) * SubsectionX) + PrevMipVertX;
						float fPrevMipTexY = (float)((PrevMipSubsectionSizeQuads + 1) * SubsectionY) + PrevMipVertY;

						int32 PrevMipTexX = FMath::FloorToInt(fPrevMipTexX);
						float fPrevMipTexFracX = FMath::Fractional(fPrevMipTexX);
						int32 PrevMipTexY = FMath::FloorToInt(fPrevMipTexY);
						float fPrevMipTexFracY = FMath::Fractional(fPrevMipTexY);

						check(TexX >= 0 && TexX < MipSizeU);
						check(TexY >= 0 && TexY < MipSizeV);
						check(PrevMipTexX >= 0 && PrevMipTexX < PrevMipSizeU);
						check(PrevMipTexY >= 0 && PrevMipTexY < PrevMipSizeV);

						int32 PrevMipTexX1 = FMath::Min<int32>(PrevMipTexX + 1, PrevMipSizeU - 1);
						int32 PrevMipTexY1 = FMath::Min<int32>(PrevMipTexY + 1, PrevMipSizeV - 1);

						DataType* TexData = &(TextureMipData[Mip])[TexX + TexY * MipSizeU];
						DataType *PreMipTexData00 = &(TextureMipData[Mip - 1])[PrevMipTexX + PrevMipTexY * PrevMipSizeU];
						DataType *PreMipTexData01 = &(TextureMipData[Mip - 1])[PrevMipTexX + PrevMipTexY1 * PrevMipSizeU];
						DataType *PreMipTexData10 = &(TextureMipData[Mip - 1])[PrevMipTexX1 + PrevMipTexY * PrevMipSizeU];
						DataType *PreMipTexData11 = &(TextureMipData[Mip - 1])[PrevMipTexX1 + PrevMipTexY1 * PrevMipSizeU];

						// Lerp weightmap data
						BiLerpTextureData<DataType>(TexData, PreMipTexData00, PreMipTexData10, PreMipTexData01, PreMipTexData11, fPrevMipTexFracX, fPrevMipTexFracY);
					}
				}

				// Record the areas we updated
				if (TextureDataInfo)
				{
					int32 TexX1 = (MipSubsectionSizeQuads + 1) * SubsectionX + VertX1;
					int32 TexY1 = (MipSubsectionSizeQuads + 1) * SubsectionY + VertY1;
					int32 TexX2 = (MipSubsectionSizeQuads + 1) * SubsectionX + VertX2;
					int32 TexY2 = (MipSubsectionSizeQuads + 1) * SubsectionY + VertY2;
					TextureDataInfo->AddMipUpdateRegion(Mip, TexX1, TexY1, TexX2, TexY2);
				}

				// Copy current mip values to prev as we move to the next mip.
				PrevMipSubsectionSizeQuads = MipSubsectionSizeQuads;
				InvPrevMipSubsectionSizeQuads = InvMipSubsectionSizeQuads;

				PrevMipSizeU = MipSizeU;
				PrevMipSizeV = MipSizeV;

				// Use this mip's area as we move to the next mip
				PrevMipSubX1 = MipSubX1;
				PrevMipSubY1 = MipSubY1;
				PrevMipSubX2 = MipSubX2;
				PrevMipSubY2 = MipSubY2;
			}
		}
	}

	// Handle mips that have texels from multiple subsections
	// not valid weight data, so just average the texels of the previous mip.
	for (int32 Mip = MaxWholeSubsectionMip + 1; Mip < TextureMipData.Num(); ++Mip)
	{
		int32 MipSubsectionSizeQuads = ((InSubsectionSizeQuads + 1) >> Mip) - 1;
		checkSlow(MipSubsectionSizeQuads <= 0);

		int32 MipSizeU = FMath::Max<int32>(WeightmapSizeU >> Mip, 1);
		int32 MipSizeV = FMath::Max<int32>(WeightmapSizeV >> Mip, 1);

		int32 PrevMipSizeU = FMath::Max<int32>(WeightmapSizeU >> (Mip - 1), 1);
		int32 PrevMipSizeV = FMath::Max<int32>(WeightmapSizeV >> (Mip - 1), 1);

		for (int32 Y = 0; Y < MipSizeV; Y++)
		{
			for (int32 X = 0; X < MipSizeU; X++)
			{
				DataType* TexData = &(TextureMipData[Mip])[X + Y * MipSizeU];

				DataType *PreMipTexData00 = &(TextureMipData[Mip - 1])[(X * 2 + 0) + (Y * 2 + 0)  * PrevMipSizeU];
				DataType *PreMipTexData01 = &(TextureMipData[Mip - 1])[(X * 2 + 0) + (Y * 2 + 1)  * PrevMipSizeU];
				DataType *PreMipTexData10 = &(TextureMipData[Mip - 1])[(X * 2 + 1) + (Y * 2 + 0)  * PrevMipSizeU];
				DataType *PreMipTexData11 = &(TextureMipData[Mip - 1])[(X * 2 + 1) + (Y * 2 + 1)  * PrevMipSizeU];

				AverageTexData<DataType>(TexData, PreMipTexData00, PreMipTexData10, PreMipTexData01, PreMipTexData11);
			}
		}

		if (TextureDataInfo)
		{
			// These mip sizes are small enough that we may as well just update the whole mip.
			TextureDataInfo->AddMipUpdateRegion(Mip, 0, 0, MipSizeU - 1, MipSizeV - 1);
		}

		if (MipSizeU == 1 && MipSizeV == 1)
		{
			break;
		}
	}
}

void ULandscapeComponent::UpdateWeightmapMips(int32 InNumSubsections, int32 InSubsectionSizeQuads, UTexture2D* WeightmapTexture, TArray<FColor*>& WeightmapTextureMipData, int32 ComponentX1/*=0*/, int32 ComponentY1/*=0*/, int32 ComponentX2/*=MAX_int32*/, int32 ComponentY2/*=MAX_int32*/, struct FLandscapeTextureDataInfo* TextureDataInfo/*=nullptr*/)
{
	UpdateMipsTempl<FColor>(InNumSubsections, InSubsectionSizeQuads, WeightmapTexture, WeightmapTextureMipData, ComponentX1, ComponentY1, ComponentX2, ComponentY2, TextureDataInfo);
}

void ULandscapeComponent::UpdateDataMips(int32 InNumSubsections, int32 InSubsectionSizeQuads, UTexture2D* Texture, TArray<uint8*>& TextureMipData, int32 ComponentX1/*=0*/, int32 ComponentY1/*=0*/, int32 ComponentX2/*=MAX_int32*/, int32 ComponentY2/*=MAX_int32*/, struct FLandscapeTextureDataInfo* TextureDataInfo/*=nullptr*/)
{
	UpdateMipsTempl<uint8>(InNumSubsections, InSubsectionSizeQuads, Texture, TextureMipData, ComponentX1, ComponentY1, ComponentX2, ComponentY2, TextureDataInfo);
}

float ULandscapeComponent::GetLayerWeightAtLocation(const FVector& InLocation, ULandscapeLayerInfoObject* LayerInfo, TArray<uint8>* LayerCache, bool bUseEditingWeightmap)
{
	// Allocate and discard locally if no external cache is passed in.
	TArray<uint8> LocalCache;
	if (LayerCache == nullptr)
	{
		LayerCache = &LocalCache;
	}

	// Fill the cache if necessary
	if (LayerCache->Num() == 0)
	{
		FLandscapeComponentDataInterface CDI(this);
		if (!CDI.GetWeightmapTextureData(LayerInfo, *LayerCache, bUseEditingWeightmap))
		{
			// no data for this layer for this component.
			return 0.0f;
		}
	}

	// Find location
	const FVector TestLocation = GetComponentToWorld().InverseTransformPosition(InLocation);
	
	// Abort if the test location is not on this component
	if (TestLocation.X < 0 || TestLocation.Y < 0 || TestLocation.X > ComponentSizeQuads || TestLocation.Y > ComponentSizeQuads)
	{
		return 0.0f;
	}

	// Find data
	int32 X1 = FMath::FloorToInt32(TestLocation.X);
	int32 Y1 = FMath::FloorToInt32(TestLocation.Y);
	int32 X2 = FMath::CeilToInt32(TestLocation.X);
	int32 Y2 = FMath::CeilToInt32(TestLocation.Y);

	int32 Stride = (SubsectionSizeQuads + 1) * NumSubsections;

	// Min is to prevent the sampling of the final column from overflowing
	int32 IdxX1 = FMath::Min<int32>(((X1 / SubsectionSizeQuads) * (SubsectionSizeQuads + 1)) + (X1 % SubsectionSizeQuads), Stride - 1);
	int32 IdxY1 = FMath::Min<int32>(((Y1 / SubsectionSizeQuads) * (SubsectionSizeQuads + 1)) + (Y1 % SubsectionSizeQuads), Stride - 1);
	int32 IdxX2 = FMath::Min<int32>(((X2 / SubsectionSizeQuads) * (SubsectionSizeQuads + 1)) + (X2 % SubsectionSizeQuads), Stride - 1);
	int32 IdxY2 = FMath::Min<int32>(((Y2 / SubsectionSizeQuads) * (SubsectionSizeQuads + 1)) + (Y2 % SubsectionSizeQuads), Stride - 1);

	// sample
	float Sample11 = (float)((*LayerCache)[IdxX1 + Stride * IdxY1]) / 255.0f;
	float Sample21 = (float)((*LayerCache)[IdxX2 + Stride * IdxY1]) / 255.0f;
	float Sample12 = (float)((*LayerCache)[IdxX1 + Stride * IdxY2]) / 255.0f;
	float Sample22 = (float)((*LayerCache)[IdxX2 + Stride * IdxY2]) / 255.0f;

	float LerpX = FMath::Fractional(static_cast<float>(TestLocation.X));
	float LerpY = FMath::Fractional(static_cast<float>(TestLocation.Y));

	// Bilinear interpolate
	return FMath::Lerp(
		FMath::Lerp(Sample11, Sample21, LerpX),
		FMath::Lerp(Sample12, Sample22, LerpX),
		LerpY);

}

void ULandscapeComponent::GetComponentExtent(int32& MinX, int32& MinY, int32& MaxX, int32& MaxY) const
{
	MinX = FMath::Min(SectionBaseX, MinX);
	MinY = FMath::Min(SectionBaseY, MinY);
	MaxX = FMath::Max(SectionBaseX + ComponentSizeQuads, MaxX);
	MaxY = FMath::Max(SectionBaseY + ComponentSizeQuads, MaxY);
}

FIntRect ULandscapeComponent::GetComponentExtent() const
{
	int32 MinX = MAX_int32, MinY = MAX_int32, MaxX = MIN_int32, MaxY = MIN_int32;
	GetComponentExtent(MinX, MinY, MaxX, MaxY);
	return FIntRect(MinX, MinY, MaxX, MaxY);
}

//
// ALandscape
//
bool ULandscapeInfo::SupportsLandscapeEditing() const
{
	// Don't let landscapes from level instances be edited : they can only be edited in their source level (note that technically, the IsEditing test is not necessary as they cannot be edited 
	//  in level instance mode, since it's mutually exclusive with landscape mode, but let's keep it for describing the intention here) :
	if (LandscapeActor.IsValid() && LandscapeActor->IsInLevelInstance() && !LandscapeActor->IsInEditLevelInstance())
	{
		return false;
	}

	bool bSupportsEditing = true;
	ForEachLandscapeProxy([&bSupportsEditing](ALandscapeProxy* Proxy)
	{
		if(Proxy->GetOutermost()->bIsCookedForEditor)
		{
			bSupportsEditing = false;
			return false;
		}

		return true;

	});
	return bSupportsEditing;
}

bool ULandscapeInfo::IsLandscapeEditableWorld() const
{
	if (GIsEditor && LandscapeActor.IsValid())
	{
		UWorld* World = LandscapeActor.IsValid() ? LandscapeActor->GetWorld() : nullptr;
		return World && SupportsLandscapeEditing() && !IsRunningCommandlet() && !World->IsPlayInEditor() && GEditor->PlayWorld == nullptr;
	}
	return false;
}

bool ULandscapeInfo::AreAllComponentsRegistered() const
{
	bool bAllRegistered = true;
	ForEachLandscapeProxy([&bAllRegistered, this](ALandscapeProxy* LandscapeProxy) {
		if (!IsValid(LandscapeProxy))
		{
			return true;
		}

		if (LandscapeProxy->GetLandscapeGuid() == LandscapeGuid)
		{	
			for (ULandscapeComponent* LandscapeComponent : LandscapeProxy->LandscapeComponents)
			{
				if (LandscapeComponent && !LandscapeComponent->IsRegistered())
				{
					bAllRegistered = false;
				}
			}
		}
		return true;
	});

	if (!bAllRegistered)
	{
		return false;
	}
		
	for (TScriptInterface<ILandscapeSplineInterface> SplineOwner : SplineActors)
	{
		if (!SplineOwner.GetObject() || !IsValidChecked(SplineOwner.GetObject()))
		{
			continue;
		}

		if (SplineOwner->GetLandscapeGuid() == LandscapeGuid)
		{
			if (SplineOwner->GetSplinesComponent() && !SplineOwner->GetSplinesComponent()->IsRegistered())
			{
				return false;
			}
		}
	}


	return true;
}

bool ULandscapeInfo::HasUnloadedComponentsInRegion(int32 X1, int32 Y1, int32 X2, int32 Y2) const
{
	bool bResult = false;

	if (LandscapeActor.IsValid())
	{
		UWorld* World = LandscapeActor->GetWorld();
		ULevel* Level = LandscapeActor->GetLevel();
		FGuid OriginalGUID = LandscapeActor->GetOriginalLandscapeGuid();

		if (UWorldPartition* WorldPartition = World->GetWorldPartition())
		{
			if (Level == World->PersistentLevel)
			{
				int32 ComponentIndexX1, ComponentIndexY1, ComponentIndexX2, ComponentIndexY2;
				ALandscape::CalcComponentIndicesOverlap(X1, Y1, X2, Y2, ComponentSizeQuads, ComponentIndexX1, ComponentIndexY1, ComponentIndexX2, ComponentIndexY2);

				const UActorPartitionSubsystem::FCellCoord MinCoord = UActorPartitionSubsystem::FCellCoord::GetCellCoord(FIntPoint(ComponentIndexX1 * ComponentSizeQuads, ComponentIndexY1 * ComponentSizeQuads), Level, LandscapeActor->GetGridSize());
				const UActorPartitionSubsystem::FCellCoord MaxCoord = UActorPartitionSubsystem::FCellCoord::GetCellCoord(FIntPoint(ComponentIndexX2 * ComponentSizeQuads, ComponentIndexY2 * ComponentSizeQuads), Level, LandscapeActor->GetGridSize());

				FWorldPartitionHelpers::ForEachActorDescInstance<ALandscapeProxy>(WorldPartition, [this, World, &MinCoord, &MaxCoord, OriginalGUID, &bResult](const FWorldPartitionActorDescInstance* ActorDescInstance)
				{
					FLandscapeActorDesc* LandscapeActorDesc = (FLandscapeActorDesc*)ActorDescInstance->GetActorDesc();

					if (LandscapeActorDesc->GridGuid == OriginalGUID)
					{
						const UActorPartitionSubsystem::FCellCoord ActorCoord(LandscapeActorDesc->GridIndexX, LandscapeActorDesc->GridIndexY, LandscapeActorDesc->GridIndexZ, World->PersistentLevel);
						if (ActorCoord.X >= MinCoord.X && ActorCoord.Y >= MinCoord.Y && ActorCoord.X <= MaxCoord.X && ActorCoord.Y <= MaxCoord.Y)
						{
							if (!ActorDescInstance->IsLoaded())
							{
								bResult = true;
								return false;
							}
						}
					}

					return true;
				});
			}
		}
	}

	return bResult;
}

void ULandscapeInfo::GetComponentsInRegion(int32 X1, int32 Y1, int32 X2, int32 Y2, TSet<ULandscapeComponent*>& OutComponents, bool bOverlap) const
{
	// Find component range for this block of data
	// X2/Y2 Coordinates are "inclusive" max values
	int32 ComponentIndexX1, ComponentIndexY1, ComponentIndexX2, ComponentIndexY2;
	if (bOverlap)
	{
		ALandscape::CalcComponentIndicesOverlap(X1, Y1, X2, Y2, ComponentSizeQuads, ComponentIndexX1, ComponentIndexY1, ComponentIndexX2, ComponentIndexY2);
	}
	else
	{
		ALandscape::CalcComponentIndicesNoOverlap(X1, Y1, X2, Y2, ComponentSizeQuads, ComponentIndexX1, ComponentIndexY1, ComponentIndexX2, ComponentIndexY2);
	}

	for (int32 ComponentIndexY = ComponentIndexY1; ComponentIndexY <= ComponentIndexY2; ComponentIndexY++)
	{
		for (int32 ComponentIndexX = ComponentIndexX1; ComponentIndexX <= ComponentIndexX2; ComponentIndexX++)
		{
			ULandscapeComponent* Component = XYtoComponentMap.FindRef(FIntPoint(ComponentIndexX, ComponentIndexY));
			if (Component && !FLevelUtils::IsLevelLocked(Component->GetLandscapeProxy()->GetLevel()) && FLevelUtils::IsLevelVisible(Component->GetLandscapeProxy()->GetLevel()))
			{
				OutComponents.Add(Component);
			}
		}
	}
}

// A struct to remember where we have spare texture channels.
struct FWeightmapTextureAllocation
{
	int32 X;
	int32 Y;
	int32 ChannelsInUse;
	UTexture2D* Texture;
	FColor* TextureData;

	FWeightmapTextureAllocation(int32 InX, int32 InY, int32 InChannels, UTexture2D* InTexture, FColor* InTextureData)
		: X(InX)
		, Y(InY)
		, ChannelsInUse(InChannels)
		, Texture(InTexture)
		, TextureData(InTextureData)
	{}
};

// A struct to hold the info about each texture chunk of the total heightmap
struct FHeightmapInfo
{
	int32 HeightmapSizeU;
	int32 HeightmapSizeV;
	UTexture2D* HeightmapTexture;
	TArray<FColor*> HeightmapTextureMipData;
};

// Deprecated
const TArray<FName> ALandscapeProxy::GetLayersFromMaterial(UMaterialInterface* MaterialInterface)
{
	return UE::Landscape::RetrieveTargetLayerNamesFromMaterial(MaterialInterface);
}

// Deprecated
const TArray<FName> ALandscapeProxy::GetLayersFromMaterial() const
{
	return RetrieveTargetLayerNamesFromMaterials();
}

TArray<FName> ALandscapeProxy::RetrieveTargetLayerNamesFromMaterials(bool bIncludeVisibilityLayer) const
{
	TSet<UMaterialInterface*> LandscapeMaterials;
	RetrieveAllLandscapeMaterials(LandscapeMaterials);

	TSet<FName> LayerNames;
	for (UMaterialInterface* Material : LandscapeMaterials)
	{
		LayerNames.Append(UE::Landscape::RetrieveTargetLayerNamesFromMaterial(Material, bIncludeVisibilityLayer));
	}
	return LayerNames.Array();
}

TMap<FName, ULandscapeLayerInfoObject*> ALandscapeProxy::RetrieveTargetLayerInfosFromAllocations() const
{
	TMap<FName, ULandscapeLayerInfoObject*> InfoObjects;
	for (const TObjectPtr<ULandscapeComponent>& Component : LandscapeComponents)
	{
		if (Component != nullptr) // can be null in server builds that strip landscape components
		{
			for (const FWeightmapLayerAllocationInfo& Allocation : Component->GetWeightmapLayerAllocations())
			{
				if (Allocation.LayerInfo)
				{
					InfoObjects.Add(Allocation.GetLayerName(), Allocation.LayerInfo);
				}
			}
		}
	}
	
	return InfoObjects;
}

#define HEIGHTDATA(X,Y) (HeightData.Num() == 0 ? LandscapeDataAccess::GetTexHeight(0.0f) : HeightData[ FMath::Clamp<int32>(Y,0,VertsY) * VertsX + FMath::Clamp<int32>(X,0,VertsX) ])
ENGINE_API extern bool GDisableAutomaticTextureMaterialUpdateDependencies;

void ALandscapeProxy::Import(const FGuid& InGuid, int32 InMinX, int32 InMinY, int32 InMaxX, int32 InMaxY, int32 InNumSubsections, int32 InSubsectionSizeQuads, const TMap<FGuid, TArray<uint16>>& InImportHeightData,
	const TCHAR* const InHeightmapFileName, const TMap<FGuid, TArray<FLandscapeImportLayerInfo>>& InImportMaterialLayerInfos, ELandscapeImportAlphamapType InImportMaterialLayerType, const TArrayView<const FLandscapeLayer>& InImportLayers)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ALandscapeProxy::Import);
	
	check(InGuid.IsValid());
	check(InImportHeightData.Num() == InImportMaterialLayerInfos.Num());

	check(!IsTemplate());

	FScopedSlowTask SlowTask(2, LOCTEXT("BeingImportingLandscapeTask", "Importing Landscape"));
	SlowTask.MakeDialog();

	SlowTask.EnterProgressFrame(1.0f);

	const int32 VertsX = InMaxX - InMinX + 1;
	const int32 VertsY = InMaxY - InMinY + 1;

	ComponentSizeQuads = InNumSubsections * InSubsectionSizeQuads;
	NumSubsections = InNumSubsections;
	SubsectionSizeQuads = InSubsectionSizeQuads;
	SetLandscapeGuid(InGuid);

	Modify();

	const int32 NumPatchesX = (VertsX - 1);
	const int32 NumPatchesY = (VertsY - 1);

	const int32 NumComponentsX = NumPatchesX / ComponentSizeQuads;
	const int32 NumComponentsY = NumPatchesY / ComponentSizeQuads;

	// currently only support importing into a new/blank landscape actor/proxy
	check(LandscapeComponents.Num() == 0);
	LandscapeComponents.Empty(NumComponentsX * NumComponentsY);

	for (int32 Y = 0; Y < NumComponentsY; Y++)
	{
		for (int32 X = 0; X < NumComponentsX; X++)
		{
			const int32 BaseX = InMinX + X * ComponentSizeQuads;
			const int32 BaseY = InMinY + Y * ComponentSizeQuads;

			ULandscapeComponent* LandscapeComponent = NewObject<ULandscapeComponent>(this, NAME_None, RF_Transactional);
			LandscapeComponent->Init(BaseX, BaseY, ComponentSizeQuads, NumSubsections, SubsectionSizeQuads);
		}
	}

	// Ensure that we don't pack so many heightmaps into a texture that their lowest LOD isn't guaranteed to be resident
#define MAX_HEIGHTMAP_TEXTURE_SIZE 512
	const int32 ComponentSizeVerts = NumSubsections * (SubsectionSizeQuads + 1);
	const int32 ComponentsPerHeightmap = FMath::Min(MAX_HEIGHTMAP_TEXTURE_SIZE / ComponentSizeVerts, 1 << (UTexture2D::GetStaticMinTextureResidentMipCount() - 2));
	check(ComponentsPerHeightmap > 0);

	// Count how many heightmaps we need and the X dimension of the final heightmap
	int32 NumHeightmapsX = 1;
	int32 FinalComponentsX = NumComponentsX;
	while (FinalComponentsX > ComponentsPerHeightmap)
	{
		FinalComponentsX -= ComponentsPerHeightmap;
		NumHeightmapsX++;
	}
	// Count how many heightmaps we need and the Y dimension of the final heightmap
	int32 NumHeightmapsY = 1;
	int32 FinalComponentsY = NumComponentsY;
	while (FinalComponentsY > ComponentsPerHeightmap)
	{
		FinalComponentsY -= ComponentsPerHeightmap;
		NumHeightmapsY++;
	}

	TArray<FHeightmapInfo> HeightmapInfos;

	for (int32 HmY = 0; HmY < NumHeightmapsY; HmY++)
	{
		for (int32 HmX = 0; HmX < NumHeightmapsX; HmX++)
		{
			FHeightmapInfo& HeightmapInfo = HeightmapInfos[HeightmapInfos.AddZeroed()];

			// make sure the heightmap UVs are powers of two.
			HeightmapInfo.HeightmapSizeU = (1 << FMath::CeilLogTwo(((HmX == NumHeightmapsX - 1) ? FinalComponentsX : ComponentsPerHeightmap) * ComponentSizeVerts));
			HeightmapInfo.HeightmapSizeV = (1 << FMath::CeilLogTwo(((HmY == NumHeightmapsY - 1) ? FinalComponentsY : ComponentsPerHeightmap) * ComponentSizeVerts));

			// Construct the heightmap textures
			HeightmapInfo.HeightmapTexture = CreateLandscapeTexture(HeightmapInfo.HeightmapSizeU, HeightmapInfo.HeightmapSizeV, TEXTUREGROUP_Terrain_Heightmap, TSF_BGRA8);

			int32 MipSubsectionSizeQuads = SubsectionSizeQuads;
			int32 MipSizeU = HeightmapInfo.HeightmapSizeU;
			int32 MipSizeV = HeightmapInfo.HeightmapSizeV;
			while (MipSizeU > 1 && MipSizeV > 1 && MipSubsectionSizeQuads >= 1)
			{
				int32 MipIndex = HeightmapInfo.HeightmapTextureMipData.Num();
				FColor* HeightmapTextureData = (FColor*)HeightmapInfo.HeightmapTexture->Source.LockMip(MipIndex);
				FMemory::Memzero(HeightmapTextureData, MipSizeU*MipSizeV*sizeof(FColor));
				HeightmapInfo.HeightmapTextureMipData.Add(HeightmapTextureData);

				MipSizeU >>= 1;
				MipSizeV >>= 1;

				MipSubsectionSizeQuads = ((MipSubsectionSizeQuads + 1) >> 1) - 1;
			}
		}
	}

	const FVector DrawScale3D = GetRootComponent()->GetRelativeScale3D();

	// layer to import data (Final or 1st layer)
	const FGuid FinalLayerGuid = FGuid();
	const TArray<uint16>& HeightData = InImportHeightData.FindChecked(FinalLayerGuid);
	const TArray<FLandscapeImportLayerInfo>& ImportLayerInfos = InImportMaterialLayerInfos.FindChecked(FinalLayerGuid);

	// Calculate the normals for each of the two triangles per quad.
	TArray<FVector> VertexNormals;
	VertexNormals.AddZeroed(VertsX * VertsY);
	for (int32 QuadY = 0; QuadY < NumPatchesY; QuadY++)
	{
		for (int32 QuadX = 0; QuadX < NumPatchesX; QuadX++)
		{
			const FVector Vert00 = FVector(0.0f, 0.0f, LandscapeDataAccess::GetLocalHeight(HEIGHTDATA(QuadX + 0, QuadY + 0))) * DrawScale3D;
			const FVector Vert01 = FVector(0.0f, 1.0f, LandscapeDataAccess::GetLocalHeight(HEIGHTDATA(QuadX + 0, QuadY + 1))) * DrawScale3D;
			const FVector Vert10 = FVector(1.0f, 0.0f, LandscapeDataAccess::GetLocalHeight(HEIGHTDATA(QuadX + 1, QuadY + 0))) * DrawScale3D;
			const FVector Vert11 = FVector(1.0f, 1.0f, LandscapeDataAccess::GetLocalHeight(HEIGHTDATA(QuadX + 1, QuadY + 1))) * DrawScale3D;

			const FVector FaceNormal1 = ((Vert00 - Vert10) ^ (Vert10 - Vert11)).GetSafeNormal();
			const FVector FaceNormal2 = ((Vert11 - Vert01) ^ (Vert01 - Vert00)).GetSafeNormal();

			// contribute to the vertex normals.
			VertexNormals[(QuadX + 1 + VertsX * (QuadY + 0))] += FaceNormal1;
			VertexNormals[(QuadX + 0 + VertsX * (QuadY + 1))] += FaceNormal2;
			VertexNormals[(QuadX + 0 + VertsX * (QuadY + 0))] += FaceNormal1 + FaceNormal2;
			VertexNormals[(QuadX + 1 + VertsX * (QuadY + 1))] += FaceNormal1 + FaceNormal2;
		}
	}

	// Weight values for each layer for each component.
	TArray<TArray<TArray<uint8>>> ComponentWeightValues;
	ComponentWeightValues.AddZeroed(NumComponentsX * NumComponentsY);

	for (int32 ComponentY = 0; ComponentY < NumComponentsY; ComponentY++)
	{
		for (int32 ComponentX = 0; ComponentX < NumComponentsX; ComponentX++)
		{
			ULandscapeComponent* const LandscapeComponent = LandscapeComponents[ComponentX + ComponentY*NumComponentsX];
			TArray<TArray<uint8>>& WeightValues = ComponentWeightValues[ComponentX + ComponentY*NumComponentsX];

			// Import alphamap data into local array and check for unused layers for this component.
			TArray<FLandscapeComponentAlphaInfo, TInlineAllocator<16>> EditingAlphaLayerData;
			for (int32 LayerIndex = 0; LayerIndex < ImportLayerInfos.Num(); LayerIndex++)
			{
				FLandscapeComponentAlphaInfo* NewAlphaInfo = new(EditingAlphaLayerData) FLandscapeComponentAlphaInfo(LandscapeComponent, LayerIndex);

				if (ImportLayerInfos[LayerIndex].LayerData.Num())
				{
					for (int32 AlphaY = 0; AlphaY <= LandscapeComponent->ComponentSizeQuads; AlphaY++)
					{
						const uint8* const OldAlphaRowStart = &ImportLayerInfos[LayerIndex].LayerData[(AlphaY + LandscapeComponent->GetSectionBase().Y - InMinY) * VertsX + (LandscapeComponent->GetSectionBase().X - InMinX)];
						uint8* const NewAlphaRowStart = &NewAlphaInfo->AlphaValues[AlphaY * (LandscapeComponent->ComponentSizeQuads + 1)];
						FMemory::Memcpy(NewAlphaRowStart, OldAlphaRowStart, LandscapeComponent->ComponentSizeQuads + 1);
					}
				}
			}

			for (int32 AlphaMapIndex = 0; AlphaMapIndex < EditingAlphaLayerData.Num(); AlphaMapIndex++)
			{
				if (EditingAlphaLayerData[AlphaMapIndex].IsLayerAllZero())
				{
					EditingAlphaLayerData.RemoveAt(AlphaMapIndex);
					AlphaMapIndex--;
				}
			}


			UE_LOGF(LogLandscape, VeryVerbose, "%ls needs %d alphamaps", *LandscapeComponent->GetName(), EditingAlphaLayerData.Num());

			TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = LandscapeComponent->GetWeightmapLayerAllocations();

			// Calculate weightmap weights for this component
			WeightValues.Empty(EditingAlphaLayerData.Num());
			WeightValues.AddZeroed(EditingAlphaLayerData.Num());
			ComponentWeightmapLayerAllocations.Empty(EditingAlphaLayerData.Num());
			TArray<bool, TInlineAllocator<16>> IsNoBlendArray;
			IsNoBlendArray.Empty(EditingAlphaLayerData.Num());
			IsNoBlendArray.AddZeroed(EditingAlphaLayerData.Num());

			for (int32 WeightLayerIndex = 0; WeightLayerIndex < WeightValues.Num(); WeightLayerIndex++)
			{
				// Lookup the original layer name
				WeightValues[WeightLayerIndex] = EditingAlphaLayerData[WeightLayerIndex].AlphaValues;
				new(ComponentWeightmapLayerAllocations) FWeightmapLayerAllocationInfo(ImportLayerInfos[EditingAlphaLayerData[WeightLayerIndex].LayerIndex].LayerInfo);
				IsNoBlendArray[WeightLayerIndex] = (ImportLayerInfos[EditingAlphaLayerData[WeightLayerIndex].LayerIndex].LayerInfo->GetBlendMethod() == ELandscapeTargetLayerBlendMethod::None);
			}

			// Discard the temporary alpha data
			EditingAlphaLayerData.Empty();
			
			// Handle the special import method where each alpha map specifies the reminder from previous layers : apply it only to weight-blended layers
			if (InImportMaterialLayerType == ELandscapeImportAlphamapType::Layered)
			{
				// For each layer...
				for (int32 WeightLayerIndex = WeightValues.Num() - 1; WeightLayerIndex >= 0; WeightLayerIndex--)
				{
					// ... multiply all lower layers'...
					for (int32 BelowWeightLayerIndex = WeightLayerIndex - 1; BelowWeightLayerIndex >= 0; BelowWeightLayerIndex--)
					{
						int32 TotalWeight = 0;

						if (IsNoBlendArray[BelowWeightLayerIndex])
						{
							continue; // skip no blend
						}

						// ... values by...
						for (int32 Idx = 0; Idx < WeightValues[WeightLayerIndex].Num(); Idx++)
						{
							// ... one-minus the current layer's values
							int32 NewValue = (int32)WeightValues[BelowWeightLayerIndex][Idx] * (int32)(255 - WeightValues[WeightLayerIndex][Idx]) / 255;
							WeightValues[BelowWeightLayerIndex][Idx] = (uint8)NewValue;
							TotalWeight += NewValue;
						}

						if (TotalWeight == 0)
						{
							// Remove the layer as it has no contribution
							WeightValues.RemoveAt(BelowWeightLayerIndex);
							ComponentWeightmapLayerAllocations.RemoveAt(BelowWeightLayerIndex);
							IsNoBlendArray.RemoveAt(BelowWeightLayerIndex);

							// The current layer has been re-numbered
							WeightLayerIndex--;
						}
					}
				}
			}
		}
	}

	// Remember where we have spare texture channels.
	TArray<FWeightmapTextureAllocation> TextureAllocations;

	for (int32 ComponentY = 0; ComponentY < NumComponentsY; ComponentY++)
	{
		const int32 HmY = ComponentY / ComponentsPerHeightmap;
		const int32 HeightmapOffsetY = (ComponentY - ComponentsPerHeightmap*HmY) * NumSubsections * (SubsectionSizeQuads + 1);

		for (int32 ComponentX = 0; ComponentX < NumComponentsX; ComponentX++)
		{
			const int32 HmX = ComponentX / ComponentsPerHeightmap;
			const FHeightmapInfo& HeightmapInfo = HeightmapInfos[HmX + HmY * NumHeightmapsX];

			ULandscapeComponent* LandscapeComponent = LandscapeComponents[ComponentX + ComponentY*NumComponentsX];

			// Lookup array of weight values for this component.
			const TArray<TArray<uint8>>& WeightValues = ComponentWeightValues[ComponentX + ComponentY*NumComponentsX];

			// Heightmap offsets
			const int32 HeightmapOffsetX = (ComponentX - ComponentsPerHeightmap*HmX) * NumSubsections * (SubsectionSizeQuads + 1);

			LandscapeComponent->HeightmapScaleBias = FVector4(1.0f / (float)HeightmapInfo.HeightmapSizeU, 1.0f / (float)HeightmapInfo.HeightmapSizeV, (float)((HeightmapOffsetX)) / (float)HeightmapInfo.HeightmapSizeU, ((float)(HeightmapOffsetY)) / (float)HeightmapInfo.HeightmapSizeV);
			LandscapeComponent->SetHeightmap(HeightmapInfo.HeightmapTexture);

			// Weightmap is sized the same as the component
			const int32 WeightmapSize = (SubsectionSizeQuads + 1) * NumSubsections;
			// Should be power of two
			check(FMath::IsPowerOfTwo(WeightmapSize));

			LandscapeComponent->WeightmapScaleBias = FVector4(1.0f / (float)WeightmapSize, 1.0f / (float)WeightmapSize, 0.5f / (float)WeightmapSize, 0.5f / (float)WeightmapSize);
			LandscapeComponent->WeightmapSubsectionOffset = (float)(SubsectionSizeQuads + 1) / (float)WeightmapSize;

			// Pointers to the texture data where we'll store each layer. Stride is 4 (FColor)
			TArray<uint8*> WeightmapTextureDataPointers;

			UE_LOGF(LogLandscape, VeryVerbose, "%ls needs %d weightmap channels", *LandscapeComponent->GetName(), WeightValues.Num());

			// Find texture channels to store each layer.
			int32 LayerIndex = 0;
			while (LayerIndex < WeightValues.Num())
			{
				const int32 RemainingLayers = WeightValues.Num() - LayerIndex;

				int32 BestAllocationIndex = -1;

				// if we need less than 4 channels, try to find them somewhere to put all of them
				if (RemainingLayers < 4)
				{
					int32 BestDistSquared = MAX_int32;
					for (int32 TryAllocIdx = 0; TryAllocIdx < TextureAllocations.Num(); TryAllocIdx++)
					{
						if (TextureAllocations[TryAllocIdx].ChannelsInUse + RemainingLayers <= 4)
						{
							FWeightmapTextureAllocation& TryAllocation = TextureAllocations[TryAllocIdx];
							const int32 TryDistSquared = FMath::Square(TryAllocation.X - ComponentX) + FMath::Square(TryAllocation.Y - ComponentY);
							if (TryDistSquared < BestDistSquared)
							{
								BestDistSquared = TryDistSquared;
								BestAllocationIndex = TryAllocIdx;
							}
						}
					}
				}

				TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = LandscapeComponent->GetWeightmapLayerAllocations();
				TArray<TObjectPtr<UTexture2D>>& ComponentWeightmapTextures = LandscapeComponent->GetWeightmapTextures();
				TArray<TObjectPtr<ULandscapeWeightmapUsage>>& ComponentWeightmapTexturesUsage = LandscapeComponent->GetWeightmapTexturesUsage();

				if (BestAllocationIndex != -1)
				{
					FWeightmapTextureAllocation& Allocation = TextureAllocations[BestAllocationIndex];
					ULandscapeWeightmapUsage* WeightmapUsage = WeightmapUsageMap.FindChecked(Allocation.Texture);
					ComponentWeightmapTexturesUsage.Add(WeightmapUsage);

					UE_LOGF(LogLandscape, VeryVerbose, "  ==> Storing %d channels starting at %ls[%d]", RemainingLayers, *Allocation.Texture->GetName(), Allocation.ChannelsInUse);

					for (int32 i = 0; i < RemainingLayers; i++)
					{
						ComponentWeightmapLayerAllocations[LayerIndex + i].WeightmapTextureIndex = static_cast<uint8>(ComponentWeightmapTextures.Num());
						ComponentWeightmapLayerAllocations[LayerIndex + i].WeightmapTextureChannel = static_cast<uint8>(Allocation.ChannelsInUse);
						WeightmapUsage->ChannelUsage[Allocation.ChannelsInUse] = LandscapeComponent;
						switch (Allocation.ChannelsInUse)
						{
						case 1:
							WeightmapTextureDataPointers.Add((uint8*)&Allocation.TextureData->G);
							break;
						case 2:
							WeightmapTextureDataPointers.Add((uint8*)&Allocation.TextureData->B);
							break;
						case 3:
							WeightmapTextureDataPointers.Add((uint8*)&Allocation.TextureData->A);
							break;
						default:
							// this should not occur.
							check(0);

						}
						Allocation.ChannelsInUse++;
					}

					LayerIndex += RemainingLayers;
					ComponentWeightmapTextures.Add(Allocation.Texture);
				}
				else
				{
					// We couldn't find a suitable place for these layers, so lets make a new one.
					UTexture2D* const WeightmapTexture = CreateLandscapeTexture(WeightmapSize, WeightmapSize, TEXTUREGROUP_Terrain_Weightmap, TSF_BGRA8);
					FColor* const MipData = (FColor*)WeightmapTexture->Source.LockMip(0);

					const int32 ThisAllocationLayers = FMath::Min<int32>(RemainingLayers, 4);
					new(TextureAllocations) FWeightmapTextureAllocation(ComponentX, ComponentY, ThisAllocationLayers, WeightmapTexture, MipData);
					ULandscapeWeightmapUsage* WeightmapUsage = WeightmapUsageMap.Add(WeightmapTexture, CreateWeightmapUsage());
					ComponentWeightmapTexturesUsage.Add(WeightmapUsage);

					UE_LOGF(LogLandscape, VeryVerbose, "  ==> Storing %d channels in new texture %ls", ThisAllocationLayers, *WeightmapTexture->GetName());

					WeightmapTextureDataPointers.Add((uint8*)&MipData->R);
					ComponentWeightmapLayerAllocations[LayerIndex + 0].WeightmapTextureIndex = static_cast<uint8>(ComponentWeightmapTextures.Num());
					ComponentWeightmapLayerAllocations[LayerIndex + 0].WeightmapTextureChannel = 0;
					WeightmapUsage->ChannelUsage[0] = LandscapeComponent;

					if (ThisAllocationLayers > 1)
					{
						WeightmapTextureDataPointers.Add((uint8*)&MipData->G);
						ComponentWeightmapLayerAllocations[LayerIndex + 1].WeightmapTextureIndex = static_cast<uint8>(ComponentWeightmapTextures.Num());
						ComponentWeightmapLayerAllocations[LayerIndex + 1].WeightmapTextureChannel = 1;
						WeightmapUsage->ChannelUsage[1] = LandscapeComponent;

						if (ThisAllocationLayers > 2)
						{
							WeightmapTextureDataPointers.Add((uint8*)&MipData->B);
							ComponentWeightmapLayerAllocations[LayerIndex + 2].WeightmapTextureIndex = static_cast<uint8>(ComponentWeightmapTextures.Num());
							ComponentWeightmapLayerAllocations[LayerIndex + 2].WeightmapTextureChannel = 2;
							WeightmapUsage->ChannelUsage[2] = LandscapeComponent;

							if (ThisAllocationLayers > 3)
							{
								WeightmapTextureDataPointers.Add((uint8*)&MipData->A);
								ComponentWeightmapLayerAllocations[LayerIndex + 3].WeightmapTextureIndex = static_cast<uint8>(ComponentWeightmapTextures.Num());
								ComponentWeightmapLayerAllocations[LayerIndex + 3].WeightmapTextureChannel = 3;
								WeightmapUsage->ChannelUsage[3] = LandscapeComponent;
							}
						}
					}
					ComponentWeightmapTextures.Add(WeightmapTexture);

					LayerIndex += ThisAllocationLayers;
				}
			}
			check(WeightmapTextureDataPointers.Num() == WeightValues.Num());

			FBox LocalBox(ForceInit);
			for (int32 SubsectionY = 0; SubsectionY < NumSubsections; SubsectionY++)
			{
				for (int32 SubsectionX = 0; SubsectionX < NumSubsections; SubsectionX++)
				{
					for (int32 SubY = 0; SubY <= SubsectionSizeQuads; SubY++)
					{
						for (int32 SubX = 0; SubX <= SubsectionSizeQuads; SubX++)
						{
							// X/Y of the vertex we're looking at in component's coordinates.
							const int32 CompX = SubsectionSizeQuads * SubsectionX + SubX;
							const int32 CompY = SubsectionSizeQuads * SubsectionY + SubY;

							// X/Y of the vertex we're looking indexed into the texture data
							const int32 TexX = (SubsectionSizeQuads + 1) * SubsectionX + SubX;
							const int32 TexY = (SubsectionSizeQuads + 1) * SubsectionY + SubY;

							const int32 WeightSrcDataIdx = CompY * (ComponentSizeQuads + 1) + CompX;
							const int32 HeightTexDataIdx = (HeightmapOffsetX + TexX) + (HeightmapOffsetY + TexY) * (HeightmapInfo.HeightmapSizeU);

							const int32 WeightTexDataIdx = (TexX)+(TexY)* (WeightmapSize);

							// copy height and normal data
							const uint16 HeightValue = HEIGHTDATA(CompX + LandscapeComponent->GetSectionBase().X - InMinX, CompY + LandscapeComponent->GetSectionBase().Y - InMinY);
							const FVector Normal = VertexNormals[CompX + LandscapeComponent->GetSectionBase().X - InMinX + VertsX * (CompY + LandscapeComponent->GetSectionBase().Y - InMinY)].GetSafeNormal();

							HeightmapInfo.HeightmapTextureMipData[0][HeightTexDataIdx].R = HeightValue >> 8;
							HeightmapInfo.HeightmapTextureMipData[0][HeightTexDataIdx].G = HeightValue & 255;
							HeightmapInfo.HeightmapTextureMipData[0][HeightTexDataIdx].B = static_cast<uint8>(FMath::RoundToInt32(127.5f * (Normal.X + 1.0f)));
							HeightmapInfo.HeightmapTextureMipData[0][HeightTexDataIdx].A = static_cast<uint8>(FMath::RoundToInt32(127.5f * (Normal.Y + 1.0f)));

							for (int32 WeightmapIndex = 0; WeightmapIndex < WeightValues.Num(); WeightmapIndex++)
							{
								WeightmapTextureDataPointers[WeightmapIndex][WeightTexDataIdx * 4] = WeightValues[WeightmapIndex][WeightSrcDataIdx];
							}

							// Get local space verts
							const FVector LocalVertex(CompX, CompY, LandscapeDataAccess::GetLocalHeight(HeightValue));
							LocalBox += LocalVertex;
						}
					}
				}
			}

			LandscapeComponent->CachedLocalBox = LocalBox;
		}
	}

	TArray<UTexture2D*> PendingTexturePlatformDataCreation;

	// Unlock the weightmaps' base mips
	for (int32 AllocationIndex = 0; AllocationIndex < TextureAllocations.Num(); AllocationIndex++)
	{
		UTexture2D* const WeightmapTexture = TextureAllocations[AllocationIndex].Texture;
		FColor* const BaseMipData = TextureAllocations[AllocationIndex].TextureData;

		// Generate mips for weightmaps
		ULandscapeComponent::GenerateWeightmapMips(NumSubsections, SubsectionSizeQuads, WeightmapTexture, BaseMipData);

		WeightmapTexture->Source.UnlockMip(0);

		WeightmapTexture->BeginCachePlatformData();
		WeightmapTexture->ClearAllCachedCookedPlatformData();
		PendingTexturePlatformDataCreation.Add(WeightmapTexture);
	}

	// Generate mipmaps for the components, and create the collision components
	for (int32 ComponentY = 0; ComponentY < NumComponentsY; ComponentY++)
	{
		for (int32 ComponentX = 0; ComponentX < NumComponentsX; ComponentX++)
		{
			const int32 HmX = ComponentX / ComponentsPerHeightmap;
			const int32 HmY = ComponentY / ComponentsPerHeightmap;
			FHeightmapInfo& HeightmapInfo = HeightmapInfos[HmX + HmY * NumHeightmapsX];

			ULandscapeComponent* LandscapeComponent = LandscapeComponents[ComponentX + ComponentY*NumComponentsX];
			LandscapeComponent->GenerateHeightmapMips(HeightmapInfo.HeightmapTextureMipData, ComponentX == NumComponentsX - 1 ? MAX_int32 : 0, ComponentY == NumComponentsY - 1 ? MAX_int32 : 0);
			LandscapeComponent->UpdateCollisionHeightData(
				HeightmapInfo.HeightmapTextureMipData[LandscapeComponent->CollisionMipLevel],
				LandscapeComponent->SimpleCollisionMipLevel > LandscapeComponent->CollisionMipLevel ? HeightmapInfo.HeightmapTextureMipData[LandscapeComponent->SimpleCollisionMipLevel] : nullptr);
			LandscapeComponent->UpdateCollisionLayerData();
		}
	}

	for (int32 HmIdx = 0; HmIdx < HeightmapInfos.Num(); HmIdx++)
	{
		FHeightmapInfo& HeightmapInfo = HeightmapInfos[HmIdx];

		// Add remaining mips down to 1x1 to heightmap texture. These do not represent quads and are just a simple averages of the previous mipmaps. 
		// These mips are not used for sampling in the vertex shader but could be sampled in the pixel shader.
		int32 Mip = HeightmapInfo.HeightmapTextureMipData.Num();
		int32 MipSizeU = (HeightmapInfo.HeightmapTexture->Source.GetSizeX()) >> Mip;
		int32 MipSizeV = (HeightmapInfo.HeightmapTexture->Source.GetSizeY()) >> Mip;
		while (MipSizeU > 1 && MipSizeV > 1)
		{
			HeightmapInfo.HeightmapTextureMipData.Add((FColor*)HeightmapInfo.HeightmapTexture->Source.LockMip(Mip));
			const int32 PrevMipSizeU = (HeightmapInfo.HeightmapTexture->Source.GetSizeX()) >> (Mip - 1);
			const int32 PrevMipSizeV = (HeightmapInfo.HeightmapTexture->Source.GetSizeY()) >> (Mip - 1);

			for (int32 Y = 0; Y < MipSizeV; Y++)
			{
				for (int32 X = 0; X < MipSizeU; X++)
				{
					FColor* const TexData = &(HeightmapInfo.HeightmapTextureMipData[Mip])[X + Y * MipSizeU];

					const FColor* const PreMipTexData00 = &(HeightmapInfo.HeightmapTextureMipData[Mip - 1])[(X * 2 + 0) + (Y * 2 + 0) * PrevMipSizeU];
					const FColor* const PreMipTexData01 = &(HeightmapInfo.HeightmapTextureMipData[Mip - 1])[(X * 2 + 0) + (Y * 2 + 1) * PrevMipSizeU];
					const FColor* const PreMipTexData10 = &(HeightmapInfo.HeightmapTextureMipData[Mip - 1])[(X * 2 + 1) + (Y * 2 + 0) * PrevMipSizeU];
					const FColor* const PreMipTexData11 = &(HeightmapInfo.HeightmapTextureMipData[Mip - 1])[(X * 2 + 1) + (Y * 2 + 1) * PrevMipSizeU];

					TexData->R = (((int32)PreMipTexData00->R + (int32)PreMipTexData01->R + (int32)PreMipTexData10->R + (int32)PreMipTexData11->R) >> 2);
					TexData->G = (((int32)PreMipTexData00->G + (int32)PreMipTexData01->G + (int32)PreMipTexData10->G + (int32)PreMipTexData11->G) >> 2);
					TexData->B = (((int32)PreMipTexData00->B + (int32)PreMipTexData01->B + (int32)PreMipTexData10->B + (int32)PreMipTexData11->B) >> 2);
					TexData->A = (((int32)PreMipTexData00->A + (int32)PreMipTexData01->A + (int32)PreMipTexData10->A + (int32)PreMipTexData11->A) >> 2);
				}
			}
			Mip++;
			MipSizeU >>= 1;
			MipSizeV >>= 1;
		}

		for (int32 i = 0; i < HeightmapInfo.HeightmapTextureMipData.Num(); i++)
		{
			HeightmapInfo.HeightmapTexture->Source.UnlockMip(i);
		}
		ULandscapeTextureHash::UpdateHash(HeightmapInfo.HeightmapTexture, ELandscapeTextureUsage::FinalData, ELandscapeTextureType::Heightmap);
		HeightmapInfo.HeightmapTexture->BeginCachePlatformData();
		HeightmapInfo.HeightmapTexture->ClearAllCachedCookedPlatformData();
		PendingTexturePlatformDataCreation.Add(HeightmapInfo.HeightmapTexture);
	}

	// Build a list of all unique materials the landscape uses
	TArray<UMaterialInterface*> LandscapeMaterials;

	for (ULandscapeComponent* Component : LandscapeComponents)
	{
		const int8 MaxLOD = IntCastChecked<int8>(FMath::CeilLogTwo(Component->SubsectionSizeQuads + 1) - 1);

		for (int8 LODIndex = 0; LODIndex < MaxLOD; ++LODIndex)
		{
			UMaterialInterface* Material = Component->GetLandscapeMaterial(LODIndex);
			LandscapeMaterials.AddUnique(Material);
		}
	}

	// Update all materials and recreate render state of all landscape components
	TArray<FComponentRecreateRenderStateContext> RecreateRenderStateContexts;

	SlowTask.EnterProgressFrame(1.0f);

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ALandscapeProxy::Import - Update Materials);
		
		// We disable automatic material update context, to manage it manually
		GDisableAutomaticTextureMaterialUpdateDependencies = true;
	
		FMaterialUpdateContext UpdateContext(FMaterialUpdateContext::EOptions::Default & ~FMaterialUpdateContext::EOptions::RecreateRenderStates);

		for (UTexture2D* Texture : PendingTexturePlatformDataCreation)
		{
			Texture->FinishCachePlatformData();
			Texture->PostEditChange();
			
			TSet<UMaterial*> BaseMaterialsThatUseThisTexture;

			for (UMaterialInterface* MaterialInterface : LandscapeMaterials)
			{
				if (DoesMaterialUseTexture(MaterialInterface, Texture))
				{
					UMaterial* Material = MaterialInterface->GetMaterial();
					bool MaterialAlreadyCompute = false;
					BaseMaterialsThatUseThisTexture.Add(Material, &MaterialAlreadyCompute);

					if (!MaterialAlreadyCompute)
					{
						if (Material->IsTextureForceRecompileCacheRessource(Texture))
						{
							UpdateContext.AddMaterial(Material);
							Material->UpdateMaterialShaderCacheAndTextureReferences();
						}
					}
				}
			}
		}
		
		GDisableAutomaticTextureMaterialUpdateDependencies = false;

		// Update MaterialInstances (must be done after textures are fully initialized)
		UpdateAllComponentMaterialInstances(UpdateContext, RecreateRenderStateContexts);
	}

	// Recreate the render state for this component, needed to update the static drawlist which has cached the MaterialRenderProxies
	// Must be after the FMaterialUpdateContext is destroyed
	RecreateRenderStateContexts.Reset();

	ALandscape* LandscapeActor = GetLandscapeActor();
	check(LandscapeActor != nullptr);

	// Create and initialize landscape info object after default layer exists
	ULandscapeInfo* LandscapeInfo = CreateLandscapeInfo();

	// Components need to be registered to be able to import the layer content and we will remove them if they should have not been visible
	bool ShouldComponentBeRegistered = GetLevel()->bIsVisible;
	RegisterAllComponents();

	// Create the default layer after components are registered
	if (LandscapeActor->GetEditLayersConst().IsEmpty() && InImportLayers.IsEmpty())
	{
		LandscapeActor->CreateDefaultLayer();
	}

	TSet<ULandscapeComponent*> ComponentsToProcess;

	struct FLayerImportSettings
	{
		FGuid SourceLayerGuid;
		FGuid DestinationLayerGuid;
	};

	TArray<FLayerImportSettings> LayerImportSettings;		

	// Only create Layers on main Landscape
	if (LandscapeActor == this && !InImportLayers.IsEmpty())
	{
		for (const FLandscapeLayer& OldLayer : InImportLayers)
		{
			const FLandscapeLayer* NewLayer = LandscapeActor->DuplicateLayerAndMoveBrushes(OldLayer);
			if (NewLayer != nullptr)
			{
				check(NewLayer->EditLayer != nullptr); // it's possible DuplicateLayerAndMoveBrushes fails (e.g. max number of layers reached), but if not, we should always have an EditLayer

				FLayerImportSettings ImportSettings;
				ImportSettings.SourceLayerGuid = OldLayer.EditLayer->GetGuid();
				ImportSettings.DestinationLayerGuid = NewLayer->EditLayer->GetGuid();
				LayerImportSettings.Add(ImportSettings);
			}
		}

		LandscapeInfo->GetComponentsInRegion(InMinX, InMinY, InMaxX, InMaxY, ComponentsToProcess);
	}
	else
	{
		// In the case of a streaming proxy, we will generate the layer data for each components that the proxy hold so no need of the grid min/max to calculate the components to update
		if (LandscapeActor != this)
		{
			LandscapeActor->AddLayersToProxy(this);
		}

		// And we will fill all the landscape components with the provided final layer content put into the default layer (aka layer index 0)
		const ULandscapeEditLayerBase* DefaultEditLayer = LandscapeActor->GetEditLayerConst(0);
		check(DefaultEditLayer != nullptr);

		FLayerImportSettings ImportSettings;
		ImportSettings.SourceLayerGuid = FinalLayerGuid;
		ImportSettings.DestinationLayerGuid = DefaultEditLayer->GetGuid();
		LayerImportSettings.Add(ImportSettings);

		ComponentsToProcess.Append(ToRawPtrTArrayUnsafe(LandscapeComponents));
	}
	check(LandscapeActor->GetEditLayersConst().IsValidIndex(LandscapeActor->GetSelectedEditLayerIndex()));
	check(LayerImportSettings.Num() != 0);
	// Currently only supports reimporting heightmap data into a single edit layer, which will always be the default layer
	ReimportDestinationLayerGuid = LayerImportSettings[0].DestinationLayerGuid;

	TSet<UTexture2D*> LayersTextures;

	for (const FLayerImportSettings& ImportSettings : LayerImportSettings)
	{
		FLandscapeEditDataInterface LandscapeEdit(LandscapeInfo, false);
		FScopedSetLandscapeEditingLayer Scope(LandscapeActor, ImportSettings.DestinationLayerGuid);

		const TArray<uint16>* ImportHeightData = InImportHeightData.Find(ImportSettings.SourceLayerGuid);

		if (ImportHeightData != nullptr && ImportHeightData->Num() != 0)
		{
			LandscapeEdit.SetHeightData(InMinX, InMinY, InMaxX, InMaxY, (uint16*)ImportHeightData->GetData(), 0, false, nullptr);
		}

		const TArray<FLandscapeImportLayerInfo>* ImportWeightData = InImportMaterialLayerInfos.Find(ImportSettings.SourceLayerGuid);

		if (ImportWeightData != nullptr)
		{
			for (const FLandscapeImportLayerInfo& MaterialLayerInfo : *ImportWeightData)
			{
				if (MaterialLayerInfo.LayerInfo != nullptr && MaterialLayerInfo.LayerData.Num() > 0)
				{
					LandscapeEdit.SetAlphaData(MaterialLayerInfo.LayerInfo, InMinX, InMinY, InMaxX, InMaxY, MaterialLayerInfo.LayerData.GetData(), /*Stride = */0, ELandscapeLayerPaintingRestriction::None);

					// Add to TargetLayers if needed.
					if (!HasTargetLayer(MaterialLayerInfo.LayerInfo->GetLayerName()))
					{
						AddTargetLayer(MaterialLayerInfo.LayerInfo->GetLayerName(), FLandscapeTargetLayerSettings(MaterialLayerInfo.LayerInfo));
					}
				}
			}
		}

		for (ULandscapeComponent* Component : ComponentsToProcess)
		{
			FLandscapeLayerComponentData* ComponentLayerData = Component->GetLayerData(ImportSettings.DestinationLayerGuid);
			check(ComponentLayerData != nullptr);

			LayersTextures.Add(ComponentLayerData->HeightmapData.Texture);
			LayersTextures.Append(ToRawPtrTArrayUnsafe(ComponentLayerData->WeightmapData.Textures));
		}
	}

	// Retrigger a caching of the platform data as we wrote again in the textures
	for (UTexture2D* Texture : LayersTextures)
	{
		Texture->UpdateResource();
	}

	LandscapeActor->RequestLayersContentUpdateForceAll();

	if (!ShouldComponentBeRegistered)
	{
		UnregisterAllComponents();
	}

	ReimportHeightmapFilePath = InHeightmapFileName;

	LandscapeInfo->UpdateLayerInfoMap();
	
}


// ----------------------------------------------------------------------------------

ALandscapeProxy::FRawMeshExportParams::FUVConfiguration::FUVConfiguration()
{
	ExportUVMappingTypes.SetNumZeroed(2);
	// For legacy reasons, this is what used to be exported by default on UV channel 0-1 : 
	ExportUVMappingTypes[0] = EUVMappingType::RelativeToProxyBoundsUV;
	ExportUVMappingTypes[1] = EUVMappingType::RelativeToProxyBoundsUV;
}

int32 ALandscapeProxy::FRawMeshExportParams::FUVConfiguration::GetNumUVChannelsNeeded() const
{
	int32 Result = 0;
	const int32 NumMappingTypes = ExportUVMappingTypes.Num();
	for (int32 Index = 0; Index < NumMappingTypes; ++Index)
	{
		EUVMappingType MappingType = ExportUVMappingTypes[Index];
		if ((MappingType != EUVMappingType::None) && (MappingType != EUVMappingType::Num))
		{
			Result = FMath::Max(Index + 1, Result);
		}
	}
	return Result;
}


// ----------------------------------------------------------------------------------

const ALandscapeProxy::FRawMeshExportParams::FUVConfiguration& ALandscapeProxy::FRawMeshExportParams::GetUVConfiguration(int32 InComponentIndex) const
{
	return ComponentsUVConfiguration.IsSet() ? (*ComponentsUVConfiguration)[InComponentIndex] : UVConfiguration;
}

const FName& ALandscapeProxy::FRawMeshExportParams::GetMaterialSlotName(int32 InComponentIndex) const
{
	return ComponentsMaterialSlotName.IsSet() ? (*ComponentsMaterialSlotName)[InComponentIndex] : MaterialSlotName;
}

int32 ALandscapeProxy::FRawMeshExportParams::GetNumUVChannelsNeeded() const
{
	int32 Result = UVConfiguration.GetNumUVChannelsNeeded();
	if (ComponentsUVConfiguration.IsSet())
	{
		for (const FUVConfiguration& ComponentUVConfiguration : *ComponentsUVConfiguration)
		{
			Result = FMath::Max(Result, ComponentUVConfiguration.GetNumUVChannelsNeeded());
		}
	}
	return Result;
}


// ----------------------------------------------------------------------------------

namespace UE::Landscape
{
	const FIntPoint QuadPattern[4] =
	{
		FIntPoint(0, 0),
		FIntPoint(0, 1),
		FIntPoint(1, 1),
		FIntPoint(1, 0)
	};

	// Generate geometry which calculates where along the quad edge the visibility would cross the Threshold and connect these crossings with a straight line. 
	// https://en.wikipedia.org/wiki/Marching_squares
	// 16 = 2 ^ 4 cases of each of vertex of the quad being above or below the threshold. 
	// don.boogert-todo: perhaps we could reduce the copies of positions but there are more important performance issues in the export function than this.
	// don.boogert-todo: we don't consider saddle points correctly but it doesn't seem to be an issue for our application in landscape geometry export.
	void GenerateMarchingSquaresGeometry(const TStaticArray<float, UE_ARRAY_COUNT(QuadPattern)>& InVisibilities, float InThreshold, const TStaticArray<FVector, UE_ARRAY_COUNT(QuadPattern)>& InPositions, TArray<int32, TInlineAllocator<12>>& OutIndices, TArray<FVector, TInlineAllocator<6>>& OutPositions)
	{
		static_assert(UE_ARRAY_COUNT(QuadPattern) == 4, "Invalid quad pattern");
		uint8 V0 = InVisibilities[0] < InThreshold ? 1 : 0;
		uint8 V1 = InVisibilities[1] < InThreshold ? 1 : 0;
		uint8 V2 = InVisibilities[2] < InThreshold ? 1 : 0;
		uint8 V3 = InVisibilities[3] < InThreshold ? 1 : 0;

		uint8 Case = (V1 << 3) | (V2 << 2) | (V3 << 1) | V0;

		auto InterpolateWithVisibility = [&InVisibilities, &InPositions, InThreshold](uint8 InPointIndex) -> FVector
			{
				const uint32 OtherPointIndex = (InPointIndex + 1) % 4;
				float Visibility = InVisibilities[InPointIndex];
				float OtherVisibility = InVisibilities[OtherPointIndex];
				const float Alpha = (Visibility != OtherVisibility) 
					? FMath::Clamp((InThreshold - Visibility) / (OtherVisibility - Visibility), 0.0f, 1.0f)
					: 0.5f;
				return FMath::Lerp(InPositions[InPointIndex], InPositions[OtherPointIndex], Alpha);
			};
			
		switch (Case)
		{
		case 0:
			OutPositions.Empty();
			OutIndices.Empty();
			break;
		case 1:
			OutPositions = { InPositions[0], InterpolateWithVisibility(0), InterpolateWithVisibility(3)};
			OutIndices = { 0, 1, 2 };
			break;
		case 2:
			OutPositions = { InPositions[3],  InterpolateWithVisibility(3),  InterpolateWithVisibility(2) };
			OutIndices = { 0, 1, 2 };
			break;
		case 3:
			OutPositions = { InPositions[0],  InterpolateWithVisibility(0),  InterpolateWithVisibility(2), InPositions[3] };
			OutIndices = { 0, 1, 2, 0, 2, 3 };
			break;
		case 4:
			OutPositions = { InPositions[2],  InterpolateWithVisibility(2),  InterpolateWithVisibility(1) };
			OutIndices = { 0, 1, 2 };
			break;
		case 5:
			OutPositions = { InPositions[0],  InterpolateWithVisibility(0),  InterpolateWithVisibility(1), InPositions[2], InterpolateWithVisibility(2),  InterpolateWithVisibility(3) };
			OutIndices = { 0, 1, 5, 1, 2, 5, 2, 4, 5, 2, 3, 4 };
			break;
		case 6:
			OutPositions = { InPositions[3],  InterpolateWithVisibility(3),  InterpolateWithVisibility(1), InPositions[2] };
			OutIndices = { 0, 1, 2, 0, 2, 3 };
			break;
		case 7:
			OutPositions = { InPositions[0],  InterpolateWithVisibility(0),  InterpolateWithVisibility(1), InPositions[2], InPositions[3] };
			OutIndices = { 0, 1, 2, 0, 2, 3, 0, 3, 4 };
			break;
		case 8:
			OutPositions = { InPositions[1],  InterpolateWithVisibility(1),  InterpolateWithVisibility(0) };
			OutIndices = { 0, 1, 2 };
			break;
		case 9:
			OutPositions = { InPositions[0], InPositions[1],  InterpolateWithVisibility(1),  InterpolateWithVisibility(3) };
			OutIndices = { 0, 1, 2, 0, 2, 3 };
			break;
		case 10:
			OutPositions = { InPositions[1],  InterpolateWithVisibility(1),  InterpolateWithVisibility(2), InPositions[3], InterpolateWithVisibility(3),  InterpolateWithVisibility(0) };
			OutIndices = { 0, 1, 5, 1, 2, 5, 2, 4, 5, 2, 3, 4 };
			break;
		case 11:
			OutPositions = { InPositions[1],  InterpolateWithVisibility(1),  InterpolateWithVisibility(2), InPositions[3], InPositions[0] };
			OutIndices = { 0, 1, 2, 0, 2, 3, 0, 3, 4 };
			break;
		case 12:
			OutPositions = { InPositions[2],  InterpolateWithVisibility(2),  InterpolateWithVisibility(0), InPositions[1] };
			OutIndices = { 0, 1, 2, 0, 2, 3 };
			break;
		case 13:
			OutPositions = { InPositions[2],  InterpolateWithVisibility(2),  InterpolateWithVisibility(3), InPositions[0], InPositions[1] };
			OutIndices = { 0, 1, 2, 0, 2, 3, 0, 3, 4 };
			break;
		case 14:
			OutPositions = { InPositions[3],  InterpolateWithVisibility(3),  InterpolateWithVisibility(0), InPositions[1], InPositions[2] };
			OutIndices = { 0, 1, 2, 0, 2, 3, 0, 3, 4 };
			break;
		case 15:
			OutPositions = { InPositions[0], InPositions[1], InPositions[2], InPositions[3] };
			OutIndices = { 0, 1, 2, 0, 2, 3 };
			break;
		default:
			check(false);
		}
	}
} // namespace  UE::Landscape


TSharedRef<UE::Landscape::Nanite::FAsyncBuildData> ALandscapeProxy::MakeAsyncNaniteBuildData(int32 InLODToExport, const TArray<ULandscapeComponent*>& InComponentsToExport) const
{
	TSharedRef<UE::Landscape::Nanite::FAsyncBuildData> AsyncBuildData = MakeShared<UE::Landscape::Nanite::FAsyncBuildData>();

	if (UWorld* World = GetWorld())
	{
		// Make sure the requested LOD is valid
		int32 FinalLODToExport = FMath::Clamp<int32>(InLODToExport, 0, FMath::CeilLogTwo(SubsectionSizeQuads + 1) - 1);

		AsyncBuildData->LOD = FinalLODToExport;
		AsyncBuildData->LandscapeWeakRef = MakeWeakObjectPtr(const_cast<ALandscapeProxy*>(this));
		AsyncBuildData->LandscapeSubSystemWeakRef = MakeWeakObjectPtr(World->GetSubsystem<ULandscapeSubsystem>());

		for (ULandscapeComponent* Component : InComponentsToExport)
		{
			check(LandscapeComponents.Contains(Component)); // component we're requesting to export has to be in the proxy.
			UMaterialInterface* Material = nullptr;
			if (Component)
			{
				Material = Component->GetMaterialInstance(0u);
				AsyncBuildData->InputMaterialSlotNames.Add(FName(*FString::Format(TEXT("LandscapeMat_{0}"), { AsyncBuildData->InputComponents.Num() })));
				AsyncBuildData->InputMaterials.Add(Material ? Material : UMaterial::GetDefaultMaterial(MD_Surface));
				AsyncBuildData->InputComponents.Add(Component);
			}
		}

		if (AsyncBuildData->InputComponents.Num() == 0)
		{
			UE_LOGF(LogLandscape, Verbose, "%ls : no Nanite mesh to export", *GetActorNameOrLabel());
		}

		if (AsyncBuildData->InputMaterials.Num() > NANITE_MAX_CLUSTER_MATERIALS)
		{
			UE_LOGF(LogLandscape, Warning, "%ls : Nanite landscape mesh would have more than %i materials, which is currently not supported. Please reduce the number of components in this landscape actor to enable Nanite.", *GetActorNameOrLabel(), NANITE_MAX_CLUSTER_MATERIALS)
		}

		for (ULandscapeComponent* LandscapeComponent : InComponentsToExport)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ALandscapeProxy::MakeAsyncBuildData-CopyHeightAndVisibility);
			FLandscapeComponentDataInterface DataInterface(LandscapeComponent, FinalLODToExport, false);

			UE::Landscape::Nanite::FAsyncComponentData AsyncComponentData;

			DataInterface.GetHeightmapTextureData(AsyncComponentData.HeightAndNormalData, false);
			DataInterface.GetWeightmapTextureData(LandscapeComponent->GetVisibilityLayer(), AsyncComponentData.Visibility);

			AsyncComponentData.ComponentDataInterface = MakeShared<FLandscapeComponentDataInterfaceBase>(LandscapeComponent, FinalLODToExport, false);
			int32 HeightmapSize = ((LandscapeComponent->SubsectionSizeQuads + 1) * LandscapeComponent->NumSubsections) >> FinalLODToExport;
			AsyncComponentData.ComponentDataInterface->HeightmapStride = HeightmapSize;
			AsyncComponentData.ComponentDataInterface->HeightmapComponentOffsetX = 0;
			AsyncComponentData.ComponentDataInterface->HeightmapComponentOffsetY = 0;
			AsyncBuildData->ComponentData.Add(LandscapeComponent, AsyncComponentData);
		}

		// Create the UStaticMesh in advance, from the game thread.
		UPackage* Package = GetPackage();
		AsyncBuildData->NaniteStaticMesh.Reset(NewObject<UStaticMesh>(/*Outer = */Package, MakeUniqueObjectName(/*Parent = */Package, UStaticMesh::StaticClass(), TEXT("LandscapeNaniteMesh"))));
	}

	return AsyncBuildData;
}

void ALandscape::SetNanitePositionPrecision(int32 InPrecision, bool bInShouldDirtyPackage)
{
	FProperty* NanitePositionPrecisionProperty = StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(ALandscapeProxy, NanitePositionPrecision));
	check(NanitePositionPrecisionProperty);

	// PostEditChangeProperty propagates to proxies and invalidates nanite/render state
	if (NanitePositionPrecision != InPrecision)
	{
		Modify(bInShouldDirtyPackage);
		NanitePositionPrecision = InPrecision;

		FPropertyChangedEvent PropertyChangedEvent(NanitePositionPrecisionProperty);
		PostEditChangeProperty(PropertyChangedEvent);
	}
}

bool ALandscapeProxy::ExportToRawMesh(const FRawMeshExportParams& InExportParams, FMeshDescription& OutRawMesh) const
{
	FRawMeshExportParams ExportParams = InExportParams;
	ExportParams.ExportLOD = FMath::Clamp<int32>(InExportParams.ExportLOD, 0, FMath::CeilLogTwo(SubsectionSizeQuads + 1) - 1);

	TArray<ULandscapeComponent*> LandscapeComponentsCopy(LandscapeComponents);
	TSharedRef<UE::Landscape::Nanite::FAsyncBuildData> AsyncBuildData = MakeAsyncNaniteBuildData(ExportParams.ExportLOD, LandscapeComponentsCopy);
	return ExportToRawMeshDataCopy(ExportParams, OutRawMesh, AsyncBuildData.Get());
}

bool ALandscapeProxy::ExportToRawMeshDataCopy(const FRawMeshExportParams& InExportParams, FMeshDescription& OutRawMesh, const UE::Landscape::Nanite::FAsyncBuildData& AsyncData) const
{
	using EUVMappingType = FRawMeshExportParams::EUVMappingType;

	const double StartTime = FPlatformTime::Seconds();
	const FMeshDescription& Mesh = OutRawMesh;
	ON_SCOPE_EXIT
	{
		double StartupDuration = FPlatformTime::Seconds() - StartTime;
		UE_LOGF(LogLandscape, Verbose, "ExportToRawMeshDataCopy took %0.4f seconds for %i vertices, %i polygons", StartupDuration, Mesh.Vertices().GetArraySize(), Mesh.Polygons().GetArraySize());
	};

	check(InExportParams.ExportLOD == AsyncData.LOD);
	
	TRACE_CPUPROFILER_EVENT_SCOPE(ALandscapeProxy::ExportToRawMeshDataCopy);

	TArray<ULandscapeComponent*> ComponentsToExport;
	if (InExportParams.ComponentsToExport.IsSet())
	{
		ComponentsToExport = *InExportParams.ComponentsToExport;
	}
	else
	{
		GetComponents(ComponentsToExport);
	}

	if (ComponentsToExport.IsEmpty())
	{
		// Nothing to export :
		return false;
	}
	
	checkf(!InExportParams.ComponentsUVConfiguration.IsSet() || InExportParams.ComponentsUVConfiguration->Num() == ComponentsToExport.Num(), TEXT("If ComponentsUVConfiguration is passed (per-component UV configuration), it must have the same number of entries as the number of components to export."))
	checkf(!InExportParams.ComponentsMaterialSlotName.IsSet() || InExportParams.ComponentsMaterialSlotName->Num() == ComponentsToExport.Num(), TEXT("If ComponentsMaterialSlotName is passed (per-component material slot), it must have the same number of entries as the number of components to export."))

	// Get the tight bounds around the proxy's component (in quads, relative to the proxy's origin) :
	const FIntRect LandscapeProxyBoundsRect = GetBoundingRect();
	const FVector2f LandscapeProxyBoundsRectUVScale = FVector2f(1.0f, 1.0f) / FVector2f(LandscapeProxyBoundsRect.Size());

	FStaticMeshAttributes Attributes(OutRawMesh);
	TVertexAttributesRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
	TEdgeAttributesRef<bool> EdgeHardnesses = Attributes.GetEdgeHardnesses();
	TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
	TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();

	const int32 NumUVChannels = InExportParams.GetNumUVChannelsNeeded();
	if (VertexInstanceUVs.GetNumChannels() < NumUVChannels)
	{
		VertexInstanceUVs.SetNumChannels(NumUVChannels);
	}

	const bool bGenerateOnePolygroupPerComponent = InExportParams.ComponentsMaterialSlotName.IsSet();

	// Export data for each component
	int32 ComponentIndex = 0;
	FPolygonGroupID PolygonGroupID = INDEX_NONE;
	OutRawMesh.ReserveNewPolygonGroups(bGenerateOnePolygroupPerComponent ? ComponentsToExport.Num() : 1);

	const int32 ComponentSizeQuadsLOD = ((ComponentSizeQuads + 1) >> InExportParams.ExportLOD) - 1;
	const float LODScale = (float) ComponentSizeQuadsLOD / (float) ComponentSizeQuads;
	const float InvLODScale = (float)ComponentSizeQuads / (float)ComponentSizeQuadsLOD;

	// min/max section bases of all exported components
	FIntPoint MinSectionBase(INT_MAX, INT_MAX);
	FIntPoint MaxSectionBase(-INT_MAX, -INT_MAX);
	for (ULandscapeComponent* Component : ComponentsToExport)
	{
		check(Component->ComponentSizeQuads == ComponentSizeQuads);		// must all be the same number of quads

		MinSectionBase = MinSectionBase.ComponentMin(Component->SectionBaseX);
		MaxSectionBase = MaxSectionBase.ComponentMax(Component->SectionBaseY);
	}

	FVector2f ExportedSectionMinQuadCoord(MinSectionBase.X, MinSectionBase.Y);
	FVector2f ExportedSectionMaxQuadCoord(MaxSectionBase.X + ComponentSizeQuads, MaxSectionBase.Y + ComponentSizeQuads);

	// setup single UV chart scale and offset
	float LightmapEdgePadding = 4;
	FVector2f LightmapUVScale(
		1.0f / (ExportedSectionMaxQuadCoord.X - ExportedSectionMinQuadCoord.X + LightmapEdgePadding * 2.0f),
		1.0f / (ExportedSectionMaxQuadCoord.Y - ExportedSectionMinQuadCoord.Y + LightmapEdgePadding * 2.0f));
	FVector2f LightmapUVOffset = (-ExportedSectionMinQuadCoord + LightmapEdgePadding) * LightmapUVScale;
	
	const bool bExportSkirt = InExportParams.SkirtDepth.IsSet();
	for (ULandscapeComponent* Component : ComponentsToExport)
	{
		TStaticArray<ULandscapeComponent*, 9> NeighborComponents;
		Component->GetLandscapeComponentNeighbors3x3(NeighborComponents);

		// Export an edge if there isn't a neighoring component or if the neighbor is not a part of set of components
		// to export
		auto ExportEdgePadding = [&NeighborComponents, &ComponentsToExport, bExportSkirt] (int32 Edge)
		{
			const ULandscapeComponent* Neighbor = NeighborComponents[Edge];
			return bExportSkirt && (!Neighbor || !ComponentsToExport.Contains(Neighbor)) ? 1 : 0;
		};
		
		// compute skirt padding
		const int32 MinXPadding = ExportEdgePadding(3); // left
		const int32 MaxXPadding = ExportEdgePadding(5); // right
		const int32 XPadding = MinXPadding + MaxXPadding;
		
		const int32 MinYPadding = ExportEdgePadding(1); // bottom
		const int32 MaxYPadding = ExportEdgePadding(7); // top
		const int32 YPadding = MinYPadding + MaxYPadding;

		TRACE_CPUPROFILER_EVENT_SCOPE(ExportComponent);

		ON_SCOPE_EXIT
		{
			++ComponentIndex;
		};

		// Early out if the Landscape bounds and given bounds do not overlap at all
		if (InExportParams.ExportBounds.IsSet() && !FBoxSphereBounds::BoxesIntersect(Component->Bounds, *InExportParams.ExportBounds))
		{
			continue;
		}

		FTransform ComponentToExportCoordinatesTransform = FTransform::Identity;
		switch (InExportParams.ExportCoordinatesType)
		{
		case FRawMeshExportParams::EExportCoordinatesType::Absolute:
			ComponentToExportCoordinatesTransform = Component->GetComponentTransform();
			break;
		case FRawMeshExportParams::EExportCoordinatesType::RelativeToProxy:
			ComponentToExportCoordinatesTransform = Component->GetComponentTransform() * GetTransform().Inverse(); // component to world to proxy
			break;
		default:
			break;
		}

		// For this component, what unique UV mapping types should we compute?
		const FRawMeshExportParams::FUVConfiguration& ComponentUVConfiguration = InExportParams.GetUVConfiguration(ComponentIndex);

		// determine if we need to split each subsection - so they don't share vertices, and can have a UV seam between them.
		int32 NumSplits = 1;
		int32 SplitSizeQuads = ComponentSizeQuadsLOD;
		if (NumSubsections > 1)
		{
			// and only if a Heightmap or Weightmap UV is requested
			for (int32 UVChannel = 0; UVChannel < NumUVChannels; ++UVChannel)
			{
				EUVMappingType UVMappingType = ComponentUVConfiguration.ExportUVMappingTypes.IsValidIndex(UVChannel) ? ComponentUVConfiguration.ExportUVMappingTypes[UVChannel] : FRawMeshExportParams::EUVMappingType::None;
				if (UVMappingType == EUVMappingType::HeightmapUV || UVMappingType == EUVMappingType::WeightmapUV)
				{
					const int32 SubsectionSizeQuadsLOD = ((SubsectionSizeQuads + 1) >> InExportParams.ExportLOD) - 1;
					NumSplits = NumSubsections;
					SplitSizeQuads = SubsectionSizeQuadsLOD;
					break;
				}
			}
		}

		const FLandscapeComponentDataInterfaceBase& CDI = *AsyncData.ComponentData.Find(Component)->ComponentDataInterface;

		const FIntPoint ComponentOffsetRelativeToProxyBoundsQuads = Component->GetSectionBase() - SectionBase - LandscapeProxyBoundsRect.Min;
		const FVector2f ComponentOffsetRelativeToProxyBoundsQuadsLOD = FVector2f(ComponentOffsetRelativeToProxyBoundsQuads) * LODScale;

		const FVector2f GlobalQuadCoordsOffset = FVector2f(Component->GetSectionBase());

		const FVector2f ComponentUVScaleRelativeToProxyBoundsLOD = LandscapeProxyBoundsRectUVScale * InvLODScale;

		const FVector2f ComponentHeightmapUVBias = FVector2f(static_cast<float>(Component->HeightmapScaleBias.Z), static_cast<float>(Component->HeightmapScaleBias.W));
		const FVector2f ComponentHeightmapUVScale = FVector2f(static_cast<float>(Component->HeightmapScaleBias.X), static_cast<float>(Component->HeightmapScaleBias.Y));
		const FVector2f ComponentHeightmapUVScaleLOD = ComponentHeightmapUVScale * InvLODScale;
		const FVector2f ComponentHeightmapUVPixelOffset = ComponentHeightmapUVScale * 0.5f;
		const FVector2f ComponentWeightmapUVScale = FVector2f(static_cast<float>(Component->WeightmapScaleBias.X), static_cast<float>(Component->WeightmapScaleBias.Y));
		const FVector2f ComponentWeightmapUVScaleLOD = ComponentWeightmapUVScale * InvLODScale;
		const FVector2f ComponentWeightmapUVPixelOffset = ComponentWeightmapUVScale * 0.5f; // I could have used Component->WeightmapScaleBias.ZW but then it would be confusing because it doesn't have the same signification as the heightmap UV bias

		const int32 PaddedComponentSizeXQuadsLOD = ComponentSizeQuadsLOD + XPadding;
		const int32 PaddedComponentSizeYQuadsLOD = ComponentSizeQuadsLOD + YPadding;

		const int32 NumQuads = PaddedComponentSizeXQuadsLOD * PaddedComponentSizeYQuadsLOD;
		const int32 NumFaces = NumQuads * 2;
		const int32 MaxNumVertices = NumFaces * 3;	// this is an upper bound, it doesn't take into account shared vertices

		OutRawMesh.ReserveNewVertices(MaxNumVertices);
		OutRawMesh.ReserveNewPolygons(NumFaces);
		OutRawMesh.ReserveNewVertexInstances(MaxNumVertices);
		OutRawMesh.ReserveNewEdges(MaxNumVertices);

		if (bGenerateOnePolygroupPerComponent || (OutRawMesh.PolygonGroups().Num() == 0))
		{
			PolygonGroupID = OutRawMesh.CreatePolygonGroup();
			PolygonGroupImportedMaterialSlotNames[PolygonGroupID] = InExportParams.GetMaterialSlotName(ComponentIndex);;
		}
		check(PolygonGroupID != INDEX_NONE);

		// Check if there are any holes
		const int32 VisThreshold = 170;

		const TArray<FColor>& HeightAndNormals = AsyncData.ComponentData.Find(Component)->HeightAndNormalData;
		const TArray<uint8>& VisDataMap = AsyncData.ComponentData.Find(Component)->Visibility;

		const float SquaredSphereRadius = InExportParams.ExportBounds.IsSet() ? FMath::Square(static_cast<float>(InExportParams.ExportBounds->SphereRadius)) : 0.0f;

		const float SkirtDepth = InExportParams.SkirtDepth.Get(0.0f);

		auto GetVertex = [&CDI, ComponentSizeQuadsLOD, SkirtDepth, &HeightAndNormals](int32 VertexX, int32 VertexY) -> FVector
		{
			const int32 ClampedVertexX = FMath::Clamp(VertexX, 0, ComponentSizeQuadsLOD);
			const int32 ClampedVertexY = FMath::Clamp(VertexY, 0, ComponentSizeQuadsLOD);
			const bool bIsInside = ClampedVertexX == VertexX && ClampedVertexY == VertexY;

			FVector Position = CDI.GetLocalVertex(ClampedVertexX, ClampedVertexY, HeightAndNormals);
			
			if (!bIsInside)
			{
				// in a skirt around the edge
				int32 SignX = FMath::Sign(VertexX - ClampedVertexX);
				int32 SignY = FMath::Sign(VertexY - ClampedVertexY);

				const float Diff = CDI.GetLocalHeight(ClampedVertexX - SignX, ClampedVertexY - SignY, HeightAndNormals) - Position.Z;
				
				Position.X = CDI.GetScaleFactor() * VertexX;
				Position.Y = CDI.GetScaleFactor() * VertexY;

				// Maintain the slope at the edge by extrapolating the skirt vertex position but only if the slope is going downwards from the edge vertex to the skirt vertex
				//  (otherwise, in case of steep slopes, the skirt depth might not be enough to bring the skirt's vertex underneath the neighboring landscape proxy)
				Position -= FVector(0.0f, 0.0f, FMath::Max(Diff, 0.0f) + SkirtDepth);
			}
			return Position;
		};

		auto GetVisibilityValue = [&CDI, ComponentSizeQuadsLOD, &VisDataMap](int32 X, int32 Y) -> float
		{
			X = FMath::Clamp(X, 0, ComponentSizeQuadsLOD);
			Y = FMath::Clamp(Y, 0, ComponentSizeQuadsLOD);

			int32 TexelX, TexelY;
			CDI.VertexXYToTexelXY(X, Y, TexelX, TexelY);
			return VisDataMap[CDI.TexelXYToIndex(TexelX, TexelY)] / 255.0f;
		};

		auto GetBasis = [&CDI, ComponentSizeQuadsLOD, &HeightAndNormals, InvScaleFactor = 1.0f / CDI.GetScaleFactor()](float X, float Y, FVector& OutLocalTangentX, FVector& OutLocalTangentY, FVector& OutLocalTangentZ)
		{
			X = X * InvScaleFactor;
			Y = Y * InvScaleFactor;

			int32 VertexX = X;
			int32 VertexY = Y;
			float Alpha = X - VertexX;
			float Beta = Y - VertexY;

			FVector TangentX[4];
			FVector TangentY[4];
			FVector TangentZ[4];

			for (int32 i = 0; i < 4; ++i)
			{
				int32 SX = FMath::Clamp(VertexX + UE::Landscape::QuadPattern[i].X, 0, ComponentSizeQuadsLOD);
				int32 SY = FMath::Clamp(VertexY + UE::Landscape::QuadPattern[i].Y, 0, ComponentSizeQuadsLOD);

				CDI.GetLocalTangentVectors(SX, SY, TangentX[i], TangentY[i], TangentZ[i], HeightAndNormals);
			}

			// don.boogert-todo: better rotation of a basis here 
			OutLocalTangentX = FMath::BiLerp(TangentX[0], TangentX[3], TangentX[1], TangentX[2], Alpha, Beta);
			OutLocalTangentY = FMath::BiLerp(TangentY[0], TangentY[3], TangentY[1], TangentY[2], Alpha, Beta);
			OutLocalTangentZ = FMath::BiLerp(TangentZ[0], TangentZ[3], TangentZ[1], TangentZ[2], Alpha, Beta);

			OutLocalTangentX.Normalize();
			OutLocalTangentY.Normalize();
			OutLocalTangentZ.Normalize();

		};

		// size the TPointHashGrid2 cells to be the size to something smaller than a landscape quad
		FBox Box(FVector(0, 0, 0), FVector(0.1f, 0.1f, 0.1f));
		FBox TransformedBox = Box.TransformBy(ComponentToExportCoordinatesTransform.ToMatrixWithScale());
		const float CellSize = TransformedBox.GetSize().GetAbsMax();
		check(CellSize > 0.0f);

		// We don't want to deduplicate vertices across splits (as there are sometimes UV seams between subsections)
		// So we build a separate deduplication structure per split.
		check(NumSplits >= 1 && NumSplits <= 2);	// the inline allocator only supports up to 2x2 total
		TArray<UE::Geometry::TPointHashGrid3<FVertexID, float>, TInlineAllocator<4>> SplitDeduplicatedVertexIDs;
		const int32 TotalSplits = NumSplits * NumSplits;
		for (int32 SplitIndex = 0; SplitIndex < TotalSplits; SplitIndex++)
		{
			new(SplitDeduplicatedVertexIDs) UE::Geometry::TPointHashGrid3<FVertexID, float>(CellSize, INDEX_NONE);
			SplitDeduplicatedVertexIDs.Last().Reserve((PaddedComponentSizeXQuadsLOD + 1) * (PaddedComponentSizeYQuadsLOD + 1) / TotalSplits);
		}
		
		// Export to MeshDescription
		for (int32 PaddedY = 0; PaddedY < PaddedComponentSizeYQuadsLOD; PaddedY++)
		{
			int32 y = PaddedY - MinYPadding;
			for (int32 PaddedX = 0; PaddedX < PaddedComponentSizeXQuadsLOD; PaddedX++)
			{
				int32 x = PaddedX - MinXPadding;

				TArray<FVector, TInlineAllocator<6>> NewLocalPositions;
				TArray<int32, TInlineAllocator<12>> NewIndices;

				bool bProcess = !InExportParams.ExportBounds.IsSet();		// no export bounds => always process

				if (VisDataMap.IsEmpty())
				{
					for (int32 i = 0; i < UE_ARRAY_COUNT(UE::Landscape::QuadPattern); i++)
					{
						int32 VertexX = x + UE::Landscape::QuadPattern[i].X;
						int32 VertexY = y + UE::Landscape::QuadPattern[i].Y;
						NewLocalPositions.Add(GetVertex(VertexX, VertexY));
					}
					NewIndices = { 0, 1, 2, 0, 2, 3 };
				}
				else
				{
					TStaticArray<FVector, UE_ARRAY_COUNT(UE::Landscape::QuadPattern)> LocalPositions;
					TStaticArray<float, UE_ARRAY_COUNT(UE::Landscape::QuadPattern)> Visibilities;
					for (int32 i = 0; i < UE_ARRAY_COUNT(UE::Landscape::QuadPattern); i++)
					{
						int32 VertexX = x + UE::Landscape::QuadPattern[i].X;
						int32 VertexY = y + UE::Landscape::QuadPattern[i].Y;

						LocalPositions[i] = GetVertex(VertexX, VertexY);
						Visibilities[i] = GetVisibilityValue(VertexX, VertexY);
					}
					UE::Landscape::GenerateMarchingSquaresGeometry(Visibilities, LANDSCAPE_VISIBILITY_THRESHOLD, LocalPositions, NewIndices, NewLocalPositions);
				}

				// transform all the positions
				TArray<FVector, TInlineAllocator<6>> NewPositions;
				NewPositions.SetNumUninitialized(NewLocalPositions.Num());
				for (int32 i = 0; i < NewLocalPositions.Num(); ++i)
				{
					NewPositions[i] = ComponentToExportCoordinatesTransform.TransformPosition(NewLocalPositions[i]);

					// If at least one vertex is within the given bounds we should process the quad
					if (!bProcess && InExportParams.ExportBounds->ComputeSquaredDistanceFromBoxToPoint(NewPositions[i]) < SquaredSphereRadius)
					{
						bProcess = true;
					}
				}

				if (bProcess)
				{
					//Fill the vertexID we need
					TArray<FVertexID, TInlineAllocator<6>> VertexIDs;
					VertexIDs.Reserve(NewPositions.Num());

					TArray<FVertexInstanceID, TInlineAllocator<12>> VertexInstanceIDs;
					VertexInstanceIDs.Reserve(NewIndices.Num());

					// determine which deduplication structure to use (don't share vertices across splits)
					int32 SplitX = FMath::Clamp(x / SplitSizeQuads, 0, NumSplits - 1);
					int32 SplitY = FMath::Clamp(y / SplitSizeQuads, 0, NumSplits - 1);
					int32 SplitIndex = SplitX + NumSplits * SplitY;
					UE::Geometry::TPointHashGrid3<FVertexID, float>& DeduplicatedVertexIDs = SplitDeduplicatedVertexIDs[SplitIndex];

					// Fill positions
					for (int32 i = 0; i < NewPositions.Num(); i++)
					{
						const FVector3f NewPos (NewPositions[i]);
						TPair<FVertexID, float> ExistingVertexID = DeduplicatedVertexIDs.FindNearestInRadius(
							NewPos, TMathUtilConstants<float>::ZeroTolerance, 
							[&VertexPositions, NewPos](const FVertexID& VertexID)
							{
								return FVector3f::DistSquared(VertexPositions[VertexID], NewPos);
							});
						FVertexID VertexID;
						if (ExistingVertexID.Key == INDEX_NONE)
						{
							VertexID = OutRawMesh.CreateVertex();
							DeduplicatedVertexIDs.InsertPointUnsafe(VertexID, NewPos);
							VertexPositions[VertexID] = NewPos;
						}
						else
						{
							VertexID = ExistingVertexID.Key;
						}
						
						VertexIDs.Add(VertexID);
					}

					// Create triangles
					int32 NumTris = NewIndices.Num() / 3;
					TArray<bool, TInlineAllocator<4>> DegenerateTriangles;
					DegenerateTriangles.Init(false, NumTris);
					for (int32 Triangle = 0; Triangle < NumTris; ++Triangle)
					{
						const FVertexID VertexID0 = VertexIDs[NewIndices[Triangle * 3 + 0]];
						const FVertexID VertexID1 = VertexIDs[NewIndices[Triangle * 3 + 1]];
						const FVertexID VertexID2 = VertexIDs[NewIndices[Triangle * 3 + 2]];

						// it's possible now for the marching squares to generate vertices which end up degenerate after the 
						DegenerateTriangles[Triangle] = VertexID0 == VertexID1 || VertexID0 == VertexID2 || VertexID1 == VertexID2;

						VertexInstanceIDs.Add(OutRawMesh.CreateVertexInstance(VertexID0));
						VertexInstanceIDs.Add(OutRawMesh.CreateVertexInstance(VertexID1));
						VertexInstanceIDs.Add(OutRawMesh.CreateVertexInstance(VertexID2));
					}

					// allocate 4 uv channels inline.
					TArray<FVector2f, TInlineAllocator<6 * 4>> UVs;
					TArray<FVector3f, TInlineAllocator<6>> Tangents;
					TArray<float, TInlineAllocator<6>> BinormalSigns;
					TArray<FVector3f, TInlineAllocator<6>> Normals;

					// Fill other vertex data
					for (int32 i = 0; i < NewPositions.Num(); i++)
					{
						const float VertexX = NewLocalPositions[i].X;
						const float VertexY = NewLocalPositions[i].Y;

						FVector LocalTangentX, LocalTangentY, LocalTangentZ;
						
						GetBasis(VertexX, VertexY, LocalTangentX, LocalTangentY, LocalTangentZ);

						Tangents.Add(FVector3f(LocalTangentX));
						BinormalSigns.Add(GetBasisDeterminantSign(LocalTangentX, LocalTangentY, LocalTangentZ));
						Normals.Add(FVector3f(LocalTangentZ));

						// Compute all UV values that we need :
						// quad coordinates (local position is just quad coords, scaled)
						const FVector2f QuadCoords = FVector2f(VertexX, VertexY) * (1.0f / CDI.GetScaleFactor());
						for (int32 UVChannel = 0; UVChannel < NumUVChannels; ++UVChannel)
						{
							EUVMappingType UVMappingType = ComponentUVConfiguration.ExportUVMappingTypes.IsValidIndex(UVChannel) ? ComponentUVConfiguration.ExportUVMappingTypes[UVChannel] : FRawMeshExportParams::EUVMappingType::None;
							switch (UVMappingType)
							{
							case EUVMappingType::RelativeToProxyBoundsUV:
							{
								FVector2f UV = (ComponentOffsetRelativeToProxyBoundsQuadsLOD + QuadCoords) * ComponentUVScaleRelativeToProxyBoundsLOD;
								UVs.Add(UV);
								break;
							}
							case EUVMappingType::HeightmapUV:
							{
								// multiple subsections have a UV seam between them -- we offset the UV by one pixel per subsection
								// (technically this should use subsection X/Y instead of split X/Y, but they are the same when it matters)
								FVector2f SplitOffsetQuadCoords = QuadCoords + FVector2f(SplitX, SplitY) * LODScale;
								FVector2f UV = SplitOffsetQuadCoords * ComponentHeightmapUVScaleLOD + ComponentHeightmapUVPixelOffset + ComponentHeightmapUVBias;
								UVs.Add(UV);
								break;
							}
							case EUVMappingType::WeightmapUV:
							{
								// multiple subsections have a UV seam between them -- we offset the UV by one pixel per subsection
								// (technically this should use subsection X/Y instead of split X/Y, but they are the same when it matters)
								FVector2f SplitOffsetQuadCoords = QuadCoords + FVector2f(SplitX, SplitY) * LODScale;
								FVector2f UV = SplitOffsetQuadCoords * ComponentWeightmapUVScaleLOD + ComponentWeightmapUVPixelOffset;
								UVs.Add(UV);
								break;
							}
							case EUVMappingType::TerrainCoordMapping_XY:
							case EUVMappingType::TerrainCoordMapping_XZ:
							case EUVMappingType::TerrainCoordMapping_YZ:
							{
								FVector2f UV = (GlobalQuadCoordsOffset + QuadCoords * InvLODScale);
								if (UVMappingType == EUVMappingType::TerrainCoordMapping_XY)
								{
									// Already in XY mapping
								}
								else
								{
									UV[0] = (UVMappingType == EUVMappingType::TerrainCoordMapping_XZ) ? UV[0] : UV[1];
									UV[1] = static_cast<float>(NewLocalPositions[i].Z);
								}
								UVs.Add(UV);
								break;
							}
							case EUVMappingType::LightmapUV:
							{
								// convert LOD quad coord to quad coord
								FVector2f UV = (ComponentOffsetRelativeToProxyBoundsQuadsLOD + QuadCoords * InvLODScale);
								UV = UV * LightmapUVScale + LightmapUVOffset;
								UVs.Add(UV);
								break;
							}
							default:
								// Valid case: we might not be computing a UV channel for this component
								break;
							}
						}
					}

					for (int32 Vertex = 0; Vertex < NewIndices.Num(); ++Vertex)
					{
						FVertexInstanceID VertexInstanceID = VertexInstanceIDs[Vertex];
						VertexInstanceTangents[VertexInstanceID] = Tangents[NewIndices[Vertex]];
						VertexInstanceBinormalSigns[VertexInstanceID] = BinormalSigns[NewIndices[Vertex]];
						VertexInstanceNormals[VertexInstanceID] = Normals[NewIndices[Vertex]];

						for (int32 UVChannel = 0; UVChannel < NumUVChannels; ++UVChannel)
						{
							VertexInstanceUVs.Set(VertexInstanceID, UVChannel, UVs[UVChannel + NumUVChannels * NewIndices[Vertex]]);
						}
					}

					auto AddTriangle = [&OutRawMesh, &EdgeHardnesses, &PolygonGroupID, &VertexIDs, &VertexInstanceIDs](int32 BaseIndex)
					{
						//Create a polygon from this triangle
						TStaticArray<FVertexInstanceID, 3> PerimeterVertexInstances;
						for (int32 Corner = 0; Corner < 3; ++Corner)
						{
							PerimeterVertexInstances[Corner] = VertexInstanceIDs[BaseIndex + Corner];
						}
						// Insert a polygon into the mesh
						TArray<FEdgeID> NewEdgeIDs;
						const FPolygonID NewPolygonID = OutRawMesh.CreatePolygon(PolygonGroupID, PerimeterVertexInstances, &NewEdgeIDs);
						for (const FEdgeID& NewEdgeID : NewEdgeIDs)
						{
							EdgeHardnesses[NewEdgeID] = false;
						}
					};

					
					for (int32 Tri = 0; Tri < NumTris; ++Tri)
					{
						if (!DegenerateTriangles[Tri])
						{
							AddTriangle(Tri * 3);
						}
						
					}
				}
			}
		}
	}

	return OutRawMesh.Polygons().Num() > 0;
}

FIntRect ALandscapeProxy::GetBoundingRect() const
{
	if (LandscapeComponents.Num() > 0)
	{
		FIntRect Rect(MAX_int32, MAX_int32, MIN_int32, MIN_int32);
		for (ULandscapeComponent* Component : LandscapeComponents)
		{
			if (Component != nullptr)
			{
				Rect.Include(Component->GetSectionBase());
			}
		}
		Rect.Max += FIntPoint(ComponentSizeQuads, ComponentSizeQuads);
		Rect -= SectionBase;
		return Rect;
	}

	return FIntRect();
}

bool ALandscape::HasAllComponent()
{
	ULandscapeInfo* Info = GetLandscapeInfo();
	if (Info && Info->XYtoComponentMap.Num() == LandscapeComponents.Num())
	{
		// all components are owned by this Landscape actor (no Landscape Proxies)
		return true;
	}
	return false;
}

bool ULandscapeInfo::GetLandscapeExtent(ALandscapeProxy* LandscapeProxy, FIntRect& ProxyExtent) const
{
	ProxyExtent.Min.X = INT32_MAX;
	ProxyExtent.Min.Y = INT32_MAX;
	ProxyExtent.Max.X = INT32_MIN;
	ProxyExtent.Max.Y = INT32_MIN;

	for (ULandscapeComponent* LandscapeComponent : LandscapeProxy->LandscapeComponents)
	{
		if (LandscapeComponent != nullptr)
		{
			LandscapeComponent->GetComponentExtent(ProxyExtent.Min.X, ProxyExtent.Min.Y, ProxyExtent.Max.X, ProxyExtent.Max.Y);
		}
	}
	
	return ProxyExtent.Min.X != INT32_MAX;
}

bool ULandscapeInfo::GetLandscapeExtent(FIntRect& LandscapeExtent) const
{
	return GetLandscapeExtent(LandscapeExtent.Min.X, LandscapeExtent.Min.Y, LandscapeExtent.Max.X, LandscapeExtent.Max.Y);
}

bool ULandscapeInfo::GetLandscapeExtent(int32& MinX, int32& MinY, int32& MaxX, int32& MaxY) const
{
	MinX = MAX_int32;
	MinY = MAX_int32;
	MaxX = MIN_int32;
	MaxY = MIN_int32;

	// Find range of entire landscape
	for (auto& XYComponentPair : XYtoComponentMap)
	{
		const ULandscapeComponent* Comp = XYComponentPair.Value;
		Comp->GetComponentExtent(MinX, MinY, MaxX, MaxY);
	}
	return (MinX != MAX_int32);
}

// Returns the full landscape extent regardless of which proxies are loaded.  Similar behavior to GetCompleteBounds.
// Extent is measured in quads, matching GetLandscapeExtent.
FIntRect ULandscapeInfo::GetCompleteLandscapeExtent() const
{
	FIntRect Extent(MAX_int32, MAX_int32, MIN_int32, MIN_int32);
	ALandscape* Landscape = LandscapeActor.Get();

	// In WP mode, examine each proxy
	if (Landscape && Landscape->GetWorld() && Landscape->GetWorld()->GetWorldPartition())
	{
		FWorldPartitionHelpers::ForEachActorDescInstance<ALandscapeProxy>(Landscape->GetWorld()->GetWorldPartition(), [this, &Extent, Landscape](const FWorldPartitionActorDescInstance* ActorDescInstance)
		{
			FLandscapeActorDesc* LandscapeActorDesc = (FLandscapeActorDesc*)ActorDescInstance->GetActorDesc();
			ALandscapeProxy* LandscapeProxy = Cast<ALandscapeProxy>(ActorDescInstance->GetActor());

			if (LandscapeProxy && (LandscapeProxy->GetGridGuid() == LandscapeGuid))
			{
				// Prioritize extent of loaded proxies, as the ActorDesc is not updated after edits until the level is saved.
				// Note that the parent ALandscape actor will have an extent in the ActorDesc (which does not match the true extent) but usually
				// no components.  That will hit this case and return false from GetLandscapeExtent.  Ignore this phantom extent.
				FIntRect ProxyExtent;
				bool bRet = GetLandscapeExtent(LandscapeProxy, ProxyExtent);
				if (bRet)
				{
					Extent.Union(ProxyExtent);
				}
			}
			else if(LandscapeActorDesc->GridGuid == LandscapeGuid)
			{
				// If not loaded, use the values from the FLandscapeActorDesc
				int32 GridSize = IntCastChecked<int32>(LandscapeActorDesc->GridSize);
				int32 X = IntCastChecked<int32>(LandscapeActorDesc->GridIndexX);
				int32 Y = IntCastChecked<int32>(LandscapeActorDesc->GridIndexY);
				FIntRect DescExtent(X * GridSize, Y * GridSize,
					(X + 1) * GridSize, (Y + 1) * GridSize);

				Extent.Union(DescExtent);
			}

			return true;
		});
	}
	else
	{
		// Otherwise use the extent straight from the components.
		GetLandscapeExtent(Extent);
	}

	return Extent;
}

bool ULandscapeInfo::GetLandscapeXYComponentBounds(FIntRect& OutXYComponentBounds) const
{
	OutXYComponentBounds = XYComponentBounds;

	return (OutXYComponentBounds.Min.X != MIN_int32) && (OutXYComponentBounds.Min.Y != MIN_int32)
		&& (OutXYComponentBounds.Max.X != MAX_int32) && (OutXYComponentBounds.Max.Y != MAX_int32);
}

void ULandscapeInfo::ForAllLandscapeComponents(TFunctionRef<void(ULandscapeComponent*)> Fn) const
{
	ForEachLandscapeProxy([&](ALandscapeProxy* Proxy)
	{
		for (ULandscapeComponent* Component : Proxy->LandscapeComponents)
		{
			if (Component != nullptr)
			{
				Fn(Component);
			}
		}
		return true;
	});
}

bool ULandscapeInfo::GetSelectedExtent(int32& MinX, int32& MinY, int32& MaxX, int32& MaxY) const
{
	MinX = MinY = MAX_int32;
	MaxX = MaxY = MIN_int32;
	for (auto& SelectedPointPair : SelectedRegion)
	{
		const FIntPoint Key = SelectedPointPair.Key;
		if (MinX > Key.X) MinX = Key.X;
		if (MaxX < Key.X) MaxX = Key.X;
		if (MinY > Key.Y) MinY = Key.Y;
		if (MaxY < Key.Y) MaxY = Key.Y;
	}
	if (MinX != MAX_int32)
	{
		return true;
	}
	// if SelectedRegion is empty, try SelectedComponents
	for (const ULandscapeComponent* Comp : SelectedComponents)
	{
		Comp->GetComponentExtent(MinX, MinY, MaxX, MaxY);
	}
	return MinX != MAX_int32;
}

FVector ULandscapeInfo::GetLandscapeCenterPos(float& LengthZ, int32 MinX /*= MAX_INT*/, int32 MinY /*= MAX_INT*/, int32 MaxX /*= MIN_INT*/, int32 MaxY /*= MIN_INT*/)
{
	// MinZ, MaxZ is Local coordinate
	float MaxZ = -UE_OLD_HALF_WORLD_MAX, MinZ = UE_OLD_HALF_WORLD_MAX;
	const float ScaleZ = static_cast<float>(DrawScale.Z);

	if (MinX == MAX_int32)
	{
		// Find range of entire landscape
		for (auto It = XYtoComponentMap.CreateIterator(); It; ++It)
		{
			ULandscapeComponent* Comp = It.Value();
			Comp->GetComponentExtent(MinX, MinY, MaxX, MaxY);
		}

		const int32 Dist = (ComponentSizeQuads + 1) >> 1; // Should be same in ALandscapeGizmoActiveActor::SetTargetLandscape
		FVector2f MidPoint(((float)(MinX + MaxX)) / 2.0f, ((float)(MinY + MaxY)) / 2.0f);
		MinX = FMath::FloorToInt(MidPoint.X) - Dist;
		MaxX = FMath::CeilToInt(MidPoint.X) + Dist;
		MinY = FMath::FloorToInt(MidPoint.Y) - Dist;
		MaxY = FMath::CeilToInt(MidPoint.Y) + Dist;
		check(MidPoint.X == ((float)(MinX + MaxX)) / 2.0f && MidPoint.Y == ((float)(MinY + MaxY)) / 2.0f);
	}

	check(MinX != MAX_int32);
	//if (MinX != MAX_int32)
	{
		int32 CompX1, CompX2, CompY1, CompY2;
		ALandscape::CalcComponentIndicesOverlap(MinX, MinY, MaxX, MaxY, ComponentSizeQuads, CompX1, CompY1, CompX2, CompY2);
		for (int32 IndexY = CompY1; IndexY <= CompY2; ++IndexY)
		{
			for (int32 IndexX = CompX1; IndexX <= CompX2; ++IndexX)
			{
				ULandscapeComponent* Comp = XYtoComponentMap.FindRef(FIntPoint(IndexX, IndexY));
				if (Comp)
				{
					ULandscapeHeightfieldCollisionComponent* CollisionComp = Comp->GetCollisionComponent();
					if (CollisionComp)
					{
						uint16* Heights = (uint16*)CollisionComp->CollisionHeightData.Lock(LOCK_READ_ONLY);
						int32 CollisionSizeVerts = CollisionComp->CollisionSizeQuads + 1;

						int32 StartX = FMath::Max(0, MinX - CollisionComp->GetSectionBase().X);
						int32 StartY = FMath::Max(0, MinY - CollisionComp->GetSectionBase().Y);
						int32 EndX = FMath::Min(CollisionSizeVerts, MaxX - CollisionComp->GetSectionBase().X + 1);
						int32 EndY = FMath::Min(CollisionSizeVerts, MaxY - CollisionComp->GetSectionBase().Y + 1);

						for (int32 Y = StartY; Y < EndY; ++Y)
						{
							for (int32 X = StartX; X < EndX; ++X)
							{
								float Height = LandscapeDataAccess::GetLocalHeight(Heights[X + Y*CollisionSizeVerts]);
								MaxZ = FMath::Max(Height, MaxZ);
								MinZ = FMath::Min(Height, MinZ);
							}
						}
						CollisionComp->CollisionHeightData.Unlock();
					}
				}
			}
		}
	}

	const float MarginZ = 3;
	if (MaxZ < MinZ)
	{
		MaxZ = +MarginZ;
		MinZ = -MarginZ;
	}
	LengthZ = (MaxZ - MinZ + 2 * MarginZ) * ScaleZ;

	const FVector LocalPosition(((float)(MinX + MaxX)) / 2.0f, ((float)(MinY + MaxY)) / 2.0f, MinZ - MarginZ);
	return GetLandscapeProxy()->LandscapeActorToWorld().TransformPosition(LocalPosition);
}

bool ULandscapeInfo::IsValidPosition(int32 X, int32 Y)
{
	int32 CompX1, CompX2, CompY1, CompY2;
	ALandscape::CalcComponentIndicesOverlap(X, Y, X, Y, ComponentSizeQuads, CompX1, CompY1, CompX2, CompY2);
	if (XYtoComponentMap.FindRef(FIntPoint(CompX1, CompY1)))
	{
		return true;
	}
	if (XYtoComponentMap.FindRef(FIntPoint(CompX2, CompY2)))
	{
		return true;
	}
	return false;
}

void ULandscapeInfo::ExportHeightmap(const FString& Filename)
{
	FIntRect ExportRegion;
	if (!GetLandscapeExtent(ExportRegion))
	{
		return;
	}

	ExportHeightmap(Filename, ExportRegion);
}

void ULandscapeInfo::ExportHeightmap(const FString& Filename, const FIntRect& ExportRegion)
{
	FScopedSlowTask Progress(1, LOCTEXT("ExportingLandscapeHeightmapTask", "Exporting Landscape Heightmap..."));
	Progress.MakeDialog();

	ILandscapeEditorModule& LandscapeEditorModule = FModuleManager::GetModuleChecked<ILandscapeEditorModule>("LandscapeEditor");
	FLandscapeEditDataInterface LandscapeEdit(this);

	TArray<uint16> HeightData;
	int32 ExportWidth = ExportRegion.Width() + 1;
	int32 ExportHeight = ExportRegion.Height() + 1;
	HeightData.AddZeroed(ExportWidth * ExportHeight);
	LandscapeEdit.GetHeightDataFast(ExportRegion.Min.X, ExportRegion.Min.Y, ExportRegion.Max.X, ExportRegion.Max.Y, HeightData.GetData(), 0);

	const ILandscapeHeightmapFileFormat* HeightmapFormat = LandscapeEditorModule.GetHeightmapFormatByExtension(*FPaths::GetExtension(Filename, true));
	if (HeightmapFormat)
	{
		HeightmapFormat->Export(*Filename, NAME_None, HeightData, {(uint32)ExportWidth, (uint32)ExportHeight}, DrawScale * FVector(1, 1, LANDSCAPE_ZSCALE));
	}
}

void ULandscapeInfo::ExportLayer(ULandscapeLayerInfoObject* LayerInfo, const FString& Filename)
{
	FIntRect ExportRegion;
	if (!GetLandscapeExtent(ExportRegion))
	{
		return;
	}

	ExportLayer(LayerInfo, Filename, ExportRegion);
}

void ULandscapeInfo::ExportLayer(ULandscapeLayerInfoObject* LayerInfo, const FString& Filename, const FIntRect& ExportRegion)
{
	FScopedSlowTask Progress(1, LOCTEXT("ExportingLandscapeWeightmapTask", "Exporting Landscape Layer Weightmap..."));
	Progress.MakeDialog();

	check(LayerInfo);
	
	ILandscapeEditorModule& LandscapeEditorModule = FModuleManager::GetModuleChecked<ILandscapeEditorModule>("LandscapeEditor");

	TArray<uint8> WeightData;
	int32 ExportWidth = ExportRegion.Width() + 1;
	int32 ExportHeight = ExportRegion.Height() + 1;
	WeightData.AddZeroed(ExportWidth * ExportHeight);

	FLandscapeEditDataInterface LandscapeEdit(this);
	LandscapeEdit.GetWeightDataFast(LayerInfo, ExportRegion.Min.X, ExportRegion.Min.Y, ExportRegion.Max.X, ExportRegion.Max.Y, WeightData.GetData(), 0);

	const ILandscapeWeightmapFileFormat* WeightmapFormat = LandscapeEditorModule.GetWeightmapFormatByExtension(*FPaths::GetExtension(Filename, true));
	if (WeightmapFormat)
	{
		WeightmapFormat->Export(*Filename, LayerInfo->GetLayerName(), WeightData, {(uint32)ExportWidth, (uint32)ExportHeight}, DrawScale * FVector(1, 1, LANDSCAPE_ZSCALE));
	}
}

void ULandscapeInfo::DeleteLayer(ULandscapeLayerInfoObject* LayerInfo, const FName& LayerName)
{
	checkf(LandscapeActor != nullptr, TEXT("ULandscapeInfo::DeleteLayer: Failed to delete landscape layer. Main LandscapeActor needs to be loaded and registered to the landscape info!"));

	GWarn->BeginSlowTask(LOCTEXT("BeginDeletingLayerTask", "Deleting Layer"), true);

	LandscapeActor->Modify();
	// Remove from layer settings array
	int32 LayerIndex = Layers.IndexOfByPredicate([LayerInfo, LayerName](const FLandscapeInfoLayerSettings& LayerSettings) { return LayerSettings.LayerInfoObj == LayerInfo && LayerSettings.LayerName == LayerName; });
	if (LayerIndex != INDEX_NONE)
	{
		Layers.RemoveAt(LayerIndex);
	}
	
	LandscapeActor->RemoveTargetLayer(LayerName);
	
	// Remove data from all components
	FLandscapeEditDataInterface LandscapeEdit(this);
	LandscapeEdit.DeleteLayer(LayerInfo);
	
	UpdateLayerInfoMap();

	GWarn->EndSlowTask();
}

void ULandscapeInfo::ReplaceLayer(ULandscapeLayerInfoObject* FromLayerInfo, ULandscapeLayerInfoObject* ToLayerInfo)
{
	// From and To may both be null if user tries to clear an already empty layer
	if (FromLayerInfo != ToLayerInfo)
	{
		GWarn->BeginSlowTask(LOCTEXT("BeginReplacingLayerTask", "Replacing Layer"), true);

		// Remove data from all components
		FLandscapeEditDataInterface LandscapeEdit(this);
		LandscapeEdit.ReplaceLayer(FromLayerInfo, ToLayerInfo);

		// Convert array
		for (int32 j = 0; j < Layers.Num(); j++)
		{
			if (Layers[j].LayerInfoObj == FromLayerInfo)
			{
				Layers[j].LayerInfoObj = ToLayerInfo;
			}
		}

		LandscapeActor->RemoveTargetLayer(FromLayerInfo->GetLayerName());

		// When a layer is cleared ToLayerInfo will be invalid
		const FLandscapeTargetLayerSettings NewSettings = ToLayerInfo != nullptr ? FLandscapeTargetLayerSettings(ToLayerInfo) : FLandscapeTargetLayerSettings();
		const FName NewName = ToLayerInfo != nullptr ? ToLayerInfo->GetLayerName() : FromLayerInfo->GetLayerName();
		LandscapeActor->AddTargetLayer(NewName, NewSettings);
		UpdateLayerInfoMap();

		GWarn->EndSlowTask();
	}
}

void ULandscapeInfo::GetUsedPaintLayers(const FGuid& InLayerGuid, TArray<ULandscapeLayerInfoObject*>& OutUsedLayerInfos) const
{
	OutUsedLayerInfos.Empty();
	ForEachLandscapeProxy([&](ALandscapeProxy* Proxy)
	{
		for (ULandscapeComponent* Component : Proxy->LandscapeComponents)
		{
			if (Component != nullptr)
			{
				Component->GetUsedPaintLayers(InLayerGuid, OutUsedLayerInfos);
			}
		}

		return true;
	});
}

void ALandscapeProxy::EditorApplyScale(const FVector& DeltaScale, const FVector* PivotLocation, bool bAltDown, bool bShiftDown, bool bCtrlDown)
{
	FVector ModifiedDeltaScale = DeltaScale;
	FVector CurrentScale = GetRootComponent()->GetRelativeScale3D();
	
	// Lock X and Y scaling to the same value :
	FVector2f XYDeltaScaleAbs(FMath::Abs(static_cast<float>(DeltaScale.X)), FMath::Abs(static_cast<float>(DeltaScale.Y)));
	// Preserve the sign of the chosen delta :
	bool bFavorX = (XYDeltaScaleAbs.X > XYDeltaScaleAbs.Y);

	if (AActor::bUsePercentageBasedScaling)
	{
		// Correct for attempts to scale to 0 on any axis
		double XYDeltaScale = bFavorX ? DeltaScale.X : DeltaScale.Y;
		if (XYDeltaScale == -1.0f)
		{
			XYDeltaScale = -(CurrentScale.X - 1) / CurrentScale.X;
		}
		if (ModifiedDeltaScale.Z == -1)
		{
			ModifiedDeltaScale.Z = -(CurrentScale.Z - 1) / CurrentScale.Z;
		}

		ModifiedDeltaScale.X = ModifiedDeltaScale.Y = XYDeltaScale;
	}
	else
	{
		// The absolute value of X and Y must be preserved so make sure they are preserved in case they flip from positive to negative (e.g.: a (-X, X) scale is accepted) : 
		const float SignMultiplier = static_cast<float>(FMath::Sign(CurrentScale.X) * FMath::Sign(CurrentScale.Y));
		FVector2d NewScale(FVector2f::ZeroVector);
		if (bFavorX)
		{
			NewScale.X = CurrentScale.X + DeltaScale.X;
			if (NewScale.X == 0.0f)
			{
				// Correct for attempts to scale to 0 on this axis : doubly-increment the scale to avoid reaching 0 :
				NewScale.X += DeltaScale.X;
			}
			NewScale.Y = SignMultiplier * NewScale.X;
		}
		else
		{
			NewScale.Y = CurrentScale.Y + DeltaScale.Y;
			if (NewScale.Y == 0.0f)
			{
				// Correct for attempts to scale to 0 on this axis : doubly-increment the scale to avoid reaching 0 :
				NewScale.Y += DeltaScale.Y;
			}
			NewScale.X = SignMultiplier * NewScale.Y;
		}

		ModifiedDeltaScale.X = NewScale.X - CurrentScale.X;
		ModifiedDeltaScale.Y = NewScale.Y - CurrentScale.Y;

		if (ModifiedDeltaScale.Z == -CurrentScale.Z)
		{
			ModifiedDeltaScale.Z += 1;
		}
	}

	Super::EditorApplyScale(ModifiedDeltaScale, PivotLocation, bAltDown, bShiftDown, bCtrlDown);

	// We need to regenerate collision objects, they depend on scale value 
	for (ULandscapeHeightfieldCollisionComponent* Comp : CollisionComponents)
	{
		if (Comp)
		{
			Comp->RecreateCollision();
		}
	}
}

void ALandscapeProxy::EditorApplyMirror(const FVector& MirrorScale, const FVector& PivotLocation)
{
	Super::EditorApplyMirror(MirrorScale, PivotLocation);

	// We need to regenerate collision objects, they depend on scale value 
	for (ULandscapeHeightfieldCollisionComponent* Comp : CollisionComponents)
	{
		if (Comp)
		{
			Comp->RecreateCollision();
		}
	}
}

void ALandscapeProxy::PostEditMove(bool bFinished)
{
	// This point is only reached when Copy and Pasted
	Super::PostEditMove(bFinished);

	if (bFinished)
	{
		ULandscapeInfo::RecreateLandscapeInfo(GetWorld(), /* bMapCheck = */ true, /* bKeepRegistrationStatus = */ true);
		RecreateComponentsState();

		if (SplineComponent)
		{
			SplineComponent->CheckSplinesValid();
		}
	}
}

void ALandscapeProxy::OnLoadedActorRemovedFromLevel()
{
	Super::OnLoadedActorRemovedFromLevel();
	ActorDescReferences.Empty();
}

void ALandscapeProxy::PostEditImport()
{
	Super::PostEditImport();

	// during import this gets called multiple times, without a valid guid the first time
	if (LandscapeGuid.IsValid())
	{
		CreateLandscapeInfo();
	}

	UpdateAllComponentMaterialInstances();
}

void ALandscape::PostEditMove(bool bFinished)
{
	if (bFinished)
	{
		// align all proxies to landscape actor
		ULandscapeInfo* LandscapeInfo = GetLandscapeInfo();
		if (LandscapeInfo)
		{
			LandscapeInfo->FixupProxiesTransform(true);
		}
	}

	// Some edit layers could be affected by BP brushes, which might need to be updated when the landscape is transformed :
	RequestLayersContentUpdate(bFinished ? ELandscapeLayerUpdateMode::Update_All : ELandscapeLayerUpdateMode::Update_All_Editing_NoCollision);

	Super::PostEditMove(bFinished);
}

void ALandscape::PostEditUndo()
{
	Super::PostEditUndo();

	RequestLayersContentUpdate(ELandscapeLayerUpdateMode::Update_All);
}

void ALandscape::UnregisterAllComponents(const bool bForReregister)
{
	// Unregister from edit layer data changed events
	for (FLandscapeLayer& Layer : LandscapeEditLayers)
	{
		Layer.EditLayer->OnLayerDataChanged().RemoveAll(this);
	}

	Super::UnregisterAllComponents(bForReregister);
}

void ALandscape::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

	auto HasValidBrush = [this]()
	{
		for (const FLandscapeLayer& Layer : LandscapeEditLayers)
		{
			for (const FLandscapeLayerBrush& Brush : Layer.Brushes)
			{
				if (IsValid(Brush.GetBrush()))
				{
					return true;
				}
			}
		}
		return false;
	};

	// Until it is properly supported, ALandscape with layer brushes will force all of its proxies to be loaded in editor
	if (UWorld* World = GetWorld(); GEditor && (World != nullptr) && !World->IsGameWorld() && LandscapeGuid.IsValid() && HasValidBrush() && !IsRunningCommandlet())
	{
		if (UWorldPartition* WorldPartition = World->GetWorldPartition())
		{
			FWorldPartitionHelpers::ForEachActorDescInstance<ALandscapeProxy>(WorldPartition, [this, WorldPartition](const FWorldPartitionActorDescInstance* ActorDescInstance)
			{
				FLandscapeActorDesc* LandscapeActorDesc = (FLandscapeActorDesc*)ActorDescInstance->GetActorDesc();

				if (LandscapeActorDesc->GridGuid == LandscapeGuid)
				{
					ActorDescReferences.Add(FWorldPartitionReference(WorldPartition, ActorDescInstance->GetGuid()));
				}
				return true;
			});
		}
	}

	// Register to data change events on the edit layers so that we can update the landscape accordingly : 
	for (FLandscapeLayer& Layer : LandscapeEditLayers)
	{
		Layer.EditLayer->OnLayerDataChanged().AddUObject(this, &ALandscape::OnEditLayerDataChanged);
	}
}

void ALandscape::PostActorCreated()
{
	Super::PostActorCreated();

	// Newly spawned Landscapes always set this value to true
	bIncludeGridSizeInNameForLandscapeActors = true;
}

bool ALandscape::ShouldImport(FStringView ActorPropString, bool IsMovingLevel)
{
	return GetWorld() != nullptr && !GetWorld()->IsGameWorld();
}

void ALandscape::PostEditImport()
{
	check(GetWorld() && !GetWorld()->IsGameWorld());

	for (ALandscape* Landscape : TActorRange<ALandscape>(GetWorld()))
	{
		if (Landscape && Landscape != this && !Landscape->HasAnyFlags(RF_BeginDestroyed) && Landscape->LandscapeGuid == LandscapeGuid)
		{
			// Copy/Paste case, need to generate new GUID
			LandscapeGuid = FGuid::NewGuid();
			OriginalLandscapeGuid = LandscapeGuid;
			break;
		}
	}

	// We need to reparent brushes that may have been part of the copy/pasted actors :
	for (FLandscapeLayer& Layer : LandscapeEditLayers)
	{
		for (FLandscapeLayerBrush& Brush : Layer.Brushes)
		{
			Brush.SetOwner(this);
		}
	}

	// Even if the component's UPROPERTY is TextExportTransient/NonPIEDuplicate, it still gets duplicated and added to the OwnedComponents so we need to remove it after duplicating the actor : 
	check(!HasNaniteComponents());
	TInlineComponentArray<ULandscapeNaniteComponent*> OwnedNaniteComponents;
	GetComponents<ULandscapeNaniteComponent>(OwnedNaniteComponents, /*bIncludeFromChildActors = */false);
	for (ULandscapeNaniteComponent* OwnedNaniteComponent : OwnedNaniteComponents)
	{
		OwnedNaniteComponent->DestroyComponent();
	}

	// Some edit layers could be affected by BP brushes, which might need to be updated when the landscape is transformed :
	RequestLayersContentUpdate(ELandscapeLayerUpdateMode::Update_All);

	Super::PostEditImport();
}

void ALandscape::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	if (!bDuplicateForPIE)
	{
		// Some edit layers could be affected by BP brushes, which might need to be updated when the landscape is transformed :
		RequestLayersContentUpdate(ELandscapeLayerUpdateMode::Update_All);
	}
}
#endif	//WITH_EDITOR


#if WITH_EDITOR

void ALandscapeProxy::RemoveXYOffsets()
{
	bool bFoundXYOffset = false;

	for (int32 i = 0; i < LandscapeComponents.Num(); ++i)
	{
		ULandscapeComponent* Comp = LandscapeComponents[i];
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (Comp && Comp->XYOffsetmapTexture_DEPRECATED)
		{
			Comp->XYOffsetmapTexture_DEPRECATED->SetFlags(RF_Transactional);
			Comp->XYOffsetmapTexture_DEPRECATED->Modify();
			Comp->XYOffsetmapTexture_DEPRECATED->MarkPackageDirty();
			Comp->XYOffsetmapTexture_DEPRECATED->ClearFlags(RF_Standalone);
			Comp->Modify();
			Comp->MarkPackageDirty();
			Comp->XYOffsetmapTexture_DEPRECATED = nullptr;
			Comp->MarkRenderStateDirty();
			bFoundXYOffset = true;
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	if (bFoundXYOffset)
	{
		RecreateCollisionComponents();
	}
}



void ALandscapeProxy::RecreateCollisionComponents()
{
	// We can assume these are all junk; they recreate as needed
	FlushGrassComponents();

	// Clear old CollisionComponent containers
	CollisionComponents.Empty();

	// Destroy any owned collision components
	TInlineComponentArray<ULandscapeHeightfieldCollisionComponent*> CollisionComps;
	GetComponents(CollisionComps);
	for (ULandscapeHeightfieldCollisionComponent* Component : CollisionComps)
	{
		Component->DestroyComponent();
	}

	TArray<USceneComponent*> AttachedCollisionComponents = RootComponent->GetAttachChildren().FilterByPredicate(
		[](USceneComponent* Component)
	{
		return Cast<ULandscapeHeightfieldCollisionComponent>(Component);
	});

	// Destroy any attached but un-owned collision components
	for (USceneComponent* Component : AttachedCollisionComponents)
	{
		Component->DestroyComponent();
	}

	// Recreate collision
	CollisionMipLevel = FMath::Clamp<int32>(CollisionMipLevel, 0, FMath::CeilLogTwo(SubsectionSizeQuads + 1) - 1);
	SimpleCollisionMipLevel = FMath::Clamp<int32>(SimpleCollisionMipLevel, 0, FMath::CeilLogTwo(SubsectionSizeQuads + 1) - 1);
	for (ULandscapeComponent* Comp : LandscapeComponents)
	{
		if (Comp)
		{
			Comp->CollisionMipLevel = CollisionMipLevel;
			Comp->SimpleCollisionMipLevel = SimpleCollisionMipLevel;
			Comp->DestroyCollisionData();
			Comp->UpdateCollisionData();
		}
	}
}

void ULandscapeInfo::RecreateCollisionComponents()
{
	ForEachLandscapeProxy([](ALandscapeProxy* Proxy)
	{
		Proxy->RecreateCollisionComponents();
		return true;
	});
}

void ULandscapeInfo::RemoveXYOffsets()
{
	ForEachLandscapeProxy([](ALandscapeProxy* Proxy)
	{
		Proxy->RemoveXYOffsets();
		return true;
	});
}

// Deprecated
bool ULandscapeInfo::CanHaveLayersContent() const
{
	return true;
}

void ULandscapeInfo::ClearDirtyData()
{
	if (ALandscape* Landscape = LandscapeActor.Get())
	{
		ForAllLandscapeComponents([=](ULandscapeComponent* InLandscapeComponent)
		{
			Landscape->ClearDirtyData(InLandscapeComponent);
		});
	}
}

void ULandscapeInfo::UpdateAllComponentMaterialInstances(bool bInInvalidateCombinationMaterials)
{
	ForEachLandscapeProxy([=](ALandscapeProxy* Proxy)
	{
		Proxy->UpdateAllComponentMaterialInstances(bInInvalidateCombinationMaterials);
		return true;
	});
}

uint32 ULandscapeInfo::GetGridSize(uint32 InGridSizeInComponents) const
{
	return InGridSizeInComponents * ComponentSizeQuads;
}

bool ULandscapeInfo::AreNewLandscapeActorsSpatiallyLoaded() const
{
	if (!bForceNonSpatiallyLoadedByDefault)
	{
		if (ALandscape* Landscape = LandscapeActor.Get())
		{
			return Landscape->bAreNewLandscapeActorsSpatiallyLoaded;
		}
	}
	
	return false;
}

ALandscapeProxy* ULandscapeInfo::MoveComponentsToLevel(const TArray<ULandscapeComponent*>& InComponents, ULevel* TargetLevel, FName NewProxyName)
{
	ALandscape* Landscape = LandscapeActor.Get();
	check(Landscape != nullptr);

	// Make sure references are in a different package (should be fixed up before calling this method)
	// Check the Physical Material is same package with Landscape
	if (Landscape->DefaultPhysMaterial && Landscape->DefaultPhysMaterial->GetOutermost() == Landscape->GetOutermost())
	{
		return nullptr;
	}

	// Check the LayerInfoObjects are not in same package as Landscape
	for (int32 i = 0; i < Layers.Num(); ++i)
	{
		ULandscapeLayerInfoObject* LayerInfo = Layers[i].LayerInfoObj;
		if (LayerInfo && LayerInfo->GetOutermost() == Landscape->GetOutermost())
		{
			return nullptr;
		}
	}

	// Check the Landscape Materials are not in same package as moved components
	for (ULandscapeComponent* Component : InComponents)
	{
		UMaterialInterface* LandscapeMaterial = Component->GetLandscapeMaterial();
		if (LandscapeMaterial && LandscapeMaterial->GetOutermost() == Component->GetOutermost())
		{
			return nullptr;
		}
	}

	ALandscapeProxy* LandscapeProxy = GetLandscapeProxyForLevel(TargetLevel);
	bool bSetPositionAndOffset = false;
	if (!LandscapeProxy)
	{
		FActorSpawnParameters SpawnParams;
		SpawnParams.Name = NewProxyName;
		SpawnParams.OverrideLevel = TargetLevel;
		LandscapeProxy = TargetLevel->GetWorld()->SpawnActor<ALandscapeStreamingProxy>(SpawnParams);

		LandscapeProxy->SynchronizeSharedProperties(Landscape);
		LandscapeProxy->CreateLandscapeInfo();
		LandscapeProxy->SetActorLabel(LandscapeProxy->GetName());
		bSetPositionAndOffset = true;
	}

	return MoveComponentsToProxy(InComponents, LandscapeProxy, bSetPositionAndOffset, TargetLevel);
}

ALandscapeProxy* ULandscapeInfo::MoveComponentsToProxy(const TArray<ULandscapeComponent*>& InComponents, ALandscapeProxy* LandscapeProxy, bool bSetPositionAndOffset, ULevel* TargetLevel)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeInfo::MoveComponentsToProxy);
	
	ALandscape* Landscape = LandscapeActor.Get();
	check(Landscape != nullptr && LandscapeProxy != nullptr);
	
	struct FCompareULandscapeComponentBySectionBase
	{
		FORCEINLINE bool operator()(const ULandscapeComponent& A, const ULandscapeComponent& B) const
		{
			return (A.GetSectionBase().X == B.GetSectionBase().X) ? (A.GetSectionBase().Y < B.GetSectionBase().Y) : (A.GetSectionBase().X < B.GetSectionBase().X);
		}
	};
	TArray<ULandscapeComponent*> ComponentsToMove(InComponents);
	ComponentsToMove.Sort(FCompareULandscapeComponentBySectionBase());
		
	const int32 ComponentSizeVerts = Landscape->NumSubsections * (Landscape->SubsectionSizeQuads + 1);
	const int32 NeedHeightmapSize = 1 << FMath::CeilLogTwo(ComponentSizeVerts);

	TSet<ALandscapeProxy*> SelectProxies;
	TSet<ULandscapeComponent*> TargetSelectedComponents;
	TArray<ULandscapeHeightfieldCollisionComponent*> TargetSelectedCollisionComponents;
	for (ULandscapeComponent* Component : ComponentsToMove)
	{
		SelectProxies.Add(Component->GetLandscapeProxy());
		if (Component->GetLandscapeProxy() != LandscapeProxy && (!TargetLevel || Component->GetLandscapeProxy()->GetOuter() != TargetLevel))
		{
			TargetSelectedComponents.Add(Component);
		}

		ULandscapeHeightfieldCollisionComponent* CollisionComp = Component->GetCollisionComponent();
		SelectProxies.Add(CollisionComp->GetLandscapeProxy());
		if (CollisionComp->GetLandscapeProxy() != LandscapeProxy && (!TargetLevel || CollisionComp->GetLandscapeProxy()->GetOuter() != TargetLevel))
		{
			TargetSelectedCollisionComponents.Add(CollisionComp);
		}
	}

	// Check which heightmap will need to be renewed :
	TSet<UTexture2D*> OldHeightmapTextures;
	for (ULandscapeComponent* Component : TargetSelectedComponents)
	{
		Component->Modify();
		OldHeightmapTextures.Add(Component->GetHeightmap());
		// Also process all edit layers heightmaps :
		Component->ForEachLayer([&](const FGuid& LayerGuid, FLandscapeLayerComponentData& LayerData)
		{
			OldHeightmapTextures.Add(Component->GetHeightmap(LayerGuid));
		});
	}

	// Need to split all the component which share Heightmap with selected components
	TMap<ULandscapeComponent*, bool> HeightmapUpdateComponents;
	HeightmapUpdateComponents.Reserve(TargetSelectedComponents.Num() * 4); // worst case
	for (ULandscapeComponent* Component : TargetSelectedComponents)
	{
		// Search neighbor only
		const int32 SearchX = Component->GetHeightmap()->Source.GetSizeX() / NeedHeightmapSize - 1;
		const int32 SearchY = Component->GetHeightmap()->Source.GetSizeY() / NeedHeightmapSize - 1;
		const FIntPoint ComponentBase = Component->GetSectionBase() / Component->ComponentSizeQuads;

		for (int32 Y = -SearchY; Y <= SearchY; ++Y)
		{
			for (int32 X = -SearchX; X <= SearchX; ++X)
			{
				ULandscapeComponent* const Neighbor = XYtoComponentMap.FindRef(ComponentBase + FIntPoint(X, Y));
				if (Neighbor && Neighbor->GetHeightmap() == Component->GetHeightmap() && !HeightmapUpdateComponents.Contains(Neighbor))
				{
					Neighbor->Modify();
					bool bNeedsMoveToCurrentLevel = TargetSelectedComponents.Contains(Neighbor);
					HeightmapUpdateComponents.Add(Neighbor, bNeedsMoveToCurrentLevel);
				}
			}
		}
	}

	// Proxy position/offset needs to be set
	if(bSetPositionAndOffset)
	{
		// set proxy location
		// by default first component location
		ULandscapeComponent* FirstComponent = *TargetSelectedComponents.CreateConstIterator();
		LandscapeProxy->GetRootComponent()->SetWorldLocationAndRotation(FirstComponent->GetComponentLocation(), FirstComponent->GetComponentRotation());
		LandscapeProxy->SetSectionBase(FirstComponent->GetSectionBase());

		// update the landscape info's proxy transform after changing the location and section base
		FixupProxiesTransform(/*bDirty =*/true);
	}

	// Hide(unregister) the new landscape if owning level currently in hidden state
	if (LandscapeProxy->GetLevel()->bIsVisible == false)
	{
		LandscapeProxy->UnregisterAllComponents();
	}

	// Changing Heightmap format for selected components
	for (const auto& HeightmapUpdateComponentPair : HeightmapUpdateComponents)
	{
		ALandscape::SplitHeightmap(HeightmapUpdateComponentPair.Key, HeightmapUpdateComponentPair.Value ? LandscapeProxy : nullptr);
	}

	// Delete if textures are not referenced anymore...
	for (UTexture2D* Texture : OldHeightmapTextures)
	{
		check(Texture != nullptr);
		Texture->SetFlags(RF_Transactional);
		Texture->Modify();
		Texture->MarkPackageDirty();
		Texture->ClearFlags(RF_Standalone);
	}

	for (ALandscapeProxy* Proxy : SelectProxies)
	{
		Proxy->Modify();
	}

	LandscapeProxy->Modify();
	LandscapeProxy->MarkPackageDirty();

	// Change Weight maps...
	{
		FLandscapeEditDataInterface LandscapeEdit(this);
		for (ULandscapeComponent* Component : TargetSelectedComponents)
		{
			Component->ReallocateWeightmaps(/*DataInterface = */&LandscapeEdit, /*InEditLayerGuid = */FGuid(), /*bInSaveToTransactionBuffer = */true, /*bool bInForceReallocate = */true, /*InTargetProxy = */LandscapeProxy, /*InRestrictSharingToComponents = */nullptr);
			Component->ForEachLayer([&](const FGuid& LayerGuid, FLandscapeLayerComponentData& LayerData)
			{
				FScopedSetLandscapeEditingLayer Scope(Landscape, LayerGuid);
				Component->ReallocateWeightmaps(/*DataInterface = */&LandscapeEdit, LayerGuid, /*bInSaveToTransactionBuffer = */true, /*bool bInForceReallocate = */true, /*InTargetProxy = */LandscapeProxy, /*InRestrictSharingToComponents = */nullptr);
			});
			Landscape->RequestLayersContentUpdateForceAll();
		}

		// Need to re-pack all the Weight map (to have it optimally re-packed...)
		for (ALandscapeProxy* Proxy : SelectProxies)
		{
			Proxy->RemoveInvalidWeightmaps();
		}
	}

	// Move the components to the Proxy actor
	// This does not use the MoveSelectedActorsToCurrentLevel path as there is no support to only move certain components.
	for (ULandscapeComponent* Component : TargetSelectedComponents)
	{
		// Need to move or recreate all related data (Height map, Weight map, maybe collision components, allocation info)
		Component->GetLandscapeProxy()->LandscapeComponents.Remove(Component);
		Component->UnregisterComponent();
		Component->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);
		Component->InvalidateLightingCache();
		Component->Rename(nullptr, LandscapeProxy);
		LandscapeProxy->LandscapeComponents.Add(Component);
		Component->AttachToComponent(LandscapeProxy->GetRootComponent(), FAttachmentTransformRules::KeepWorldTransform);

		// clear transient mobile data
		Component->MobileDataSourceHash.Invalidate();
		Component->MobileMaterialInterfaces.Reset();
		Component->MobileWeightmapTextures.Reset();
		Component->MobileWeightmapTextureArray = nullptr; 

		Component->UpdateMaterialInstances();
	}

	for (ULandscapeHeightfieldCollisionComponent* Component : TargetSelectedCollisionComponents)
	{
		// Need to move or recreate all related data (Height map, Weight map, maybe collision components, allocation info)

		Component->GetLandscapeProxy()->CollisionComponents.Remove(Component);
		Component->UnregisterComponent();
		Component->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);
		Component->Rename(nullptr, LandscapeProxy);
		LandscapeProxy->CollisionComponents.Add(Component);
		Component->AttachToComponent(LandscapeProxy->GetRootComponent(), FAttachmentTransformRules::KeepWorldTransform);

		// Move any foliage associated
		AInstancedFoliageActor::MoveInstancesForComponentToLevel(Component, LandscapeProxy->GetLevel());
	}
		
	// Register our new components if destination landscape is registered in scene 
	if (LandscapeProxy->GetRootComponent()->IsRegistered())
	{
		LandscapeProxy->RegisterAllComponents();
	}

	for (ALandscapeProxy* Proxy : SelectProxies)
	{
		if (Proxy->GetRootComponent()->IsRegistered())
		{
			Proxy->RegisterAllComponents();
		}
	}

	return LandscapeProxy;
}

void ALandscape::SplitHeightmap(ULandscapeComponent* Comp, ALandscapeProxy* TargetProxy, FMaterialUpdateContext* InOutUpdateContext, TArray<FComponentRecreateRenderStateContext>* InOutRecreateRenderStateContext, bool InReregisterComponent)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ALandscape::SplitHeightmap);
	
	ULandscapeInfo* Info = Comp->GetLandscapeInfo();

	// Make sure the heightmap UVs are powers of two.
	int32 ComponentSizeVerts = Comp->NumSubsections * (Comp->SubsectionSizeQuads + 1);
	int32 HeightmapSizeU = (1 << FMath::CeilLogTwo(ComponentSizeVerts));
	int32 HeightmapSizeV = (1 << FMath::CeilLogTwo(ComponentSizeVerts));

	ALandscapeProxy* SrcProxy = Comp->GetLandscapeProxy();
	ALandscapeProxy* DstProxy = TargetProxy ? TargetProxy : SrcProxy;
	SrcProxy->Modify();
	DstProxy->Modify();

	UTexture2D* OldHeightmapTexture = Comp->GetHeightmap(false);
	UTexture2D* NewHeightmapTexture = NULL;
	FVector4 OldHeightmapScaleBias = Comp->HeightmapScaleBias;
	FVector4 NewHeightmapScaleBias = FVector4(1.0f / (float)HeightmapSizeU, 1.0f / (float)HeightmapSizeV, 0.0f, 0.0f);

	// TODO-LS: Although we are only reading from the existing heightmap texture, the current code in FLandscapeEditDataInterface
	// is performing a call to LockMip(), which results in the UnlockMip() modifying the TextureSource CompressionFormat.
	// Modification to textures MUST be wrapped in PreEditChange/PostEditChange() in order to prevent texture compilation issues.
	// But really, the ideal fix here would be to call LockMipReadOnly() instead of LockMip(), and completely avoid modifying the texture.
	OldHeightmapTexture->PreEditChange(nullptr);

	{
		// Read old data and split
		FLandscapeEditDataInterface LandscapeEdit(Info);
		TArray<uint8> HeightData;
		HeightData.AddZeroed((1 + Comp->ComponentSizeQuads) * (1 + Comp->ComponentSizeQuads) * sizeof(uint16));
		// Because of edge problem, normal would be just copy from old component data
		TArray<uint8> NormalData;
		NormalData.AddZeroed((1 + Comp->ComponentSizeQuads) * (1 + Comp->ComponentSizeQuads) * sizeof(uint16));
		LandscapeEdit.GetHeightDataFast(Comp->GetSectionBase().X, Comp->GetSectionBase().Y, Comp->GetSectionBase().X + Comp->ComponentSizeQuads, Comp->GetSectionBase().Y + Comp->ComponentSizeQuads, (uint16*)HeightData.GetData(), 0, (uint16*)NormalData.GetData());

		// Create the new heightmap texture
		NewHeightmapTexture = DstProxy->CreateLandscapeTexture(HeightmapSizeU, HeightmapSizeV, TEXTUREGROUP_Terrain_Heightmap, TSF_BGRA8);
		const bool bClear = true;
		ULandscapeComponent::CreateEmptyTextureMips(NewHeightmapTexture, ELandscapeTextureUsage::FinalData, ELandscapeTextureType::Heightmap, bClear);
		NewHeightmapTexture->PostEditChange();

		Comp->HeightmapScaleBias = NewHeightmapScaleBias;
		Comp->SetHeightmap(NewHeightmapTexture);

		// Don't bother copying the data for edit layer based landscapes because that will be done in the next UpdateLayersContent.
		check(Comp->GetHeightmap(false) == Comp->GetHeightmap(true));
	}

	OldHeightmapTexture->PostEditChange();

	// End material update
	if (InOutUpdateContext != nullptr && InOutRecreateRenderStateContext != nullptr)
	{
		Comp->UpdateMaterialInstances(*InOutUpdateContext, *InOutRecreateRenderStateContext);
	}
	else
	{
		Comp->UpdateMaterialInstances();
	}

	// We disable automatic material update context, to manage it manually if we have a custom update context specified
	GDisableAutomaticTextureMaterialUpdateDependencies = (InOutUpdateContext != nullptr);

	if (InOutUpdateContext != nullptr)
	{
		// Build a list of all unique materials the landscape uses
		TArray<UMaterialInterface*> LandscapeMaterials;

		int8 MaxLOD = static_cast<int8>(FMath::CeilLogTwo(Comp->SubsectionSizeQuads + 1) - 1);

		for (int8 LODIndex = 0; LODIndex < MaxLOD; ++LODIndex)
		{
			UMaterialInterface* Material = Comp->GetLandscapeMaterial(LODIndex);
			LandscapeMaterials.AddUnique(Material);
		}

		TSet<UMaterial*> BaseMaterialsThatUseThisTexture;

		for (UMaterialInterface* MaterialInterface : LandscapeMaterials)
		{
			if (DoesMaterialUseTexture(MaterialInterface, NewHeightmapTexture))
			{
				UMaterial* Material = MaterialInterface->GetMaterial();
				bool MaterialAlreadyCompute = false;
				BaseMaterialsThatUseThisTexture.Add(Material, &MaterialAlreadyCompute);

				if (!MaterialAlreadyCompute)
				{
					if (Material->IsTextureForceRecompileCacheRessource(NewHeightmapTexture))
					{
						InOutUpdateContext->AddMaterial(Material);
						Material->UpdateMaterialShaderCacheAndTextureReferences();
					}
				}
			}
		}
	}

	GDisableAutomaticTextureMaterialUpdateDependencies = false;

#if WITH_EDITORONLY_DATA

	FLandscapeEditLayerReadback* NewCPUReadback = new FLandscapeEditLayerReadback();
	bool bHashIsValid = true;
	const uint64 Hash = ULandscapeTextureHash::CalculateTextureHash64(NewHeightmapTexture, ELandscapeTextureType::Heightmap, bHashIsValid);
	NewCPUReadback->SetHash(Hash);
	DstProxy->HeightmapsCPUReadback.Add(NewHeightmapTexture, NewCPUReadback);

	// Free OldHeightmapTexture's CPUReadBackResource if not used by any component
	bool FreeCPUReadBack = true;
	for (ULandscapeComponent* Component : SrcProxy->LandscapeComponents)
	{
		if (Component != Comp && Component->GetHeightmap(false) == OldHeightmapTexture)
		{
			FreeCPUReadBack = false;
			break;
		}
	}
	if (FreeCPUReadBack)
	{
		FLandscapeEditLayerReadback** OldCPUReadback = SrcProxy->HeightmapsCPUReadback.Find(OldHeightmapTexture);
		if (OldCPUReadback)
		{
			if (FLandscapeEditLayerReadback* ResourceToDelete = *OldCPUReadback)
			{
				delete ResourceToDelete;
				SrcProxy->HeightmapsCPUReadback.Remove(OldHeightmapTexture);
			}
		}
	}


	// TODO-LS: Although we are only reading from the existing heightmap texture, the current code in FLandscapeEditDataInterface
	// is performing a call to LockMip(), which results in the UnlockMip() modifying the TextureSource CompressionFormat.
	// Modification to textures MUST be wrapped in PreEditChange/PostEditChange() in order to prevent texture compilation issues.
	// But really, the ideal fix here would be to call LockMipReadOnly() instead of LockMip(), and completely avoid modifying the texture.
	TArray<UTexture2D*> OldLayerHeightmaps;

	{
		// Move layer content to new layer heightmap
		FLandscapeEditDataInterface LandscapeEdit(Info);
		ALandscape* Landscape = Info->LandscapeActor.Get();

		Comp->ForEachLayer([&](const FGuid& LayerGuid, FLandscapeLayerComponentData& LayerData)
		{
			UTexture2D* OldLayerHeightmap = LayerData.HeightmapData.Texture;
			if (OldLayerHeightmap != nullptr)
			{
				FScopedSetLandscapeEditingLayer Scope(Landscape, LayerGuid);

				OldLayerHeightmaps.Add(OldLayerHeightmap);
				OldLayerHeightmap->PreEditChange(nullptr);

				// Read old data and split
				TArray<uint8> LayerHeightData;
				LayerHeightData.AddZeroed((1 + Comp->ComponentSizeQuads) * (1 + Comp->ComponentSizeQuads) * sizeof(uint16));
				// Because of edge problem, normal would be just copy from old component data
				TArray<uint8> LayerNormalData;
				LayerNormalData.AddZeroed((1 + Comp->ComponentSizeQuads) * (1 + Comp->ComponentSizeQuads) * sizeof(uint16));

				// Read using old heightmap scale/bias
				Comp->HeightmapScaleBias = OldHeightmapScaleBias;
				LandscapeEdit.GetHeightDataFast(Comp->GetSectionBase().X, Comp->GetSectionBase().Y, Comp->GetSectionBase().X + Comp->ComponentSizeQuads, Comp->GetSectionBase().Y + Comp->ComponentSizeQuads, (uint16*)LayerHeightData.GetData(), 0, (uint16*)LayerNormalData.GetData());
				// Restore new heightmap scale/bias
				Comp->HeightmapScaleBias = NewHeightmapScaleBias;
				{
					// no mipchain required as these layer weight maps are used in layer compositing to generate a final set of weight maps to be used for rendering
					UTexture2D* LayerHeightmapTexture = DstProxy->CreateLandscapeTexture(HeightmapSizeU, HeightmapSizeV, TEXTUREGROUP_Terrain_Heightmap, TSF_BGRA8, /* OptionalOverrideOuter = */ nullptr, /* bCompress = */ false, /* bMipChain = */ false);
					const bool bClear = true;
					ULandscapeComponent::CreateEmptyTextureMips(LayerHeightmapTexture, ELandscapeTextureUsage::FinalData, ELandscapeTextureType::Heightmap, bClear);
					LayerHeightmapTexture->PostEditChange();
					// Set Layer heightmap texture
					LayerData.HeightmapData.Texture = LayerHeightmapTexture;
					LandscapeEdit.SetHeightData(Comp->GetSectionBase().X, Comp->GetSectionBase().Y, Comp->GetSectionBase().X + Comp->ComponentSizeQuads, Comp->GetSectionBase().Y + Comp->ComponentSizeQuads, (uint16*)LayerHeightData.GetData(), 0, false, (uint16*)LayerNormalData.GetData());
				}
			}
		});

		Landscape->RequestLayersContentUpdateForceAll();
	}

	Algo::ForEach(OldLayerHeightmaps, [](UTexture2D* Tex) { Tex->PostEditChange(); });

#endif

	// Reregister
	if (InReregisterComponent)
	{
		FComponentReregisterContext ReregisterContext(Comp);
	}
}

namespace
{
	inline float AdjustStaticLightingResolution(float StaticLightingResolution, int32 NumSubsections, int32 SubsectionSizeQuads, int32 ComponentSizeQuads)
	{
		// Change Lighting resolution to proper one...
		if (StaticLightingResolution > 1.0f)
		{
			StaticLightingResolution = static_cast<float>(static_cast<int32>(StaticLightingResolution));
		}
		else if (StaticLightingResolution < 1.0f)
		{
			// Restrict to 1/16
			if (StaticLightingResolution < 0.0625)
			{
				StaticLightingResolution = 0.0625;
			}

			// Adjust to 1/2^n
			int32 i = 2;
			int32 LightmapSize = (NumSubsections * (SubsectionSizeQuads + 1)) >> 1;
			while (StaticLightingResolution < (1.0f / i) && LightmapSize > 4)
			{
				i <<= 1;
				LightmapSize >>= 1;
			}
			StaticLightingResolution = 1.0f / i;

			int32 PixelPaddingX = GPixelFormats[PF_DXT1].BlockSizeX;

			int32 DestSize = (int32)((2 * PixelPaddingX + ComponentSizeQuads + 1) * StaticLightingResolution);
			StaticLightingResolution = (float)DestSize / (2 * PixelPaddingX + ComponentSizeQuads + 1);
		}

		return StaticLightingResolution;
	}
};

bool ALandscapeProxy::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	if (IsTemplate())
	{
		return true;
	}

	// Don't allow editing of properties that are shared with the parent landscape properties
	// See ALandscapeProxy::FixupSharedData(ALandscape* Landscape)
	if (GetLandscapeActor() != this)
	{
		return !IsPropertyInherited(InProperty);
	}

	return true;
}

void ALandscapeProxy::PostTransacted(const FTransactionObjectEvent& InTransactionEvent)
{
	if (InTransactionEvent.GetEventType() == ETransactionObjectEventType::UndoRedo)
	{
		const TArray<FName>& ChangedProperties = InTransactionEvent.GetChangedProperties();
		
		for (const FName& ChangedProperty : ChangedProperties)
		{
			FProperty* Property = FindFProperty<FProperty>(GetClass(), ChangedProperty);

			if (Property == nullptr)
			{
				continue;
			}
			
			FPropertyChangedEvent PropertyChangedEvent(Property);
			PostEditChangeProperty(PropertyChangedEvent);
		}
	}
}

void ALandscapeProxy::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	ensureAlwaysMsgf(!GetLevel() || (GetLevel() && !GetLevel()->HasAnyFlags(EObjectFlags::RF_NeedPostLoad)), TEXT("Executing ALandscapeProxy::PostEditChangeProperty before the landscape actor has finished loading with the world can result in registration events occuring in PostLoad, which could leave the landscape proxy in an temporary invalid state"));

	const FName MemberPropertyName = PropertyChangedEvent.MemberProperty ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;
	const FName SubPropertyName = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	bool bChangedPhysMaterial = false;
	bool bChangedMaterial = false;

	// In some cases, GetWorld() can return nullptr here (e.g. when replacing references when deleting a world asset from the content browser)
	//  , so we should not assume the World is valid here
	UWorld* World = GetWorld();

	if (MemberPropertyName == FName("RelativeScale3D"))
	{
		// RelativeScale3D isn't even a property of ALandscapeProxy, it's a property of the root component
		if (RootComponent)
		{
			const FVector OriginalScale = RootComponent->GetRelativeScale3D();
			FVector ModifiedScale = OriginalScale;

			// Lock X and Y scaling to the same value (but preserving their original sign)
			if (SubPropertyName == FName("Y"))
			{
				ModifiedScale.X = FMath::Abs(OriginalScale.Y) * FMath::Sign(OriginalScale.X);
			}
			else if (SubPropertyName == FName("X"))
			{
				ModifiedScale.Y = FMath::Abs(OriginalScale.X) * FMath::Sign(OriginalScale.Y);
			} 
			else if (SubPropertyName == FName("Z"))
			{
				// no adjustment necessary for Z, it is unrestricted
			}
			else
			{
				// When changing all axis values at once (e.g. when copy/pasting the scale), we receive only one event and the sub-property is not set
				check(SubPropertyName == MemberPropertyName); // Any other combination of property / sub-property is invalid
				// ensure that X and Y are locked (except for sign)
				if (!FMath::IsNearlyEqual(FMath::Abs(OriginalScale.X), FMath::Abs(OriginalScale.Y)))
				{
					// Arbitrarily favor the X axis as the uniform scale value (but retain the sign of Y) : 
					ModifiedScale.Y = FMath::Abs(OriginalScale.X) * FMath::Sign(OriginalScale.Y);
					UE_LOGF(LogLandscape, Warning, "Non-uniform XY scale for landscape (%f, %f) : scale will be forced to (%f, %f)", OriginalScale.X, OriginalScale.Y, ModifiedScale.X, ModifiedScale.Y);
				}
			}

			ULandscapeInfo* Info = GetLandscapeInfo();

			// Correct for attempts to scale to 0 on any axis
			if (ModifiedScale.X == 0)
			{
				if (Info && Info->DrawScale.X < 0)
				{
					ModifiedScale.Y = ModifiedScale.X = -1;
				}
				else
				{
					ModifiedScale.Y = ModifiedScale.X = 1;
				}
			}
			if (ModifiedScale.Z == 0)
			{
				if (Info && Info->DrawScale.Z < 0)
				{
					ModifiedScale.Z = -1;
				}
				else
				{
					ModifiedScale.Z = 1;
				}
			}

			RootComponent->SetRelativeScale3D(ModifiedScale);

			// Update ULandscapeInfo cached DrawScale
			if (Info)
			{
				Info->DrawScale = ModifiedScale;
			}

			// We need to regenerate collision objects, they depend on scale value
			if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
			{
				for (int32 ComponentIndex = 0; ComponentIndex < CollisionComponents.Num(); ComponentIndex++)
				{
					ULandscapeHeightfieldCollisionComponent* Comp = CollisionComponents[ComponentIndex];
					if (Comp)
					{
						Comp->RecreateCollision();
					}
				}
			}
		}
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, StreamingDistanceMultiplier))
	{
		// Recalculate in a few seconds.
		if (World != nullptr)
		{
			World->TriggerStreamingDataRebuild();
		}
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, DefaultPhysMaterial))
	{
		bChangedPhysMaterial = true;
	}
	else if ((MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, CollisionMipLevel))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, SimpleCollisionMipLevel))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, bBakeMaterialPositionOffsetIntoCollision))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, bGenerateOverlapEvents)))
	{
		if (bBakeMaterialPositionOffsetIntoCollision)
		{
			MarkComponentsRenderStateDirty();
		}
		if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
		{
			RecreateCollisionComponents();
		}
	}
	else if ((MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, ScalableLODDistributionSetting))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, ScalableLOD0DistributionSetting))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, ScalableLOD0ScreenSize))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, LODDistributionSetting))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, LOD0DistributionSetting))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, LOD0ScreenSize))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, bUseScalableLODSettings))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, LODBlendRange))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, RuntimeVirtualTextures))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, VirtualTextureRenderPassType))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, bVirtualTextureRenderWithQuad))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, bVirtualTextureRenderWithQuadHQ))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, VirtualTextureNumLods))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, VirtualTextureLodBias))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, bUseDynamicMaterialInstance))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, NonNaniteVirtualShadowMapConstantDepthBias))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, NonNaniteVirtualShadowMapInvalidationHeightErrorThreshold))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, NonNaniteVirtualShadowMapInvalidationScreenSizeLimit)))
	{		
		MarkComponentsRenderStateDirty();
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, bUseMaterialPositionOffsetInStaticLighting))
	{
		InvalidateLightingCache();
	}
	// Replicate properties shared with components to all of them :
	else if ((MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, CastShadow))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, bCastDynamicShadow))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, ShadowCacheInvalidationBehavior))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, bCastStaticShadow))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, bCastContactShadow))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, bCastFarShadow))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, bCastHiddenShadow))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, bCastShadowAsTwoSided))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, bAffectDistanceFieldLighting))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, bAffectDynamicIndirectLighting))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, bAffectIndirectLightingWhileHidden))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, bRenderCustomDepth))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, CustomDepthStencilWriteMask))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, CustomDepthStencilValue))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, LightingChannels))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, LDMaxDrawDistance))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, bUsedForNavigation))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, bFillCollisionUnderLandscapeForNavmesh))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, bHoldout)))
	{
		// TODO [jonathan.bard] : Move to its own function so that we can streamline the setters of these properties when we start exposing them
		for (ULandscapeComponent* Comp : LandscapeComponents)
		{
			if (Comp)
			{
				Comp->UpdatedSharedPropertiesFromActor();
			}
		}
		if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, bHoldout))
		{
			// Match holdout property on the grass components as well
			for (UHierarchicalInstancedStaticMeshComponent* FoliageComp : FoliageComponents)
			{
				if (FoliageComp)
				{
					FoliageComp->SetHoldout(bHoldout);
				}
			}
		}

		UpdateNaniteSharedPropertiesFromActor();
	}
	// Nanite-related changes require invalidating the generated component data :
	else if ((MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, bEnableNanite))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, bNaniteSkirtEnabled))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, NaniteSkirtDepth))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, NaniteLODIndex))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, NanitePositionPrecision))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, NaniteMaxEdgeLengthFactor))
)
	{
		InvalidateOrUpdateNaniteRepresentation(/* bInCheckContentId = */true, /*InTargetPlatform = */nullptr);
		UpdateRenderingMethod();
		MarkComponentsRenderStateDirty();
	}
	else if ((MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, LandscapeMaterial))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, LandscapeHoleMaterial))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, PerLODOverrideMaterials))
		|| (SubPropertyName == GET_MEMBER_NAME_CHECKED(FLandscapePerLODMaterialOverride, Material))
		|| (SubPropertyName == GET_MEMBER_NAME_CHECKED(FLandscapePerLODMaterialOverride, LODIndex)))
	{
		bChangedMaterial = true;

		// Ignore changes when LODIndex slider is active or adding new PerLODMaterialOverride (Material is initially null, does not require rebuild)
		if ((MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeStreamingProxy, PerLODOverrideMaterials) && PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayAdd)
			|| PropertyChangedEvent.ChangeType == EPropertyChangeType::Interactive)
		{
			bChangedMaterial = false;
		}

		if (ULandscapeInfo* Info = GetLandscapeInfo(); Info != nullptr && bChangedMaterial)
		{
			FMaterialUpdateContext MaterialUpdateContext;
			Info->UpdateLayerInfoMap(/*this*/);

			// Clear the parents out of combination material instances
			for (const auto& MICPair : MaterialInstanceConstantMap)
			{
				UMaterialInstanceConstant* MaterialInstance = MICPair.Value;
				MaterialInstance->BasePropertyOverrides.bOverride_BlendMode = false;
				MaterialInstance->SetParentEditorOnly(nullptr);
				MaterialUpdateContext.AddMaterialInstance(MaterialInstance);
			}

			// Remove our references to any material instances
			MaterialInstanceConstantMap.Empty();

			UpdateAllComponentMaterialInstances();

			if ((World != nullptr) && (World->GetFeatureLevel() == ERHIFeatureLevel::ES3_1))
			{
				for (ULandscapeComponent* Component : LandscapeComponents)
				{
					if (Component != nullptr)
					{
						Component->CheckGenerateMobilePlatformData(/*bIsCooking = */ false, /*TargetPlatform = */ nullptr);
					}
				}
			}

			InvalidateOrUpdateNaniteRepresentation(/* bInCheckContentId = */true, /*InTargetPlatform = */nullptr);
			UpdateRenderingMethod();
		}
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, GrassTypesOverrides))
	{
		for (ULandscapeComponent* Component : LandscapeComponents)
		{
			if (Component != nullptr)
			{
				Component->UpdateGrassTypes();
			}
		}
	}

	// Remove any null landscape components
	LandscapeComponents.RemoveAll([](const ULandscapeComponent* Component) { return Component == nullptr; });

	// Must do this *after* correcting the scale or reattaching the landscape components will crash!
	// Must do this *after* clamping values / propogating values to components
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Call that posteditchange when components are registered
	if (bChangedPhysMaterial)
	{
		ChangedPhysMaterial();
	}

	if (bChangedMaterial)
	{
		if (World != nullptr)
		{
			ULandscapeSubsystem* Subsystem = World->GetSubsystem<ULandscapeSubsystem>();
			check(Subsystem != nullptr);
			Subsystem->GetDelegateAccess().OnLandscapeProxyMaterialChangedDelegate.Broadcast(this, FOnLandscapeProxyMaterialChangedParams());
		}
	}
}

void ALandscapeStreamingProxy::CheckOverriddenSharedProperties()
{
	ALandscape* ParentLandscape = GetLandscapeActor();
	if (ParentLandscape == nullptr)
	{
		// Cannot validate anything without the parent 
		return;
	}

	TArray<FName> PropertiesToUnOverride;
	auto AddSharedPropertyOverrideMapCheckMessage = [this, &PropertiesToUnOverride](const FName& InOverriddenProperty)
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("PropertyName"), FText::FromString(InOverriddenProperty.ToString()));
			TWeakObjectPtr<ALandscapeStreamingProxy> LandscapeStreamingProxyWeakPtr = this;

			FMessageLog MessageLog("MapCheck");
			TSharedRef<FTokenizedMessage> TokenizedMessage = CVarLandscapeSilenceInvalidPropertyOverridesMapCheckWarnings->GetBool() ? MessageLog.Info() : MessageLog.Warning();
			TokenizedMessage->AddToken(FUObjectToken::Create(this, FText::FromString(GetActorNameOrLabel())))
				->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_InvalidLandscapePropertyOverride", "The Landscape property {PropertyName} has been overridden but points to an invalid asset. It's most likely a mistake. The engine default material will be used. "), Arguments)))
				->AddToken(FMapErrorToken::Create(FMapErrors::LandscapeProxyInvalidPropertyOverride_Warning))
				->AddToken(FActionToken::Create(LOCTEXT("MapCheck_UnOverrideProperty", "Un-override property"), FText::Format(LOCTEXT("MapCheck_UnOverrideProperty_Desc", "Un-override {PropertyName}."), Arguments),
					FOnActionTokenExecuted::CreateLambda([LandscapeStreamingProxyWeakPtr, PropertyName = InOverriddenProperty]()
						{
							if (LandscapeStreamingProxyWeakPtr.IsValid())
							{
								LandscapeStreamingProxyWeakPtr->SetSharedPropertyOverride(PropertyName, false);
							}
						}),
					/*bInSingleUse = */true));
		};

	// If the user has set a material override but left the material unset, this is most likely a mistake : 
	if ((ParentLandscape->LandscapeMaterial != nullptr) && (LandscapeMaterial == nullptr) && IsSharedPropertyOverridden(GET_MEMBER_NAME_CHECKED(ALandscapeProxy, LandscapeMaterial)))
	{
		AddSharedPropertyOverrideMapCheckMessage(GET_MEMBER_NAME_CHECKED(ALandscapeProxy, LandscapeMaterial));
	}

	// If the user has set a material override but left the material unset, this is most likely a mistake : 
	if ((ParentLandscape->LandscapeHoleMaterial != nullptr) && (LandscapeHoleMaterial == nullptr) && IsSharedPropertyOverridden(GET_MEMBER_NAME_CHECKED(ALandscapeProxy, LandscapeHoleMaterial)))
	{
		AddSharedPropertyOverrideMapCheckMessage(GET_MEMBER_NAME_CHECKED(ALandscapeProxy, LandscapeHoleMaterial));
	}
}

void ALandscapeStreamingProxy::CheckForErrors()
{
	Super::CheckForErrors();

	CheckOverriddenSharedProperties();
}

bool ALandscapeStreamingProxy::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	if (InProperty && InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(ALandscapeStreamingProxy, LandscapeActorRef))
	{
		if (UWorld* World = GetWorld())
		{
			return !World->GetSubsystem<ULandscapeSubsystem>()->IsGridBased();
		}
	}
	else if (InProperty && IsPropertyOverridable(InProperty))
	{
		return IsSharedPropertyOverridden(InProperty->GetFName());
	}

	return true;
}

void ALandscapeStreamingProxy::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName MemberPropertyName = PropertyChangedEvent.MemberProperty ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;

	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeStreamingProxy, LandscapeActorRef))
	{
		// Landscape Actor reference was changed .. need to update LandscapeGUIDs to match
		if (LandscapeActorRef && IsValidLandscapeActor(LandscapeActorRef.Get()))
		{
			LandscapeGuid = LandscapeActorRef->GetLandscapeGuid();
			OriginalLandscapeGuid = LandscapeActorRef->GetOriginalLandscapeGuid();
			if (GIsEditor && GetWorld() && !GetWorld()->IsPlayInEditor())
			{
				// TODO - only need to refresh the old and new landscape info
				ULandscapeInfo::RecreateLandscapeInfo(GetWorld(), false);
				FixupWeightmaps();
				InitializeProxyLayersWeightmapUsage();
				// TODO [jonathan.bard] : Call FixupSharedData?
			}
		}
		else
		{
			LandscapeActorRef = nullptr;
		}
	}

	// Must do this *after* clamping values
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void ALandscapeStreamingProxy::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

#if WITH_EDITORONLY_DATA
	// If the landscape actor is not set yet and we're transferring the property from the lazy object pointer it was previously stored as to the soft object ptr it is now stored as :
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!LandscapeActorRef)
	{
		// Because of how lazy object pointers were made, the only way we can deprecate them is if the object they're pointing to is currently loaded :
		if (LandscapeActor_DEPRECATED.IsValid())
		{
			LandscapeActorRef = LandscapeActor_DEPRECATED.Get();
			LandscapeActor_DEPRECATED = nullptr;
		}
		else if (LandscapeActor_DEPRECATED.IsPending())
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("LevelName"), FText::FromString(GetLevel()->GetOutermost()->GetName()));
			Arguments.Add(TEXT("ProxyName"), FText::FromString(GetName()));
			FMessageLog("MapCheck").Error()
				->AddToken(FUObjectToken::Create(this, FText::FromString(GetActorNameOrLabel())))
				->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_LandscapeLazyObjectPtrDeprecation_Warning", "Landscape proxy {ProxyName} of {LevelName} points to a LandscapeActor that is not currently loaded. "
				"This will lose the property upon save. Please make sure to load the level containing the parent landscape actor prior to {LevelName} so that data deprecation can be performed adequately. "
				"It is advised to reassign the \"Landscape Actor\" property of LandscapeStreamingProxies and resave these actors."), Arguments)))
				->AddToken(FMapErrorToken::Create(FMapErrors::LandscapeLazyObjectPtrDeprecation_Warning));

			// Show MapCheck window
			FMessageLog("MapCheck").Open(EMessageSeverity::Warning);
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA

	if (LandscapeGuid.IsValid())
	{
		ULandscapeInfo* LandscapeInfo = GetLandscapeInfo();
		check(LandscapeInfo);
		if (UWorld* World = GetWorld(); GEditor && (World != nullptr) && !World->IsGameWorld() && !IsRunningCommandlet())
		{
			if (UWorldPartition* WorldPartition = World->GetWorldPartition(); WorldPartition && WorldPartition->IsInitialized())
			{
				const FVector ActorLocation = GetActorLocation();
				FBox Bounds(ActorLocation, ActorLocation + (GetGridSize() * LandscapeInfo->DrawScale));

				// all actors that intersect Landscape in 2D need to be considered
				Bounds.Min.Z = -HALF_WORLD_MAX;
				Bounds.Max.Z = HALF_WORLD_MAX;

				FWorldPartitionHelpers::ForEachIntersectingActorDescInstance(WorldPartition, Bounds, [this, WorldPartition](const FWorldPartitionActorDescInstance* ActorDescInstance) mutable
				{
					FName PropertyValue;
					if (ActorDescInstance->GetProperty(ALandscape::AffectsLandscapeActorDescProperty, &PropertyValue))
					{
						// If no Guid specified then consider actor as affecting all landscapes
						if(FGuid ParsedGuid; PropertyValue.IsNone() || (FGuid::Parse(PropertyValue.ToString(), ParsedGuid) && ParsedGuid == LandscapeGuid))
						{
							ActorDescReferences.Add(FWorldPartitionReference(WorldPartition, ActorDescInstance->GetGuid()));
						}
					}
					return true;
				});
			}
		}
	}
}

AActor* ALandscapeStreamingProxy::GetSceneOutlinerParent() const
{
	if (ULandscapeInfo* LandscapeInfo = GetLandscapeInfo())
	{
		return LandscapeInfo->LandscapeActor.Get();
	}

	return Super::GetSceneOutlinerParent();
}

bool ALandscapeStreamingProxy::CanDeleteSelectedActor(FText& OutReason) const
{
	return true;
}


bool ALandscapeStreamingProxy::GetReferencedContentObjects(TArray<UObject*>& Objects) const
{
	Super::GetReferencedContentObjects(Objects);

	// Also return the objects referenced by our parent landscape : 
	if (const ALandscape* LandscapeActor = GetLandscapeActor())
	{
		LandscapeActor->GetReferencedContentObjects(Objects);
	}

	return true;
}

void ALandscapeStreamingProxy::GetActorDescProperties(FPropertyPairsMap& PropertyPairsMap) const
{
	Super::GetActorDescProperties(PropertyPairsMap);

	for (URuntimeVirtualTexture* RuntimeVirtualTexture : RuntimeVirtualTextures)
	{
		if (RuntimeVirtualTexture)
		{
			PropertyPairsMap.AddProperty(UPrimitiveComponent::RVTActorDescProperty);
			return;
		}
	}
}

bool ALandscapeStreamingProxy::ShouldIncludeGridSizeInName(UWorld* InWorld, const FActorPartitionIdentifier& InIdentifier) const
{
	// Always return true if this world setting flag is true
	if (Super::ShouldIncludeGridSizeInName(InWorld, InIdentifier))
	{
		return true;
	}

	if (ULandscapeInfo* LandscapeInfo = ULandscapeInfo::Find(InWorld, InIdentifier.GetGridGuid()))
	{
		if (ALandscape* Landscape = LandscapeInfo->LandscapeActor.Get())
		{
			// This new flag is to support Landscapes that were created with bIncludeGridSizeInNameForPartitionedActors == false or 
			// that were reconfigured with FLandscapeConfigHelper::ChangeGridSize
			return Landscape->bIncludeGridSizeInNameForLandscapeActors;
		}
	}

	return false;
}

bool ALandscape::CanDeleteSelectedActor(FText& OutReason) const
{
	if (!IsUserManaged())
	{
		// Allow Delete of Actor if all other related actors have been deleted
		ULandscapeInfo* Info = GetLandscapeInfo();
		return Info ? Info->CanDeleteLandscape(OutReason) : false;
	}

	return true;
}

bool ULandscapeInfo::CanDeleteLandscape(FText& OutReason) const
{
	check(LandscapeActor != nullptr);
	int32 UndeletedProxyCount = 0;
	int32 UndeletedSplineCount = 0;

	// Check Registered Proxies
	for (TWeakObjectPtr<ALandscapeStreamingProxy> ProxyPtr : SortedStreamingProxies)
	{
		ALandscapeProxy* RegisteredProxy = ProxyPtr.Get();
		if (!RegisteredProxy || RegisteredProxy == LandscapeActor)
		{
			continue;
		}

		check(IsValidChecked(RegisteredProxy));
		UndeletedProxyCount++;
	}

	// Then check for Unloaded Proxies
	if (AActor* Actor = LandscapeActor.Get())
	{
		UWorld* World = Actor->GetWorld();
		if (UWorldPartition* WorldPartition = (World != nullptr) ? World->GetWorldPartition() : nullptr)
		{
			FWorldPartitionHelpers::ForEachActorDescInstance<ALandscapeProxy>(WorldPartition, [this, &UndeletedProxyCount](const FWorldPartitionActorDescInstance* ActorDescInstance)
			{
				FLandscapeActorDesc* LandscapeActorDesc = (FLandscapeActorDesc*)ActorDescInstance->GetActorDesc();

				if (LandscapeActorDesc->GridGuid == LandscapeGuid)
				{
					ALandscapeProxy* LandscapeProxy = Cast<ALandscapeProxy>(ActorDescInstance->GetActor());
					if (LandscapeProxy != LandscapeActor)
					{
						const bool bIsLoaded = ActorDescInstance->IsLoaded();
						const bool bIsAllocated = (LandscapeProxy != nullptr);
						const bool bIsGarbage = bIsAllocated && !IsValidChecked(LandscapeProxy);
						const bool bIsRegistered = bIsAllocated && SortedStreamingProxies.Contains(CastChecked<ALandscapeStreamingProxy>(LandscapeProxy));
						const bool bRegisteredComps = bIsAllocated && LandscapeProxy->HasActorRegisteredAllComponents();
						check(bIsRegistered == bRegisteredComps);

						if (bIsGarbage)
						{
							// proxy has been deleted, WP just hasn't cleaned up the actor desc yet.
							// so don't count it as an undeleted proxy
						}
						else if (bIsLoaded && !bIsRegistered)
						{
							// this is a bit of a messed up state that we can encounter after an initial save
							// where the proxy is loaded but not registered with the world
							// TODO [chris.tchou] : figure out how to fix this for real so we don't get into this state
							UE_LOGF(LogLandscape, Display, "The Landscape Proxy '%p' is loaded and valid, but not registered. This may cause incorrect display in the outliner view. You can reload the level to correct this.",
								LandscapeProxy);

							// since this proxy was not counted in the registered proxy loop above, count it here
							UndeletedProxyCount++;
						}
						else if (!bIsLoaded)
						{
							// unloaded proxies still exist (just not in memory at the moment)
							UndeletedProxyCount++;
						}
					}
				}

				return true;
			});
		}
	}

	// Check Registered Splines
	for (TScriptInterface<ILandscapeSplineInterface> SplineOwner : SplineActors)
	{
		// Only check for ALandscapeSplineActor type because ALandscapeProxy also implement the ILandscapeSplineInterface for non WP worlds
		if(ALandscapeSplineActor* SplineActor = Cast<ALandscapeSplineActor>(SplineOwner.GetObject()))
		{ 
			check(IsValidChecked(SplineActor));
			UndeletedSplineCount++;
		}
	}

	// Then check for Unloaded Splines
	if (AActor* Actor = LandscapeActor.Get())
	{
		UWorld* World = Actor->GetWorld();
		if (UWorldPartition* WorldPartition = (World != nullptr) ? World->GetWorldPartition() : nullptr)
		{
			FWorldPartitionHelpers::ForEachActorDescInstance<ALandscapeSplineActor>(WorldPartition, [this, &UndeletedSplineCount](const FWorldPartitionActorDescInstance* ActorDescInstance)
			{
				FName AffectsLandscapeProperty;
				if (ActorDescInstance->GetProperty(ALandscape::AffectsLandscapeActorDescProperty, &AffectsLandscapeProperty))
				{
					FGuid ParsedLandscapeGuid;
					if (FGuid::Parse(AffectsLandscapeProperty.ToString(), ParsedLandscapeGuid) && ParsedLandscapeGuid == LandscapeGuid)
				{
						ALandscapeSplineActor* SplineActor = Cast<ALandscapeSplineActor>(ActorDescInstance->GetActor());
		
					// If SplineActor is null then it is not loaded/deleted. If it's loaded then it needs to be pending kill.
					if (!SplineActor)
					{
						++UndeletedSplineCount;
					}
					else
					{
						// If Actor is loaded it should be Registered and not pending kill (already accounted for) or pending kill (deleted)
						check(SplineActors.Contains(SplineActor) == IsValidChecked(SplineActor));
					}
				}
				}

				return true;
			});
		}
	}

	if (UndeletedProxyCount > 0 || UndeletedSplineCount > 0)
	{
		OutReason = FText::Format(LOCTEXT("CanDeleteLandscapeReason", "Landscape can't be deleted because it still has {0} LandscapeStreamingProxies and {1} LandscapeSplineActors"), FText::AsNumber(UndeletedProxyCount), FText::AsNumber(UndeletedSplineCount));
		return false;
	}

	return true;
}

void ALandscape::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ALandscape::PostEditChangeProperty);

	// Clamp all values of a FPerQualityLevelFloat
	auto Clamp = [](const FPerQualityLevelFloat& QualityLevelFloat, float Min, float Max) -> FPerQualityLevelFloat
	{
		FPerQualityLevelFloat Clamped = QualityLevelFloat;
		Clamped.Default = FMath::Clamp(Clamped.GetDefault(), Min, Max);

		for (auto It = Clamped.PerQuality.CreateIterator(); It; ++It)
		{
			float& Value = It.Value();
			Value = FMath::Clamp(Value, Min, Max);
		}
		
		return Clamped;
	};

	const FName MemberPropertyName = PropertyChangedEvent.MemberProperty ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;
	const FName SubPropertyName = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	bool bMaterialChanged = false;
	bool bNeedsRecalcBoundingBox = false;
	bool bChangedLighting = false;
	bool bPropagateToProxies = false;
	bool bMarkAllLandscapeRenderStateDirty = false;
	bool bNaniteToggled = false;

	ULandscapeInfo* Info = GetLandscapeInfo();

	if ((MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, LandscapeMaterial))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, LandscapeHoleMaterial))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, PerLODOverrideMaterials))
		|| (SubPropertyName == GET_MEMBER_NAME_CHECKED(FLandscapePerLODMaterialOverride, Material))
		|| (SubPropertyName == GET_MEMBER_NAME_CHECKED(FLandscapePerLODMaterialOverride, LODIndex)))
	{
		bMaterialChanged = true;

		// Ignore changes when LODIndex slider is active or adding new PerLODMaterialOverride (Material is initially null, does not require rebuild)
		if ((MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeStreamingProxy, PerLODOverrideMaterials) && PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayAdd)
			|| PropertyChangedEvent.ChangeType == EPropertyChangeType::Interactive)
		{
			bMaterialChanged = false;
		}
	}
	else if (MemberPropertyName == FName("RelativeScale3D") ||
			 MemberPropertyName == FName("RelativeLocation") ||
			 MemberPropertyName == FName("RelativeRotation"))
	{
		if (Info != nullptr)
		{
			// update transformations for all linked proxies 
			Info->FixupProxiesTransform(true);
			bNeedsRecalcBoundingBox = true;
		}

		// Some edit layers could be affected by BP brushes, which might need to be updated when the landscape is transformed :
		RequestLayersContentUpdate(ELandscapeLayerUpdateMode::Update_All);
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, MaxLODLevel))
	{
		MaxLODLevel = FMath::Clamp<int32>(MaxLODLevel, -1, FMath::CeilLogTwo(SubsectionSizeQuads + 1) - 1);
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, LODDistributionSetting))
	{
		LODDistributionSetting = FMath::Clamp(LODDistributionSetting, 1.0f, 10.0f);
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, LOD0DistributionSetting))
	{
		LOD0DistributionSetting = FMath::Clamp(LOD0DistributionSetting, 1.0f, 10.0f);
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, LOD0ScreenSize))
	{
		LOD0ScreenSize = FMath::Clamp(LOD0ScreenSize, 0.1f, 10.0f);
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, ScalableLODDistributionSetting))
	{
		ScalableLODDistributionSetting = Clamp(ScalableLODDistributionSetting, 1.0f, 10.0f);
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, ScalableLOD0DistributionSetting))
	{
		ScalableLOD0DistributionSetting = Clamp(ScalableLOD0DistributionSetting, 1.0f, 10.0f);
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, ScalableLOD0ScreenSize))
	{
		ScalableLOD0ScreenSize = Clamp(ScalableLOD0ScreenSize, 0.1f, 10.0f);
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, CollisionMipLevel))
	{
		CollisionMipLevel = FMath::Clamp<int32>(CollisionMipLevel, 0, FMath::CeilLogTwo(SubsectionSizeQuads + 1) - 1);
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, SimpleCollisionMipLevel))
	{
		SimpleCollisionMipLevel = FMath::Clamp<int32>(SimpleCollisionMipLevel, 0, FMath::CeilLogTwo(SubsectionSizeQuads + 1) - 1);
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, StaticLightingResolution))
	{
		StaticLightingResolution = ::AdjustStaticLightingResolution(StaticLightingResolution, NumSubsections, SubsectionSizeQuads, ComponentSizeQuads);
		bChangedLighting = true;
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, StaticLightingLOD))
	{
		StaticLightingLOD = FMath::Clamp<int32>(StaticLightingLOD, 0, FMath::CeilLogTwo(SubsectionSizeQuads + 1) - 1);
		bChangedLighting = true;
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscapeProxy, ExportLOD))
	{
		ExportLOD = FMath::Clamp<int32>(ExportLOD, 0, FMath::CeilLogTwo(SubsectionSizeQuads + 1) - 1);
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscape, bEnableNanite))
	{
		bNaniteToggled = true;

		// Generate Nanite data for a landscape with components on it, and recreate render state
		// Streaming proxies won't be built here, but the bPropagateToProxies path will.
		InvalidateOrUpdateNaniteRepresentation(/* bInCheckContentId = */true, /*InTargetPlatform = */nullptr);
		UpdateRenderingMethod();
		MarkComponentsRenderStateDirty();
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscape, LODGroupKey))
	{
		bMarkAllLandscapeRenderStateDirty = true;
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ALandscape, bGrassEnabled))
	{
		if (!bGrassEnabled)
		{
			// This should not technically be required but is done to expedite the flush and mirror the default logic in ALandscape::SetGrassEnabled().
			// This may leave the grass builder in a non-sync'd state as it thinks it is populated but the grass maps are gone, but the builder does detect that
			// and will fix itself.
			FlushGrassComponents(/* OnlyForComponents = */nullptr, /* bFlushGrassMaps = */true);
		}
	}

	// If the property that has changed is overridable or inherited and not interactive (dragging a slider)
	// synchronize the change on all landscape proxies :
	if (IsSharedProperty(MemberPropertyName) && PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		bPropagateToProxies = true;
	}

	// Must do this *after* clamping values
	Super::PostEditChangeProperty(PropertyChangedEvent);

	bPropagateToProxies = bPropagateToProxies || bNeedsRecalcBoundingBox || bChangedLighting || bNaniteToggled || bMaterialChanged;

	if (Info != nullptr)
	{
		if (bPropagateToProxies)
		{
			// Propagate Event to Proxies...
			Info->ForEachLandscapeProxy([this, &PropertyChangedEvent](ALandscapeProxy* Proxy)
			{
				// The ALandscape is included in ForEachLandscapeProxy, avoid getting stuck in a PostEditChange loop
				if (Proxy != this)
				{
					Proxy->SynchronizeSharedProperties(this, /*bInExecutePostEditChangeProperty =*/true);
				}
				return true;
			});
		}

		if (bMarkAllLandscapeRenderStateDirty)
		{
			MarkAllLandscapeRenderStateDirty();
		}

		if (bNeedsRecalcBoundingBox || bMaterialChanged || bChangedLighting)
		{
			// We cannot iterate the XYtoComponentMap directly because reregistering components modifies the array.
			TArray<ULandscapeComponent*> AllComponents;
			Info->XYtoComponentMap.GenerateValueArray(AllComponents);
			for (ULandscapeComponent* Comp : AllComponents)
			{
				if (ensure(Comp))
				{
					Comp->Modify();

					if (bNeedsRecalcBoundingBox)
					{
						Comp->UpdateCachedBounds();
						Comp->UpdateBounds();
					}

					if (bChangedLighting)
					{
						Comp->InvalidateLightingCache();
					}
				}
			}

			if (bMaterialChanged)
			{
				UpdateAllComponentMaterialInstances();

				UWorld* World = GetWorld();

				if (World != nullptr)
				{
					if (World->GetFeatureLevel() == ERHIFeatureLevel::ES3_1)
					{
						for (ULandscapeComponent* Component : LandscapeComponents)
						{
							if (Component != nullptr)
							{
								Component->CheckGenerateMobilePlatformData(/*bIsCooking = */ false, /*TargetPlatform = */ nullptr);
							}
						}
					}

					InvalidateOrUpdateNaniteRepresentation(/* bInCheckContentId = */true, /*InTargetPlatform = */nullptr);
					UpdateRenderingMethod();
				}
			}
		}

		// Need to update Gizmo scene proxy
		if (bNeedsRecalcBoundingBox && GetWorld())
		{
			for (ALandscapeGizmoActiveActor* Gizmo : TActorRange<ALandscapeGizmoActiveActor>(GetWorld()))
			{
				Gizmo->MarkComponentsRenderStateDirty();
			}
		}
	}
}

void ALandscapeProxy::ChangedPhysMaterial()
{
	for (ULandscapeComponent* LandscapeComponent : LandscapeComponents)
	{
		if (LandscapeComponent && LandscapeComponent->IsRegistered())
		{
			ULandscapeHeightfieldCollisionComponent* CollisionComponent = LandscapeComponent->GetCollisionComponent();
			if (CollisionComponent)
			{
				LandscapeComponent->UpdateCollisionLayerData();
				// Physical materials cooked into collision object, so we need to recreate it
				CollisionComponent->RecreateCollision();
			}
		}
	}
}

void ULandscapeComponent::PreEditChange(FProperty* PropertyThatWillChange)
{
	Super::PreEditChange(PropertyThatWillChange);
	if (PropertyThatWillChange && (PropertyThatWillChange->GetFName() == GET_MEMBER_NAME_CHECKED(ULandscapeComponent, ForcedLOD) || PropertyThatWillChange->GetFName() == GET_MEMBER_NAME_CHECKED(ULandscapeComponent, LODBias)))
	{
		// PreEdit unregister component and re-register after PostEdit so we will lose XYtoComponentMap for this component
		ULandscapeInfo* Info = GetLandscapeInfo();
		if (Info)
		{
			FIntPoint ComponentKey = GetSectionBase() / ComponentSizeQuads;
			auto RegisteredComponent = Info->XYtoComponentMap.FindRef(ComponentKey);

			if (RegisteredComponent == nullptr)
			{
				Info->XYtoComponentMap.Add(ComponentKey, this);
			}
		}
	}
}

void ULandscapeComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName MemberPropertyName = PropertyChangedEvent.MemberProperty ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;
	const FName SubPropertyName = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	if ((MemberPropertyName == GET_MEMBER_NAME_CHECKED(ULandscapeComponent, OverrideMaterial))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ULandscapeComponent, PerLODOverrideMaterials))
		|| (SubPropertyName == FName("MaterialPerLOD_Key")))
	{
		bool RecreateMaterialInstances = true;

		if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ULandscapeComponent, PerLODOverrideMaterials) && PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayAdd)
		{
			RecreateMaterialInstances = false;
		}

		if (RecreateMaterialInstances)
		{
			UpdateMaterialInstances();

			UWorld* World = GetWorld();

			if (World != nullptr && World->GetFeatureLevel() == ERHIFeatureLevel::ES3_1)
			{
				CheckGenerateMobilePlatformData(/*bIsCooking = */ false, /*TargetPlatform = */ nullptr);
			}
		}
	}
	else if ((MemberPropertyName == GET_MEMBER_NAME_CHECKED(ULandscapeComponent, ForcedLOD))
		|| (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ULandscapeComponent, LODBias)))
	{
		bool bForcedLODChanged = (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ULandscapeComponent, ForcedLOD));
		SetLOD(bForcedLODChanged, bForcedLODChanged ? ForcedLOD : LODBias);
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ULandscapeComponent, StaticLightingResolution))
	{
		if (StaticLightingResolution > 0.0f)
		{
			StaticLightingResolution = ::AdjustStaticLightingResolution(StaticLightingResolution, NumSubsections, SubsectionSizeQuads, ComponentSizeQuads);
		}
		else
		{
			StaticLightingResolution = 0;
		}
		InvalidateLightingCache();
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ULandscapeComponent, LightingLODBias))
	{
		int32 MaxLOD = FMath::CeilLogTwo(SubsectionSizeQuads + 1) - 1;
		LightingLODBias = FMath::Clamp<int32>(LightingLODBias, -1, MaxLOD);
		InvalidateLightingCache();
	}
	else if ((MemberPropertyName == GET_MEMBER_NAME_CHECKED(ULandscapeComponent, CollisionMipLevel))
		 || (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ULandscapeComponent, SimpleCollisionMipLevel)))
	{
		CollisionMipLevel = FMath::Clamp<int32>(CollisionMipLevel, 0, FMath::CeilLogTwo(SubsectionSizeQuads + 1) - 1);
		SimpleCollisionMipLevel = FMath::Clamp<int32>(SimpleCollisionMipLevel, 0, FMath::CeilLogTwo(SubsectionSizeQuads + 1) - 1);
		if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
		{
			DestroyCollisionData();
			UpdateCollisionData(); // Rebuild for new CollisionMipLevel
		}
	}

	// Must do this *after* clamping values
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

TSet<class ULandscapeComponent*> ULandscapeInfo::GetSelectedComponents() const
{
	return SelectedComponents;
}

TSet<class ULandscapeComponent*> ULandscapeInfo::GetSelectedRegionComponents() const
{
	return SelectedRegionComponents;
}

void ULandscapeInfo::UpdateSelectedComponents(TSet<ULandscapeComponent*>& NewComponents, bool bIsComponentwise /*=true*/)
{
	int32 InSelectType = bIsComponentwise ? FLandscapeEditToolRenderData::ST_COMPONENT : FLandscapeEditToolRenderData::ST_REGION;

	if (bIsComponentwise)
	{
		for (TSet<ULandscapeComponent*>::TIterator It(NewComponents); It; ++It)
		{
			ULandscapeComponent* Comp = *It;
			if ((Comp->EditToolRenderData.SelectedType & InSelectType) == 0)
			{
				Comp->Modify(false);
				int32 SelectedType = Comp->EditToolRenderData.SelectedType;
				SelectedType |= InSelectType;
				Comp->EditToolRenderData.UpdateSelectionMaterial(SelectedType, Comp);
				Comp->UpdateEditToolRenderData();
			}
		}

		// Remove the material from any old components that are no longer in the region
		TSet<ULandscapeComponent*> RemovedComponents = SelectedComponents.Difference(NewComponents);
		for (TSet<ULandscapeComponent*>::TIterator It(RemovedComponents); It; ++It)
		{
			ULandscapeComponent* Comp = *It;
			Comp->Modify(false);
			int32 SelectedType = Comp->EditToolRenderData.SelectedType;
			SelectedType &= ~InSelectType;
			Comp->EditToolRenderData.UpdateSelectionMaterial(SelectedType, Comp);
			Comp->UpdateEditToolRenderData();
		}
		SelectedComponents = NewComponents;
	}
	else
	{
		// Only add components...
		if (NewComponents.Num())
		{
			for (TSet<ULandscapeComponent*>::TIterator It(NewComponents); It; ++It)
			{
				ULandscapeComponent* Comp = *It;
				if ((Comp->EditToolRenderData.SelectedType & InSelectType) == 0)
				{
					Comp->Modify(false);
					int32 SelectedType = Comp->EditToolRenderData.SelectedType;
					SelectedType |= InSelectType;
					Comp->EditToolRenderData.UpdateSelectionMaterial(SelectedType, Comp);
					Comp->UpdateEditToolRenderData();
				}

				SelectedRegionComponents.Add(*It);
			}
		}
		else
		{
			// Remove the material from any old components that are no longer in the region
			for (TSet<ULandscapeComponent*>::TIterator It(SelectedRegionComponents); It; ++It)
			{
				ULandscapeComponent* Comp = *It;
				Comp->Modify(false);
				int32 SelectedType = Comp->EditToolRenderData.SelectedType;
				SelectedType &= ~InSelectType;
				Comp->EditToolRenderData.UpdateSelectionMaterial(SelectedType, Comp);
				Comp->UpdateEditToolRenderData();
			}
			SelectedRegionComponents = NewComponents;
		}
	}
}

void ULandscapeInfo::ClearSelectedRegion(bool bIsComponentwise /*= true*/)
{
	TSet<ULandscapeComponent*> NewComponents;
	UpdateSelectedComponents(NewComponents, bIsComponentwise);
	if (!bIsComponentwise)
	{
		SelectedRegion.Empty();
	}

	// Refresh the detail panel since the selected region affects the visibility of some landscape tool settings : 
	ILandscapeModule& LandscapeModule = FModuleManager::GetModuleChecked<ILandscapeModule>("Landscape");
	LandscapeModule.GetLandscapeEditorServices()->RefreshDetailPanel();
}

TArray<UTexture*> ULandscapeComponent::ReallocateWeightmaps(FLandscapeEditDataInterface* DataInterface, const FGuid& InEditLayerGuid, bool bInSaveToTransactionBuffer, bool bInForceReallocate, 
	ALandscapeProxy* InTargetProxy, const TSet<ULandscapeComponent*>* InRestrictSharingToComponents)
{
	TArray<UTexture*> CreatedTextures;
	int32 NeededNewChannels = 0;
	ALandscapeProxy* TargetProxy = InTargetProxy ? InTargetProxy : GetLandscapeProxy();
	
	const bool bIsFinalWeightmap = !InEditLayerGuid.IsValid();

	TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = GetWeightmapLayerAllocations(InEditLayerGuid);
	TArray<TObjectPtr<UTexture2D>>& ComponentWeightmapTextures = GetWeightmapTextures(InEditLayerGuid);
	TArray<TObjectPtr<ULandscapeWeightmapUsage>>& ComponentWeightmapTexturesUsage = GetWeightmapTexturesUsage(InEditLayerGuid);

	// When force reallocating, skip tests to see if allocations are necessary based on Component's WeightmapLayeAllocInfo
	if (!bInForceReallocate)
	{
		for (int32 LayerIdx = 0; LayerIdx < ComponentWeightmapLayerAllocations.Num(); LayerIdx++)
		{
			if (!ComponentWeightmapLayerAllocations[LayerIdx].IsAllocated())
			{
				NeededNewChannels++;
			}
		}

		// All channels allocated!
		if (NeededNewChannels == 0)
		{
			return {};
		}
	}

	ULandscapeInfo* LandscapeInfo = GetLandscapeInfo();

	bool bMarkPackageDirty = DataInterface == nullptr ? true : DataInterface->GetShouldDirtyPackage();
	if (bInSaveToTransactionBuffer)
	{
		LandscapeInfo->ModifyObject(this, bMarkPackageDirty);
		LandscapeInfo->ModifyObject(TargetProxy, bMarkPackageDirty);
	}

	if (!bInForceReallocate)
	{
		// UE_LOGF(LogLandscape, Log, "----------------------");
		// UE_LOGF(LogLandscape, Log, "Component %ls needs %d layers (%d new)", *GetName(), WeightmapLayerAllocations.Num(), NeededNewChannels);

		// See if our existing textures have sufficient space
		int32 ExistingTexAvailableChannels = 0;
		for (int32 TexIdx = 0; TexIdx < ComponentWeightmapTextures.Num(); TexIdx++)
		{
			ULandscapeWeightmapUsage* Usage = ComponentWeightmapTexturesUsage[TexIdx];
			check(Usage);
			check(Usage->LayerGuid == InEditLayerGuid);
			ExistingTexAvailableChannels += Usage->FreeChannelCount();

			if (ExistingTexAvailableChannels >= NeededNewChannels)
			{
				break;
			}
		}

		if (ExistingTexAvailableChannels >= NeededNewChannels)
		{
			// UE_LOGF(LogLandscape, Log, "Existing texture has available channels");

			// Allocate using our existing textures' spare channels.
			int32 CurrentAlloc = 0;
			for (int32 TexIdx = 0; TexIdx < ComponentWeightmapTextures.Num(); TexIdx++)
			{
				ULandscapeWeightmapUsage* Usage = ComponentWeightmapTexturesUsage[TexIdx];

				for (int32 ChanIdx = 0; ChanIdx < 4; ChanIdx++)
				{
					if (Usage->ChannelUsage[ChanIdx] == nullptr)
					{
						// Find next allocation to treat
						for (; CurrentAlloc < ComponentWeightmapLayerAllocations.Num(); ++CurrentAlloc)
						{
							FWeightmapLayerAllocationInfo& AllocInfo = ComponentWeightmapLayerAllocations[CurrentAlloc];

							if (!AllocInfo.IsAllocated())
							{
								break;
							}
						}

						FWeightmapLayerAllocationInfo& AllocInfo = ComponentWeightmapLayerAllocations[CurrentAlloc];
						check(!AllocInfo.IsAllocated());

						// Zero out the data for this texture channel
						if (DataInterface)
						{
							DataInterface->ZeroTextureChannel(ComponentWeightmapTextures[TexIdx], ChanIdx);
						}

						AllocInfo.WeightmapTextureIndex = IntCastChecked<uint8>(TexIdx);
						AllocInfo.WeightmapTextureChannel = IntCastChecked<uint8>(ChanIdx);

						Usage->ChannelUsage[ChanIdx] = this;

						NeededNewChannels--;

						if (NeededNewChannels == 0)
						{
							return {};
						}
					}
				}
			}
			// we should never get here.
			check(false);
		}
	}

	// UE_LOGF(LogLandscape, Log, "Reallocating.");

	// We are totally reallocating the weightmap
	int32 TotalNeededChannels = ComponentWeightmapLayerAllocations.Num();
	int32 CurrentLayer = 0;
	TArray<UTexture2D*> NewWeightmapTextures;
	TArray<ULandscapeWeightmapUsage*> NewComponentWeightmapTexturesUsage;
	
	while (TotalNeededChannels > 0)
	{
		// UE_LOGF(LogLandscape, Log, "Still need %d channels", TotalNeededChannels);

		UTexture2D* CurrentWeightmapTexture = nullptr;
		ULandscapeWeightmapUsage* CurrentWeightmapUsage = nullptr;

		// Testing TotalNeededChannels here is not correct.  There's no apparent reason to do it and it prevents direct re-use of available channels in
		// WeightmapUsageMap if more than one texture is needed.  In simple cases, this works if this TotalNeededChannels test (and the one ten lines down)
		// are removed. But... this bug is at least 14 years old in this very central function.  Changing this has some risk and it solves no known problems.
		// For now, this will be left as is.
		if (TotalNeededChannels < 4)
		{
			// UE_LOGF(LogLandscape, Log, "Looking for nearest");

			// see if we can find a suitable existing weightmap texture with sufficient channels
			int32 BestDistanceSquared = MAX_int32;
			for (auto& ItPair : TargetProxy->WeightmapUsageMap)
			{
				ULandscapeWeightmapUsage* TryWeightmapUsage = ItPair.Value;
				//
				if (TryWeightmapUsage->FreeChannelCount() >= TotalNeededChannels && TryWeightmapUsage->LayerGuid == InEditLayerGuid)
				{
					if (TryWeightmapUsage->IsEmpty())
					{
						CurrentWeightmapTexture = ItPair.Key;
						CurrentWeightmapUsage = TryWeightmapUsage;
						break;
					}
					else
					{
						// See if this candidate is closer than any others we've found
						for (int32 ChanIdx = 0; ChanIdx < ULandscapeWeightmapUsage::NumChannels; ChanIdx++)
						{
							if (TryWeightmapUsage->ChannelUsage[ChanIdx] != nullptr)
							{
								int32 TryDistanceSquared = (TryWeightmapUsage->ChannelUsage[ChanIdx]->GetSectionBase() - GetSectionBase()).SizeSquared();
								if (TryDistanceSquared < BestDistanceSquared)
								{
									if (InRestrictSharingToComponents != nullptr)
									{
										TSet<ULandscapeComponent*, DefaultKeyFuncs<ULandscapeComponent*>, TInlineSetAllocator<4>> WeightmapComponents(MakeArrayView(TryWeightmapUsage->GetUniqueValidComponents()));
										int32 NumWeightmapComponents = WeightmapComponents.Num();
										// Don't pick this candidate if it will lead the texture to be shared with a component that's not in the provided list :
										if ((NumWeightmapComponents > 0) && WeightmapComponents.Intersect(*InRestrictSharingToComponents).Num() != NumWeightmapComponents)
										{
											break;
										}
									}

									CurrentWeightmapTexture = ItPair.Key;
									CurrentWeightmapUsage = TryWeightmapUsage;
									BestDistanceSquared = TryDistanceSquared;
								}
							}
						}
					}
				}
			}
		}

		bool NeedsUpdateResource = false;
		// No suitable weightmap texture
		if (CurrentWeightmapTexture == nullptr)
		{
			if (bMarkPackageDirty)
			{
				LandscapeInfo->MarkObjectDirty(this);
			}

			// Weightmap is sized the same as the component
			int32 WeightmapSize = (SubsectionSizeQuads + 1) * NumSubsections;

			// We need a new weightmap texture
			CurrentWeightmapTexture = TargetProxy->CreateLandscapeTexture(WeightmapSize, WeightmapSize, TEXTUREGROUP_Terrain_Weightmap, TSF_BGRA8, nullptr, /* bCompress = */ false, /* bMipChain= */ bIsFinalWeightmap); 

			// Alloc dummy mips
			if (bIsFinalWeightmap)
			{
				const bool bClear = true;
				CreateEmptyTextureMips(CurrentWeightmapTexture, ELandscapeTextureUsage::FinalData, ELandscapeTextureType::Weightmap, bClear);
			}
			else
			{
				// sanity check - only final weightmaps are allowed more than one mip after creation
				check(CurrentWeightmapTexture->Source.GetNumMips() == 1);
			}

			CurrentWeightmapTexture->PostEditChange();

			CreatedTextures.Add(CurrentWeightmapTexture);

			// Store it in the usage map
			CurrentWeightmapUsage = TargetProxy->WeightmapUsageMap.Add(CurrentWeightmapTexture, TargetProxy->CreateWeightmapUsage());
			CurrentWeightmapUsage->LayerGuid = InEditLayerGuid;
			// UE_LOGF(LogLandscape, Log, "Making a new texture %ls", *CurrentWeightmapTexture->GetName());
		}
		else
		{
			// sanity check - only final weightmaps are allowed more than one mip (except if we load old data with CVarStripLayerTextureMipsOnLoad set to false)
			check(!CVarStripLayerTextureMipsOnLoad->GetBool() || (CurrentWeightmapTexture->Source.GetNumMips() == 1) || bIsFinalWeightmap);
		}

		NewComponentWeightmapTexturesUsage.Add(CurrentWeightmapUsage);
		NewWeightmapTextures.Add(CurrentWeightmapTexture);

		for (int32 ChanIdx = 0; ChanIdx < 4 && TotalNeededChannels > 0; ChanIdx++)
		{
			// UE_LOGF(LogLandscape, Log, "Finding allocation for layer %d", CurrentLayer);
			if (CurrentWeightmapUsage->ChannelUsage[ChanIdx] == nullptr)
			{
				// Use this allocation
				FWeightmapLayerAllocationInfo& AllocInfo = ComponentWeightmapLayerAllocations[CurrentLayer];

				if (!AllocInfo.IsAllocated())
				{
					// New layer - zero out the data for this texture channel
					if (DataInterface)
					{
						DataInterface->ZeroTextureChannel(CurrentWeightmapTexture, ChanIdx);
						// UE_LOGF(LogLandscape, Log, "Zeroing out channel %ls.%d", *CurrentWeightmapTexture->GetName(), ChanIdx);
					}
				}
				else
				{
					UTexture2D* OldWeightmapTexture = ComponentWeightmapTextures[AllocInfo.WeightmapTextureIndex];

					// Copy the data
					if (ensure(DataInterface != nullptr)) // it's not safe to skip the copy
					{
						DataInterface->CopyTextureChannel(CurrentWeightmapTexture, ChanIdx, OldWeightmapTexture, AllocInfo.WeightmapTextureChannel);
						DataInterface->ZeroTextureChannel(OldWeightmapTexture, AllocInfo.WeightmapTextureChannel);
						// UE_LOGF(LogLandscape, Log, "Copying old channel (%ls).%d to new channel (%ls).%d", *OldWeightmapTexture->GetName(), AllocInfo.WeightmapTextureChannel, *CurrentWeightmapTexture->GetName(), ChanIdx);
					}

					// Remove the old allocation
					ULandscapeWeightmapUsage* OldWeightmapUsage = ComponentWeightmapTexturesUsage[AllocInfo.WeightmapTextureIndex];
					OldWeightmapUsage->ChannelUsage[AllocInfo.WeightmapTextureChannel] = nullptr;
				}

				// Assign the new allocation
				CurrentWeightmapUsage->ChannelUsage[ChanIdx] = this;
				AllocInfo.WeightmapTextureIndex = IntCastChecked<uint8>(NewWeightmapTextures.Num() - 1);
				AllocInfo.WeightmapTextureChannel = IntCastChecked<uint8>(ChanIdx);
				CurrentLayer++;
				TotalNeededChannels--;
			}
		}
	}

	if (DataInterface && bIsFinalWeightmap)
	{
		// Update the mipmaps for the textures we edited
		for (int32 Idx = 0; Idx < NewWeightmapTextures.Num(); Idx++)
		{
			UTexture2D* WeightmapTexture = NewWeightmapTextures[Idx];
			FLandscapeTextureDataInfo* WeightmapDataInfo = DataInterface->GetTextureDataInfo(WeightmapTexture);

			int32 NumMips = WeightmapTexture->Source.GetNumMips();
			TArray<FColor*> WeightmapTextureMipData;
			WeightmapTextureMipData.AddUninitialized(NumMips);
			for (int32 MipIdx = 0; MipIdx < NumMips; MipIdx++)
			{
				WeightmapTextureMipData[MipIdx] = (FColor*)WeightmapDataInfo->GetMipData(MipIdx);
			}

			ULandscapeComponent::UpdateWeightmapMips(NumSubsections, SubsectionSizeQuads, WeightmapTexture, WeightmapTextureMipData, 0, 0, MAX_int32, MAX_int32, WeightmapDataInfo);
		}
	}

	// Replace the weightmap textures
	SetWeightmapTexturesInternal(MoveTemp(NewWeightmapTextures), InEditLayerGuid);
	SetWeightmapTexturesUsageInternal(MoveTemp(NewComponentWeightmapTexturesUsage), InEditLayerGuid);

	TargetProxy->ValidateProxyLayersWeightmapUsage();

	return CreatedTextures;
}

void ALandscapeProxy::RemoveInvalidWeightmaps()
{
	if (GIsEditor)
	{
		for (decltype(WeightmapUsageMap)::TIterator It(WeightmapUsageMap); It; ++It)
		{
			UTexture2D* Tex = It.Key();
			ULandscapeWeightmapUsage* Usage = It.Value();
			if (Usage->IsEmpty()) // Invalid Weight-map
			{
				if (Tex)
				{
					Tex->SetFlags(RF_Transactional);
					Tex->Modify();
					Tex->MarkPackageDirty();
					Tex->ClearFlags(RF_Standalone);
				}

				It.RemoveCurrent();
			}
		}

		// Remove Unused Weightmaps...
		for (ULandscapeComponent* Component : LandscapeComponents)
		{
			if (Component != nullptr)
			{
				Component->RemoveInvalidWeightmaps();
			}
		}
	}
}

void ULandscapeComponent::RemoveInvalidWeightmaps()
{
	// Process the final weightmaps
	RemoveInvalidWeightmaps(FGuid());

	// Also process all edit layers weightmaps :
	ForEachLayer([&](const FGuid& LayerGuid, FLandscapeLayerComponentData& LayerData)
	{
		RemoveInvalidWeightmaps(LayerGuid);
	});
}

void ULandscapeComponent::RemoveInvalidWeightmaps(const FGuid& InEditLayerGuid)
{
	TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = GetWeightmapLayerAllocations(InEditLayerGuid);
	TArray<TObjectPtr<UTexture2D>>& ComponentWeightmapTextures = GetWeightmapTextures(InEditLayerGuid);
	TArray<TObjectPtr<ULandscapeWeightmapUsage>>& ComponentWeightmapTexturesUsage = GetWeightmapTexturesUsage(InEditLayerGuid);

	// Adjust WeightmapTextureIndex index for other layers
	TSet<int32> UnUsedTextureIndices;
	{
		TSet<int32> UsedTextureIndices;
		for (int32 LayerIdx = 0; LayerIdx < ComponentWeightmapLayerAllocations.Num(); LayerIdx++)
		{
			UsedTextureIndices.Add(ComponentWeightmapLayerAllocations[LayerIdx].WeightmapTextureIndex);
		}

		for (int32 WeightIdx = 0; WeightIdx < ComponentWeightmapTextures.Num(); ++WeightIdx)
		{
			if (!UsedTextureIndices.Contains(WeightIdx))
			{
				UnUsedTextureIndices.Add(WeightIdx);
			}
		}
	}

	int32 RemovedTextures = 0;
	for (int32 UnusedIndex : UnUsedTextureIndices)
	{
		int32 WeightmapTextureIndexToRemove = UnusedIndex - RemovedTextures;
		ComponentWeightmapTextures[WeightmapTextureIndexToRemove]->SetFlags(RF_Transactional);
		ComponentWeightmapTextures[WeightmapTextureIndexToRemove]->Modify();
		ComponentWeightmapTextures[WeightmapTextureIndexToRemove]->MarkPackageDirty();
		ComponentWeightmapTextures[WeightmapTextureIndexToRemove]->ClearFlags(RF_Standalone);
		ComponentWeightmapTextures.RemoveAt(WeightmapTextureIndexToRemove);

		ComponentWeightmapTexturesUsage.RemoveAt(WeightmapTextureIndexToRemove);

		// Adjust WeightmapTextureIndex index for other layers
		for (int32 LayerIdx = 0; LayerIdx < ComponentWeightmapLayerAllocations.Num(); LayerIdx++)
		{
			FWeightmapLayerAllocationInfo& Allocation = ComponentWeightmapLayerAllocations[LayerIdx];

			if (Allocation.WeightmapTextureIndex > WeightmapTextureIndexToRemove)
			{
				Allocation.WeightmapTextureIndex--;
			}

			checkSlow(Allocation.WeightmapTextureIndex < WeightmapTextures.Num());
		}
		RemovedTextures++;
	}
}

void ULandscapeComponent::InitHeightmapData(TArray<FColor>& Heights, bool bUpdateCollision)
{
	int32 ComponentSizeVerts = NumSubsections * (SubsectionSizeQuads + 1);

	if (Heights.Num() != FMath::Square(ComponentSizeVerts))
	{
		return;
	}

	// Handling old Height map....
	if (HeightmapTexture && HeightmapTexture->GetOutermost() != GetTransientPackage()
		&& HeightmapTexture->GetOutermost() == GetOutermost()
		&& HeightmapTexture->Source.GetSizeX() >= ComponentSizeVerts) // if Height map is not valid...
	{
		HeightmapTexture->SetFlags(RF_Transactional);
		HeightmapTexture->Modify();
		HeightmapTexture->MarkPackageDirty();
		HeightmapTexture->ClearFlags(RF_Standalone); // Delete if no reference...
	}

	// New Height map
	TArray<FColor*> HeightmapTextureMipData;
	// make sure the heightmap UVs are powers of two.
	int32 HeightmapSizeU = (1 << FMath::CeilLogTwo(ComponentSizeVerts));
	int32 HeightmapSizeV = (1 << FMath::CeilLogTwo(ComponentSizeVerts));

	// Height map construction
	SetHeightmap(GetLandscapeProxy()->CreateLandscapeTexture(HeightmapSizeU, HeightmapSizeV, TEXTUREGROUP_Terrain_Heightmap, TSF_BGRA8));

	int32 MipSubsectionSizeQuads = SubsectionSizeQuads;
	int32 MipSizeU = HeightmapSizeU;
	int32 MipSizeV = HeightmapSizeV;

	HeightmapScaleBias = FVector4(1.0f / (float)HeightmapSizeU, 1.0f / (float)HeightmapSizeV, 0.0f, 0.0f);

	int32 Mip = 0;
	while (MipSizeU > 1 && MipSizeV > 1 && MipSubsectionSizeQuads >= 1)
	{
		FColor* HeightmapTextureData = (FColor*)GetHeightmap()->Source.LockMip(Mip);
		if (Mip == 0)
		{
			FMemory::Memcpy(HeightmapTextureData, Heights.GetData(), MipSizeU*MipSizeV*sizeof(FColor));
		}
		else
		{
			FMemory::Memzero(HeightmapTextureData, MipSizeU*MipSizeV*sizeof(FColor));
		}
		HeightmapTextureMipData.Add(HeightmapTextureData);

		MipSizeU >>= 1;
		MipSizeV >>= 1;
		Mip++;

		MipSubsectionSizeQuads = ((MipSubsectionSizeQuads + 1) >> 1) - 1;
	}
	ULandscapeComponent::GenerateHeightmapMips(HeightmapTextureMipData);

	if (bUpdateCollision)
	{
		UpdateCollisionHeightData(
			HeightmapTextureMipData[CollisionMipLevel],
			SimpleCollisionMipLevel > CollisionMipLevel ? HeightmapTextureMipData[SimpleCollisionMipLevel] : nullptr);
	}

	UTexture2D* Heightmap = GetHeightmap();
	for (int32 i = 0; i < HeightmapTextureMipData.Num(); i++)
	{
		Heightmap->Source.UnlockMip(i);
	}
	ULandscapeTextureHash::UpdateHash(Heightmap, ELandscapeTextureUsage::FinalData, ELandscapeTextureType::Heightmap);
	Heightmap->PostEditChange();
}

void ULandscapeComponent::InitWeightmapData(TArray<ULandscapeLayerInfoObject*>& LayerInfos, TArray<TArray<uint8> >& WeightmapData)
{
	if (LayerInfos.Num() != WeightmapData.Num() || LayerInfos.Num() <= 0)
	{
		return;
	}

	int32 ComponentSizeVerts = NumSubsections * (SubsectionSizeQuads + 1);

	// Validation..
	for (int32 Idx = 0; Idx < WeightmapData.Num(); ++Idx)
	{
		if (WeightmapData[Idx].Num() != FMath::Square(ComponentSizeVerts))
		{
			return;
		}
	}

	for (int32 Idx = 0; Idx < WeightmapTextures.Num(); ++Idx)
	{
		if (WeightmapTextures[Idx] && WeightmapTextures[Idx]->GetOutermost() != GetTransientPackage()
			&& WeightmapTextures[Idx]->GetOutermost() == GetOutermost()
			&& WeightmapTextures[Idx]->Source.GetSizeX() == ComponentSizeVerts)
		{
			WeightmapTextures[Idx]->SetFlags(RF_Transactional);
			WeightmapTextures[Idx]->Modify();
			WeightmapTextures[Idx]->MarkPackageDirty();
			WeightmapTextures[Idx]->ClearFlags(RF_Standalone); // Delete if no reference...
		}
	}
	WeightmapTextures.Empty();

	WeightmapLayerAllocations.Empty(LayerInfos.Num());
	for (int32 Idx = 0; Idx < LayerInfos.Num(); ++Idx)
	{
		new (WeightmapLayerAllocations)FWeightmapLayerAllocationInfo(LayerInfos[Idx]);
	}

	ReallocateWeightmaps(/*DataInterface = */nullptr, GetEditingLayerGUID(), /*bInSaveToTransactionBuffer = */true, /*bool bInForceReallocate = */false, /*InTargetProxy = */nullptr, /*InRestrictSharingToComponents = */nullptr);

	check(WeightmapLayerAllocations.Num() > 0 && WeightmapTextures.Num() > 0);

	int32 WeightmapSize = ComponentSizeVerts;
	WeightmapScaleBias = FVector4(1.0f / (float)WeightmapSize, 1.0f / (float)WeightmapSize, 0.5f / (float)WeightmapSize, 0.5f / (float)WeightmapSize);
	WeightmapSubsectionOffset = (float)(SubsectionSizeQuads + 1) / (float)WeightmapSize;

	TArray<void*> WeightmapDataPtrs;
	WeightmapDataPtrs.AddUninitialized(WeightmapTextures.Num());
	for (int32 WeightmapIdx = 0; WeightmapIdx < WeightmapTextures.Num(); ++WeightmapIdx)
	{
		// Calling modify here makes sure that async texture compilation finishes (triggered by ReallocateWeightmaps) so we can Lock the mip
		WeightmapTextures[WeightmapIdx]->Modify();
		WeightmapDataPtrs[WeightmapIdx] = WeightmapTextures[WeightmapIdx]->Source.LockMip(0);
	}

	for (int32 LayerIdx = 0; LayerIdx < WeightmapLayerAllocations.Num(); ++LayerIdx)
	{
		void* DestDataPtr = WeightmapDataPtrs[WeightmapLayerAllocations[LayerIdx].WeightmapTextureIndex];
		uint8* DestTextureData = (uint8*)DestDataPtr + ChannelOffsets[WeightmapLayerAllocations[LayerIdx].WeightmapTextureChannel];
		uint8* SrcTextureData = (uint8*)&WeightmapData[LayerIdx][0];

		for (int32 i = 0; i < WeightmapData[LayerIdx].Num(); i++)
		{
			DestTextureData[i * 4] = SrcTextureData[i];
		}
	}

	for (int32 Idx = 0; Idx < WeightmapTextures.Num(); Idx++)
	{
		UTexture2D* WeightmapTexture = WeightmapTextures[Idx];
		WeightmapTexture->Source.UnlockMip(0);
	}

	for (int32 Idx = 0; Idx < WeightmapTextures.Num(); Idx++)
	{
		UTexture2D* WeightmapTexture = WeightmapTextures[Idx];
		{
			const bool bShouldDirtyPackage = true;
			FLandscapeTextureDataInfo WeightmapDataInfo(WeightmapTexture, bShouldDirtyPackage);

			int32 NumMips = WeightmapTexture->Source.GetNumMips();
			TArray<FColor*> WeightmapTextureMipData;
			WeightmapTextureMipData.AddUninitialized(NumMips);
			for (int32 MipIdx = 0; MipIdx < NumMips; MipIdx++)
			{
				WeightmapTextureMipData[MipIdx] = (FColor*)WeightmapDataInfo.GetMipData(MipIdx);
			}

			ULandscapeComponent::UpdateWeightmapMips(NumSubsections, SubsectionSizeQuads, WeightmapTexture, WeightmapTextureMipData, 0, 0, MAX_int32, MAX_int32, &WeightmapDataInfo);
		}

		WeightmapTexture->PostEditChange();
	}

	FlushRenderingCommands();

	MaterialInstances.Empty(1);
	MaterialInstances.Add(nullptr);

	LODIndexToMaterialIndex.Empty(1);
	LODIndexToMaterialIndex.Add(0);

	//  TODO: need to update layer system?
}



bool ALandscapeStreamingProxy::IsValidLandscapeActor(ALandscape* Landscape)
{
	if (Landscape)
	{
		if (!Landscape->HasAnyFlags(RF_BeginDestroyed))
		{
			if (!LandscapeActorRef && !LandscapeGuid.IsValid())
			{
				return true; // always valid for newly created Proxy
			}
			if ((LandscapeActorRef && (LandscapeActorRef == Landscape))
				|| (!LandscapeActorRef && LandscapeGuid.IsValid() && (LandscapeGuid == Landscape->GetLandscapeGuid())))
			{
				const bool bCompatibleSize = (ComponentSizeQuads == Landscape->ComponentSizeQuads)
					&& (NumSubsections == Landscape->NumSubsections)
					&& (SubsectionSizeQuads == Landscape->SubsectionSizeQuads);

				if (!bCompatibleSize)
				{
					UE_LOGF(LogLandscape, Warning, "Landscape streaming proxy %ls's setup (Num Quads = %i, Num Subsections = %i, Num Quads per Subsection = %i) is not compatible with landscape actor %ls (Num Quads = %i, Num Subsections = %i, Num Quads per Subsection = %i)", 
						*GetName(), ComponentSizeQuads, NumSubsections, SubsectionSizeQuads, *Landscape->GetName(), Landscape->ComponentSizeQuads, Landscape->NumSubsections, Landscape->SubsectionSizeQuads);
				}
				return bCompatibleSize;
			}
		}
	}
	return false;
}

/* Returns the list of layer names relevant to mobile platforms. Walks the material tree following feature level switch nodes. */
static void GetAllMobileRelevantLayerNames(TSet<FName>& OutLayerNames, UMaterial* InMaterial)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GetAllMobileRelevantLayerNames);

	TArray<FName> LayerNames;

	bool bMobileUseRuntimeGrassMapGeneration = false;
	{
		// if ANY mobile platform has runtime generation (because we don't calculate mobile weightmaps per platform, just mobile/non-mobile)
		static FShaderPlatformCachedIniValue<bool> UseRuntimeGenerationCVar(TEXT("grass.GrassMap.UseRuntimeGeneration"));
		FGenericDataDrivenShaderPlatformInfo::Initialize();
		for (int32 SPIndex = 0; SPIndex < SP_NumPlatforms; SPIndex++)
		{
			EShaderPlatform SP = static_cast<EShaderPlatform>(SPIndex);
			if (FGenericDataDrivenShaderPlatformInfo::IsValid(SP) && IsMobilePlatform(SP) && UseRuntimeGenerationCVar.Get(SP))
			{
				bMobileUseRuntimeGrassMapGeneration = true;
				break;
			}
		}
	}

	TSet<UClass*> MobileCustomOutputExpressionTypesToQuery;
	if (bMobileUseRuntimeGrassMapGeneration)
	{
		MobileCustomOutputExpressionTypesToQuery.Add(UMaterialExpressionLandscapeGrassOutput::StaticClass());
	}

	const bool bRecurseIntoMaterialFunctions = true;
	TArray<UMaterialExpression*> ES31Expressions;
	InMaterial->GetAllReferencedExpressions(ES31Expressions, nullptr, ERHIFeatureLevel::ES3_1, EMaterialQualityLevel::Num, ERHIShadingPath::Num, bRecurseIntoMaterialFunctions, &MobileCustomOutputExpressionTypesToQuery);

	TArray<UMaterialExpression*> MobileExpressions = MoveTemp(ES31Expressions);
	for (UMaterialExpression* Expression : MobileExpressions)
	{
		if (Expression)
		{
			Expression->GetLandscapeLayerNames(LayerNames);
		}
	}

	for (const FName& Name : LayerNames)
	{
		OutLayerNames.Add(Name);
	}
}

void ULandscapeComponent::GenerateMobileWeightmapLayerAllocations()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeComponent::GenerateMobileWeightmapLayerAllocations);

	// Early-out if the component doesn't even have weightmap allocations in the first place, since GetAllMobileRelevantLayerNames can be quite expensive :
	if (WeightmapLayerAllocations.IsEmpty())
	{
		return;
	}

	const bool bComponentHasHoles = ComponentHasVisibilityPainted();
	UMaterialInterface* const HoleMaterial = bComponentHasHoles ? GetLandscapeHoleMaterial() : nullptr;
	UMaterialInterface* const MaterialToUse = bComponentHasHoles && HoleMaterial ? HoleMaterial : GetLandscapeMaterial();
		
	TSet<FName> LayerNames;
	GetAllMobileRelevantLayerNames(LayerNames, MaterialToUse->GetMaterial());
	MobileWeightmapLayerAllocations = WeightmapLayerAllocations.FilterByPredicate([&](const FWeightmapLayerAllocationInfo& Allocation) -> bool 
		{
			return Allocation.LayerInfo && LayerNames.Contains(Allocation.GetLayerName());
		}
	);
	MobileWeightmapLayerAllocations.StableSort(([&](const FWeightmapLayerAllocationInfo& A, const FWeightmapLayerAllocationInfo& B) -> bool
	{
		ULandscapeLayerInfoObject* LhsLayerInfo = A.LayerInfo;
		ULandscapeLayerInfoObject* RhsLayerInfo = B.LayerInfo;

		if (!LhsLayerInfo && !RhsLayerInfo) return false; // equally broken :P
		if (!LhsLayerInfo && RhsLayerInfo) return false; // broken layers sort to the end
		if (!RhsLayerInfo && LhsLayerInfo) return true;

		// Sort visibility layer to the front
		if (LhsLayerInfo == ALandscapeProxy::VisibilityLayer && RhsLayerInfo != ALandscapeProxy::VisibilityLayer) return true;
		if (RhsLayerInfo == ALandscapeProxy::VisibilityLayer && LhsLayerInfo != ALandscapeProxy::VisibilityLayer) return false;

		return false; // equal, preserve order
	}));
}

void ULandscapeComponent::GenerateMobilePlatformPixelData(bool bIsCooking, const ITargetPlatform* TargetPlatform)
{
	check(!IsTemplate());
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeComponent::GenerateMobilePlatformPixelData);

	GenerateMobileWeightmapLayerAllocations();

	int32 WeightmapSize = (SubsectionSizeQuads + 1) * NumSubsections;
	UE::Landscape::FBatchTextureCopy CopyRequests;

	MobileWeightmapTextures.Empty();
	MobileWeightmapTextureArray = nullptr;

	const int32 NumWeightTextures = FMath::DivideAndRoundUp(static_cast<int32>(Algo::CountIf(MobileWeightmapLayerAllocations, [](const FWeightmapLayerAllocationInfo& AllocationInfo) { return AllocationInfo.LayerInfo; })), 4);

	const bool MobileWeightmapTextureArrayEnabled = UE::Landscape::Private::IsMobileWeightmapTextureArrayEnabled();
	
	if (MobileWeightmapTextureArrayEnabled && NumWeightTextures > 0)
	{
		MobileWeightmapTextureArray = GetLandscapeProxy()->CreateLandscapeTextureArray(WeightmapSize, WeightmapSize, NumWeightTextures, TEXTUREGROUP_Terrain_Weightmap, TSF_BGRA8);
		MobileWeightmapTextureArray->PostEditChange();
		MobileWeightmapTextureArray->UpdateResource();
	}
	else
	{
		MobileWeightmapTextures.SetNum(NumWeightTextures);
		for (int32 i = 0; i < NumWeightTextures; ++i)
		{
			UTexture2D* CurrentWeightmapTexture  = GetLandscapeProxy()->CreateLandscapeTexture(WeightmapSize, WeightmapSize, TEXTUREGROUP_Terrain_Weightmap, TSF_BGRA8);
			const bool bClear = true;
			CreateEmptyTextureMips(CurrentWeightmapTexture, ELandscapeTextureUsage::FinalData, ELandscapeTextureType::Weightmap, bClear);
			MobileWeightmapTextures[i] = CurrentWeightmapTexture;
		}
	}

	{
		FLandscapeTextureDataInterface LandscapeData;
		UTexture2D* CurrentWeightmapTexture = nullptr;
		int32 CurrentChannel = 0;
		int32 RemainingChannels = 0;
		int32 index = 0;
		for (FWeightmapLayerAllocationInfo& Allocation : MobileWeightmapLayerAllocations)
		{
			if (!Allocation.LayerInfo)
			{
				continue;
			}
			
			if (RemainingChannels == 0)
			{
				CurrentChannel = 0;
				RemainingChannels = 4;
			}

			int32 Slice = FMath::DivideAndRoundDown(index,4);
			if (MobileWeightmapTextureArrayEnabled)
			{
				CopyRequests.AddWeightmapCopy(MobileWeightmapTextureArray, Slice, IntCastChecked<int8>(CurrentChannel), this, Allocation.LayerInfo);	
			}
			else
			{
				CopyRequests.AddWeightmapCopy(MobileWeightmapTextures[Slice], 0, IntCastChecked<int8>(CurrentChannel), this, Allocation.LayerInfo);
			}
			
			// update Allocation
			Allocation.WeightmapTextureIndex = IntCastChecked<uint8>(Slice);
			Allocation.WeightmapTextureChannel = IntCastChecked<uint8>(CurrentChannel);
			CurrentChannel++;
			RemainingChannels--;
			index++;
		}
		
	}
	
	CopyRequests.ProcessTextureCopies();
	
	if (MobileWeightmapTextureArray)
	{
		GDisableAutomaticTextureMaterialUpdateDependencies = true;
		MobileWeightmapTextureArray->PostEditChange();
		MobileWeightmapTextureArray->UpdateResource();
		MobileWeightmapTextureArray->SetDeterministicLightingGuid();
		GDisableAutomaticTextureMaterialUpdateDependencies = false;
	}
	else
	{
		GDisableAutomaticTextureMaterialUpdateDependencies = true;
		for (int TextureIdx = 0; TextureIdx < MobileWeightmapTextures.Num(); TextureIdx++)
		{
			UTexture* Texture = MobileWeightmapTextures[TextureIdx];
			Texture->PostEditChange();

			// PostEditChange() will assign a random GUID to the texture, which leads to non-deterministic builds.
			Texture->SetDeterministicLightingGuid();
		}
		GDisableAutomaticTextureMaterialUpdateDependencies = false;
	}

	FLinearColor Masks[4];
	Masks[0] = FLinearColor(1, 0, 0, 0);
	Masks[1] = FLinearColor(0, 1, 0, 0);
	Masks[2] = FLinearColor(0, 0, 1, 0);
	Masks[3] = FLinearColor(0, 0, 0, 1);


	if (!GIsEditor)
	{
		// This path is used by game mode running with uncooked data, eg standalone executable Mobile Preview.
		// Game mode cannot create MICs, so we use a MaterialInstanceDynamic here.
		
		// Fallback to use non mobile materials if there is no mobile one
		if (MobileCombinationMaterialInstances.Num() == 0)
		{
			MobileCombinationMaterialInstances.Append(MaterialInstances);
		}

		MobileMaterialInterfaces.Reset();
		MobileMaterialInterfaces.Reserve(MobileCombinationMaterialInstances.Num());

		for (int32 MaterialIndex = 0; MaterialIndex < MobileCombinationMaterialInstances.Num(); ++MaterialIndex)
		{
			UMaterialInstanceDynamic* NewMobileMaterialInstance = UMaterialInstanceDynamic::Create(MobileCombinationMaterialInstances[MaterialIndex], this);

			// Set the layer mask
			for (const FWeightmapLayerAllocationInfo& Allocation : MobileWeightmapLayerAllocations)
			{
				if (Allocation.LayerInfo)
				{
					NewMobileMaterialInstance->SetVectorParameterValue(FName(*FString::Printf(TEXT("LayerMask_%s"), *Allocation.GetLayerName().ToString())), Masks[Allocation.WeightmapTextureChannel]);
				}
			}

			if (MobileWeightmapTextureArray)
			{
				NewMobileMaterialInstance->SetTextureParameterValue(TEXT("WeightmapArray"), MobileWeightmapTextureArray);	
			}
			else
			{
				for (int TextureIdx = 0; TextureIdx < MobileWeightmapTextures.Num(); TextureIdx++)
				{
					NewMobileMaterialInstance->SetTextureParameterValue(FName(*FString::Printf(TEXT("Weightmap%d"), TextureIdx)), MobileWeightmapTextures[TextureIdx]);
				}	
			}
			
			MobileMaterialInterfaces.Add(NewMobileMaterialInstance);
		}
	}
	else
	{
		// When cooking, we need to make a persistent MIC. In the editor we also do so in
		// case we start a Cook in Editor operation, which will reuse the MIC we create now.

		check(LODIndexToMaterialIndex.Num() > 0);		

		if (MaterialPerLOD.Num() == 0)
		{
			const int8 MaxLOD = static_cast<int8>(FMath::CeilLogTwo(SubsectionSizeQuads + 1) - 1);

			for (int8 LODIndex = 0; LODIndex <= MaxLOD; ++LODIndex)
			{
				UMaterialInterface* CurrentMaterial = GetLandscapeMaterial(LODIndex);

				if (MaterialPerLOD.Find(CurrentMaterial) == nullptr)
				{
					MaterialPerLOD.Add(CurrentMaterial, LODIndex);
				}
			}
		}

		MobileCombinationMaterialInstances.SetNumZeroed(MaterialPerLOD.Num());
		MobileMaterialInterfaces.Reset();
		MobileMaterialInterfaces.Reserve(MaterialPerLOD.Num());
		int8 MaterialIndex = 0;

		for (auto It = MaterialPerLOD.CreateConstIterator(); It; ++It)
		{
			const int8 MaterialLOD = It.Value();

			// Find or set a matching MIC in the Landscape's map.
			MobileCombinationMaterialInstances[MaterialIndex] = GetCombinationMaterial(nullptr, MobileWeightmapLayerAllocations, MaterialLOD, true);
			if (MobileCombinationMaterialInstances[MaterialIndex] == nullptr)
			{
				continue;
			}

			if (bIsCooking)
			{
				// If we are cooking ensure we are caching shader maps.
				MobileCombinationMaterialInstances[MaterialIndex]->BeginCacheForCookedPlatformData(TargetPlatform);
			}

			UMaterialInstanceConstant* NewMobileMaterialInstance = NewObject<ULandscapeMaterialInstanceConstant>(this);

			NewMobileMaterialInstance->SetParentEditorOnly(MobileCombinationMaterialInstances[MaterialIndex]);

			// Set the layer mask
			for (const FWeightmapLayerAllocationInfo& Allocation : MobileWeightmapLayerAllocations)
			{
				if (Allocation.LayerInfo)
				{
					NewMobileMaterialInstance->SetVectorParameterValueEditorOnly(FName(*FString::Printf(TEXT("LayerMask_%s"), *Allocation.GetLayerName().ToString())), Masks[Allocation.WeightmapTextureChannel]);
				}
			}

			if (MobileWeightmapTextureArray)
			{
				NewMobileMaterialInstance->SetTextureParameterValueEditorOnly(TEXT("WeightmapArray"), MobileWeightmapTextureArray);
			}
			else
			{
				for (int TextureIdx = 0; TextureIdx < MobileWeightmapTextures.Num(); TextureIdx++)
				{
					NewMobileMaterialInstance->SetTextureParameterValueEditorOnly(FName(*FString::Printf(TEXT("Weightmap%d"), TextureIdx)), MobileWeightmapTextures[TextureIdx]);
				}
			}
			
			NewMobileMaterialInstance->PostEditChange();

			MobileMaterialInterfaces.Add(NewMobileMaterialInstance);
			++MaterialIndex;
		}
	}
}

FName ALandscapeProxy::GenerateUniqueLandscapeTextureName(UObject* InOuter, TextureGroup InLODGroup) const
{
	FName BaseName;
	if (InLODGroup == TEXTUREGROUP_Terrain_Heightmap)
	{
		BaseName = "Heightmap";
	}
	else if (InLODGroup == TEXTUREGROUP_Terrain_Weightmap)
	{
		BaseName = "Weightmap";
	}
	return MakeUniqueObjectName(InOuter, UTexture2D::StaticClass(), BaseName);
}

UTexture2D* ALandscapeProxy::CreateLandscapeTexture(int32 InSizeX, int32 InSizeY, TextureGroup InLODGroup, ETextureSourceFormat InFormat, UObject* OptionalOverrideOuter, bool bCompress, bool bMipChain) const
{
	UObject* TexOuter = OptionalOverrideOuter ? OptionalOverrideOuter : const_cast<ALandscapeProxy*>(this);
	UTexture2D* NewTexture = NewObject<UTexture2D>(TexOuter, GenerateUniqueLandscapeTextureName(TexOuter, InLODGroup));
	if (bMipChain)
	{
		NewTexture->Source.Init2DWithMipChain(InSizeX, InSizeY, InFormat);
	}
	else
	{
		NewTexture->Source.Init(InSizeX, InSizeY, 1, 1, InFormat);
	}
	
	NewTexture->SRGB = false;
	NewTexture->CompressionNone = !bCompress;
	NewTexture->CompressionQuality = TCQ_Highest;
	NewTexture->MipGenSettings = bMipChain ? TMGS_LeaveExistingMips : TMGS_NoMipmaps;
	NewTexture->AddressX = TA_Clamp;
	NewTexture->AddressY = TA_Clamp;
	NewTexture->LODGroup = InLODGroup;

	return NewTexture;
}

UTexture2DArray* ALandscapeProxy::CreateLandscapeTextureArray(int32 InSizeX, int32 InSizeY, int32 Slices, TextureGroup InLODGroup, ETextureSourceFormat InFormat, UObject* OptionalOverrideOuter )
{
	UObject* TexOuter = OptionalOverrideOuter ? OptionalOverrideOuter : const_cast<ALandscapeProxy*>(this);
	UTexture2DArray* NewTextureArray = NewObject<UTexture2DArray>(TexOuter);
	
	const int32 NumMips = FMath::FloorLog2(FMath::Max(InSizeX, InSizeY)) + 1;
	NewTextureArray->Source.Init(InSizeX, InSizeY, Slices, NumMips, InFormat);

	NewTextureArray->SRGB = false;
	NewTextureArray->CompressionNone = true;
	NewTextureArray->MipGenSettings = TMGS_LeaveExistingMips;
	NewTextureArray->AddressX = TA_Clamp;
	NewTextureArray->AddressY = TA_Clamp;
	NewTextureArray->LODGroup = InLODGroup;
	
	return NewTextureArray;
}

UTexture2D* ALandscapeProxy::CreateLandscapeToolTexture(int32 InSizeX, int32 InSizeY, TextureGroup InLODGroup, ETextureSourceFormat InFormat) const
{
	UObject* TexOuter = const_cast<ALandscapeProxy*>(this);
	UTexture2D* NewTexture = NewObject<UTexture2D>(TexOuter, GenerateUniqueLandscapeTextureName(TexOuter, InLODGroup));

	int32 BytesPerPixel = FTextureSource::GetBytesPerPixel(InFormat);
	int32 ZeroBufferSize = BytesPerPixel * InSizeX * InSizeY;
	uint8* ZeroBuffer = reinterpret_cast<uint8*>(FMemory::MallocZeroed(ZeroBufferSize));
	NewTexture->Source.Init(InSizeX, InSizeY, 1, 1, InFormat, ZeroBuffer);
	FMemory::Free(ZeroBuffer);

	// Note : for TSF_G8, use TC_Grayscale even though CompressionNone is set to true, otherwise, the texture ends up being created as BGRA instead 
	NewTexture->CompressionSettings = (InFormat == TSF_G8) ? TC_Grayscale : TC_Default;
	NewTexture->SRGB = false;
	NewTexture->CompressionNone = true;
	NewTexture->MipGenSettings = TMGS_NoMipmaps;
	NewTexture->AddressX = TA_Clamp;
	NewTexture->AddressY = TA_Clamp;
	NewTexture->LODGroup = InLODGroup;

	return NewTexture;
}

ULandscapeWeightmapUsage* ALandscapeProxy::CreateWeightmapUsage()
{
	// NonTransactional on purpose : it's too much trouble to have usages transactional since they're present in the proxies and duplicated in possibly multiple components, 
	//  plus some edit layers (the splines layer, namely, which is procedural) are non-transactional, which complicates things further. Instead, we just regenerate the usages
	//  on undo
	return NewObject<ULandscapeWeightmapUsage>(this, ULandscapeWeightmapUsage::StaticClass(), NAME_None, RF_NoFlags);
}

void ALandscapeProxy::RemoveOverlappingComponent(ULandscapeComponent* Component)
{
	Modify();
	Component->Modify();

	ULandscapeHeightfieldCollisionComponent* CollisionComponent = Component->GetCollisionComponent();
	if ((CollisionComponent != nullptr) && (CollisionComponent->GetRenderComponent() == Component || (CollisionComponent->GetRenderComponent() == nullptr)))
	{
		CollisionComponent->Modify();
		CollisionComponents.Remove(CollisionComponent);
		CollisionComponent->DestroyComponent();
	}
	LandscapeComponents.Remove(Component);
	Component->DestroyComponent();
}

TArray<FLinearColor> ALandscapeProxy::SampleRTData(UTextureRenderTarget2D* InRenderTarget, FLinearColor InRect)
{
	if (!InRenderTarget)
	{
		FMessageLog("Blueprint").Warning(LOCTEXT("SampleRTData_InvalidRenderTarget", "SampleRTData: Render Target must be non-null."));
		return { FLinearColor(0,0,0,0) };
	}
	else if (!InRenderTarget->GetResource())
	{
		FMessageLog("Blueprint").Warning(LOCTEXT("SampleRTData_ReleasedRenderTarget", "SampleRTData: Render Target has been released."));
		return { FLinearColor(0,0,0,0) };
	}
	else
	{
		InRect.R = static_cast<float>(FMath::Clamp(int(InRect.R), 0, InRenderTarget->SizeX - 1));
		InRect.G = static_cast<float>(FMath::Clamp(int(InRect.G), 0, InRenderTarget->SizeY - 1));
		InRect.B = static_cast<float>(FMath::Clamp(int(InRect.B), int(InRect.R + 1), InRenderTarget->SizeX));
		InRect.A = static_cast<float>(FMath::Clamp(int(InRect.A), int(InRect.G + 1), InRenderTarget->SizeY));
		FIntRect Rect = FIntRect(static_cast<int32>(InRect.R), static_cast<int32>(InRect.G), static_cast<int32>(InRect.B), static_cast<int32>(InRect.A));

		FImage Image;
		if ( ! FImageUtils::GetRenderTargetImage(InRenderTarget,Image,Rect) )
		{
			FMessageLog("Blueprint").Warning(LOCTEXT("SampleRTData_FailedGetRenderTarget", "SampleRTData: GetRenderTargetImage failed."));
			return { FLinearColor(0,0,0,0) };
		}

		Image.ChangeFormat(ERawImageFormat::RGBA32F,EGammaSpace::Linear);
		
		TArrayView64<FLinearColor> Colors = Image.AsRGBA32F();

		return TArray<FLinearColor>( Colors );
	}
}

bool ALandscapeProxy::LandscapeImportHeightmapFromRenderTarget(UTextureRenderTarget2D* InRenderTarget, bool InImportHeightFromRGChannel, int32 InEditLayerIndex)
{
	uint64 StartCycle = FPlatformTime::Cycles64();

	ALandscape* Landscape = GetLandscapeActor();
	ULandscapeInfo* LandscapeInfo = Landscape ? Landscape->GetLandscapeInfo() : nullptr;
	if (Landscape == nullptr || LandscapeInfo == nullptr)
	{
		FMessageLog("Blueprint").Error(LOCTEXT("LandscapeImportHeightmapFromRenderTarget_NullLandscape", "LandscapeImportHeightmapFromRenderTarget: Landscape must be non-null."));
		return false;
	}

	const ULandscapeEditLayerBase* EditLayer = LandscapeInfo->LandscapeActor->GetEditLayer(InEditLayerIndex);
	if (EditLayer == nullptr)
	{
		FMessageLog("Blueprint").Error(LOCTEXT("LandscapeImportHeightmapFromRenderTarget_InvalidEditLayer", "LandscapeImportHeightmapFromRenderTarget: Invalid edit layer index for import."));
		return false;
	}
	const FGuid& EditLayerGuid = LandscapeInfo->LandscapeActor->GetEditLayer(InEditLayerIndex)->GetGuid();

	int32 MinX, MinY, MaxX, MaxY;

	if (!LandscapeInfo->GetLandscapeExtent(MinX, MinY, MaxX, MaxY))
	{
		FMessageLog("Blueprint").Error(LOCTEXT("LandscapeImportHeightmapFromRenderTarget_InvalidLandscapeExtends", "LandscapeImportHeightmapFromRenderTarget: The landscape min extends are invalid."));
		return false;
	}

	if (InRenderTarget == nullptr || InRenderTarget->GetResource() == nullptr)
	{
		FMessageLog("Blueprint").Error(LOCTEXT("LandscapeImportHeightmapFromRenderTarget_InvalidRT", "LandscapeImportHeightmapFromRenderTarget: Render Target must be non null and not released."));
		return false;
	}

	FTextureRenderTargetResource* RenderTargetResource = InRenderTarget->GameThread_GetRenderTargetResource();
	FIntRect SampleRect = FIntRect(0, 0, FMath::Min(1 + MaxX - MinX, InRenderTarget->SizeX), FMath::Min(1 + MaxY - MinY, InRenderTarget->SizeY));

	TArray<uint16> HeightData;

	switch (InRenderTarget->RenderTargetFormat)
	{
		case RTF_RGBA16f:
		case RTF_RGBA32f:
		{
			TArray<FLinearColor> OutputRTHeightmap;
			OutputRTHeightmap.Reserve(SampleRect.Width() * SampleRect.Height());

			RenderTargetResource->ReadLinearColorPixels(OutputRTHeightmap, FReadSurfaceDataFlags(RCM_MinMax, CubeFace_MAX), SampleRect);
			HeightData.Reserve(OutputRTHeightmap.Num());

			for (const FLinearColor& LinearColor : OutputRTHeightmap)
			{
				if (InImportHeightFromRGChannel)
				{
					FColor Color = LinearColor.ToFColor(false);
					uint16 Height = ((Color.R << 8) | Color.G);
					HeightData.Add(Height);
				}
				else
				{
					HeightData.Add((uint16)LinearColor.R);
				}
			}
		}
		break;			

		case RTF_RGBA8:
		{
			TArray<FColor> OutputRTHeightmap;
			OutputRTHeightmap.Reserve(SampleRect.Width() * SampleRect.Height());

			RenderTargetResource->ReadPixels(OutputRTHeightmap, FReadSurfaceDataFlags(RCM_MinMax, CubeFace_MAX), SampleRect);
			HeightData.Reserve(OutputRTHeightmap.Num());

			for (FColor Color : OutputRTHeightmap)
			{
				uint16 Height = ((Color.R << 8) | Color.G);
				HeightData.Add(Height);
			}
		}
		break;

		default:
		{
			FMessageLog("Blueprint").Error(LOCTEXT("LandscapeImportHeightmapFromRenderTarget_InvalidRTFormat", "LandscapeImportHeightmapFromRenderTarget: The Render Target format is invalid. We only support RTF_RGBA16f, RTF_RGBA32f, RTF_RGBA8"));
			return false;
		}
	}	

	FScopedTransaction Transaction(LOCTEXT("Undo_ImportHeightmap", "Importing Landscape Heightmap"));

	FHeightmapAccessor<false> HeightmapAccessor(LandscapeInfo);
	
	HeightmapAccessor.SetEditLayer(EditLayerGuid);
	HeightmapAccessor.SetData(MinX, MinY, SampleRect.Width() - 1, SampleRect.Height() - 1, HeightData.GetData());

	double SecondsTaken = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartCycle);
	UE_LOGF(LogLandscapeBP, Display, "Took %f seconds to import heightmap from render target.", SecondsTaken);

	return true;
}
#endif

bool ALandscapeProxy::LandscapeExportHeightmapToRenderTarget(UTextureRenderTarget2D* InRenderTarget, bool bInExportHeightIntoRGChannel, bool InExportLandscapeProxies)
{
	// TODO [jonathan.bard] : This is redundant with RenderWeightmap, which is available in engine as well, so it'd probably best to deprecate this
#if WITH_EDITOR
	uint64 StartCycle = FPlatformTime::Cycles64();

	UMaterial* HeightmapRenderMaterial = LoadObject<UMaterial>(nullptr, TEXT("/Engine/EditorLandscapeResources/Landscape_Heightmap_To_RenderTarget2D.Landscape_Heightmap_To_RenderTarget2D"));
	if (HeightmapRenderMaterial == nullptr)
	{
		FMessageLog("Blueprint").Error(LOCTEXT("LandscapeExportHeightmapToRenderTarget_Landscape_Heightmap_To_RenderTarget2D.", "LandscapeExportHeightmapToRenderTarget: Material Landscape_Heightmap_To_RenderTarget2D not found in engine content."));
		return false;
	}

	TArray<ULandscapeComponent*> LandscapeComponentsToExport;
	//  Export the component of the specified proxy
	LandscapeComponentsToExport.Append(LandscapeComponents);

	// If requested, export all proxies
	if (InExportLandscapeProxies && (GetLandscapeActor() == this))
	{
		ULandscapeInfo* LandscapeInfo = GetLandscapeInfo();

		LandscapeInfo->ForEachLandscapeProxy([&](ALandscapeProxy* Proxy)
		{
			LandscapeComponentsToExport.Append(Proxy->LandscapeComponents);
			return true;
		});
	}

	if (LandscapeComponentsToExport.Num() == 0)
	{
		return true;
	}

	UWorld* World = GEditor->GetEditorWorldContext().World();
	FTextureRenderTargetResource* RenderTargetResource = InRenderTarget->GameThread_GetRenderTargetResource();

	// Create a canvas for the render target and clear it to black
	FCanvas Canvas(RenderTargetResource, nullptr, FGameTime(), World->GetFeatureLevel());
	Canvas.Clear(FLinearColor::Black);

	// Find exported component's base offset
	FIntRect ComponentsExtent(MAX_int32, MAX_int32, MIN_int32, MIN_int32);
	for (ULandscapeComponent* Component : LandscapeComponentsToExport)
	{
		Component->GetComponentExtent(ComponentsExtent.Min.X, ComponentsExtent.Min.Y, ComponentsExtent.Max.X, ComponentsExtent.Max.Y);
	}
	FIntPoint ExportBaseOffset = ComponentsExtent.Min;

	struct FTrianglePerMID
	{
		UMaterialInstanceDynamic* HeightmapMID;
		TArray<FCanvasUVTri> TriangleList;
	};

	TMap<UTexture*, FTrianglePerMID> TrianglesPerHeightmap;

	for (const ULandscapeComponent* Component : LandscapeComponentsToExport)
	{
		FTrianglePerMID* TrianglesPerMID = TrianglesPerHeightmap.Find(Component->GetHeightmap());

		if (TrianglesPerMID == nullptr)
		{
			FTrianglePerMID Data;
			Data.HeightmapMID = UMaterialInstanceDynamic::Create(HeightmapRenderMaterial, this);
			Data.HeightmapMID->SetTextureParameterValue(TEXT("Heightmap"), Component->GetHeightmap());
			Data.HeightmapMID->SetScalarParameterValue(TEXT("ExportHeightIntoRGChannel"), bInExportHeightIntoRGChannel);
			TrianglesPerMID = &TrianglesPerHeightmap.Add(Component->GetHeightmap(), Data);
		}

		FIntPoint ComponentSectionBase = Component->GetSectionBase();
		FIntPoint ComponentHeightmapTextureSize(Component->GetHeightmap()->Source.GetSizeX(), Component->GetHeightmap()->Source.GetSizeY());
		int32 SubsectionSizeVerts = Component->SubsectionSizeQuads + 1;
		float HeightmapSubsectionOffsetU = (float)(SubsectionSizeVerts) / (float)ComponentHeightmapTextureSize.X;
		float HeightmapSubsectionOffsetV = (float)(SubsectionSizeVerts) / (float)ComponentHeightmapTextureSize.Y;

		for (int8 SubY = 0; SubY < NumSubsections; ++SubY)
		{
			for (int8 SubX = 0; SubX < NumSubsections; ++SubX)
			{
				FIntPoint SubSectionSectionBase = ComponentSectionBase - ExportBaseOffset;
				SubSectionSectionBase.X += Component->SubsectionSizeQuads * SubX;
				SubSectionSectionBase.Y += Component->SubsectionSizeQuads * SubY;

				// Offset for this component's data in heightmap texture
				const float HeightmapOffsetU = static_cast<float>(Component->HeightmapScaleBias.Z) + HeightmapSubsectionOffsetU * SubX;
				const float HeightmapOffsetV = static_cast<float>(Component->HeightmapScaleBias.W) + HeightmapSubsectionOffsetV * SubY;

				FCanvasUVTri Tri1;
				Tri1.V0_Pos = FVector2D(SubSectionSectionBase.X, SubSectionSectionBase.Y);
				Tri1.V1_Pos = FVector2D(SubSectionSectionBase.X + SubsectionSizeVerts, SubSectionSectionBase.Y);
				Tri1.V2_Pos = FVector2D(SubSectionSectionBase.X + SubsectionSizeVerts, SubSectionSectionBase.Y + SubsectionSizeVerts);

				Tri1.V0_UV = FVector2D(HeightmapOffsetU, HeightmapOffsetV);
				Tri1.V1_UV = FVector2D(HeightmapOffsetU + HeightmapSubsectionOffsetU, HeightmapOffsetV);
				Tri1.V2_UV = FVector2D(HeightmapOffsetU + HeightmapSubsectionOffsetU, HeightmapOffsetV + HeightmapSubsectionOffsetV);
				TrianglesPerMID->TriangleList.Add(Tri1);

				FCanvasUVTri Tri2;
				Tri2.V0_Pos = FVector2D(SubSectionSectionBase.X + SubsectionSizeVerts, SubSectionSectionBase.Y + SubsectionSizeVerts);
				Tri2.V1_Pos = FVector2D(SubSectionSectionBase.X, SubSectionSectionBase.Y + SubsectionSizeVerts);
				Tri2.V2_Pos = FVector2D(SubSectionSectionBase.X, SubSectionSectionBase.Y);

				Tri2.V0_UV = FVector2D(HeightmapOffsetU + HeightmapSubsectionOffsetU, HeightmapOffsetV + HeightmapSubsectionOffsetV);
				Tri2.V1_UV = FVector2D(HeightmapOffsetU, HeightmapOffsetV + HeightmapSubsectionOffsetV);
				Tri2.V2_UV = FVector2D(HeightmapOffsetU, HeightmapOffsetV);

				TrianglesPerMID->TriangleList.Add(Tri2);
			}
		}
	}

	for (auto& TriangleList : TrianglesPerHeightmap)
	{
		FCanvasTriangleItem TriItemList(MoveTemp(TriangleList.Value.TriangleList), nullptr);
		TriItemList.MaterialRenderProxy = TriangleList.Value.HeightmapMID->GetRenderProxy();
		TriItemList.BlendMode = SE_BLEND_Opaque;
		TriItemList.SetColor(FLinearColor::White);

		TriItemList.Draw(&Canvas);
	}

	TrianglesPerHeightmap.Reset();

	// Tell the rendering thread to draw any remaining batched elements
	Canvas.Flush_GameThread(true);

	ENQUEUE_RENDER_COMMAND(DrawHeightmapRTCommand)(
		[RenderTargetResource](FRHICommandListImmediate& RHICmdList)
		{
			TransitionAndCopyTexture(RHICmdList, RenderTargetResource->GetRenderTargetTexture(), RenderTargetResource->TextureRHI, {});
		});


	FlushRenderingCommands();

	double SecondsTaken = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartCycle);
	UE_LOGF(LogLandscapeBP, Display, "Took %f seconds to export heightmap to render target.", SecondsTaken);
#endif
	return true;
}

TArray<FName> ALandscape::GetTargetLayerNames(bool bInIncludeVisibilityLayer) const
{
	TArray<FName> Result;

#if WITH_EDITOR
	if (ULandscapeInfo* LandscapeInfo = GetLandscapeInfo())
	{
		Algo::TransformIf(LandscapeInfo->Layers, Result,
			[bInIncludeVisibilityLayer](const FLandscapeInfoLayerSettings& InSettings) 
			{ 
				return (InSettings.LayerInfoObj != nullptr) 
				&& (bInIncludeVisibilityLayer || (InSettings.LayerInfoObj != ALandscapeProxy::VisibilityLayer)); 
			},
			[](const FLandscapeInfoLayerSettings& InSettings) { return InSettings.GetLayerName(); });
	}
#else // WITH_EDITOR
	FMessageLog("Blueprint").Error(LOCTEXT("GetTargetLayerNames_Runtime.", "GetTargetLayerNames_EditorOnly: this cannot be called at runtime"));
#endif // !WITH_EDITOR

	return Result;
}

const TArray<ULandscapeEditLayerBase*> ALandscape::GetEditLayersBP() const
{
#if WITH_EDITOR
		return GetEditLayers();
#else // WITH_EDITOR
		return {};
#endif // !WITH_EDITOR
}

const ULandscapeEditLayerBase* ALandscape::GetEditLayerByNameBP(const FName& InLayerName) const
{
#if WITH_EDITOR
	return ObjectPtrWrap(GetEditLayer(InLayerName));
#else // WITH_EDITOR
	return nullptr;
#endif // !WITH_EDITOR
}


#if WITH_EDITOR

bool ALandscapeProxy::LandscapeImportWeightmapFromRenderTarget(UTextureRenderTarget2D* InRenderTarget, FName InLayerName, int32 InEditLayerIndex)
{
	if (ALandscape* Landscape = GetLandscapeActor())
	{
		ULandscapeInfo* LandscapeInfo = Landscape->GetLandscapeInfo();

		int32 MinX, MinY, MaxX, MaxY;
		if (LandscapeInfo && LandscapeInfo->GetLandscapeExtent(MinX, MinY, MaxX, MaxY))
		{
			const ULandscapeEditLayerBase* EditLayer = LandscapeInfo->LandscapeActor->GetEditLayer(InEditLayerIndex);
			if (EditLayer == nullptr)
			{
				FMessageLog("Blueprint").Error(LOCTEXT("LandscapeImportRenderTarget_InvalidEditLayer", "LandscapeImportWeightmapFromRenderTarget: Invalid edit layer index for import."));
				return false;
			}
			const FGuid& EditLayerGuid = LandscapeInfo->LandscapeActor->GetEditLayer(InEditLayerIndex)->GetGuid();

			const uint32 LandscapeWidth = (uint32)(1 + MaxX - MinX);
			const uint32 LandscapeHeight = (uint32)(1 + MaxY - MinY);
			const FLinearColor SampleRect = FLinearColor(0.0f, 0.0f, static_cast<float>(LandscapeWidth), static_cast<float>(LandscapeHeight));

			const uint32 RTWidth = InRenderTarget->SizeX;
			const uint32 RTHeight = InRenderTarget->SizeY;
			ETextureRenderTargetFormat format = (InRenderTarget->RenderTargetFormat);

			if (RTWidth >= LandscapeWidth && RTHeight >= LandscapeHeight)
			{
				TArray<FLinearColor> RTData;
				RTData = SampleRTData(InRenderTarget, SampleRect);

				TArray<uint8> LayerData;
				LayerData.Reserve( RTData.Num() );

				for (const FLinearColor & RTColor : RTData)
				{
					LayerData.Add( FColor::QuantizeUNormFloatTo8(RTColor.R) );
				}

				FLandscapeInfoLayerSettings CurWeightmapInfo;

				int32 Index = LandscapeInfo->GetLayerInfoIndex(InLayerName, LandscapeInfo->GetLandscapeProxy());

				if (ensure(Index != INDEX_NONE))
				{
					CurWeightmapInfo = LandscapeInfo->Layers[Index];
				}

				if (CurWeightmapInfo.LayerInfoObj == nullptr)
				{
					FMessageLog("Blueprint").Error(LOCTEXT("LandscapeImportRenderTarget_InvalidLayerInfoObject", "LandscapeImportWeightmapFromRenderTarget: Layers must first have Layer Info Objects assigned before importing."));
					return false;
				}

				FScopedTransaction Transaction(LOCTEXT("Undo_ImportWeightmap", "Importing Landscape Layer"));

				TAlphamapAccessor</*bInUseInterp = */false> AlphamapAccessor(LandscapeInfo, CurWeightmapInfo.LayerInfoObj);
				AlphamapAccessor.SetEditLayer(EditLayerGuid);
				AlphamapAccessor.SetData(MinX, MinY, MaxX, MaxY, LayerData.GetData(), ELandscapeLayerPaintingRestriction::None);

				uint64 CycleEnd = FPlatformTime::Cycles64();
				UE_LOGF(LogLandscape, Verbose, "Took %f seconds to import heightmap from render target", FPlatformTime::ToSeconds64(CycleEnd));

				return true;
			}
			else
			{
				FMessageLog("Blueprint").Error(LOCTEXT("LandscapeImportRenderTarget_InvalidRenderTarget", "LandscapeImportWeightmapFromRenderTarget: Render target must be at least as large as landscape on each axis."));
				return false;
			}
		}
		else
		{
			return false;
		}
	}
	FMessageLog("Blueprint").Error(LOCTEXT("LandscapeImportRenderTarget_NullLandscape.", "LandscapeImportWeightmapFromRenderTarget: Landscape must be non-null."));
	return false;
}

// Deprecated
bool ALandscapeProxy::LandscapeExportWeightmapToRenderTarget(UTextureRenderTarget2D* InRenderTarget, FName InLayerName)
{
	return false;
}

#endif //WITH_EDITOR

#undef LOCTEXT_NAMESPACE
