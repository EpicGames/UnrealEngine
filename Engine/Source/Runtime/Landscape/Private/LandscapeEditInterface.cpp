// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
LandscapeEditInterface.cpp: Landscape editing interface
=============================================================================*/

#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformStackWalk.h"
#include "Templates/Greater.h"
#include "Containers/ArrayView.h"
#include "RenderingThread.h"
#include "LandscapeProxy.h"
#include "Landscape.h"
#include "LandscapeEditLayer.h"
#include "LandscapeInfo.h"
#include "LandscapeComponent.h"
#include "LandscapeLayerInfoObject.h"

#if WITH_EDITOR

#include "LandscapeDataAccess.h"
#include "LandscapeEdit.h"
#include "LandscapeRender.h"
#include "LandscapePrivate.h"
#include "ComponentReregisterContext.h"
#include "LandscapeTextureHash.h"
#include "Algo/Transform.h"
#include "TextureCompiler.h"

// Channel remapping
extern const size_t ChannelOffsets[4] = {STRUCT_OFFSET(FColor,R), STRUCT_OFFSET(FColor,G), STRUCT_OFFSET(FColor,B), STRUCT_OFFSET(FColor,A)};

//
// FLandscapeEditDataInterface
//
FLandscapeEditDataInterface::FLandscapeEditDataInterface(ULandscapeInfo* InLandscapeInfo, bool bInUploadTextureChangesToGPU)
	: FLandscapeTextureDataInterface(bInUploadTextureChangesToGPU)
	, bUseSharedLandscapeEditLayer(true)
	, LocalEditLayerGUID()
{
	if (InLandscapeInfo)
	{
		LandscapeInfo			= InLandscapeInfo;
		ComponentSizeQuads		= InLandscapeInfo->ComponentSizeQuads;
		SubsectionSizeQuads		= InLandscapeInfo->SubsectionSizeQuads;
		ComponentNumSubsections = InLandscapeInfo->ComponentNumSubsections;
		DrawScale				= InLandscapeInfo->DrawScale;
	}
}


FLandscapeEditDataInterface::FLandscapeEditDataInterface(ULandscapeInfo* InLandscapeInfo, const FGuid& InEditLayerGUID, bool bInUploadTextureChangesToGPU)
	: FLandscapeTextureDataInterface(bInUploadTextureChangesToGPU),
	bUseSharedLandscapeEditLayer(false),
	LocalEditLayerGUID(InEditLayerGUID)

{
	if (InLandscapeInfo)
	{
		LandscapeInfo = InLandscapeInfo;
		ComponentSizeQuads = InLandscapeInfo->ComponentSizeQuads;
		SubsectionSizeQuads = InLandscapeInfo->SubsectionSizeQuads;
		ComponentNumSubsections = InLandscapeInfo->ComponentNumSubsections;
		DrawScale = InLandscapeInfo->DrawScale;
	}
}

void FLandscapeEditDataInterface::SetEditLayer(const FGuid& InEditLayerGUID)
{
	bUseSharedLandscapeEditLayer = false;
	LocalEditLayerGUID = InEditLayerGUID;
}

FGuid FLandscapeEditDataInterface::GetEditLayer() const
{
	if (bUseSharedLandscapeEditLayer)
	{
		FGuid LandscapeEditLayerGuid;
		if ((LandscapeInfo != nullptr) && LandscapeInfo->LandscapeActor.IsValid())
		{
			LandscapeEditLayerGuid = LandscapeInfo->LandscapeActor->GetEditingLayer();
		}
		return LandscapeEditLayerGuid;
	}
	return LocalEditLayerGUID;
}

FLandscapeTextureDataInterface::FLandscapeTextureDataInterface(bool bInUploadTextureChangesToGPU)
	: bUploadTextureChangesToGPU(bInUploadTextureChangesToGPU), bShouldDirtyPackage(true)
{}

FLandscapeTextureDataInterface::~FLandscapeTextureDataInterface()
{
	Flush();
}

void FLandscapeTextureDataInterface::Flush()
{
	bool bNeedToWaitForUpdate = false;

	if (bUploadTextureChangesToGPU)
	{
		// Update all textures
		for (TMap<UTexture2D*, FLandscapeTextureDataInfo*>::TIterator It(TextureDataMap); It; ++It)
		{
			if (It.Value()->UpdateTextureData())
			{
				bNeedToWaitForUpdate = true;
			}
		}
	}

	if( bNeedToWaitForUpdate )
	{
		FlushRenderingCommands();
	}

	// delete all the FLandscapeTextureDataInfo allocations
	for( TMap<UTexture2D*, FLandscapeTextureDataInfo*>::TIterator It(TextureDataMap); It;  ++It )
	{
		delete It.Value();	// FLandscapeTextureDataInfo destructors will unlock any texture data
	}

	TextureDataMap.Empty();
}

#include "LevelUtils.h"

void ALandscape::CalcComponentIndicesOverlap(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, const int32 ComponentSizeQuads, 
											 int32& ComponentIndexX1, int32& ComponentIndexY1, int32& ComponentIndexX2, int32& ComponentIndexY2)
{
	// Find component range for this block of data
	ComponentIndexX1 = (X1-1 >= 0) ? (X1-1) / ComponentSizeQuads : (X1) / ComponentSizeQuads - 1;	// -1 because we need to pick up vertices shared between components
	ComponentIndexY1 = (Y1-1 >= 0) ? (Y1-1) / ComponentSizeQuads : (Y1) / ComponentSizeQuads - 1;
	ComponentIndexX2 = (X2 >= 0) ? X2 / ComponentSizeQuads : (X2+1) / ComponentSizeQuads - 1;
	ComponentIndexY2 = (Y2 >= 0) ? Y2 / ComponentSizeQuads : (Y2+1) / ComponentSizeQuads - 1;
}

void ALandscape::CalcComponentIndicesNoOverlap(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, const int32 ComponentSizeQuads, 
									  int32& ComponentIndexX1, int32& ComponentIndexY1, int32& ComponentIndexX2, int32& ComponentIndexY2)
{
	// Find component range for this block of data
	ComponentIndexX1 = (X1 >= 0) ? X1 / ComponentSizeQuads : (X1+1) / ComponentSizeQuads - 1;	// -1 because we need to pick up vertices shared between components
	ComponentIndexY1 = (Y1 >= 0) ? Y1 / ComponentSizeQuads : (Y1+1) / ComponentSizeQuads - 1;
	ComponentIndexX2 = (X2-1 >= 0) ? (X2-1) / ComponentSizeQuads : (X2) / ComponentSizeQuads - 1;
	ComponentIndexY2 = (Y2-1 >= 0) ? (Y2-1) / ComponentSizeQuads : (Y2) / ComponentSizeQuads - 1;
	// Shrink indices for shared values
	if ( ComponentIndexX2 < ComponentIndexX1)
	{
		ComponentIndexX2 = ComponentIndexX1;
	}
	if ( ComponentIndexY2 < ComponentIndexY1)
	{
		ComponentIndexY2 = ComponentIndexY1;
	}
}

namespace
{
	// Ugly helper function, all arrays should be only size 4
	template<typename T, typename F>
	FORCEINLINE void CalcInterpValue(const int32* Dist, const bool* Exist, const T* Value, F& ValueX, F& ValueY)
	{
		if (Exist[0] && Exist[1])
		{
			ValueX = (F)(Dist[1] * Value[0] + Dist[0] * Value[1]) / (Dist[0] + Dist[1]);
		}
		else
		{
			if (Exist[0])
			{
				ValueX = Value[0];
			}
			else if (Exist[1])
			{
				ValueX = Value[1];
			}
		}

		if (Exist[2] && Exist[3])
		{
			ValueY = (F)(Dist[3] * Value[2] + Dist[2] * Value[3]) / (Dist[2] + Dist[3]);
		}
		else
		{
			if (Exist[2])
			{
				ValueY = Value[2];
			}
			else if (Exist[3])
			{
				ValueY = Value[3];
			}
		}
	}

	template<typename T, typename F>
	FORCEINLINE T CalcValueFromValueXY( const int32* Dist, const F& ValueX, const F& ValueY, const uint8& CornerSet, const T* CornerValues )
	{
		T FinalValue;
		int32 DistX = FMath::Min(Dist[0], Dist[1]);
		int32 DistY = FMath::Min(Dist[2], Dist[3]);
		if (DistX+DistY > 0)
		{
			FinalValue = static_cast<T>(((ValueX * DistY) + (ValueY * DistX)) / static_cast<F>((DistX + DistY)));
		}
		else
		{
			if ((CornerSet & 1) && Dist[0] == 0 && Dist[2] == 0)
			{
				FinalValue = CornerValues[0];
			}
			else if ((CornerSet & 1 << 1) && Dist[1] == 0 && Dist[2] == 0)
			{
				FinalValue = CornerValues[1];
			}
			else if ((CornerSet & 1 << 2) && Dist[0] == 0 && Dist[3] == 0)
			{
				FinalValue = CornerValues[2];
			}
			else if ((CornerSet & 1 << 3) && Dist[1] == 0 && Dist[3] == 0)
			{
				FinalValue = CornerValues[3];
			}
			else
			{
				FinalValue = static_cast<T>(ValueX);
			}
		}
		return FinalValue;
	}

};

bool FLandscapeEditDataInterface::GetComponentsInRegion(int32 X1, int32 Y1, int32 X2, int32 Y2, TSet<ULandscapeComponent*>* OutComponents /*= nullptr*/)
{
	if (ComponentSizeQuads <= 0 || !LandscapeInfo)
	{
		return false;
	}
	// Find component range for this block of data
	int32 ComponentIndexX1, ComponentIndexY1, ComponentIndexX2, ComponentIndexY2;
	ALandscape::CalcComponentIndicesOverlap(X1, Y1, X2, Y2, ComponentSizeQuads, ComponentIndexX1, ComponentIndexY1, ComponentIndexX2, ComponentIndexY2);

	bool bNotLocked = true;
	for (int32 ComponentIndexY = ComponentIndexY1; ComponentIndexY <= ComponentIndexY2; ComponentIndexY++)
	{
		for (int32 ComponentIndexX = ComponentIndexX1; ComponentIndexX <= ComponentIndexX2; ComponentIndexX++)
		{
			ULandscapeComponent* Component = LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(ComponentIndexX, ComponentIndexY));
			if (Component)
			{
				bNotLocked = bNotLocked && (!FLevelUtils::IsLevelLocked(Component->GetLandscapeProxy()->GetLevel())) && FLevelUtils::IsLevelVisible(Component->GetLandscapeProxy()->GetLevel());
				if (OutComponents)
				{
					OutComponents->Add(Component);
				}
			}
		}
	}
	return bNotLocked;
}

void FLandscapeEditDataInterface::SetHeightData(int32 X1, int32 Y1, int32 X2, int32 Y2, const uint16* InData, int32 InStride, bool InCalcNormals, const uint16* InNormalData, const uint16* InHeightAlphaBlendData, const uint8* InHeightFlagsData, bool InCreateComponents, UTexture2D* InHeightmap, UTexture2D* InXYOffsetmapTexture,
											   bool InUpdateBounds, bool InUpdateCollision, bool InGenerateMips)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeEditDataInterface::SetHeightData);

	const int32 NumVertsX = 1 + X2 - X1;
	const int32 NumVertsY = 1 + Y2 - Y1;

	if (InStride == 0)
	{
		InStride = NumVertsX;
	}

	check(ComponentSizeQuads > 0);
	// Find component range for this block of data
	int32 ComponentIndexX1, ComponentIndexY1, ComponentIndexX2, ComponentIndexY2;
	ALandscape::CalcComponentIndicesNoOverlap(X1, Y1, X2, Y2, ComponentSizeQuads, ComponentIndexX1, ComponentIndexY1, ComponentIndexX2, ComponentIndexY2);

	FVector* VertexNormals = nullptr;
	if (InCalcNormals)
	{
		// Calculate the normals for each of the two triangles per quad.
		// Note that the normals at the edges are not correct because they include normals
		// from triangles outside the current area. They are not updated
		VertexNormals = new FVector[NumVertsX*NumVertsY];
		FMemory::Memzero(VertexNormals, NumVertsX*NumVertsY * sizeof(FVector));

		for (int32 Y = 0; Y < NumVertsY - 1; Y++)
		{
			for (int32 X = 0; X < NumVertsX - 1; X++)
			{
				FVector Vert00 = FVector(0.0f, 0.0f, LandscapeDataAccess::GetLocalHeight(InData[(X + 0) + InStride*(Y + 0)])) * DrawScale;
				FVector Vert01 = FVector(0.0f, 1.0f, LandscapeDataAccess::GetLocalHeight(InData[(X + 0) + InStride*(Y + 1)])) * DrawScale;
				FVector Vert10 = FVector(1.0f, 0.0f, LandscapeDataAccess::GetLocalHeight(InData[(X + 1) + InStride*(Y + 0)])) * DrawScale;
				FVector Vert11 = FVector(1.0f, 1.0f, LandscapeDataAccess::GetLocalHeight(InData[(X + 1) + InStride*(Y + 1)])) * DrawScale;

				FVector FaceNormal1 = ((Vert00 - Vert10) ^ (Vert10 - Vert11)).GetSafeNormal();
				FVector FaceNormal2 = ((Vert11 - Vert01) ^ (Vert01 - Vert00)).GetSafeNormal();

				// contribute to the vertex normals.
				VertexNormals[(X + 1 + NumVertsX*(Y + 0))] += FaceNormal1;
				VertexNormals[(X + 0 + NumVertsX*(Y + 1))] += FaceNormal2;
				VertexNormals[(X + 0 + NumVertsX*(Y + 0))] += FaceNormal1 + FaceNormal2;
				VertexNormals[(X + 1 + NumVertsX*(Y + 1))] += FaceNormal1 + FaceNormal2;
			}
		}
	}

	for (int32 ComponentIndexY = ComponentIndexY1; ComponentIndexY <= ComponentIndexY2; ComponentIndexY++)
	{
		for (int32 ComponentIndexX = ComponentIndexX1; ComponentIndexX <= ComponentIndexX2; ComponentIndexX++)
		{
			FIntPoint ComponentKey(ComponentIndexX, ComponentIndexY);
			ULandscapeComponent* Component = LandscapeInfo->XYtoComponentMap.FindRef(ComponentKey);

			// if nullptr, it was painted away
			if (Component == nullptr)
			{
				if (InCreateComponents)
				{
					// not yet implemented
					continue;
				}
				else
				{
					continue;
				}
			}

			UTexture2D* Heightmap = InHeightmap != nullptr ? InHeightmap : Component->GetHeightmap(GetEditLayer());

			Component->Modify(GetShouldDirtyPackage());

			FLandscapeTextureDataInfo* TexDataInfo = GetTextureDataInfo(Heightmap);
			FColor* HeightmapTextureData = (FColor*)TexDataInfo->GetMipData(/*InMipIndex = */0);

			// Find the texture data corresponding to this vertex
			int32 SizeU = Heightmap->Source.GetSizeX();
			int32 SizeV = Heightmap->Source.GetSizeY();
			int32 HeightmapOffsetX = static_cast<int32>(Component->HeightmapScaleBias.Z * SizeU);
			int32 HeightmapOffsetY = static_cast<int32>(Component->HeightmapScaleBias.W * SizeV);

			// Find coordinates of box that lies inside component
			int32 ComponentX1 = FMath::Clamp<int32>(X1 - ComponentIndexX*ComponentSizeQuads, 0, ComponentSizeQuads);
			int32 ComponentY1 = FMath::Clamp<int32>(Y1 - ComponentIndexY*ComponentSizeQuads, 0, ComponentSizeQuads);
			int32 ComponentX2 = FMath::Clamp<int32>(X2 - ComponentIndexX*ComponentSizeQuads, 0, ComponentSizeQuads);
			int32 ComponentY2 = FMath::Clamp<int32>(Y2 - ComponentIndexY*ComponentSizeQuads, 0, ComponentSizeQuads);

			// Find subsection range for this box
			int32 SubIndexX1 = FMath::Clamp<int32>((ComponentX1 - 1) / SubsectionSizeQuads, 0, ComponentNumSubsections - 1);	// -1 because we need to pick up vertices shared between subsections
			int32 SubIndexY1 = FMath::Clamp<int32>((ComponentY1 - 1) / SubsectionSizeQuads, 0, ComponentNumSubsections - 1);
			int32 SubIndexX2 = FMath::Clamp<int32>(ComponentX2 / SubsectionSizeQuads, 0, ComponentNumSubsections - 1);
			int32 SubIndexY2 = FMath::Clamp<int32>(ComponentY2 / SubsectionSizeQuads, 0, ComponentNumSubsections - 1);

			// To adjust bounding box
			uint16 MinHeight = MAX_uint16;
			uint16 MaxHeight = 0;

			for (int32 SubIndexY = SubIndexY1; SubIndexY <= SubIndexY2; SubIndexY++)
			{
				for (int32 SubIndexX = SubIndexX1; SubIndexX <= SubIndexX2; SubIndexX++)
				{
					// Find coordinates of box that lies inside subsection
					int32 SubX1 = FMath::Clamp<int32>(ComponentX1 - SubsectionSizeQuads*SubIndexX, 0, SubsectionSizeQuads);
					int32 SubY1 = FMath::Clamp<int32>(ComponentY1 - SubsectionSizeQuads*SubIndexY, 0, SubsectionSizeQuads);
					int32 SubX2 = FMath::Clamp<int32>(ComponentX2 - SubsectionSizeQuads*SubIndexX, 0, SubsectionSizeQuads);
					int32 SubY2 = FMath::Clamp<int32>(ComponentY2 - SubsectionSizeQuads*SubIndexY, 0, SubsectionSizeQuads);

					// Update texture data for the box that lies inside subsection
					for (int32 SubY = SubY1; SubY <= SubY2; SubY++)
					{
						for (int32 SubX = SubX1; SubX <= SubX2; SubX++)
						{
							int32 LandscapeX = SubIndexX*SubsectionSizeQuads + ComponentIndexX*ComponentSizeQuads + SubX;
							int32 LandscapeY = SubIndexY*SubsectionSizeQuads + ComponentIndexY*ComponentSizeQuads + SubY;
							checkSlow(LandscapeX >= X1 && LandscapeX <= X2);
							checkSlow(LandscapeY >= Y1 && LandscapeY <= Y2);

							// Find the input data corresponding to this vertex
							int32 DataIndex = (LandscapeX - X1) + InStride * (LandscapeY - Y1);
							const uint16& Height = InData[DataIndex];

							// for bounding box
							if (Height < MinHeight)
							{
								MinHeight = Height;
							}
							if (Height > MaxHeight)
							{
								MaxHeight = Height;
							}

							int32 TexX = HeightmapOffsetX + (SubsectionSizeQuads + 1) * SubIndexX + SubX;
							int32 TexY = HeightmapOffsetY + (SubsectionSizeQuads + 1) * SubIndexY + SubY;
							FColor& TexData = HeightmapTextureData[TexX + TexY * SizeU];

							// Update the texture
							TexData.R = Height >> 8;
							TexData.G = Height & 255;

							// Update normals if we're not on an edge vertex
							if (VertexNormals && LandscapeX > X1 && LandscapeX < X2 && LandscapeY > Y1 && LandscapeY < Y2)
							{
								const int32 NormalDataIndex = (LandscapeX - X1) + NumVertsX * (LandscapeY - Y1);
								FVector Normal = VertexNormals[NormalDataIndex].GetSafeNormal();
								TexData.B = static_cast<uint8>(FMath::RoundToInt32(127.5f * (Normal.X + 1.0f)));
								TexData.A = static_cast<uint8>(FMath::RoundToInt32(127.5f * (Normal.Y + 1.0f)));
							}
							else if (InNormalData)
							{
								// Need data validation?
								const uint16& Normal = InNormalData[DataIndex];
								TexData.B = Normal >> 8;
								TexData.A = Normal & 255;
							}
							else if (InHeightAlphaBlendData)
							{
								const uint16& HeightAlphaBlend = InHeightAlphaBlendData[DataIndex];
								TexData.B = HeightAlphaBlend >> 8;
								TexData.A = HeightAlphaBlend & 0xFC;
								TexData.A |= (InHeightFlagsData ? InHeightFlagsData[DataIndex] : 3) & 0x3;
							}
						}
					}

					// Record the areas of the texture we need to re-upload
					int32 TexX1 = HeightmapOffsetX + (SubsectionSizeQuads + 1) * SubIndexX + SubX1;
					int32 TexY1 = HeightmapOffsetY + (SubsectionSizeQuads + 1) * SubIndexY + SubY1;
					int32 TexX2 = HeightmapOffsetX + (SubsectionSizeQuads + 1) * SubIndexX + SubX2;
					int32 TexY2 = HeightmapOffsetY + (SubsectionSizeQuads + 1) * SubIndexY + SubY2;
					TexDataInfo->AddMipUpdateRegion(0, TexX1, TexY1, TexX2, TexY2);
				}
			}
												
			Component->RequestHeightmapUpdate();
			// In Edit Layer based landscapes, cumulate dirty collision region (will be used next time UpdateCollisionHeightData is called)
			Component->UpdateDirtyCollisionHeightData(FIntRect(ComponentX1, ComponentY1, ComponentX2, ComponentY2));
			
		}
	}

	if (VertexNormals)
	{
		delete[] VertexNormals;
	}
}

// Deprecated
void FLandscapeEditDataInterface::RecalculateNormals()
{
}

template<typename TStoreData>
void FLandscapeEditDataInterface::GetHeightDataTemplFast(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TStoreData& StoreData, UTexture2D* InHeightmap, TStoreData* NormalData /*= nullptr*/)
{
	if (!LandscapeInfo) return;
	int32 ComponentIndexX1, ComponentIndexY1, ComponentIndexX2, ComponentIndexY2;
	ALandscape::CalcComponentIndicesNoOverlap(X1, Y1, X2, Y2, ComponentSizeQuads, ComponentIndexX1, ComponentIndexY1, ComponentIndexX2, ComponentIndexY2);

	for( int32 ComponentIndexY=ComponentIndexY1;ComponentIndexY<=ComponentIndexY2;ComponentIndexY++ )
	{
		for( int32 ComponentIndexX=ComponentIndexX1;ComponentIndexX<=ComponentIndexX2;ComponentIndexX++ )
		{		
			ULandscapeComponent* Component = LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(ComponentIndexX, ComponentIndexY));

			if (Component == nullptr)
			{
				continue;
			}

			UTexture2D* Heightmap = InHeightmap != nullptr ? InHeightmap : Component->GetHeightmap(GetEditLayer());

			FLandscapeTextureDataInfo* TexDataInfo = nullptr;
			FColor* HeightmapTextureData = nullptr;
			TexDataInfo = GetTextureDataInfo(Heightmap);
			HeightmapTextureData = (FColor*)TexDataInfo->GetMipData(/*InMipIndex = */0);

			// Find coordinates of box that lies inside component
			int32 ComponentX1 = FMath::Clamp<int32>(X1-ComponentIndexX*ComponentSizeQuads, 0, ComponentSizeQuads);
			int32 ComponentY1 = FMath::Clamp<int32>(Y1-ComponentIndexY*ComponentSizeQuads, 0, ComponentSizeQuads);
			int32 ComponentX2 = FMath::Clamp<int32>(X2-ComponentIndexX*ComponentSizeQuads, 0, ComponentSizeQuads);
			int32 ComponentY2 = FMath::Clamp<int32>(Y2-ComponentIndexY*ComponentSizeQuads, 0, ComponentSizeQuads);

			// Find subsection range for this box
			int32 SubIndexX1 = FMath::Clamp<int32>((ComponentX1-1) / SubsectionSizeQuads,0,ComponentNumSubsections-1);	// -1 because we need to pick up vertices shared between subsections
			int32 SubIndexY1 = FMath::Clamp<int32>((ComponentY1-1) / SubsectionSizeQuads,0,ComponentNumSubsections-1);
			int32 SubIndexX2 = FMath::Clamp<int32>(ComponentX2 / SubsectionSizeQuads,0,ComponentNumSubsections-1);
			int32 SubIndexY2 = FMath::Clamp<int32>(ComponentY2 / SubsectionSizeQuads,0,ComponentNumSubsections-1);

			for( int32 SubIndexY=SubIndexY1;SubIndexY<=SubIndexY2;SubIndexY++ )
			{
				for( int32 SubIndexX=SubIndexX1;SubIndexX<=SubIndexX2;SubIndexX++ )
				{
					// Find coordinates of box that lies inside subsection
					int32 SubX1 = FMath::Clamp<int32>(ComponentX1-SubsectionSizeQuads*SubIndexX, 0, SubsectionSizeQuads);
					int32 SubY1 = FMath::Clamp<int32>(ComponentY1-SubsectionSizeQuads*SubIndexY, 0, SubsectionSizeQuads);
					int32 SubX2 = FMath::Clamp<int32>(ComponentX2-SubsectionSizeQuads*SubIndexX, 0, SubsectionSizeQuads);
					int32 SubY2 = FMath::Clamp<int32>(ComponentY2-SubsectionSizeQuads*SubIndexY, 0, SubsectionSizeQuads);

					// Update texture data for the box that lies inside subsection
					for( int32 SubY=SubY1;SubY<=SubY2;SubY++ )
					{
						for( int32 SubX=SubX1;SubX<=SubX2;SubX++ )
						{
							int32 LandscapeX = SubIndexX*SubsectionSizeQuads + ComponentIndexX*ComponentSizeQuads + SubX;
							int32 LandscapeY = SubIndexY*SubsectionSizeQuads + ComponentIndexY*ComponentSizeQuads + SubY;

							// Find the texture data corresponding to this vertex
							int32 SizeU = Heightmap->Source.GetSizeX();
							int32 SizeV = Heightmap->Source.GetSizeY();
							int32 HeightmapOffsetX = static_cast<int32>(Component->HeightmapScaleBias.Z * SizeU);
							int32 HeightmapOffsetY = static_cast<int32>(Component->HeightmapScaleBias.W * SizeV);

							int32 TexX = HeightmapOffsetX + (SubsectionSizeQuads+1) * SubIndexX + SubX;
							int32 TexY = HeightmapOffsetY + (SubsectionSizeQuads+1) * SubIndexY + SubY;
							FColor& TexData = HeightmapTextureData[ TexX + TexY * SizeU ];

							uint16 Height = (((uint16)TexData.R) << 8) | TexData.G;
							StoreData.Store(LandscapeX, LandscapeY, Height);
							if (NormalData)
							{
								uint16 Normals = (((uint16)TexData.B) << 8) | TexData.A;
								NormalData->Store(LandscapeX, LandscapeY, Normals);
							}
						}
					}
				}
			}
		}
	}
}

template<typename TData, typename TStoreData, typename FType>
void FLandscapeEditDataInterface::CalcMissingValues(const int32 X1, const int32 X2, const int32 Y1, const int32 Y2,
	const int32 ComponentIndexX1, const int32 ComponentIndexX2, const int32 ComponentIndexY1, const int32 ComponentIndexY2,
	const int32 ComponentSizeX, const int32 ComponentSizeY, TData* CornerValues,
	TArray<bool>& NoBorderY1, TArray<bool>& NoBorderY2, TArray<bool>& ComponentDataExist, TStoreData& StoreData)
{
	bool NoBorderX1 = false, NoBorderX2 = false;
	// Init data...
	FMemory::Memzero(NoBorderY1.GetData(), ComponentSizeX*sizeof(bool));
	FMemory::Memzero(NoBorderY2.GetData(), ComponentSizeX*sizeof(bool));
	int32 BorderX1 = INT_MAX, BorderX2 = INT_MIN;
	TArray<int32> BorderY1, BorderY2;
	BorderY1.Empty(ComponentSizeX);
	BorderY2.Empty(ComponentSizeX);
	for( int32 ComponentIndexX=ComponentIndexX1;ComponentIndexX<=ComponentIndexX2;ComponentIndexX++ )
	{
		new (BorderY1) int32(INT_MAX);
		new (BorderY2) int32(INT_MIN);
	}

	// fill up missing values...
	for( int32 ComponentIndexY=ComponentIndexY1;ComponentIndexY<=ComponentIndexY2;ComponentIndexY++ )
	{
		NoBorderX1 = false;
		NoBorderX2 = false;
		BorderX1 = INT_MAX;
		BorderX2 = INT_MIN;
		for( int32 ComponentIndexX=ComponentIndexX1;ComponentIndexX<=ComponentIndexX2;ComponentIndexX++ )
		{
			int32 ComponentIndexXY = ComponentSizeX*(ComponentIndexY-ComponentIndexY1) + ComponentIndexX-ComponentIndexX1;
			if (!ComponentDataExist[ComponentIndexXY])
			{
				int32 ComponentIndexXX = ComponentIndexX - ComponentIndexX1;
				int32 ComponentIndexYY = ComponentIndexY - ComponentIndexY1;

				uint8 CornerSet = 0;
				bool ExistLeft = ComponentIndexXX > 0 && ComponentDataExist[ ComponentIndexXX-1 + ComponentIndexYY * ComponentSizeX ];
				bool ExistUp = ComponentIndexYY > 0 && ComponentDataExist[ ComponentIndexXX + (ComponentIndexYY-1) * ComponentSizeX ];

				// Search for neighbor component for interpolation
				bool bShouldSearchX = (BorderX2 <= ComponentIndexX);
				bool bShouldSearchY = (BorderY2[ComponentIndexXX] <= ComponentIndexY);
				// Search for left-closest component
				if ( bShouldSearchX || (!NoBorderX1 && BorderX1 == INT_MAX) )
				{
					NoBorderX1 = true;
					BorderX1 = INT_MAX;
					for (int32 X = ComponentIndexX-1; X >= ComponentIndexX1; X--)
					{
						if (ComponentDataExist[ComponentSizeX*(ComponentIndexY-ComponentIndexY1) + X-ComponentIndexX1])
						{
							NoBorderX1 = false;
							BorderX1 = X;
							break;
						}
					}
				}
				// Search for right-closest component
				if ( bShouldSearchX || (!NoBorderX2 && BorderX2 == INT_MIN) )
				{
					NoBorderX2 = true;
					BorderX2 = INT_MIN;
					for (int32 X = ComponentIndexX+1; X <= ComponentIndexX2; X++)
					{
						if (ComponentDataExist[ComponentSizeX*(ComponentIndexY-ComponentIndexY1) + X-ComponentIndexX1])
						{
							NoBorderX2 = false;
							BorderX2 = X;
							break;
						}
					}
				}
				// Search for up-closest component
				if ( bShouldSearchY || (!NoBorderY1[ComponentIndexXX] && BorderY1[ComponentIndexXX] == INT_MAX))
				{
					NoBorderY1[ComponentIndexXX] = true;
					BorderY1[ComponentIndexXX] = INT_MAX;
					for (int32 Y = ComponentIndexY-1; Y >= ComponentIndexY1; Y--)
					{
						if (ComponentDataExist[ComponentSizeX*(Y-ComponentIndexY1) + ComponentIndexX-ComponentIndexX1])
						{
							NoBorderY1[ComponentIndexXX] = false;
							BorderY1[ComponentIndexXX] = Y;
							break;
						}
					}
				}
				// Search for bottom-closest component
				if ( bShouldSearchY || (!NoBorderY2[ComponentIndexXX] && BorderY2[ComponentIndexXX] == INT_MIN))
				{
					NoBorderY2[ComponentIndexXX] = true;
					BorderY2[ComponentIndexXX] = INT_MIN;
					for (int32 Y = ComponentIndexY+1; Y <= ComponentIndexY2; Y++)
					{
						if (ComponentDataExist[ComponentSizeX*(Y-ComponentIndexY1) + ComponentIndexX-ComponentIndexX1])
						{
							NoBorderY2[ComponentIndexXX] = false;
							BorderY2[ComponentIndexXX] = Y;
							break;
						}
					}
				}

				if (((ComponentIndexX == ComponentIndexX1) || (ComponentIndexY == ComponentIndexY1)) ? false : ComponentDataExist[ComponentSizeX*(ComponentIndexY-1-ComponentIndexY1) + ComponentIndexX-1-ComponentIndexX1])
				{
					CornerSet |= 1;
					CornerValues[0] = TData(StoreData.Load(ComponentIndexX*ComponentSizeQuads, ComponentIndexY*ComponentSizeQuads));
				}
				if (((ComponentIndexX == ComponentIndexX2) || (ComponentIndexY == ComponentIndexY1)) ? false : ComponentDataExist[ComponentSizeX*(ComponentIndexY-1-ComponentIndexY1) + ComponentIndexX+1-ComponentIndexX1])
				{
					CornerSet |= 1 << 1;
					CornerValues[1] = TData(StoreData.Load((ComponentIndexX + 1)*ComponentSizeQuads, ComponentIndexY*ComponentSizeQuads));
				}
				if (((ComponentIndexX == ComponentIndexX1) || (ComponentIndexY == ComponentIndexY2)) ? false : ComponentDataExist[ComponentSizeX*(ComponentIndexY+1-ComponentIndexY1) + ComponentIndexX-1-ComponentIndexX1])
				{
					CornerSet |= 1 << 2;
					CornerValues[2] = TData(StoreData.Load(ComponentIndexX*ComponentSizeQuads, (ComponentIndexY + 1)*ComponentSizeQuads));
				}
				if (((ComponentIndexX == ComponentIndexX2) || (ComponentIndexY == ComponentIndexY2)) ? false : ComponentDataExist[ComponentSizeX*(ComponentIndexY+1-ComponentIndexY1) + ComponentIndexX+1-ComponentIndexX1])
				{
					CornerSet |= 1 << 3;
					CornerValues[3] = TData(StoreData.Load((ComponentIndexX + 1)*ComponentSizeQuads, (ComponentIndexY + 1)*ComponentSizeQuads));
				}

				FillCornerValues(CornerSet, CornerValues);

				// Find coordinates of box that lies inside component
				int32 ComponentX1 = FMath::Clamp<int32>(X1-ComponentIndexX*ComponentSizeQuads, 0, ComponentSizeQuads);
				int32 ComponentY1 = FMath::Clamp<int32>(Y1-ComponentIndexY*ComponentSizeQuads, 0, ComponentSizeQuads);
				int32 ComponentX2 = FMath::Clamp<int32>(X2-ComponentIndexX*ComponentSizeQuads, 0, ComponentSizeQuads);
				int32 ComponentY2 = FMath::Clamp<int32>(Y2-ComponentIndexY*ComponentSizeQuads, 0, ComponentSizeQuads);

				// Find subsection range for this box
				int32 SubIndexX1 = FMath::Clamp<int32>((ComponentX1-1) / SubsectionSizeQuads,0,ComponentNumSubsections-1);	// -1 because we need to pick up vertices shared between subsections
				int32 SubIndexY1 = FMath::Clamp<int32>((ComponentY1-1) / SubsectionSizeQuads,0,ComponentNumSubsections-1);
				int32 SubIndexX2 = FMath::Clamp<int32>(ComponentX2 / SubsectionSizeQuads,0,ComponentNumSubsections-1);
				int32 SubIndexY2 = FMath::Clamp<int32>(ComponentY2 / SubsectionSizeQuads,0,ComponentNumSubsections-1);

				for( int32 SubIndexY=SubIndexY1;SubIndexY<=SubIndexY2;SubIndexY++ )
				{
					for( int32 SubIndexX=SubIndexX1;SubIndexX<=SubIndexX2;SubIndexX++ )
					{
						// Find coordinates of box that lies inside subsection
						int32 SubX1 = FMath::Clamp<int32>(ComponentX1-SubsectionSizeQuads*SubIndexX, 0, SubsectionSizeQuads);
						int32 SubY1 = FMath::Clamp<int32>(ComponentY1-SubsectionSizeQuads*SubIndexY, 0, SubsectionSizeQuads);
						int32 SubX2 = FMath::Clamp<int32>(ComponentX2-SubsectionSizeQuads*SubIndexX, 0, SubsectionSizeQuads);
						int32 SubY2 = FMath::Clamp<int32>(ComponentY2-SubsectionSizeQuads*SubIndexY, 0, SubsectionSizeQuads);

						// Update texture data for the box that lies inside subsection
						for( int32 SubY=SubY1;SubY<=SubY2;SubY++ )
						{
							for( int32 SubX=SubX1;SubX<=SubX2;SubX++ )
							{
								int32 LandscapeX = SubIndexX*SubsectionSizeQuads + ComponentIndexX*ComponentSizeQuads + SubX;
								int32 LandscapeY = SubIndexY*SubsectionSizeQuads + ComponentIndexY*ComponentSizeQuads + SubY;

								// Find the texture data corresponding to this vertex
								TData Value[4];
								FMemory::Memzero(Value, sizeof(TData)* 4);
								int32 Dist[4] = {INT_MAX, INT_MAX, INT_MAX, INT_MAX};
								FType ValueX, ValueY;
								FMemory::Memzero(&ValueX, sizeof(FType));
								FMemory::Memzero(&ValueY, sizeof(FType));
								bool Exist[4] = {false, false, false, false};

								if (ExistLeft)
								{
									Value[0] = TData(StoreData.Load(ComponentIndexX*ComponentSizeQuads, LandscapeY));
									Dist[0] = LandscapeX - (ComponentIndexX*ComponentSizeQuads);
									Exist[0] = true;
								}
								else if ( BorderX1 != INT_MAX )
								{
									int32 BorderIdxX = (BorderX1+1)*ComponentSizeQuads;
									Value[0] = TData(StoreData.Load(BorderIdxX, LandscapeY));
									Dist[0] = LandscapeX - (BorderIdxX-1);
									Exist[0] = true;
								}
								else 
								{
									if ((CornerSet & 1) && (CornerSet & 1 << 2))
									{
										int32 Dist1 = LandscapeY - (ComponentIndexY*ComponentSizeQuads);
										int32 Dist2 = ((ComponentIndexY+1)*ComponentSizeQuads) - LandscapeY;
										Value[0] = static_cast<TData>((FType)(Dist2 * CornerValues[0] + Dist1 * CornerValues[2]) / (Dist1 + Dist2));
										Dist[0] = LandscapeX - (ComponentIndexX*ComponentSizeQuads);
										Exist[0] = true;
									}
								}

								if ( BorderX2 != INT_MIN )
								{
									int32 BorderIdxX = BorderX2*ComponentSizeQuads;
									Value[1] = TData(StoreData.Load(BorderIdxX, LandscapeY));
									Dist[1] = (BorderIdxX+1) - LandscapeX;
									Exist[1] = true;
								}
								else 
								{
									if ((CornerSet & 1 << 1) && (CornerSet & 1 << 3))
									{
										int32 Dist1 = LandscapeY - (ComponentIndexY*ComponentSizeQuads);
										int32 Dist2 = ((ComponentIndexY+1)*ComponentSizeQuads) - LandscapeY;
										Value[1] = static_cast<TData>((FType)(Dist2 * CornerValues[1] + Dist1 * CornerValues[3]) / (Dist1 + Dist2));
										Dist[1] = (ComponentIndexX+1)*ComponentSizeQuads - LandscapeX;
										Exist[1] = true;
									}
								}

								if (ExistUp)
								{
									Value[2] = TData(StoreData.Load(LandscapeX, ComponentIndexY*ComponentSizeQuads));
									Dist[2] = LandscapeY - (ComponentIndexY*ComponentSizeQuads);
									Exist[2] = true;
								}
								else if ( BorderY1[ComponentIndexXX] != INT_MAX )
								{
									int32 BorderIdxY = (BorderY1[ComponentIndexXX]+1)*ComponentSizeQuads;
									Value[2] = TData(StoreData.Load(LandscapeX, BorderIdxY));
									Dist[2] = LandscapeY - BorderIdxY;
									Exist[2] = true;
								}
								else 
								{
									if ((CornerSet & 1) && (CornerSet & 1 << 1))
									{
										int32 Dist1 = LandscapeX - (ComponentIndexX*ComponentSizeQuads);
										int32 Dist2 = (ComponentIndexX+1)*ComponentSizeQuads - LandscapeX;
										Value[2] = static_cast<TData>((FType)(Dist2 * CornerValues[0] + Dist1 * CornerValues[1]) / (Dist1 + Dist2));
										Dist[2] = LandscapeY - (ComponentIndexY*ComponentSizeQuads);
										Exist[2] = true;
									}
								}

								if ( BorderY2[ComponentIndexXX] != INT_MIN )
								{
									int32 BorderIdxY = BorderY2[ComponentIndexXX]*ComponentSizeQuads;
									Value[3] = TData(StoreData.Load(LandscapeX, BorderIdxY));
									Dist[3] = BorderIdxY - LandscapeY;
									Exist[3] = true;
								}
								else 
								{
									if ((CornerSet & 1 << 2) && (CornerSet & 1 << 3))
									{
										int32 Dist1 = LandscapeX - (ComponentIndexX*ComponentSizeQuads);
										int32 Dist2 = (ComponentIndexX+1)*ComponentSizeQuads - LandscapeX;
										Value[3] = static_cast<TData>((FType)(Dist2 * CornerValues[2] + Dist1 * CornerValues[3]) / (Dist1 + Dist2));
										Dist[3] = (ComponentIndexY+1)*ComponentSizeQuads - LandscapeY;
										Exist[3] = true;
									}
								}

								CalcInterpValue<TData, FType>(Dist, Exist, Value, ValueX, ValueY);

								TData FinalValue; // Default Value
								FMemory::Memzero(&FinalValue, sizeof(TData));
								if ( (Exist[0] || Exist[1]) && (Exist[2] || Exist[3]) )
								{
									FinalValue = CalcValueFromValueXY<TData>(Dist, ValueX, ValueY, CornerSet, CornerValues);
								}
								else if ( (Exist[0] || Exist[1]) )
								{
									FinalValue = static_cast<TData>(ValueX);
								}
								else if ( (Exist[2] || Exist[3]) )
								{
									FinalValue = static_cast<TData>(ValueY);
								}

								StoreData.Store(LandscapeX, LandscapeY, FinalValue);
							}
						}
					}
				}
			}
		}
	}
}

const FColor& FLandscapeEditDataInterface::GetHeightMapColor(const ULandscapeComponent* Component, int32 TexU, int32 TexV, FColor* TextureData)
{
	UTexture2D* Heightmap = Component->GetHeightmap(GetEditLayer());
	check(Component);
	if (!TextureData)
	{
		FLandscapeTextureDataInfo* TexDataInfo = GetTextureDataInfo(Heightmap);
		TextureData = (FColor*)TexDataInfo->GetMipData(/*InMipIndex = */0);	
	}

	// All Heightmaps of component have the same texture size
	const FTextureSource& HeightmapTextureSource = Heightmap->Source;
	const int32 SizeU = HeightmapTextureSource.GetSizeX();
	const int32 SizeV = HeightmapTextureSource.GetSizeY();
	const int32 HeightmapOffsetX = static_cast<int32>(Component->HeightmapScaleBias.Z * SizeU);
	const int32 HeightmapOffsetY = static_cast<int32>(Component->HeightmapScaleBias.W * SizeV);


	const int32 TexX = HeightmapOffsetX + TexU;
	const int32 TexY = HeightmapOffsetY + TexV;
	const FColor& TexData = TextureData[ TexX + TexY * SizeU ];
	return TexData;
}

uint16 FLandscapeEditDataInterface::GetHeightMapData(const ULandscapeComponent* Component, int32 TexU, int32 TexV, FColor* TextureData /*= nullptr*/)
{
	const FColor& TexData = GetHeightMapColor(Component, TexU, TexV, TextureData);
	return ((((uint16)TexData.R) << 8) | TexData.G);
}

uint16 FLandscapeEditDataInterface::GetHeightMapAlphaBlendData(const ULandscapeComponent* Component, int32 TexU, int32 TexV, FColor* TextureData /*= nullptr*/)
{
	const FColor& TexData = GetHeightMapColor(Component, TexU, TexV, TextureData);
	return ((((uint16)TexData.B) << 8) | TexData.A) & 0xFFFC;
}

uint8 FLandscapeEditDataInterface::GetHeightMapFlagsData(const ULandscapeComponent* Component, int32 TexU, int32 TexV, FColor* TextureData /*= nullptr*/)
{
	const FColor& TexData = GetHeightMapColor(Component, TexU, TexV, TextureData);
	return TexData.A & 0x3;
}

template<typename TDataAccess, typename TGetHeightMapDataFunction>
void FLandscapeEditDataInterface::GetHeightDataInternal(int32& ValidX1, int32& ValidY1, int32& ValidX2, int32& ValidY2, TDataAccess& StoreData, TGetHeightMapDataFunction GetHeightMapDataFunction)
{
	// Copy variables
	int32 X1 = ValidX1, X2 = ValidX2, Y1 = ValidY1, Y2 = ValidY2;
	ValidX1 = INT_MAX; ValidX2 = INT_MIN; ValidY1 = INT_MAX; ValidY2 = INT_MIN;

	int32 ComponentIndexX1, ComponentIndexY1, ComponentIndexX2, ComponentIndexY2;
	ALandscape::CalcComponentIndicesNoOverlap(X1, Y1, X2, Y2, ComponentSizeQuads, ComponentIndexX1, ComponentIndexY1, ComponentIndexX2, ComponentIndexY2);
	
	int32 ComponentSizeX = ComponentIndexX2-ComponentIndexX1+1;
	int32 ComponentSizeY = ComponentIndexY2-ComponentIndexY1+1;

	// Neighbor Components
	ULandscapeComponent* BorderComponent[4] = {0, 0, 0, 0};
	ULandscapeComponent* CornerComponent[4] = {0, 0, 0, 0};
	bool NoBorderX1 = false, NoBorderX2 = false;
	TArray<bool> NoBorderY1, NoBorderY2, ComponentDataExist;
	TArray<ULandscapeComponent*> BorderComponentY1, BorderComponentY2;
	ComponentDataExist.Empty(ComponentSizeX*ComponentSizeY);
	ComponentDataExist.AddZeroed(ComponentSizeX*ComponentSizeY);
	bool bHasMissingValue = false;

	FLandscapeTextureDataInfo* NeighborTexDataInfo[4] = {0, 0, 0, 0};
	FColor* NeighborHeightmapTextureData[4] = {0, 0, 0, 0};
	typename TDataAccess::DataType CornerValues[4] = {0, 0, 0, 0};

	int32 EdgeCoord = (SubsectionSizeQuads+1) * ComponentNumSubsections - 1; //ComponentSizeQuads;

	// initial loop....
	for( int32 ComponentIndexY=ComponentIndexY1;ComponentIndexY<=ComponentIndexY2;ComponentIndexY++ )
	{
		NoBorderX1 = false;
		NoBorderX2 = false;
		BorderComponent[0] = BorderComponent[1] = nullptr;
		for( int32 ComponentIndexX=ComponentIndexX1;ComponentIndexX<=ComponentIndexX2;ComponentIndexX++ )
		{
			BorderComponent[2] = BorderComponent[3] = nullptr;
			int32 ComponentIndexXY = ComponentSizeX*(ComponentIndexY-ComponentIndexY1) + ComponentIndexX-ComponentIndexX1;
			int32 ComponentIndexXX = ComponentIndexX - ComponentIndexX1;
			int32 ComponentIndexYY = ComponentIndexY - ComponentIndexY1;
			ComponentDataExist[ComponentIndexXY] = false;
			ULandscapeComponent* Component = LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(ComponentIndexX,ComponentIndexY));

			FLandscapeTextureDataInfo* TexDataInfo = nullptr;
			FColor* HeightmapTextureData = nullptr;
			uint8 CornerSet = 0;
			bool ExistLeft = ComponentIndexXX > 0 && ComponentDataExist[ ComponentIndexXX-1 + ComponentIndexYY * ComponentSizeX ];
			bool ExistUp = ComponentIndexYY > 0 && ComponentDataExist[ ComponentIndexXX + (ComponentIndexYY-1) * ComponentSizeX ];

			if( Component )
			{
				TexDataInfo = GetTextureDataInfo(Component->GetHeightmap(GetEditLayer()));
				HeightmapTextureData = (FColor*)TexDataInfo->GetMipData(/*InMipIndex = */0);
				ComponentDataExist[ComponentIndexXY] = true;
				// Update valid region
				ValidX1 = FMath::Min<int32>(Component->GetSectionBase().X, ValidX1);
				ValidX2 = FMath::Max<int32>(Component->GetSectionBase().X+ComponentSizeQuads, ValidX2);
				ValidY1 = FMath::Min<int32>(Component->GetSectionBase().Y, ValidY1);
				ValidY2 = FMath::Max<int32>(Component->GetSectionBase().Y+ComponentSizeQuads, ValidY2);
			}
			else
			{
				if (!bHasMissingValue)
				{
					NoBorderY1.Empty(ComponentSizeX);
					NoBorderY2.Empty(ComponentSizeX);
					NoBorderY1.AddZeroed(ComponentSizeX);
					NoBorderY2.AddZeroed(ComponentSizeX);
					BorderComponentY1.Empty(ComponentSizeX);
					BorderComponentY2.Empty(ComponentSizeX);
					BorderComponentY1.AddZeroed(ComponentSizeX);
					BorderComponentY2.AddZeroed(ComponentSizeX);
					bHasMissingValue = true;
				}

				// Search for neighbor component for interpolation
				bool bShouldSearchX = (BorderComponent[1] && BorderComponent[1]->GetSectionBase().X / ComponentSizeQuads <= ComponentIndexX);
				bool bShouldSearchY = (BorderComponentY2[ComponentIndexXX] && BorderComponentY2[ComponentIndexXX]->GetSectionBase().Y / ComponentSizeQuads <= ComponentIndexY);
				// Search for left-closest component
				if ( bShouldSearchX || (!NoBorderX1 && !BorderComponent[0]))
				{
					NoBorderX1 = true;
					for (int32 X = ComponentIndexX-1; X >= ComponentIndexX1; X--)
					{
						BorderComponent[0] = LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(X,ComponentIndexY));
						if (BorderComponent[0])
						{
							NoBorderX1 = false;
							NeighborTexDataInfo[0] = GetTextureDataInfo(BorderComponent[0]->GetHeightmap(GetEditLayer()));
							NeighborHeightmapTextureData[0] = (FColor*)NeighborTexDataInfo[0]->GetMipData(/*InMipIndex = */0);
							break;
						}
					}
				}
				// Search for right-closest component
				if ( bShouldSearchX || (!NoBorderX2 && !BorderComponent[1]))
				{
					NoBorderX2 = true;
					for (int32 X = ComponentIndexX+1; X <= ComponentIndexX2; X++)
					{
						BorderComponent[1] = LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(X,ComponentIndexY));
						if (BorderComponent[1])
						{
							NoBorderX2 = false;
							NeighborTexDataInfo[1] = GetTextureDataInfo(BorderComponent[1]->GetHeightmap(GetEditLayer()));
							NeighborHeightmapTextureData[1] = (FColor*)NeighborTexDataInfo[1]->GetMipData(/*InMipIndex = */0);
							break;
						}
					}
				}
				// Search for up-closest component
				if ( bShouldSearchY || (!NoBorderY1[ComponentIndexXX] && !BorderComponentY1[ComponentIndexXX]))
				{
					NoBorderY1[ComponentIndexXX] = true;
					for (int32 Y = ComponentIndexY-1; Y >= ComponentIndexY1; Y--)
					{
						BorderComponentY1[ComponentIndexXX] = BorderComponent[2] = LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(ComponentIndexX,Y));
						if (BorderComponent[2])
						{
							NoBorderY1[ComponentIndexXX] = false;
							NeighborTexDataInfo[2] = GetTextureDataInfo(BorderComponent[2]->GetHeightmap(GetEditLayer()));
							NeighborHeightmapTextureData[2] = (FColor*)NeighborTexDataInfo[2]->GetMipData(/*InMipIndex = */0);
							break;
						}
					}
				}
				else
				{
					BorderComponent[2] = BorderComponentY1[ComponentIndexXX];
					if (BorderComponent[2])
					{
						NeighborTexDataInfo[2] = GetTextureDataInfo(BorderComponent[2]->GetHeightmap(GetEditLayer()));
						NeighborHeightmapTextureData[2] = (FColor*)NeighborTexDataInfo[2]->GetMipData(/*InMipIndex = */0);
					}
				}
				// Search for bottom-closest component
				if ( bShouldSearchY || (!NoBorderY2[ComponentIndexXX] && !BorderComponentY2[ComponentIndexXX]))
				{
					NoBorderY2[ComponentIndexXX] = true;
					for (int32 Y = ComponentIndexY+1; Y <= ComponentIndexY2; Y++)
					{
						BorderComponentY2[ComponentIndexXX] = BorderComponent[3] = LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(ComponentIndexX,Y));
						if (BorderComponent[3])
						{
							NoBorderY2[ComponentIndexXX] = false;
							NeighborTexDataInfo[3] = GetTextureDataInfo(BorderComponent[3]->GetHeightmap(GetEditLayer()));
							NeighborHeightmapTextureData[3] = (FColor*)NeighborTexDataInfo[3]->GetMipData(/*InMipIndex = */0);
							break;
						}
					}
				}
				else
				{
					BorderComponent[3] = BorderComponentY2[ComponentIndexXX];
					if (BorderComponent[3])
					{
						NeighborTexDataInfo[3] = GetTextureDataInfo(BorderComponent[3]->GetHeightmap(GetEditLayer()));
						NeighborHeightmapTextureData[3] = (FColor*)NeighborTexDataInfo[3]->GetMipData(/*InMipIndex = */0);
					}
				}

				CornerComponent[0] = ComponentIndexX >= ComponentIndexX1 && ComponentIndexY >= ComponentIndexY1 ?
					LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint((ComponentIndexX-1),(ComponentIndexY-1))) : nullptr;
				CornerComponent[1] = ComponentIndexX <= ComponentIndexX2 && ComponentIndexY >= ComponentIndexY1 ?
					LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint((ComponentIndexX+1),(ComponentIndexY-1))) : nullptr;
				CornerComponent[2] = ComponentIndexX >= ComponentIndexX1 && ComponentIndexY <= ComponentIndexY2 ?
					LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint((ComponentIndexX-1),(ComponentIndexY+1))) : nullptr;
				CornerComponent[3] = ComponentIndexX <= ComponentIndexX2 && ComponentIndexY <= ComponentIndexY2 ?
					LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint((ComponentIndexX+1),(ComponentIndexY+1))) : nullptr;

				if (CornerComponent[0])
				{
					CornerSet |= 1;
					CornerValues[0] = GetHeightMapDataFunction(CornerComponent[0], EdgeCoord, EdgeCoord, nullptr);
				}
				else if ((ExistLeft || ExistUp) && X1 <= ComponentIndexX*ComponentSizeQuads && Y1 <= ComponentIndexY*ComponentSizeQuads  )
				{
					CornerSet |= 1;
					CornerValues[0] = StoreData.Load(ComponentIndexX*ComponentSizeQuads, ComponentIndexY*ComponentSizeQuads);
				}
				else if (BorderComponent[0])
				{
					CornerSet |= 1;
					CornerValues[0] = GetHeightMapDataFunction(BorderComponent[0], EdgeCoord, 0, NeighborHeightmapTextureData[0]);
				}
				else if (BorderComponent[2])
				{
					CornerSet |= 1;
					CornerValues[0] = GetHeightMapDataFunction(BorderComponent[2], 0, EdgeCoord, NeighborHeightmapTextureData[2]);
				}

				if (CornerComponent[1])
				{
					CornerSet |= 1 << 1;
					CornerValues[1] = GetHeightMapDataFunction(CornerComponent[1], 0, EdgeCoord, nullptr);
				}
				else if (ExistUp && X2 >= (ComponentIndexX+1)*ComponentSizeQuads)
				{
					CornerSet |= 1 << 1;
					CornerValues[1] = StoreData.Load( (ComponentIndexX+1)*ComponentSizeQuads, ComponentIndexY*ComponentSizeQuads);
				}
				else if (BorderComponent[1])
				{
					CornerSet |= 1 << 1;
					CornerValues[1] = GetHeightMapDataFunction(BorderComponent[1], 0, 0, NeighborHeightmapTextureData[1]);
				}
				else if (BorderComponent[2])
				{
					CornerSet |= 1 << 1;
					CornerValues[1] = GetHeightMapDataFunction(BorderComponent[2], EdgeCoord, EdgeCoord, NeighborHeightmapTextureData[2]);
				}

				if (CornerComponent[2])
				{
					CornerSet |= 1 << 2;
					CornerValues[2] = GetHeightMapDataFunction(CornerComponent[2], EdgeCoord, 0, nullptr);
				}
				else if (ExistLeft && Y2 >= (ComponentIndexY+1)*ComponentSizeQuads) // Use data already stored for 0, 2
				{
					CornerSet |= 1 << 2;
					CornerValues[2] = StoreData.Load(ComponentIndexX*ComponentSizeQuads, (ComponentIndexY + 1)*ComponentSizeQuads);
				}
				else if (BorderComponent[0])
				{
					CornerSet |= 1 << 2;
					CornerValues[2] = GetHeightMapDataFunction(BorderComponent[0], EdgeCoord, EdgeCoord, NeighborHeightmapTextureData[0]);
				}
				else if (BorderComponent[3])
				{
					CornerSet |= 1 << 2;
					CornerValues[2] = GetHeightMapDataFunction(BorderComponent[3], 0, 0, NeighborHeightmapTextureData[3]);
				}

				if (CornerComponent[3])
				{
					CornerSet |= 1 << 3;
					CornerValues[3] = GetHeightMapDataFunction(CornerComponent[3], 0, 0, nullptr);
				}
				else if (BorderComponent[1])
				{
					CornerSet |= 1 << 3;
					CornerValues[3] = GetHeightMapDataFunction(BorderComponent[1], 0, EdgeCoord, NeighborHeightmapTextureData[1]);
				}
				else if (BorderComponent[3])
				{
					CornerSet |= 1 << 3;
					CornerValues[3] = GetHeightMapDataFunction(BorderComponent[3], EdgeCoord, 0, NeighborHeightmapTextureData[3]);
				}

				FillCornerValues(CornerSet, CornerValues);
				ComponentDataExist[ComponentIndexXY] = ExistLeft || ExistUp || (BorderComponent[0] || BorderComponent[1] || BorderComponent[2] || BorderComponent[3]) || CornerSet;
			}

			if (!ComponentDataExist[ComponentIndexXY])
			{
				continue;
			}

			// Find coordinates of box that lies inside component
			int32 ComponentX1 = FMath::Clamp<int32>(X1-ComponentIndexX*ComponentSizeQuads, 0, ComponentSizeQuads);
			int32 ComponentY1 = FMath::Clamp<int32>(Y1-ComponentIndexY*ComponentSizeQuads, 0, ComponentSizeQuads);
			int32 ComponentX2 = FMath::Clamp<int32>(X2-ComponentIndexX*ComponentSizeQuads, 0, ComponentSizeQuads);
			int32 ComponentY2 = FMath::Clamp<int32>(Y2-ComponentIndexY*ComponentSizeQuads, 0, ComponentSizeQuads);

			// Find subsection range for this box
			int32 SubIndexX1 = FMath::Clamp<int32>((ComponentX1-1) / SubsectionSizeQuads,0,ComponentNumSubsections-1);	// -1 because we need to pick up vertices shared between subsections
			int32 SubIndexY1 = FMath::Clamp<int32>((ComponentY1-1) / SubsectionSizeQuads,0,ComponentNumSubsections-1);
			int32 SubIndexX2 = FMath::Clamp<int32>(ComponentX2 / SubsectionSizeQuads,0,ComponentNumSubsections-1);
			int32 SubIndexY2 = FMath::Clamp<int32>(ComponentY2 / SubsectionSizeQuads,0,ComponentNumSubsections-1);

			for( int32 SubIndexY=SubIndexY1;SubIndexY<=SubIndexY2;SubIndexY++ )
			{
				for( int32 SubIndexX=SubIndexX1;SubIndexX<=SubIndexX2;SubIndexX++ )
				{
					// Find coordinates of box that lies inside subsection
					int32 SubX1 = FMath::Clamp<int32>(ComponentX1-SubsectionSizeQuads*SubIndexX, 0, SubsectionSizeQuads);
					int32 SubY1 = FMath::Clamp<int32>(ComponentY1-SubsectionSizeQuads*SubIndexY, 0, SubsectionSizeQuads);
					int32 SubX2 = FMath::Clamp<int32>(ComponentX2-SubsectionSizeQuads*SubIndexX, 0, SubsectionSizeQuads);
					int32 SubY2 = FMath::Clamp<int32>(ComponentY2-SubsectionSizeQuads*SubIndexY, 0, SubsectionSizeQuads);

					// Update texture data for the box that lies inside subsection
					for( int32 SubY=SubY1;SubY<=SubY2;SubY++ )
					{
						for( int32 SubX=SubX1;SubX<=SubX2;SubX++ )
						{
							int32 LandscapeX = SubIndexX*SubsectionSizeQuads + ComponentIndexX*ComponentSizeQuads + SubX;
							int32 LandscapeY = SubIndexY*SubsectionSizeQuads + ComponentIndexY*ComponentSizeQuads + SubY;

							// Find the input data corresponding to this vertex
							if( Component )
							{
								// Find the texture data corresponding to this vertex
								typename TDataAccess::DataType Height = GetHeightMapDataFunction(Component, (SubsectionSizeQuads + 1) * SubIndexX + SubX, (SubsectionSizeQuads + 1) * SubIndexY + SubY, HeightmapTextureData);
								StoreData.Store(LandscapeX, LandscapeY, Height);
							}
							else
							{
								// Find the texture data corresponding to this vertex
								typename TDataAccess::DataType Value[4] = {0, 0, 0, 0};
								int32 Dist[4] = {INT_MAX, INT_MAX, INT_MAX, INT_MAX};
								float ValueX = 0.0f, ValueY = 0.0f;
								bool Exist[4] = {false, false, false, false};

								// Use data already stored for 0, 2
								if (ExistLeft)
								{
									Value[0] = StoreData.Load( ComponentIndexX*ComponentSizeQuads, LandscapeY);
									Dist[0] = LandscapeX - (ComponentIndexX*ComponentSizeQuads);
									Exist[0] = true;
								}
								else if (BorderComponent[0])
								{
									Value[0] = GetHeightMapDataFunction(BorderComponent[0], EdgeCoord, (SubsectionSizeQuads + 1) * SubIndexY + SubY, NeighborHeightmapTextureData[0]);
									Dist[0] = LandscapeX - (BorderComponent[0]->GetSectionBase().X + ComponentSizeQuads);
									Exist[0] = true;
								}
								else
								{
									if ((CornerSet & 1) && (CornerSet & 1 << 2))
									{
										int32 Dist1 = LandscapeY - (ComponentIndexY*ComponentSizeQuads);
										int32 Dist2 = ((ComponentIndexY+1)*ComponentSizeQuads) - LandscapeY;
										Value[0] = static_cast<typename TDataAccess::DataType>((float)(Dist2 * CornerValues[0] + Dist1 * CornerValues[2]) / (Dist1 + Dist2));
										Dist[0] = LandscapeX - (ComponentIndexX*ComponentSizeQuads);
										Exist[0] = true;
									}
								}

								if (BorderComponent[1])
								{
									Value[1] = GetHeightMapDataFunction(BorderComponent[1], 0, (SubsectionSizeQuads + 1) * SubIndexY + SubY, NeighborHeightmapTextureData[1]);
									Dist[1] = (BorderComponent[1]->GetSectionBase().X) - LandscapeX;
									Exist[1] = true;
								}
								else
								{
									if ((CornerSet & 1 << 1) && (CornerSet & 1 << 3))
									{
										int32 Dist1 = LandscapeY - (ComponentIndexY*ComponentSizeQuads);
										int32 Dist2 = ((ComponentIndexY+1)*ComponentSizeQuads) - LandscapeY;
										Value[1] = static_cast<typename TDataAccess::DataType>((float)(Dist2 * CornerValues[1] + Dist1 * CornerValues[3]) / (Dist1 + Dist2));
										Dist[1] = (ComponentIndexX+1)*ComponentSizeQuads - LandscapeX;
										Exist[1] = true;
									}
								}

								if (ExistUp)
								{
									Value[2] = StoreData.Load(LandscapeX, ComponentIndexY*ComponentSizeQuads);
									Dist[2] = LandscapeY - (ComponentIndexY*ComponentSizeQuads);
									Exist[2] = true;
								}
								else if (BorderComponent[2])
								{
									Value[2] = GetHeightMapDataFunction(BorderComponent[2], (SubsectionSizeQuads + 1) * SubIndexX + SubX, EdgeCoord, NeighborHeightmapTextureData[2]);
									Dist[2] = LandscapeY - (BorderComponent[2]->GetSectionBase().Y + ComponentSizeQuads);
									Exist[2] = true;
								}
								else
								{
									if ((CornerSet & 1) && (CornerSet & 1 << 1))
									{
										int32 Dist1 = LandscapeX - (ComponentIndexX*ComponentSizeQuads);
										int32 Dist2 = (ComponentIndexX+1)*ComponentSizeQuads - LandscapeX;
										Value[2] = static_cast<typename TDataAccess::DataType>((float)(Dist2 * CornerValues[0] + Dist1 * CornerValues[1]) / (Dist1 + Dist2));
										Dist[2] = LandscapeY - (ComponentIndexY*ComponentSizeQuads);
										Exist[2] = true;
									}
								}

								if (BorderComponent[3])
								{
									Value[3] = GetHeightMapDataFunction(BorderComponent[3], (SubsectionSizeQuads + 1) * SubIndexX + SubX, 0, NeighborHeightmapTextureData[3]);
									Dist[3] = (BorderComponent[3]->GetSectionBase().Y) - LandscapeY;
									Exist[3] = true;
								}
								else
								{
									if ((CornerSet & 1 << 2) && (CornerSet & 1 << 3))
									{
										int32 Dist1 = LandscapeX - (ComponentIndexX*ComponentSizeQuads);
										int32 Dist2 = (ComponentIndexX+1)*ComponentSizeQuads - LandscapeX;
										Value[3] = static_cast<typename TDataAccess::DataType>((float)(Dist2 * CornerValues[2] + Dist1 * CornerValues[3]) / (Dist1 + Dist2));
										Dist[3] = (ComponentIndexY+1)*ComponentSizeQuads - LandscapeY;
										Exist[3] = true;
									}
								}

								CalcInterpValue<typename TDataAccess::DataType>(Dist, Exist, Value, ValueX, ValueY);

								typename TDataAccess::DataType FinalValue = 0; // Default Value
								if ( (Exist[0] || Exist[1]) && (Exist[2] || Exist[3]) )
								{
									FinalValue = CalcValueFromValueXY<typename TDataAccess::DataType>(Dist, ValueX, ValueY, CornerSet, CornerValues);
								}
								else if ( (BorderComponent[0] || BorderComponent[1]) )
								{
									FinalValue = static_cast<typename TDataAccess::DataType>(ValueX);
								}
								else if ( (BorderComponent[2] || BorderComponent[3]) )
								{
									FinalValue = static_cast<typename TDataAccess::DataType>(ValueY);
								}
								else if ( (Exist[0] || Exist[1]) )
								{
									FinalValue = static_cast<typename TDataAccess::DataType>(ValueX);
								}
								else if ( (Exist[2] || Exist[3]) )
								{
									FinalValue = static_cast<typename TDataAccess::DataType>(ValueY);
								}

								StoreData.Store(LandscapeX, LandscapeY, FinalValue);
							}
						}
					}
				}
			}
		}
	}

	if (bHasMissingValue)
	{
		// generate something to fill in any missing data within the requested region
		CalcMissingValues<typename TDataAccess::DataType, TDataAccess, float>(X1, X2, Y1, Y2,
			ComponentIndexX1, ComponentIndexX2, ComponentIndexY1, ComponentIndexY2,
			ComponentSizeX, ComponentSizeY, CornerValues,
			NoBorderY1, NoBorderY2, ComponentDataExist, StoreData);

		// clamp the returned bounds to the original request bounds
		// this means the returned bounds are allowed to shrink (if there is only a smaller part of the request that actually exists)
		// but it is not allowed to expand beyond the original request
		ValidX1 = FMath::Max<int32>(X1, ValidX1);
		ValidX2 = FMath::Min<int32>(X2, ValidX2);
		ValidY1 = FMath::Max<int32>(Y1, ValidY1);
		ValidY2 = FMath::Min<int32>(Y2, ValidY2);
	}
	else
	{
		// if there were no missing values, then the entire requested bounds is returned
		ValidX1 = X1;
		ValidX2 = X2;
		ValidY1 = Y1;
		ValidY2 = Y2;
	}
}

namespace
{
	template<typename T>
	struct TArrayDataAccess
	{
		typedef T DataType;
		int32 X1;
		int32 Y1;
		DataType* Data;
		int32 Stride;

		TArrayDataAccess(int32 InX1, int32 InY1, DataType* InData, int32 InStride)
			:	X1(InX1)
			,	Y1(InY1)
			,	Data(InData)
			,	Stride(InStride)
		{}

		inline void Store(int32 LandscapeX, int32 LandscapeY, DataType NewValue)
		{
			Data[ (LandscapeY-Y1) * Stride + (LandscapeX-X1) ] = NewValue;
		}

		inline DataType Load(int32 LandscapeX, int32 LandscapeY)
		{
			return Data[ (LandscapeY-Y1) * Stride + (LandscapeX-X1) ];
		}
	};

	template<typename T>
	struct TSparseDataAccess
	{
		typedef T DataType;
		TMap<FIntPoint, DataType>& SparseData;

		TSparseDataAccess(TMap<FIntPoint, DataType>& InSparseData)
			:	SparseData(InSparseData)
		{}

		inline void Store(int32 LandscapeX, int32 LandscapeY, DataType NewValue)
		{
			SparseData.Add(FIntPoint(LandscapeX,LandscapeY), NewValue);
		}

		inline DataType Load(int32 LandscapeX, int32 LandscapeY)
		{
			return SparseData.FindRef(FIntPoint(LandscapeX,LandscapeY));
		}
	};
};

template<typename TDataAccess>
void FLandscapeEditDataInterface::GetHeightDataTempl(int32& ValidX1, int32& ValidY1, int32& ValidX2, int32& ValidY2, TDataAccess& StoreData)
{
	GetHeightDataInternal(ValidX1, ValidY1, ValidX2, ValidY2, StoreData, [&](const ULandscapeComponent* Component, int32 TexU, int32 TexV, FColor* TextureData)->uint16 { return GetHeightMapData(Component, TexU, TexV, TextureData); });
}

template<typename TDataAccess>
void FLandscapeEditDataInterface::GetHeightAlphaBlendDataTempl(int32& ValidX1, int32& ValidY1, int32& ValidX2, int32& ValidY2, TDataAccess& StoreData)
{
	GetHeightDataInternal(ValidX1, ValidY1, ValidX2, ValidY2, StoreData, [&](const ULandscapeComponent* Component, int32 TexU, int32 TexV, FColor* TextureData)->uint16 { return GetHeightMapAlphaBlendData(Component, TexU, TexV, TextureData); });
}

template<typename TDataAccess>
void FLandscapeEditDataInterface::GetHeightFlagsDataTempl(int32& ValidX1, int32& ValidY1, int32& ValidX2, int32& ValidY2, TDataAccess& StoreData)
{
	GetHeightDataInternal(ValidX1, ValidY1, ValidX2, ValidY2, StoreData, [&](const ULandscapeComponent* Component, int32 TexU, int32 TexV, FColor* TextureData)->uint8 { return GetHeightMapFlagsData(Component, TexU, TexV, TextureData); });
}

void FLandscapeEditDataInterface::GetHeightData(int32& X1, int32& Y1, int32& X2, int32& Y2, uint16* Data, int32 Stride)
{
	if( Stride==0 )
	{
		Stride = (1+X2-X1);
	}

	TArrayDataAccess<uint16> ArrayStoreData(X1, Y1, Data, Stride);
	GetHeightDataTempl(X1, Y1, X2, Y2, ArrayStoreData);
}

void FLandscapeEditDataInterface::GetHeightAlphaBlendData(int32& X1, int32& Y1, int32& X2, int32& Y2, uint16* Data, int32 Stride)
{
	if (Stride == 0)
	{
		Stride = (1 + X2 - X1);
	}

	TArrayDataAccess<uint16> ArrayStoreData(X1, Y1, Data, Stride);
	GetHeightAlphaBlendDataTempl(X1, Y1, X2, Y2, ArrayStoreData);
}

void FLandscapeEditDataInterface::GetHeightFlagsData(int32& X1, int32& Y1, int32& X2, int32& Y2, uint8* Data, int32 Stride)
{
	if (Stride == 0)
	{
		Stride = (1 + X2 - X1);
	}

	TArrayDataAccess<uint8> ArrayStoreData(X1, Y1, Data, Stride);
	GetHeightFlagsDataTempl(X1, Y1, X2, Y2, ArrayStoreData);
}

void FLandscapeEditDataInterface::GetHeightDataFast(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, uint16* Data, int32 Stride, uint16* NormalData /*= nullptr*/, UTexture2D* InHeightmap)
{
	if( Stride==0 )
	{
		Stride = (1+X2-X1);
	}

	TArrayDataAccess<uint16> ArrayStoreData(X1, Y1, Data, Stride);
	if (NormalData)
	{
		TArrayDataAccess<uint16> ArrayNormalData(X1, Y1, NormalData, Stride);
		GetHeightDataTemplFast(X1, Y1, X2, Y2, ArrayStoreData, InHeightmap, &ArrayNormalData);
	}
	else
	{
		GetHeightDataTemplFast(X1, Y1, X2, Y2, ArrayStoreData, InHeightmap);
	}
}

void FLandscapeEditDataInterface::GetHeightData(int32& X1, int32& Y1, int32& X2, int32& Y2, TMap<FIntPoint, uint16>& SparseData)
{
	TSparseDataAccess<uint16> SparseStoreData(SparseData);
	GetHeightDataTempl(X1, Y1, X2, Y2, SparseStoreData);
}

void FLandscapeEditDataInterface::GetHeightDataFast(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TMap<FIntPoint, uint16>& SparseData, TMap<FIntPoint, uint16>* NormalData /*= nullptr*/, UTexture2D* InHeightmap)
{
	TSparseDataAccess<uint16> SparseStoreData(SparseData);
	if (NormalData)
	{
		TSparseDataAccess<uint16> SparseNormalData(*NormalData);
		GetHeightDataTemplFast(X1, Y1, X2, Y2, SparseStoreData, InHeightmap, &SparseNormalData);
	}
	else
	{
		GetHeightDataTemplFast(X1, Y1, X2, Y2, SparseStoreData, InHeightmap);
	}
}

bool ULandscapeComponent::DeleteLayerIfAllZero(const FGuid& InEditLayerGuid, const uint8* const TexDataPtr, int32 TexSize, int32 LayerIdx, bool bShouldDirtyPackage)
{
	if (TexDataPtr == nullptr)
	{
		return false;
	}
	
	// Check the data for the entire component and to see if it's all zero
	for (int32 TexY = 0; TexY < TexSize; TexY++)
	{
		for (int32 TexX = 0; TexX < TexSize; TexX++)
		{
			const int32 TexDataIndex = 4 * (TexX + TexY * TexSize);

			// Stop the first time we see any non-zero data
			uint8 Weight = TexDataPtr[TexDataIndex];
			if (Weight != 0)
			{
				return false;
			}
		}
	}

	DeleteLayerAllocation(InEditLayerGuid, LayerIdx, bShouldDirtyPackage);
	MarkRenderStateDirty();

	return true;
}

void ULandscapeComponent::DeleteLayer(ULandscapeLayerInfoObject* LayerInfo, FLandscapeEditDataInterface& LandscapeEdit)
{
	FGuid EditLayerGuid = this->GetEditingLayerGUID();
	DeleteLayerInternal(LayerInfo, LandscapeEdit, EditLayerGuid);
}

void ULandscapeComponent::DeleteLayerInternal(ULandscapeLayerInfoObject* LayerInfo, FLandscapeEditDataInterface& LandscapeEdit, const FGuid& InEditLayerGuid)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeComponent_DeleteLayer);

	ULandscapeComponent* Component = this;
	
	// This can be called during WeightmapFixup, so skip checking for updated weightmap allocations
	const TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = Component->GetWeightmapLayerAllocations(InEditLayerGuid);
	const TArray<UTexture2D*>& ComponentWeightmapTextures = Component->GetWeightmapTextures(InEditLayerGuid);

	// Delete the first LayerAllocation with a matching LayerInfo
	const int32 DeleteAllocIdx = ComponentWeightmapLayerAllocations.IndexOfByPredicate(
		[LayerInfo](const FWeightmapLayerAllocationInfo& Allocation) { return Allocation.LayerInfo == LayerInfo; });
	if (DeleteAllocIdx == INDEX_NONE)
	{
		// Layer not used for this component.
		return;
	}

	Component->DeleteLayerAllocation(InEditLayerGuid, DeleteAllocIdx, LandscapeEdit.GetShouldDirtyPackage());

	Component->RequestWeightmapUpdate();
}

void FLandscapeEditDataInterface::DeleteLayer(ULandscapeLayerInfoObject* LayerInfo)
{
	if (!LandscapeInfo)
	{
		return;
	}

	for (auto& XYComponentPair : LandscapeInfo->XYtoComponentMap)
	{
		ULandscapeComponent* Component = XYComponentPair.Value;
		Component->DeleteLayerInternal(LayerInfo, *this, GetEditLayer());
	}

	if (LandscapeInfo->LandscapeActor.IsValid())
	{
		LandscapeInfo->LandscapeActor->RequestLayersContentUpdate(ELandscapeLayerUpdateMode::Update_All);
	}
	else
	{
		TSet<ULandscapeComponent*> Components;
		Algo::Transform(LandscapeInfo->XYtoComponentMap, Components, &TPair<FIntPoint, ULandscapeComponent*>::Value);
		ALandscapeProxy::InvalidateGeneratedComponentData(Components);
	}
}

void ULandscapeComponent::FillLayer(ULandscapeLayerInfoObject* LayerInfo, FLandscapeEditDataInterface& LandscapeEdit)
{
	check(LayerInfo);

	ULandscapeComponent* Component = this;

	FGuid EditLayerGuid = LandscapeEdit.GetEditLayer();

	// TODO [jonathan.bard] : This should always be false now, as we should always use this function in the context of edit layers. Check that it's the case
	//  and look for all places where we use LandscapeEdit.GetEditLayer() (and check that the result !IsValid(), these should all go. We might even be able to 
	//  get rid of FScopedSetLandscapeEditingLayer at long last!)
	const bool bIsFinalWeightmap = !EditLayerGuid.IsValid();

	ALandscapeProxy* Proxy = Component->GetLandscapeProxy();
	Component->Modify(LandscapeEdit.GetShouldDirtyPackage());
	Proxy->Modify(LandscapeEdit.GetShouldDirtyPackage());

	TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = Component->GetWeightmapLayerAllocations(EditLayerGuid);
	TArray<TObjectPtr<UTexture2D>>& ComponentWeightmapTextures = Component->GetWeightmapTextures(EditLayerGuid);
	TArray<TObjectPtr<ULandscapeWeightmapUsage>>& ComponentWeightmapTexturesUsage = Component->GetWeightmapTexturesUsage(EditLayerGuid);

	// Find the index for this layer in this component.
	int32 FillLayerIdx = ComponentWeightmapLayerAllocations.IndexOfByPredicate(
		[LayerInfo](const FWeightmapLayerAllocationInfo& Allocation) { return Allocation.LayerInfo == LayerInfo; });

	// if the layer is still not found then we are forced to make a new allocation
	if (FillLayerIdx == INDEX_NONE)
	{
		FillLayerIdx = ComponentWeightmapLayerAllocations.Num();
		ComponentWeightmapLayerAllocations.Add(FWeightmapLayerAllocationInfo(LayerInfo));
		Component->ReallocateWeightmaps(/*DataInterface =*/&LandscapeEdit, EditLayerGuid, /*bInSaveToTransactionBuffer = */true, /*bool bInForceReallocate = */false, /*InTargetProxy = */nullptr, /*InRestrictSharingToComponents = */nullptr);
	}

	check(FillLayerIdx != INDEX_NONE);

	// fill the layer
	{
		// Find the texture data corresponding to this vertex
		const int32 SizeU = (SubsectionSizeQuads + 1) * NumSubsections;
		const int32 SizeV = (SubsectionSizeQuads + 1) * NumSubsections;
		const int32 WeightmapOffsetX = static_cast<int32>(Component->WeightmapScaleBias.Z * SizeU);
		const int32 WeightmapOffsetY = static_cast<int32>(Component->WeightmapScaleBias.W * SizeV);

		FWeightmapLayerAllocationInfo& FillLayerAllocation = ComponentWeightmapLayerAllocations[FillLayerIdx];
		uint8* MaterialLayerData = (uint8*)LandscapeEdit.GetTextureDataInfo(ComponentWeightmapTextures[FillLayerAllocation.WeightmapTextureIndex])->GetMipData(/*InMipIndex = */0);

		for (int32 Y = 0; Y < SizeV; ++Y)
		{
			const int32 TexY = WeightmapOffsetY + Y;
			uint8* RowData = MaterialLayerData + ((WeightmapOffsetY + Y) * SizeU + WeightmapOffsetX) * 4 + ChannelOffsets[FillLayerAllocation.WeightmapTextureChannel];
			for (int32 X = 0; X < SizeU; ++X)
			{
				RowData[X * 4] = 255;
			}
		}
	}

	// update mips if this texture isn't used in the layer compositing 
	if (bIsFinalWeightmap)
	{
		for (int32 TextureIdx = 0; TextureIdx < ComponentWeightmapTextures.Num(); ++TextureIdx)
		{
			UTexture2D* WeightmapTexture = ComponentWeightmapTextures[TextureIdx];
			FLandscapeTextureDataInfo* WeightmapDataInfo = LandscapeEdit.GetTextureDataInfo(WeightmapTexture);

			const int32 NumMips = WeightmapTexture->Source.GetNumMips();
			TArray<FColor*> WeightmapTextureMipData;
			WeightmapTextureMipData.AddUninitialized(NumMips);
			for (int32 MipIdx = 0; MipIdx < NumMips; MipIdx++)
			{
				WeightmapTextureMipData[MipIdx] = (FColor*)WeightmapDataInfo->GetMipData(MipIdx);
			}

			ULandscapeComponent::UpdateWeightmapMips(NumSubsections, SubsectionSizeQuads, WeightmapTexture, WeightmapTextureMipData, 0, 0, MAX_int32, MAX_int32, WeightmapDataInfo);

			WeightmapDataInfo->AddMipUpdateRegion(0, 0, 0, WeightmapTexture->Source.GetSizeX() - 1, WeightmapTexture->Source.GetSizeY() - 1);
		}
	}

	Component->RequestWeightmapUpdate();

	Proxy->ValidateProxyLayersWeightmapUsage();
}

void FLandscapeEditDataInterface::FillLayer(ULandscapeLayerInfoObject* LayerInfo)
{
	const bool bEmptyLayersOnly = false;
	FillLayer(LayerInfo, bEmptyLayersOnly);
}

void FLandscapeEditDataInterface::FillEmptyLayers(ULandscapeLayerInfoObject* LayerInfo)
{
	const bool bEmptyLayersOnly = true;
	FillLayer(LayerInfo, bEmptyLayersOnly);
}

void FLandscapeEditDataInterface::FillLayer(ULandscapeLayerInfoObject* LayerInfo, bool bEmptyLayersOnly)
{
	if (!LandscapeInfo)
	{
		return;
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	LayerInfo->IsReferencedFromLoadedData = true;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	for (auto& XYComponentPair : LandscapeInfo->XYtoComponentMap)
	{
		ULandscapeComponent* Component = XYComponentPair.Value;
		const TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = Component->GetWeightmapLayerAllocations(GetEditLayer());

		bool LayerNeedFilling = true;

		if (bEmptyLayersOnly)
		{
			for (const FWeightmapLayerAllocationInfo& Alloc : ComponentWeightmapLayerAllocations)
			{
				if (Alloc.LayerInfo != nullptr)
				{
					LayerNeedFilling = false;
					break;
				}
			}
		}

		if (LayerNeedFilling)
		{
			Component->FillLayer(LayerInfo, *this);
		}
	}

	if (LandscapeInfo->LandscapeActor.IsValid())
	{
		LandscapeInfo->LandscapeActor->RequestLayersContentUpdateForceAll();
	}
	else
	{
		TSet<ULandscapeComponent*> Components;
		Algo::Transform(LandscapeInfo->XYtoComponentMap, Components, &TPair<FIntPoint, ULandscapeComponent*>::Value);
		ALandscapeProxy::InvalidateGeneratedComponentData(Components);
	}
}

void ULandscapeComponent::ReplaceLayer(ULandscapeLayerInfoObject* FromLayerInfo, ULandscapeLayerInfoObject* ToLayerInfo, FLandscapeEditDataInterface& LandscapeEdit)
{
	ReplaceLayerInternal(FromLayerInfo, ToLayerInfo, LandscapeEdit, GetEditingLayerGUID());
}

void ULandscapeComponent::ReplaceLayerInternal(ULandscapeLayerInfoObject* FromLayerInfo, ULandscapeLayerInfoObject* ToLayerInfo, FLandscapeEditDataInterface& LandscapeEdit, const FGuid& InEditLayerGUID)
{
	check(FromLayerInfo); // ToLayerInfo may be null when clearing a layer
	if (FromLayerInfo == ToLayerInfo)
	{
		return;
	}

	TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = GetWeightmapLayerAllocations(InEditLayerGUID);
	TArray<TObjectPtr<UTexture2D>>& ComponentWeightmapTextures = GetWeightmapTextures(InEditLayerGUID);
	TArray<TObjectPtr<ULandscapeWeightmapUsage>>& ComponentWeightmapTexturesUsage = GetWeightmapTexturesUsage(InEditLayerGUID);

	// Find the index for this layer in this component.
	int32 FromLayerIdx = INDEX_NONE;
	for (int32 LayerIdx = 0; LayerIdx < ComponentWeightmapLayerAllocations.Num(); LayerIdx++)
	{
		FWeightmapLayerAllocationInfo& Allocation = ComponentWeightmapLayerAllocations[LayerIdx];
		if (Allocation.LayerInfo == FromLayerInfo)
		{
			FromLayerIdx = LayerIdx;
		}
	}
	if (FromLayerIdx == INDEX_NONE)
	{
		// Layer not used for this component, nothing to do.
		return;
	}

	bool bMerging = true;

	// Find the index for this layer in this component.
	int32 ToLayerIdx = INDEX_NONE;
	for (int32 LayerIdx = 0; LayerIdx < ComponentWeightmapLayerAllocations.Num(); LayerIdx++)
	{
		FWeightmapLayerAllocationInfo& Allocation = ComponentWeightmapLayerAllocations[LayerIdx];
		if (Allocation.LayerInfo == ToLayerInfo)
		{
			ToLayerIdx = LayerIdx;
		}
	}
	if (ToLayerIdx == INDEX_NONE)
	{
		// Layer not used for this component, so do trivial replace.
		ComponentWeightmapLayerAllocations[FromLayerIdx].LayerInfo = ToLayerInfo;
		bMerging = false;
	}

	FWeightmapLayerAllocationInfo& FromLayerAllocation = ComponentWeightmapLayerAllocations[FromLayerIdx];

	// See if we'll be able to remove the texture completely.
	bool bCanRemoveLayerTexture = false;
	if (bMerging)
	{
		bCanRemoveLayerTexture = true;
		for (int32 LayerIdx = 0; LayerIdx < ComponentWeightmapLayerAllocations.Num(); LayerIdx++)
		{
			FWeightmapLayerAllocationInfo& Allocation = ComponentWeightmapLayerAllocations[LayerIdx];

			// check if we will be able to remove the texture also
			if (LayerIdx != FromLayerIdx && Allocation.WeightmapTextureIndex == FromLayerAllocation.WeightmapTextureIndex)
			{
				bCanRemoveLayerTexture = false;
				break;
			}
		}
	}

	if (bMerging)
	{
		FWeightmapLayerAllocationInfo& ToLayerAllocation = ComponentWeightmapLayerAllocations[ToLayerIdx];

		// Lock data for all the weightmaps
		FLandscapeTextureDataInfo* FromTexDataInfo = LandscapeEdit.GetTextureDataInfo(ComponentWeightmapTextures[FromLayerAllocation.WeightmapTextureIndex]);
		FLandscapeTextureDataInfo* ToTexDataInfo = LandscapeEdit.GetTextureDataInfo(ComponentWeightmapTextures[ToLayerAllocation.WeightmapTextureIndex]);

		check(FromTexDataInfo->GetMipSizeX(0) == FromTexDataInfo->GetMipSizeY(0));
		check(ToTexDataInfo->GetMipSizeX(0) == ToTexDataInfo->GetMipSizeY(0));
		check(FromTexDataInfo->GetMipSizeX(0) == ToTexDataInfo->GetMipSizeX(0));
		const int32 MipSize = FromTexDataInfo->GetMipSizeX(0);

		uint8* const SrcTextureData = (uint8*)FromTexDataInfo->GetMipData(/*InMipIndex = */0) + ChannelOffsets[FromLayerAllocation.WeightmapTextureChannel];
		uint8* const DestTextureData = (uint8*)ToTexDataInfo->GetMipData(/*InMipIndex = */0) + ChannelOffsets[ToLayerAllocation.WeightmapTextureChannel];

		for (int32 i = 0; i < FMath::Square(MipSize); i++)
		{
			DestTextureData[i*4] = static_cast<uint8>(FMath::Min(255, (uint16)DestTextureData[i*4] + (uint16)SrcTextureData[i*4]));
		}

		// Update all mips
		if (!bCanRemoveLayerTexture)
		{
			UTexture2D* WeightmapTexture = ComponentWeightmapTextures[FromLayerAllocation.WeightmapTextureIndex];
			FLandscapeTextureDataInfo* WeightmapDataInfo = FromTexDataInfo;

			const int32 NumMips = WeightmapTexture->Source.GetNumMips();
			TArray<FColor*> WeightmapTextureMipData;
			WeightmapTextureMipData.AddUninitialized(NumMips);
			for (int32 MipIdx = 0; MipIdx < NumMips; MipIdx++)
			{
				WeightmapTextureMipData[MipIdx] = (FColor*)WeightmapDataInfo->GetMipData(MipIdx);
			}

			ULandscapeComponent::UpdateWeightmapMips(NumSubsections, SubsectionSizeQuads, WeightmapTexture, WeightmapTextureMipData, 0, 0, MAX_int32, MAX_int32, WeightmapDataInfo);

			WeightmapDataInfo->AddMipUpdateRegion(0, 0, 0, WeightmapTexture->Source.GetSizeX() - 1, WeightmapTexture->Source.GetSizeY() - 1);
		}

		if (FromTexDataInfo != ToTexDataInfo)
		{
			UTexture2D* WeightmapTexture = ComponentWeightmapTextures[ToLayerAllocation.WeightmapTextureIndex];
			FLandscapeTextureDataInfo* WeightmapDataInfo = ToTexDataInfo;

			const int32 NumMips = WeightmapTexture->Source.GetNumMips();
			TArray<FColor*> WeightmapTextureMipData;
			WeightmapTextureMipData.AddUninitialized(NumMips);
			for (int32 MipIdx = 0; MipIdx < NumMips; MipIdx++)
			{
				WeightmapTextureMipData[MipIdx] = (FColor*)WeightmapDataInfo->GetMipData(MipIdx);
			}

			ULandscapeComponent::UpdateWeightmapMips(NumSubsections, SubsectionSizeQuads, WeightmapTexture, WeightmapTextureMipData, 0, 0, MAX_int32, MAX_int32, WeightmapDataInfo);

			WeightmapDataInfo->AddMipUpdateRegion(0, 0, 0, WeightmapTexture->Source.GetSizeX() - 1, WeightmapTexture->Source.GetSizeY() - 1);
		}
	}

	// if merging into an existing layer, remove the layer and potentially the texture
	if (bMerging)
	{
		ALandscapeProxy* Proxy = GetLandscapeProxy();
		// Mark the channel as unallocated, so we can reuse it later
		ULandscapeWeightmapUsage* Usage = ComponentWeightmapTexturesUsage[FromLayerAllocation.WeightmapTextureIndex];
		//check(Usage);
		if (Usage)
		{
			Usage->ChannelUsage[FromLayerAllocation.WeightmapTextureChannel] = nullptr;
		}

		// If this layer was the last usage for this texture, we can remove it.
		if (bCanRemoveLayerTexture)
		{
			ComponentWeightmapTextures[FromLayerAllocation.WeightmapTextureIndex]->SetFlags(RF_Transactional);
			ComponentWeightmapTextures[FromLayerAllocation.WeightmapTextureIndex]->Modify(LandscapeEdit.GetShouldDirtyPackage());
			ComponentWeightmapTextures[FromLayerAllocation.WeightmapTextureIndex]->ClearFlags(RF_Standalone);

			ComponentWeightmapTextures.RemoveAt(FromLayerAllocation.WeightmapTextureIndex);
			ComponentWeightmapTexturesUsage.RemoveAt(FromLayerAllocation.WeightmapTextureIndex);

			// Adjust WeightmapTextureIndex index for other layers
			for (int32 LayerIdx = 0; LayerIdx < ComponentWeightmapLayerAllocations.Num(); LayerIdx++)
			{
				if (LayerIdx == FromLayerIdx)
				{
					continue;
				}

				FWeightmapLayerAllocationInfo& Allocation = ComponentWeightmapLayerAllocations[LayerIdx];

				if (Allocation.WeightmapTextureIndex > FromLayerAllocation.WeightmapTextureIndex)
				{
					Allocation.WeightmapTextureIndex--;
				}

				check(Allocation.WeightmapTextureIndex < ComponentWeightmapTextures.Num());
			}
		}

		// Remove the layer
		ComponentWeightmapLayerAllocations.RemoveAt(FromLayerIdx);
	}

	RequestWeightmapUpdate();
	GetLandscapeProxy()->ValidateProxyLayersWeightmapUsage();
}

void FLandscapeEditDataInterface::ReplaceLayer(ULandscapeLayerInfoObject* FromLayerInfo, ULandscapeLayerInfoObject* ToLayerInfo)
{
	if (!LandscapeInfo) return;

	auto DoReplace = [&](const FGuid& EditLayerGuid)
	{
		for (auto It = LandscapeInfo->XYtoComponentMap.CreateIterator(); It; ++It)
		{
			ULandscapeComponent* Component = It.Value();
			Component->ReplaceLayerInternal(FromLayerInfo, ToLayerInfo, *this, EditLayerGuid);
		}
	};

	if (LandscapeInfo->LandscapeActor.IsValid())
	{
		LandscapeInfo->LandscapeActor->Modify(GetShouldDirtyPackage());
		LandscapeInfo->LandscapeActor->ReplaceLayerSubstractiveBlendStatus(FromLayerInfo, ToLayerInfo, GetShouldDirtyPackage());
		LandscapeInfo->LandscapeActor->ForEachEditLayerConst([DoReplace](const ULandscapeEditLayerBase* CurrentLayer)
		{
			DoReplace(CurrentLayer->GetGuid());
			return true;
		});
		LandscapeInfo->LandscapeActor->RequestLayersContentUpdateForceAll(ELandscapeLayerUpdateMode::Update_Weightmap_All);
	}
	else
	{
		FGuid BaseLayerGuid;
		DoReplace(BaseLayerGuid);
	}
}

// simple classes for the template....
namespace
{
	template<typename TDataType>
	struct TArrayStoreData
	{
		int32 X1;
		int32 Y1;
		TDataType* Data;
		int32 Stride;
		int32 ArraySize;

		TArrayStoreData(int32 InX1, int32 InY1, TDataType* InData, int32 InStride)
			:	X1(InX1)
			,	Y1(InY1)
			,	Data(InData)
			,	Stride(InStride)
			,	ArraySize(1)
		{}

		inline void Store(int32 LandscapeX, int32 LandscapeY, uint8 Weight) {}
		inline void Store(int32 LandscapeX, int32 LandscapeY, uint8 Weight, int32 LayerIdx) {}
		inline void Store(int32 LandscapeX, int32 LandscapeY, FVector2D Offset) {}
		inline TDataType Load(int32 LandscapeX, int32 LandscapeY) { return 0; }
		inline void PreInit(int32 InArraySize) { ArraySize = InArraySize; }
	};

	template<> void TArrayStoreData<uint8>::Store(int32 LandscapeX, int32 LandscapeY, uint8 Weight)
	{
		Data[ (LandscapeY-Y1) * Stride + (LandscapeX-X1) ] = Weight;
	}

	template<> uint8 TArrayStoreData<uint8>::Load(int32 LandscapeX, int32 LandscapeY)
	{
		return Data[ (LandscapeY-Y1) * Stride + (LandscapeX-X1) ];
	}

	template<> FVector2D TArrayStoreData<FVector2D>::Load(int32 LandscapeX, int32 LandscapeY)
	{
		return Data[(LandscapeY - Y1) * Stride + (LandscapeX - X1)];
	}

	template<> FVector TArrayStoreData<FVector>::Load(int32 LandscapeX, int32 LandscapeY)
	{
		return Data[(LandscapeY - Y1) * Stride + (LandscapeX - X1)];
	}

	template<> void TArrayStoreData<FVector2D>::Store(int32 LandscapeX, int32 LandscapeY, FVector2D Offset)
	{
		Data[ (LandscapeY-Y1) * Stride + (LandscapeX-X1) ] = Offset;
	}

	template<> void TArrayStoreData<FVector>::Store(int32 LandscapeX, int32 LandscapeY, FVector2D Offset)
	{
		Data[ (LandscapeY-Y1) * Stride + (LandscapeX-X1) ] = FVector(Offset.X, Offset.Y, 0.0f);
	}

	// Data items should be initialized with ArraySize
	template<> void TArrayStoreData<TArray<uint8>>::Store(int32 LandscapeX, int32 LandscapeY, uint8 Weight, int32 LayerIdx)
	{
		TArray<uint8>& Value = Data[ ((LandscapeY-Y1) * Stride + (LandscapeX-X1)) ];
		if (Value.Num() != ArraySize)
		{
			Value.Empty(ArraySize);
			Value.AddZeroed(ArraySize);
		}
		Value[LayerIdx] = Weight;
	}

	template<typename TDataType>
	struct TSparseStoreData
	{
		TMap<FIntPoint, TDataType>& SparseData;
		int32 ArraySize;

		TSparseStoreData(TMap<FIntPoint, TDataType>& InSparseData)
			:	SparseData(InSparseData)
			,	ArraySize(1)
		{}

		inline void Store(int32 LandscapeX, int32 LandscapeY, uint8 Weight) {}
		inline void Store(int32 LandscapeX, int32 LandscapeY, uint8 Weight, int32 LayerIdx) {}
		inline void Store(int32 LandscapeX, int32 LandscapeY, FVector2D Offset) {}
		inline TDataType Load(int32 LandscapeX, int32 LandscapeY) { return 0; }
		inline void PreInit(int32 InArraySize) { ArraySize = InArraySize; }
	};

	template<> void TSparseStoreData<uint8>::Store(int32 LandscapeX, int32 LandscapeY, uint8 Weight)
	{
		SparseData.Add(FIntPoint(LandscapeX,LandscapeY), Weight);
	}

	template<> uint8 TSparseStoreData<uint8>::Load(int32 LandscapeX, int32 LandscapeY)
	{
		return SparseData.FindRef(FIntPoint(LandscapeX,LandscapeY));
	}

	template<> FVector2D TSparseStoreData<FVector2D>::Load(int32 LandscapeX, int32 LandscapeY)
	{
		return SparseData.FindRef(FIntPoint(LandscapeX, LandscapeY));
	}

	template<> FVector TSparseStoreData<FVector>::Load(int32 LandscapeX, int32 LandscapeY)
	{
		return SparseData.FindRef(FIntPoint(LandscapeX, LandscapeY));
	}

	template<> void TSparseStoreData<TArray<uint8>>::Store(int32 LandscapeX, int32 LandscapeY, uint8 Weight, int32 LayerIdx)
	{
		TArray<uint8>* Value = SparseData.Find(FIntPoint(LandscapeX,LandscapeY));
		if (Value)
		{
			(*Value)[LayerIdx] = Weight;
		}
		else
		{
			TArray<uint8> Values;
			Values.Empty(ArraySize);
			Values.AddZeroed(ArraySize);
			Values[LayerIdx] = Weight;
			SparseData.Add(FIntPoint(LandscapeX, LandscapeY), Values);
		}
	}

	template<> void TSparseStoreData<FVector2D>::Store(int32 LandscapeX, int32 LandscapeY, FVector2D Offset)
	{
		SparseData.Add(FIntPoint(LandscapeX,LandscapeY), Offset);
	}

	template<> void TSparseStoreData<FVector>::Store(int32 LandscapeX, int32 LandscapeY, FVector2D Offset)
	{
		// Preserve old Z value
		FVector* PrevValue = SparseData.Find(FIntPoint(LandscapeX,LandscapeY));
		if (PrevValue != nullptr)
		{
			PrevValue->X = Offset.X;
			PrevValue->Y = Offset.Y;
		}
		else
		{
			SparseData.Add(FIntPoint(LandscapeX,LandscapeY), FVector(Offset.X, Offset.Y, 0.0f));
		}
	}
};

inline bool FLandscapeEditDataInterface::IsLayerAllowed(const ULandscapeLayerInfoObject* const LayerInfo, const int32 ComponentIndexX, const int32 SubIndexX, const int32 SubX, const int32 ComponentIndexY, const int32 SubIndexY, const int32 SubY)
{
	// left / right
	if (SubIndexX == 0 && SubX == 0)
	{
		ULandscapeComponent* EdgeComponent = LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(ComponentIndexX - 1, ComponentIndexY));
		if (EdgeComponent && !EdgeComponent->LayerAllowList.Contains(LayerInfo))
		{
			return false;
		}
	}
	else if (SubIndexX == ComponentNumSubsections - 1 && SubX == SubsectionSizeQuads)
	{
		ULandscapeComponent* EdgeComponent = LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(ComponentIndexX + 1, ComponentIndexY));
		if (EdgeComponent && !EdgeComponent->LayerAllowList.Contains(LayerInfo))
		{
			return false;
		}
	}

	// up / down
	if (SubIndexY == 0 && SubY == 0)
	{
		ULandscapeComponent* EdgeComponent = LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(ComponentIndexX, ComponentIndexY - 1));
		if (EdgeComponent && !EdgeComponent->LayerAllowList.Contains(LayerInfo))
		{
			return false;
		}
	}
	else if (SubIndexY == ComponentNumSubsections - 1 && SubY == SubsectionSizeQuads)
	{
		ULandscapeComponent* EdgeComponent = LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(ComponentIndexX, ComponentIndexY + 1));
		if (EdgeComponent && !EdgeComponent->LayerAllowList.Contains(LayerInfo))
		{
			return false;
		}
	}

	// diagonals
	if (SubIndexY == 0 && SubY == 0 && SubIndexX == 0 && SubX == 0)
	{
		ULandscapeComponent* CornerComponent = LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(ComponentIndexX - 1, ComponentIndexY - 1));
		if (CornerComponent && !CornerComponent->LayerAllowList.Contains(LayerInfo))
		{
			return false;
		}
	}
	else if (SubIndexY == 0 && SubY == 0 && SubIndexX == ComponentNumSubsections - 1 && SubX == SubsectionSizeQuads)
	{
		ULandscapeComponent* CornerComponent = LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(ComponentIndexX + 1, ComponentIndexY - 1));
		if (CornerComponent && !CornerComponent->LayerAllowList.Contains(LayerInfo))
		{
			return false;
		}
	}
	else if (SubIndexY == ComponentNumSubsections - 1 && SubY == SubsectionSizeQuads && SubIndexX == 0 && SubX == 0)
	{
		ULandscapeComponent* CornerComponent = LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(ComponentIndexX - 1, ComponentIndexY + 1));
		if (CornerComponent && !CornerComponent->LayerAllowList.Contains(LayerInfo))
		{
			return false;
		}
	}
	else if (SubIndexY == ComponentNumSubsections - 1 && SubY == SubsectionSizeQuads && SubIndexX == ComponentNumSubsections - 1 && SubX == SubsectionSizeQuads)
	{
		ULandscapeComponent* CornerComponent = LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(ComponentIndexX + 1, ComponentIndexY + 1));
		if (CornerComponent && !CornerComponent->LayerAllowList.Contains(LayerInfo))
		{
			return false;
		}
	}

	return true;
}

void FLandscapeEditDataInterface::SetAlphaData(ULandscapeLayerInfoObject* const LayerInfo, const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, const uint8* Data, int32 Stride, ELandscapeLayerPaintingRestriction PaintingRestriction /*= None*/)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeEditDataInterface_SetAlphaData);

	check(LayerInfo != nullptr);
	check(Data != nullptr);

	if (Stride == 0)
	{
		Stride = (1+X2-X1);
	}

	check(ComponentSizeQuads > 0);
	// Find component range for this block of data
	int32 ComponentIndexX1 = (X1-1 >= 0) ? (X1-1) / ComponentSizeQuads : (X1) / ComponentSizeQuads - 1;	// -1 because we need to pick up vertices shared between components
	int32 ComponentIndexY1 = (Y1-1 >= 0) ? (Y1-1) / ComponentSizeQuads : (Y1) / ComponentSizeQuads - 1;
	int32 ComponentIndexX2 = (X2 >= 0) ? X2 / ComponentSizeQuads : (X2+1) / ComponentSizeQuads - 1;
	int32 ComponentIndexY2 = (Y2 >= 0) ? Y2 / ComponentSizeQuads : (Y2+1) / ComponentSizeQuads - 1;

	TArray<FLandscapeTextureDataInfo*, TInlineAllocator<2>> TexDataInfos;
	TArray<uint8*, TInlineAllocator<8>> LayerDataPtrs;      // Pointers to all layers' data
	TArray<bool, TInlineAllocator<8>> LayerEditDataAllZero; // Whether the data we are editing for this layer is all zero
	TArray<FColor*> CollisionWeightmapMipData;
	TArray<FColor*> SimpleCollisionWeightmapMipData;
	TArray<FColor*> WeightmapTextureMipData;

	for (int32 ComponentIndexY = ComponentIndexY1; ComponentIndexY <= ComponentIndexY2; ComponentIndexY++)
	{
		for (int32 ComponentIndexX = ComponentIndexX1; ComponentIndexX <= ComponentIndexX2; ComponentIndexX++)
		{
			FIntPoint ComponentKey(ComponentIndexX,ComponentIndexY);
			ULandscapeComponent* Component = LandscapeInfo->XYtoComponentMap.FindRef(ComponentKey);

			// if nullptr, there is no component at this location
			if (Component == nullptr)
			{
				continue;
			}

			if (LayerInfo == ALandscape::VisibilityLayer && !Component->IsLandscapeHoleMaterialValid())
			{
				continue;
			}

			if (PaintingRestriction == ELandscapeLayerPaintingRestriction::UseComponentAllowList && !Component->LayerAllowList.Contains(LayerInfo))
			{
				continue;
			}

			Component->Modify(GetShouldDirtyPackage());

			TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = Component->GetWeightmapLayerAllocations(GetEditLayer());
			int32 UpdateLayerIdx = ComponentWeightmapLayerAllocations.IndexOfByPredicate([LayerInfo](const FWeightmapLayerAllocationInfo& Allocation){ return Allocation.LayerInfo == LayerInfo; });

			// Need allocation for weightmap
			if (UpdateLayerIdx == INDEX_NONE)
			{
				const int32 LayerLimit = Component->GetLandscapeProxy()->MaxPaintedLayersPerComponent;

				// if we can't allocate a layer, then there is nothing to paint
				if (PaintingRestriction == ELandscapeLayerPaintingRestriction::ExistingOnly ||
					(PaintingRestriction == ELandscapeLayerPaintingRestriction::UseMaxLayers && LayerLimit > 0 && ComponentWeightmapLayerAllocations.Num() >= LayerLimit))
				{
					continue;
				}

				auto IsDataAllZeroForComponent = [&]() -> bool
				{
					// Find coordinates of box that lies inside component
					const int32 ComponentX1 = FMath::Clamp<int32>(X1 - ComponentIndexX * ComponentSizeQuads, 0, ComponentSizeQuads);
					const int32 ComponentY1 = FMath::Clamp<int32>(Y1 - ComponentIndexY * ComponentSizeQuads, 0, ComponentSizeQuads);
					const int32 ComponentX2 = FMath::Clamp<int32>(X2 - ComponentIndexX * ComponentSizeQuads, 0, ComponentSizeQuads);
					const int32 ComponentY2 = FMath::Clamp<int32>(Y2 - ComponentIndexY * ComponentSizeQuads, 0, ComponentSizeQuads);

					// Find subsection range for this box
					const int32 SubIndexX1 = FMath::Clamp<int32>((ComponentX1 - 1) / SubsectionSizeQuads, 0, ComponentNumSubsections - 1); // -1 because we need to pick up vertices shared between subsections
					const int32 SubIndexY1 = FMath::Clamp<int32>((ComponentY1 - 1) / SubsectionSizeQuads, 0, ComponentNumSubsections - 1);
					const int32 SubIndexX2 = FMath::Clamp<int32>(ComponentX2 / SubsectionSizeQuads, 0, ComponentNumSubsections - 1);
					const int32 SubIndexY2 = FMath::Clamp<int32>(ComponentY2 / SubsectionSizeQuads, 0, ComponentNumSubsections - 1);

					for (int32 SubIndexY = SubIndexY1; SubIndexY <= SubIndexY2; SubIndexY++)
					{
						for (int32 SubIndexX = SubIndexX1; SubIndexX <= SubIndexX2; SubIndexX++)
						{
							// Find coordinates of box that lies inside subsection
							const int32 SubX1 = FMath::Clamp<int32>(ComponentX1 - SubsectionSizeQuads * SubIndexX, 0, SubsectionSizeQuads);
							const int32 SubY1 = FMath::Clamp<int32>(ComponentY1 - SubsectionSizeQuads * SubIndexY, 0, SubsectionSizeQuads);
							const int32 SubX2 = FMath::Clamp<int32>(ComponentX2 - SubsectionSizeQuads * SubIndexX, 0, SubsectionSizeQuads);
							const int32 SubY2 = FMath::Clamp<int32>(ComponentY2 - SubsectionSizeQuads * SubIndexY, 0, SubsectionSizeQuads);

							// Update texture data for the box that lies inside subsection
							for (int32 SubY = SubY1; SubY <= SubY2; SubY++)
							{
								for (int32 SubX = SubX1; SubX <= SubX2; SubX++)
								{
									const int32 LandscapeX = SubIndexX * SubsectionSizeQuads + ComponentIndexX * ComponentSizeQuads + SubX;
									const int32 LandscapeY = SubIndexY * SubsectionSizeQuads + ComponentIndexY * ComponentSizeQuads + SubY;
									checkSlow(LandscapeX >= X1 && LandscapeX <= X2);
									checkSlow(LandscapeY >= Y1 && LandscapeY <= Y2);

									// Find the input data corresponding to this vertex
									const int32 DataIndex = (LandscapeX - X1) + Stride * (LandscapeY - Y1);
									uint8 NewWeight = Data[DataIndex];
									if (NewWeight)
									{
										return false;
									}
								}
							}
						}
					}
					return true;
				};

				// Avoid reallocating and update material instances if we're about to write all zeros and we don't have an allocated channel for this LayerInfo
				if (IsDataAllZeroForComponent())
				{
					continue;
				}

				UpdateLayerIdx = ComponentWeightmapLayerAllocations.Num();
				new (ComponentWeightmapLayerAllocations) FWeightmapLayerAllocationInfo(LayerInfo);
				Component->ReallocateWeightmaps(/*DataInterface =*/this, GetEditLayer(), /*bInSaveToTransactionBuffer = */true, /*bool bInForceReallocate = */false, /*InTargetProxy = */nullptr, /*InRestrictSharingToComponents = */nullptr);
				Component->RequestWeightmapUpdate();
			}

			check(UpdateLayerIdx < ComponentWeightmapLayerAllocations.Num());

			// Lock data for all the weightmaps
			TexDataInfos.Reset();
			const TArray<UTexture2D*>& ComponentWeightmapTextures = Component->GetWeightmapTextures(GetEditLayer());
			TexDataInfos.AddUninitialized(ComponentWeightmapTextures.Num());

			for (int32 WeightmapIdx = 0; WeightmapIdx < ComponentWeightmapTextures.Num(); ++WeightmapIdx)
			{
				TexDataInfos[WeightmapIdx] = GetTextureDataInfo(ComponentWeightmapTextures[WeightmapIdx]);
			}

			LayerDataPtrs.Reset();        // Pointers to all layers' data
			LayerDataPtrs.AddUninitialized(ComponentWeightmapLayerAllocations.Num());
			LayerEditDataAllZero.Reset(); // Whether the data we are editing for this layer is all zero 
			LayerEditDataAllZero.AddUninitialized(ComponentWeightmapLayerAllocations.Num());

			for (int32 LayerIdx = 0; LayerIdx < ComponentWeightmapLayerAllocations.Num(); LayerIdx++)
			{
				const FWeightmapLayerAllocationInfo& Allocation = ComponentWeightmapLayerAllocations[LayerIdx];

				if (Allocation.LayerInfo != nullptr) // only take into account valid layer
				{
					LayerDataPtrs[LayerIdx] = (uint8*)TexDataInfos[Allocation.WeightmapTextureIndex]->GetMipData(/*InMipIndex = */0) + ChannelOffsets[Allocation.WeightmapTextureChannel];
					LayerEditDataAllZero[LayerIdx] = true;
				}
				else
				{
					LayerDataPtrs[LayerIdx] = nullptr;
					LayerEditDataAllZero[LayerIdx] = false;
				}
			}

			// Find the texture data corresponding to this vertex
			const int32 TexSize = (SubsectionSizeQuads+1) * ComponentNumSubsections;

			// Find coordinates of box that lies inside component
			const int32 ComponentX1 = FMath::Clamp<int32>(X1-ComponentIndexX*ComponentSizeQuads, 0, ComponentSizeQuads);
			const int32 ComponentY1 = FMath::Clamp<int32>(Y1-ComponentIndexY*ComponentSizeQuads, 0, ComponentSizeQuads);
			const int32 ComponentX2 = FMath::Clamp<int32>(X2-ComponentIndexX*ComponentSizeQuads, 0, ComponentSizeQuads);
			const int32 ComponentY2 = FMath::Clamp<int32>(Y2-ComponentIndexY*ComponentSizeQuads, 0, ComponentSizeQuads);

			// Find subsection range for this box
			const int32 SubIndexX1 = FMath::Clamp<int32>((ComponentX1-1) / SubsectionSizeQuads, 0, ComponentNumSubsections-1); // -1 because we need to pick up vertices shared between subsections
			const int32 SubIndexY1 = FMath::Clamp<int32>((ComponentY1-1) / SubsectionSizeQuads, 0, ComponentNumSubsections-1);
			const int32 SubIndexX2 = FMath::Clamp<int32>(ComponentX2 / SubsectionSizeQuads, 0, ComponentNumSubsections-1);
			const int32 SubIndexY2 = FMath::Clamp<int32>(ComponentY2 / SubsectionSizeQuads, 0, ComponentNumSubsections-1);

			for (int32 SubIndexY = SubIndexY1; SubIndexY <= SubIndexY2; SubIndexY++)
			{
				for (int32 SubIndexX = SubIndexX1; SubIndexX <= SubIndexX2; SubIndexX++)
				{
					// Find coordinates of box that lies inside subsection
					const int32 SubX1 = FMath::Clamp<int32>(ComponentX1-SubsectionSizeQuads*SubIndexX, 0, SubsectionSizeQuads);
					const int32 SubY1 = FMath::Clamp<int32>(ComponentY1-SubsectionSizeQuads*SubIndexY, 0, SubsectionSizeQuads);
					const int32 SubX2 = FMath::Clamp<int32>(ComponentX2-SubsectionSizeQuads*SubIndexX, 0, SubsectionSizeQuads);
					const int32 SubY2 = FMath::Clamp<int32>(ComponentY2-SubsectionSizeQuads*SubIndexY, 0, SubsectionSizeQuads);

					// Update texture data for the box that lies inside subsection
					for (int32 SubY = SubY1; SubY <= SubY2; SubY++)
					{
						for (int32 SubX = SubX1; SubX <= SubX2; SubX++)
						{
							const int32 LandscapeX = SubIndexX*SubsectionSizeQuads + ComponentIndexX*ComponentSizeQuads + SubX;
							const int32 LandscapeY = SubIndexY*SubsectionSizeQuads + ComponentIndexY*ComponentSizeQuads + SubY;
							checkSlow( LandscapeX >= X1 && LandscapeX <= X2 );
							checkSlow( LandscapeY >= Y1 && LandscapeY <= Y2 );

							// Find the input data corresponding to this vertex
							const int32 DataIndex = (LandscapeX-X1) + Stride * (LandscapeY-Y1);
							uint8 NewWeight = Data[DataIndex];

							const int32 TexX = (SubsectionSizeQuads+1) * SubIndexX + SubX;
							const int32 TexY = (SubsectionSizeQuads+1) * SubIndexY + SubY;
							const int32 TexDataIndex = 4 * (TexX + TexY * TexSize);

							uint8 CurrentWeight = LayerDataPtrs[UpdateLayerIdx][TexDataIndex];
							if (NewWeight == CurrentWeight)
							{
								continue;
							}

							if (PaintingRestriction == ELandscapeLayerPaintingRestriction::UseComponentAllowList && NewWeight != 0)
							{
								bool bIsAllowed = IsLayerAllowed(LayerInfo, ComponentIndexX, SubIndexX, SubX, ComponentIndexY, SubIndexY, SubY);
								if (!bIsAllowed)
								{
									NewWeight = 0;
								}
							}

							// No need to check for nullptr here because UpdateLayerIdx is always valid
							uint8& Weight = LayerDataPtrs[UpdateLayerIdx][TexDataIndex];
							Weight = NewWeight;
							if (Weight != 0)
							{
								LayerEditDataAllZero[UpdateLayerIdx] = false;
							}
						}
					}

					// Record the areas of the texture we need to re-upload
					const int32 TexX1 = (SubsectionSizeQuads+1) * SubIndexX + SubX1;
					const int32 TexY1 = (SubsectionSizeQuads+1) * SubIndexY + SubY1;
					const int32 TexX2 = (SubsectionSizeQuads+1) * SubIndexX + SubX2;
					const int32 TexY2 = (SubsectionSizeQuads+1) * SubIndexY + SubY2;
					for (int32 WeightmapIdx = 0; WeightmapIdx < ComponentWeightmapTextures.Num(); WeightmapIdx++)
					{
						if (TexDataInfos[WeightmapIdx] != nullptr)
						{
							TexDataInfos[WeightmapIdx]->AddMipUpdateRegion(0,TexX1,TexY1,TexX2,TexY2);
						}
					}
				}
			}

			// Check if we need to remove weightmap allocations for layers that were completely painted away
			bool bRemovedLayer = false;
			for (int32 LayerIdx = 0; LayerIdx < ComponentWeightmapLayerAllocations.Num(); LayerIdx++)
			{
				if (LayerEditDataAllZero[LayerIdx])
				{
					bool bLayerDeleted = Component->DeleteLayerIfAllZero(GetEditLayer(), LayerDataPtrs[LayerIdx], TexSize, LayerIdx, GetShouldDirtyPackage());

					if (bLayerDeleted)
					{
						LayerEditDataAllZero.RemoveAt(LayerIdx);
						LayerDataPtrs.RemoveAt(LayerIdx);
						LayerIdx--;

						bRemovedLayer = true;
					}
				}
			}

			if (bRemovedLayer)
			{
				Component->RequestWeightmapUpdate();
			}

			Component->GetLandscapeProxy()->ValidateProxyLayersWeightmapUsage();
		}
	}
}

// Deprecated
void FLandscapeEditDataInterface::SetAlphaData(ULandscapeLayerInfoObject* const LayerInfo, const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, const uint8* Data, int32 Stride, ELandscapeLayerPaintingRestriction PaintingRestriction /*= None*/, bool bWeightAdjust /*= true*/, bool bTotalWeightAdjust /*= false*/)
{
	SetAlphaData(LayerInfo, X1, Y1, X2, Y2, Data, Stride, PaintingRestriction);
}

void FLandscapeEditDataInterface::SetAlphaData(const TSet<ULandscapeLayerInfoObject*>& DirtyLayerInfos, const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, const uint8* Data, int32 Stride, ELandscapeLayerPaintingRestriction PaintingRestriction /*= None*/)
{
	if (DirtyLayerInfos.Num() == 0)
	{
		return;
	}

	for (ULandscapeLayerInfoObject* LayerInfo : DirtyLayerInfos)
	{
		// The Data[] array passed in is indexed by LandscapeInfo->GetLayerInfoIndex(),
		// so if we're trying to write a layer which isn't in the LandscapeInfo,
		// its data is either missing or written where another layer's data should be.
		// Either way it's *very bad*.
		check(LandscapeInfo->GetLayerInfoIndex(LayerInfo) != INDEX_NONE);
	}

	if (Stride == 0)
	{
		Stride = (1+X2-X1) * LandscapeInfo->Layers.Num();
	}

	check(ComponentSizeQuads > 0);
	// Find component range for this block of data
	int32 ComponentIndexX1 = (X1-1 >= 0) ? (X1-1) / ComponentSizeQuads : (X1) / ComponentSizeQuads - 1;	// -1 because we need to pick up vertices shared between components
	int32 ComponentIndexY1 = (Y1-1 >= 0) ? (Y1-1) / ComponentSizeQuads : (Y1) / ComponentSizeQuads - 1;
	int32 ComponentIndexX2 = (X2 >= 0) ? X2 / ComponentSizeQuads : (X2+1) / ComponentSizeQuads - 1;
	int32 ComponentIndexY2 = (Y2 >= 0) ? Y2 / ComponentSizeQuads : (Y2+1) / ComponentSizeQuads - 1;

	TArray<ULandscapeLayerInfoObject*, TInlineAllocator<8>> NeedAllocationInfos;
	TArray<FLandscapeTextureDataInfo*, TInlineAllocator<2>> TexDataInfos;

	struct FLayerDataInfo
	{
		const uint8* InDataPtr;
		uint8* TexDataPtr;
	};

	TArray<FLayerDataInfo, TInlineAllocator<8>> LayerDataInfos;		// Pointers to all layers' data 
	TArray<bool, TInlineAllocator<8>> LayerEditDataAllZero; // Whether the data we are editing for this layer is all zero 
	TArray<FColor*> CollisionWeightmapMipData;
	TArray<FColor*> SimpleCollisionWeightmapMipData;
	TArray<FColor*> WeightmapTextureMipData;

	for (int32 ComponentIndexY = ComponentIndexY1; ComponentIndexY <= ComponentIndexY2; ComponentIndexY++)
	{
		for (int32 ComponentIndexX = ComponentIndexX1; ComponentIndexX <= ComponentIndexX2; ComponentIndexX++)
		{
			FIntPoint ComponentKey(ComponentIndexX,ComponentIndexY);
			ULandscapeComponent* Component = LandscapeInfo->XYtoComponentMap.FindRef(ComponentKey);

			// if nullptr, there is no component at this location
			if (Component == nullptr)
			{
				continue;
			}

			const int32 LayerLimit = Component->GetLandscapeProxy()->MaxPaintedLayersPerComponent;

			NeedAllocationInfos.Reset();

			TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = Component->GetWeightmapLayerAllocations(GetEditLayer());

			for (ULandscapeLayerInfoObject* LayerInfo : DirtyLayerInfos)
			{
				const bool bFound = ComponentWeightmapLayerAllocations.ContainsByPredicate([LayerInfo](const FWeightmapLayerAllocationInfo& Allocation){ return Allocation.LayerInfo == LayerInfo; });
				if (!bFound)
				{
					NeedAllocationInfos.Add(LayerInfo);
				}
			}

			// Need allocation for weightmaps
			if (NeedAllocationInfos.Num() > 0)
			{
				if (NeedAllocationInfos.Num() == DirtyLayerInfos.Num())
				{
					if (PaintingRestriction == ELandscapeLayerPaintingRestriction::ExistingOnly ||
						(PaintingRestriction == ELandscapeLayerPaintingRestriction::UseMaxLayers &&
							ComponentWeightmapLayerAllocations.Num() >= LayerLimit))
					{
						// nothing to paint to this component due to layer limit
						continue;
					}
				}
				if (PaintingRestriction != ELandscapeLayerPaintingRestriction::ExistingOnly)
				{
					Component->Modify(GetShouldDirtyPackage());
					for (ULandscapeLayerInfoObject* LayerInfoNeedingAllocation : NeedAllocationInfos)
					{
						if (PaintingRestriction == ELandscapeLayerPaintingRestriction::UseMaxLayers &&
							LayerLimit > 0 && ComponentWeightmapLayerAllocations.Num() >= LayerLimit)
						{
							break;
						}
						ComponentWeightmapLayerAllocations.Emplace(LayerInfoNeedingAllocation);
					}
					Component->ReallocateWeightmaps(/*DataInterface =*/this, GetEditLayer(), /*bInSaveToTransactionBuffer = */true, /*bool bInForceReallocate = */false, /*InTargetProxy = */nullptr, /*InRestrictSharingToComponents = */nullptr);
					Component->RequestWeightmapUpdate();
				}
			}

			// Lock data for all the weightmaps
			TexDataInfos.Reset();
			const TArray<UTexture2D*>& ComponentWeightmapTextures = Component->GetWeightmapTextures(GetEditLayer());
			TexDataInfos.AddUninitialized(ComponentWeightmapTextures.Num());

			for (int32 WeightmapIdx = 0; WeightmapIdx < ComponentWeightmapTextures.Num(); ++WeightmapIdx)
			{
				TexDataInfos[WeightmapIdx] = GetTextureDataInfo(ComponentWeightmapTextures[WeightmapIdx]);
			}

			LayerDataInfos.Reset();        // Pointers to all layers' data
			LayerDataInfos.AddUninitialized(ComponentWeightmapLayerAllocations.Num());
			LayerEditDataAllZero.Reset(); // Whether the data we are editing for this layer is all zero 
			LayerEditDataAllZero.AddUninitialized(ComponentWeightmapLayerAllocations.Num());

			for (int32 LayerIdx = 0; LayerIdx < ComponentWeightmapLayerAllocations.Num(); LayerIdx++)
			{
				const FWeightmapLayerAllocationInfo& Allocation = ComponentWeightmapLayerAllocations[LayerIdx];
				const int32 LayerDataIdx = LandscapeInfo->GetLayerInfoIndex(ComponentWeightmapLayerAllocations[LayerIdx].LayerInfo);
				check(LayerDataIdx != INDEX_NONE);
				LayerDataInfos[LayerIdx].InDataPtr = Data + LayerDataIdx;
				LayerDataInfos[LayerIdx].TexDataPtr = (uint8*)TexDataInfos[Allocation.WeightmapTextureIndex]->GetMipData(/*InMipIndex = */0) + ChannelOffsets[Allocation.WeightmapTextureChannel];
				LayerEditDataAllZero[LayerIdx] = true;
			}

			// Find the texture data corresponding to this vertex
			const int32 TexSize = (Component->SubsectionSizeQuads+1) * Component->NumSubsections; 

			// Find coordinates of box that lies inside component
			const int32 ComponentX1 = FMath::Clamp<int32>(X1-ComponentIndexX*ComponentSizeQuads, 0, ComponentSizeQuads);
			const int32 ComponentY1 = FMath::Clamp<int32>(Y1-ComponentIndexY*ComponentSizeQuads, 0, ComponentSizeQuads);
			const int32 ComponentX2 = FMath::Clamp<int32>(X2-ComponentIndexX*ComponentSizeQuads, 0, ComponentSizeQuads);
			const int32 ComponentY2 = FMath::Clamp<int32>(Y2-ComponentIndexY*ComponentSizeQuads, 0, ComponentSizeQuads);

			// Find subsection range for this box
			const int32 SubIndexX1 = FMath::Clamp<int32>((ComponentX1-1) / SubsectionSizeQuads,0,ComponentNumSubsections-1);	// -1 because we need to pick up vertices shared between subsections
			const int32 SubIndexY1 = FMath::Clamp<int32>((ComponentY1-1) / SubsectionSizeQuads,0,ComponentNumSubsections-1);
			const int32 SubIndexX2 = FMath::Clamp<int32>(ComponentX2 / SubsectionSizeQuads,0,ComponentNumSubsections-1);
			const int32 SubIndexY2 = FMath::Clamp<int32>(ComponentY2 / SubsectionSizeQuads,0,ComponentNumSubsections-1);

			for (int32 SubIndexY = SubIndexY1; SubIndexY <= SubIndexY2; SubIndexY++)
			{
				for (int32 SubIndexX = SubIndexX1; SubIndexX <= SubIndexX2; SubIndexX++)
				{
					// Find coordinates of box that lies inside subsection
					const int32 SubX1 = FMath::Clamp<int32>(ComponentX1-SubsectionSizeQuads*SubIndexX, 0, SubsectionSizeQuads);
					const int32 SubY1 = FMath::Clamp<int32>(ComponentY1-SubsectionSizeQuads*SubIndexY, 0, SubsectionSizeQuads);
					const int32 SubX2 = FMath::Clamp<int32>(ComponentX2-SubsectionSizeQuads*SubIndexX, 0, SubsectionSizeQuads);
					const int32 SubY2 = FMath::Clamp<int32>(ComponentY2-SubsectionSizeQuads*SubIndexY, 0, SubsectionSizeQuads);

					// Update texture data for the box that lies inside subsection
					for (int32 SubY = SubY1; SubY <= SubY2; SubY++)
					{
						for (int32 SubX = SubX1; SubX <= SubX2; SubX++)
						{
							const int32 LandscapeX = SubIndexX*SubsectionSizeQuads + ComponentIndexX*ComponentSizeQuads + SubX;
							const int32 LandscapeY = SubIndexY*SubsectionSizeQuads + ComponentIndexY*ComponentSizeQuads + SubY;
							checkSlow( LandscapeX >= X1 && LandscapeX <= X2 );
							checkSlow( LandscapeY >= Y1 && LandscapeY <= Y2 );

							// Find the input data corresponding to this vertex
							const int32 DataIndex = (LandscapeY-Y1) * Stride + (LandscapeX-X1) * LandscapeInfo->Layers.Num();

							// Adjust all layer weights
							const int32 TexX = (SubsectionSizeQuads+1) * SubIndexX + SubX;
							const int32 TexY = (SubsectionSizeQuads+1) * SubIndexY + SubY;

							const int32 TexDataIndex = 4 * (TexX + TexY * TexSize);

							// Apply weights to all layers simultaneously.
							for (int32 LayerIdx = 0; LayerIdx < ComponentWeightmapLayerAllocations.Num(); LayerIdx++)
							{
								// this is equivalent to saying if (DirtyLayerInfos.Contains(Allocation.LayerInfo))
								// which is what we really mean here, but this is quicker
								// and I've lost count of the depth we've nested for loops at this point
								if (LayerDataInfos[LayerIdx].TexDataPtr != nullptr)
								{
									uint8& Weight = LayerDataInfos[LayerIdx].TexDataPtr[TexDataIndex];

									Weight = LayerDataInfos[LayerIdx].InDataPtr[DataIndex]; // Only for whole weight
									if (Weight != 0)
									{
										LayerEditDataAllZero[LayerIdx] = false;
									}
								}
							}
						}
					}

					// Record the areas of the texture we need to re-upload
					const int32 TexX1 = (SubsectionSizeQuads+1) * SubIndexX + SubX1;
					const int32 TexY1 = (SubsectionSizeQuads+1) * SubIndexY + SubY1;
					const int32 TexX2 = (SubsectionSizeQuads+1) * SubIndexX + SubX2;
					const int32 TexY2 = (SubsectionSizeQuads+1) * SubIndexY + SubY2;
					for (int32 WeightmapIdx = 0; WeightmapIdx < ComponentWeightmapTextures.Num(); WeightmapIdx++)
					{
						if (TexDataInfos[WeightmapIdx] != nullptr)
						{
							TexDataInfos[WeightmapIdx]->AddMipUpdateRegion(0,TexX1,TexY1,TexX2,TexY2);
						}
					}
				}
			}

			// Check if we need to remove weightmap allocations for layers that were completely painted away
			bool bRemovedLayer = false;
			for (int32 LayerIdx = 0; LayerIdx < ComponentWeightmapLayerAllocations.Num(); LayerIdx++)
			{
				if (LayerEditDataAllZero[LayerIdx])
				{
					bool bLayerDeleted = Component->DeleteLayerIfAllZero(GetEditLayer(), LayerDataInfos[LayerIdx].TexDataPtr, TexSize, LayerIdx, GetShouldDirtyPackage());

					if (bLayerDeleted)
					{
						LayerEditDataAllZero.RemoveAt(LayerIdx);
						LayerDataInfos.RemoveAt(LayerIdx);
						LayerIdx--;

						bRemovedLayer = true;
					}
				}
			}

			if (bRemovedLayer)
			{
				Component->RequestWeightmapUpdate();
			}

			Component->GetLandscapeProxy()->ValidateProxyLayersWeightmapUsage();
		}
	}
}

template<typename TStoreData>
void FLandscapeEditDataInterface::GetWeightDataTemplFast(ULandscapeLayerInfoObject* LayerInfo, const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TStoreData& StoreData)
{
	int32 ComponentIndexX1, ComponentIndexY1, ComponentIndexX2, ComponentIndexY2;
	ALandscape::CalcComponentIndicesNoOverlap(X1, Y1, X2, Y2, ComponentSizeQuads, ComponentIndexX1, ComponentIndexY1, ComponentIndexX2, ComponentIndexY2);

	for( int32 ComponentIndexY=ComponentIndexY1;ComponentIndexY<=ComponentIndexY2;ComponentIndexY++ )
	{
		for( int32 ComponentIndexX=ComponentIndexX1;ComponentIndexX<=ComponentIndexX2;ComponentIndexX++ )
		{		
			ULandscapeComponent* Component = LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(ComponentIndexX,ComponentIndexY));

			if( !Component )
			{
				continue;
			}

			UTexture2D* WeightmapTexture = nullptr;
			FLandscapeTextureDataInfo* TexDataInfo = nullptr;
			uint8* WeightmapTextureData = nullptr;
			uint8 WeightmapChannelOffset = 0;
			TArray<FLandscapeTextureDataInfo*> TexDataInfos; // added for whole weight case...

			const TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = Component->GetWeightmapLayerAllocations(GetEditLayer());
			const TArray<UTexture2D*>& ComponentWeightmapTextures = Component->GetWeightmapTextures(GetEditLayer());

			if (LayerInfo != nullptr)
			{
				for( int32 LayerIdx=0;LayerIdx<ComponentWeightmapLayerAllocations.Num();LayerIdx++ )
				{
					if(ComponentWeightmapLayerAllocations[LayerIdx].LayerInfo == LayerInfo )
					{
						WeightmapTexture = ComponentWeightmapTextures[ComponentWeightmapLayerAllocations[LayerIdx].WeightmapTextureIndex];
						TexDataInfo = GetTextureDataInfo(WeightmapTexture);
						WeightmapTextureData = (uint8*)TexDataInfo->GetMipData(/*InMipIndex = */0);
						WeightmapChannelOffset = ChannelOffsets[ComponentWeightmapLayerAllocations[LayerIdx].WeightmapTextureChannel];
						break;
					}
				}
			}
			else
			{
				// Lock data for all the weightmaps
				for( int32 WeightmapIdx=0;WeightmapIdx < ComponentWeightmapTextures.Num();WeightmapIdx++ )
				{
					TexDataInfos.Add(GetTextureDataInfo(ComponentWeightmapTextures[WeightmapIdx]));
				}
			}

			// Find coordinates of box that lies inside component
			int32 ComponentX1 = FMath::Clamp<int32>(X1-ComponentIndexX*ComponentSizeQuads, 0, ComponentSizeQuads);
			int32 ComponentY1 = FMath::Clamp<int32>(Y1-ComponentIndexY*ComponentSizeQuads, 0, ComponentSizeQuads);
			int32 ComponentX2 = FMath::Clamp<int32>(X2-ComponentIndexX*ComponentSizeQuads, 0, ComponentSizeQuads);
			int32 ComponentY2 = FMath::Clamp<int32>(Y2-ComponentIndexY*ComponentSizeQuads, 0, ComponentSizeQuads);

			// Find subsection range for this box
			int32 SubIndexX1 = FMath::Clamp<int32>((ComponentX1-1) / SubsectionSizeQuads,0,ComponentNumSubsections-1);	// -1 because we need to pick up vertices shared between subsections
			int32 SubIndexY1 = FMath::Clamp<int32>((ComponentY1-1) / SubsectionSizeQuads,0,ComponentNumSubsections-1);
			int32 SubIndexX2 = FMath::Clamp<int32>(ComponentX2 / SubsectionSizeQuads,0,ComponentNumSubsections-1);
			int32 SubIndexY2 = FMath::Clamp<int32>(ComponentY2 / SubsectionSizeQuads,0,ComponentNumSubsections-1);

			for( int32 SubIndexY=SubIndexY1;SubIndexY<=SubIndexY2;SubIndexY++ )
			{
				for( int32 SubIndexX=SubIndexX1;SubIndexX<=SubIndexX2;SubIndexX++ )
				{
					// Find coordinates of box that lies inside subsection
					int32 SubX1 = FMath::Clamp<int32>(ComponentX1-SubsectionSizeQuads*SubIndexX, 0, SubsectionSizeQuads);
					int32 SubY1 = FMath::Clamp<int32>(ComponentY1-SubsectionSizeQuads*SubIndexY, 0, SubsectionSizeQuads);
					int32 SubX2 = FMath::Clamp<int32>(ComponentX2-SubsectionSizeQuads*SubIndexX, 0, SubsectionSizeQuads);
					int32 SubY2 = FMath::Clamp<int32>(ComponentY2-SubsectionSizeQuads*SubIndexY, 0, SubsectionSizeQuads);

					// Update texture data for the box that lies inside subsection
					for( int32 SubY=SubY1;SubY<=SubY2;SubY++ )
					{
						for( int32 SubX=SubX1;SubX<=SubX2;SubX++ )
						{
							int32 LandscapeX = SubIndexX*SubsectionSizeQuads + ComponentIndexX*ComponentSizeQuads + SubX;
							int32 LandscapeY = SubIndexY*SubsectionSizeQuads + ComponentIndexY*ComponentSizeQuads + SubY;

							if (LayerInfo != nullptr)
							{
								// Find the input data corresponding to this vertex
								uint8 Weight;
								if( WeightmapTexture )
								{
									// Find the texture data corresponding to this vertex
									int32 SizeU = WeightmapTexture->Source.GetSizeX();
									int32 SizeV = WeightmapTexture->Source.GetSizeY();
									int32 WeightmapOffsetX = static_cast<int32>(Component->WeightmapScaleBias.Z * SizeU);
									int32 WeightmapOffsetY = static_cast<int32>(Component->WeightmapScaleBias.W * SizeV);

									int32 TexX = WeightmapOffsetX + (SubsectionSizeQuads+1) * SubIndexX + SubX;
									int32 TexY = WeightmapOffsetY + (SubsectionSizeQuads+1) * SubIndexY + SubY;
									Weight = WeightmapTextureData[ 4 * (TexX + TexY * SizeU) + WeightmapChannelOffset ];
								}
								else
								{
									Weight = 0;
								}

								StoreData.Store(LandscapeX, LandscapeY, Weight);
							}
							else // Whole weight map case...
							{
								StoreData.PreInit(LandscapeInfo->Layers.Num());

								for( int32 LayerIdx=0;LayerIdx<ComponentWeightmapLayerAllocations.Num();LayerIdx++ )
								{
									int32 Idx = ComponentWeightmapLayerAllocations[LayerIdx].WeightmapTextureIndex;
									UTexture2D* ComponentWeightmapTexture = ComponentWeightmapTextures[Idx];
									uint8* ComponentWeightmapTextureData = (uint8*)TexDataInfos[Idx]->GetMipData(/*InMipIndex = */0);
									uint8 ComponentWeightmapChannelOffset = ChannelOffsets[ComponentWeightmapLayerAllocations[LayerIdx].WeightmapTextureChannel];

									// Find the texture data corresponding to this vertex
									int32 SizeU = ComponentWeightmapTexture->Source.GetSizeX();
									int32 SizeV = ComponentWeightmapTexture->Source.GetSizeY();
									int32 WeightmapOffsetX = static_cast<int32>(Component->WeightmapScaleBias.Z * SizeU);
									int32 WeightmapOffsetY = static_cast<int32>(Component->WeightmapScaleBias.W * SizeV);

									int32 TexX = WeightmapOffsetX + (SubsectionSizeQuads+1) * SubIndexX + SubX;
									int32 TexY = WeightmapOffsetY + (SubsectionSizeQuads+1) * SubIndexY + SubY;

									uint8 Weight = ComponentWeightmapTextureData[ 4 * (TexX + TexY * SizeU) + ComponentWeightmapChannelOffset ];

									// Find index in LayerInfos
									{
										int32 LayerInfoIdx = LandscapeInfo->GetLayerInfoIndex(ComponentWeightmapLayerAllocations[LayerIdx].LayerInfo);
										if (LayerInfoIdx != INDEX_NONE)
										{
											StoreData.Store(LandscapeX, LandscapeY, Weight, LayerInfoIdx);
										}
									}
								}
							}
						}
					}
				}
			}
		}
	}
}

uint8 FLandscapeEditDataInterface::GetWeightMapData(const ULandscapeComponent* Component, ULandscapeLayerInfoObject* LayerInfo, int32 TexU, int32 TexV, uint8 Offset /*= 0*/, UTexture2D* Texture /*= nullptr*/, uint8* TextureData /*= nullptr*/)
{
	check(Component);
	if (!Texture || !TextureData)
	{
		if (LayerInfo != nullptr)
		{
			const TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = Component->GetWeightmapLayerAllocations(GetEditLayer());
			const TArray<UTexture2D*>& ComponentWeightmapTextures = Component->GetWeightmapTextures(GetEditLayer());

			for( int32 LayerIdx=0;LayerIdx<ComponentWeightmapLayerAllocations.Num();LayerIdx++ )
			{
				if(ComponentWeightmapLayerAllocations[LayerIdx].LayerInfo == LayerInfo )
				{
					Texture = ComponentWeightmapTextures[ComponentWeightmapLayerAllocations[LayerIdx].WeightmapTextureIndex];
					FLandscapeTextureDataInfo* TexDataInfo = GetTextureDataInfo(Texture);
					TextureData = (uint8*)TexDataInfo->GetMipData(/*InMipIndex = */0);
					Offset = ChannelOffsets[ComponentWeightmapLayerAllocations[LayerIdx].WeightmapTextureChannel];
					break;
				}
			}
		}
	}

	if (Texture && TextureData)
	{
		const int32 SizeU = Texture->Source.GetSizeX();
		const int32 SizeV = Texture->Source.GetSizeY();
		const int32 WeightmapOffsetX = static_cast<int32>(Component->WeightmapScaleBias.Z * SizeU);
		const int32 WeightmapOffsetY = static_cast<int32>(Component->WeightmapScaleBias.W * SizeV);

		const int32 TexX = WeightmapOffsetX + TexU;
		const int32 TexY = WeightmapOffsetY + TexV;
		return TextureData[ 4 * (TexX + TexY * SizeU) + Offset ];
	}
	return 0;
}

template<typename TStoreData>
void FLandscapeEditDataInterface::GetWeightDataTempl(ULandscapeLayerInfoObject* LayerInfo, int32& ValidX1, int32& ValidY1, int32& ValidX2, int32& ValidY2, TStoreData& StoreData)
{
	// Copy variables
	int32 X1 = ValidX1, X2 = ValidX2, Y1 = ValidY1, Y2 = ValidY2;
	ValidX1 = INT_MAX; ValidX2 = INT_MIN; ValidY1 = INT_MAX; ValidY2 = INT_MIN;

	int32 ComponentIndexX1, ComponentIndexY1, ComponentIndexX2, ComponentIndexY2;
	ALandscape::CalcComponentIndicesNoOverlap(X1, Y1, X2, Y2, ComponentSizeQuads, ComponentIndexX1, ComponentIndexY1, ComponentIndexX2, ComponentIndexY2);

	int32 ComponentSizeX = ComponentIndexX2-ComponentIndexX1+1;
	int32 ComponentSizeY = ComponentIndexY2-ComponentIndexY1+1;

	// Neighbor Components
	ULandscapeComponent* BorderComponent[4] = {0, 0, 0, 0};
	ULandscapeComponent* CornerComponent[4] = {0, 0, 0, 0};
	bool NoBorderX1 = false, NoBorderX2 = false;
	TArray<bool> NoBorderY1, NoBorderY2, ComponentDataExist;
	TArray<ULandscapeComponent*> BorderComponentY1, BorderComponentY2;
	ComponentDataExist.Empty(ComponentSizeX*ComponentSizeY);
	ComponentDataExist.AddZeroed(ComponentSizeX*ComponentSizeY);
	bool bHasMissingValue = false;

	UTexture2D* NeighborWeightmapTexture[4] = {0, 0, 0, 0};
	FLandscapeTextureDataInfo* NeighborTexDataInfo[4] = {0, 0, 0, 0};
	uint8* NeighborWeightmapTextureData[4] = {0, 0, 0, 0};
	uint8 NeighborWeightmapChannelOffset[4] = {0, 0, 0, 0};
	uint8 CornerValues[4] = {0, 0, 0, 0};
	int32 EdgeCoord = (SubsectionSizeQuads+1) * ComponentNumSubsections - 1; //ComponentSizeQuads;

	// initial loop....
	for( int32 ComponentIndexY=ComponentIndexY1;ComponentIndexY<=ComponentIndexY2;ComponentIndexY++ )
	{
		NoBorderX1 = false;
		NoBorderX2 = false;
		BorderComponent[0] = BorderComponent[1] = nullptr;
		for( int32 ComponentIndexX=ComponentIndexX1;ComponentIndexX<=ComponentIndexX2;ComponentIndexX++ )
		{	
			BorderComponent[2] = BorderComponent[3] = nullptr;
			int32 ComponentIndexXY = ComponentSizeX*(ComponentIndexY-ComponentIndexY1) + ComponentIndexX-ComponentIndexX1;
			int32 ComponentIndexXX = ComponentIndexX - ComponentIndexX1;
			int32 ComponentIndexYY = ComponentIndexY - ComponentIndexY1;
			ComponentDataExist[ComponentIndexXY] = false;
			ULandscapeComponent* Component = LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(ComponentIndexX,ComponentIndexY));

			UTexture2D* WeightmapTexture = nullptr;
			FLandscapeTextureDataInfo* TexDataInfo = nullptr;
			uint8* WeightmapTextureData = nullptr;
			uint8 WeightmapChannelOffset = 0;
			TArray<FLandscapeTextureDataInfo*> TexDataInfos; // added for whole weight case...
			uint8 CornerSet = 0;
			bool ExistLeft = ComponentIndexXX > 0 && ComponentDataExist[ ComponentIndexXX-1 + ComponentIndexYY * ComponentSizeX ];
			bool ExistUp = ComponentIndexYY > 0 && ComponentDataExist[ ComponentIndexXX + (ComponentIndexYY-1) * ComponentSizeX ];
			bool IsBorderPosition = false;
		
			if( Component )
			{
				const TArray<UTexture2D*>& ComponentWeightmapTextures = Component->GetWeightmapTextures(GetEditLayer());

				if (LayerInfo != nullptr)
				{
					const TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = Component->GetWeightmapLayerAllocations(GetEditLayer());

					for( int32 LayerIdx=0;LayerIdx<ComponentWeightmapLayerAllocations.Num();LayerIdx++ )
					{
						if(ComponentWeightmapLayerAllocations[LayerIdx].LayerInfo == LayerInfo )
						{
							WeightmapTexture = ComponentWeightmapTextures[ComponentWeightmapLayerAllocations[LayerIdx].WeightmapTextureIndex];
							TexDataInfo = GetTextureDataInfo(WeightmapTexture);
							WeightmapTextureData = (uint8*)TexDataInfo->GetMipData(/*InMipIndex = */0);
							WeightmapChannelOffset = ChannelOffsets[ComponentWeightmapLayerAllocations[LayerIdx].WeightmapTextureChannel];
							break;
						}
					}
				}
				else
				{
					// Lock data for all the weightmaps
					for( int32 WeightmapIdx=0;WeightmapIdx < ComponentWeightmapTextures.Num();WeightmapIdx++ )
					{
						TexDataInfos.Add(GetTextureDataInfo(ComponentWeightmapTextures[WeightmapIdx]));
					}
				}
				ComponentDataExist[ComponentIndexXY] = true;
				// Update valid region
				ValidX1 = FMath::Min<int32>(Component->GetSectionBase().X, ValidX1);
				ValidX2 = FMath::Max<int32>(Component->GetSectionBase().X+ComponentSizeQuads, ValidX2);
				ValidY1 = FMath::Min<int32>(Component->GetSectionBase().Y, ValidY1);
				ValidY2 = FMath::Max<int32>(Component->GetSectionBase().Y+ComponentSizeQuads, ValidY2);
			}
			else
			{
				if (!bHasMissingValue)
				{
					NoBorderY1.Empty(ComponentSizeX);
					NoBorderY2.Empty(ComponentSizeX);
					NoBorderY1.AddZeroed(ComponentSizeX);
					NoBorderY2.AddZeroed(ComponentSizeX);
					BorderComponentY1.Empty(ComponentSizeX);
					BorderComponentY2.Empty(ComponentSizeX);
					BorderComponentY1.AddZeroed(ComponentSizeX);
					BorderComponentY2.AddZeroed(ComponentSizeX);
					bHasMissingValue = true;
				}

				if (LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(ComponentIndexX - 1, ComponentIndexY)) == nullptr
					|| LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(ComponentIndexX, ComponentIndexY - 1)) == nullptr
					|| LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(ComponentIndexX + 1, ComponentIndexY)) == nullptr
					|| LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(ComponentIndexX, ComponentIndexY + 1)) == nullptr)
				{
					IsBorderPosition = true;
				}

				// Search for neighbor component for interpolation
				bool bShouldSearchX = (BorderComponent[1] && BorderComponent[1]->GetSectionBase().X / ComponentSizeQuads <= ComponentIndexX);
				bool bShouldSearchY = (BorderComponentY2[ComponentIndexXX] && BorderComponentY2[ComponentIndexXX]->GetSectionBase().Y / ComponentSizeQuads <= ComponentIndexY);
				// Search for left-closest component
				if ( bShouldSearchX || (!NoBorderX1 && !BorderComponent[0]))
				{
					NoBorderX1 = true;
					for (int32 X = ComponentIndexX-1; X >= ComponentIndexX1; X--)
					{
						BorderComponent[0] = LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(X,ComponentIndexY));
						if (BorderComponent[0])
						{
							NoBorderX1 = false;
							if (LayerInfo != nullptr)
							{
								const TArray<FWeightmapLayerAllocationInfo>& BorderWeightmapLayerAllocations = BorderComponent[0]->GetWeightmapLayerAllocations(GetEditLayer());
								const TArray<UTexture2D*>& BorderWeightmapTextures = BorderComponent[0]->GetWeightmapTextures(GetEditLayer());

								for( int32 LayerIdx=0;LayerIdx<BorderWeightmapLayerAllocations.Num();LayerIdx++ )
								{
									if(BorderWeightmapLayerAllocations[LayerIdx].LayerInfo == LayerInfo )
									{
										NeighborWeightmapTexture[0] = BorderWeightmapTextures[BorderWeightmapLayerAllocations[LayerIdx].WeightmapTextureIndex];
										NeighborTexDataInfo[0] = GetTextureDataInfo(NeighborWeightmapTexture[0]);
										NeighborWeightmapTextureData[0] = (uint8*)NeighborTexDataInfo[0]->GetMipData(/*InMipIndex = */0);
										NeighborWeightmapChannelOffset[0] = ChannelOffsets[BorderWeightmapLayerAllocations[LayerIdx].WeightmapTextureChannel];
										break;
									}
								}
							}
							break;
						}
					}
				}
				// Search for right-closest component
				if ( bShouldSearchX || (!NoBorderX2 && !BorderComponent[1]))
				{
					NoBorderX2 = true;
					for (int32 X = ComponentIndexX+1; X <= ComponentIndexX2; X++)
					{
						BorderComponent[1] = LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(X,ComponentIndexY));
						if (BorderComponent[1])
						{
							NoBorderX2 = false;
							if (LayerInfo != nullptr)
							{
								const TArray<FWeightmapLayerAllocationInfo>& BorderWeightmapLayerAllocations = BorderComponent[1]->GetWeightmapLayerAllocations(GetEditLayer());
								const TArray<UTexture2D*>& BorderWeightmapTextures = BorderComponent[1]->GetWeightmapTextures(GetEditLayer());

								for( int32 LayerIdx=0;LayerIdx<BorderWeightmapLayerAllocations.Num();LayerIdx++ )
								{
									if(BorderWeightmapLayerAllocations[LayerIdx].LayerInfo == LayerInfo )
									{
										NeighborWeightmapTexture[1] = BorderWeightmapTextures[BorderWeightmapLayerAllocations[LayerIdx].WeightmapTextureIndex];
										NeighborTexDataInfo[1] = GetTextureDataInfo(NeighborWeightmapTexture[1]);
										NeighborWeightmapTextureData[1] = (uint8*)NeighborTexDataInfo[1]->GetMipData(/*InMipIndex = */0);
										NeighborWeightmapChannelOffset[1] = ChannelOffsets[BorderWeightmapLayerAllocations[LayerIdx].WeightmapTextureChannel];
										break;
									}
								}
							}
						}
					}
				}
				// Search for up-closest component
				if ( bShouldSearchY || (!NoBorderY1[ComponentIndexXX] && !BorderComponentY1[ComponentIndexXX]))
				{
					NoBorderY1[ComponentIndexXX] = true;
					for (int32 Y = ComponentIndexY-1; Y >= ComponentIndexY1; Y--)
					{
						BorderComponentY1[ComponentIndexXX] = BorderComponent[2] = LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(ComponentIndexX,Y));
						if (BorderComponent[2])
						{
							NoBorderY1[ComponentIndexXX] = false;
							if (LayerInfo != nullptr)
							{
								const TArray<FWeightmapLayerAllocationInfo>& BorderWeightmapLayerAllocations = BorderComponent[2]->GetWeightmapLayerAllocations(GetEditLayer());
								const TArray<UTexture2D*>& BorderWeightmapTextures = BorderComponent[2]->GetWeightmapTextures(GetEditLayer());

								for( int32 LayerIdx=0;LayerIdx<BorderWeightmapLayerAllocations.Num();LayerIdx++ )
								{
									if(BorderWeightmapLayerAllocations[LayerIdx].LayerInfo == LayerInfo )
									{
										NeighborWeightmapTexture[2] = BorderWeightmapTextures[BorderWeightmapLayerAllocations[LayerIdx].WeightmapTextureIndex];
										NeighborTexDataInfo[2] = GetTextureDataInfo(NeighborWeightmapTexture[2]);
										NeighborWeightmapTextureData[2] = (uint8*)NeighborTexDataInfo[2]->GetMipData(/*InMipIndex = */0);
										NeighborWeightmapChannelOffset[2] = ChannelOffsets[BorderWeightmapLayerAllocations[LayerIdx].WeightmapTextureChannel];
										break;
									}
								}
							}
						}
					}
				}
				else
				{
					BorderComponent[2] = BorderComponentY1[ComponentIndexXX];
					if (BorderComponent[2])
					{
						if (LayerInfo != nullptr)
						{
							const TArray<FWeightmapLayerAllocationInfo>& BorderWeightmapLayerAllocations = BorderComponent[2]->GetWeightmapLayerAllocations(GetEditLayer());
							const TArray<UTexture2D*>& BorderWeightmapTextures = BorderComponent[2]->GetWeightmapTextures(GetEditLayer());

							for( int32 LayerIdx=0;LayerIdx<BorderWeightmapLayerAllocations.Num();LayerIdx++ )
							{
								if(BorderWeightmapLayerAllocations[LayerIdx].LayerInfo == LayerInfo )
								{
									NeighborWeightmapTexture[2] = BorderWeightmapTextures[BorderWeightmapLayerAllocations[LayerIdx].WeightmapTextureIndex];
									NeighborTexDataInfo[2] = GetTextureDataInfo(NeighborWeightmapTexture[2]);
									NeighborWeightmapTextureData[2] = (uint8*)NeighborTexDataInfo[2]->GetMipData(/*InMipIndex = */0);
									NeighborWeightmapChannelOffset[2] = ChannelOffsets[BorderWeightmapLayerAllocations[LayerIdx].WeightmapTextureChannel];
									break;
								}
							}
						}
					}
				}
				// Search for bottom-closest component
				if ( bShouldSearchY || (!NoBorderY2[ComponentIndexXX] && !BorderComponentY2[ComponentIndexXX]))
				{
					NoBorderY2[ComponentIndexXX] = true;
					for (int32 Y = ComponentIndexY+1; Y <= ComponentIndexY2; Y++)
					{
						BorderComponentY2[ComponentIndexXX] = BorderComponent[3] = LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(ComponentIndexX,Y));
						if (BorderComponent[3])
						{
							NoBorderY2[ComponentIndexXX] = false;
							if (LayerInfo != nullptr)
							{
								const TArray<FWeightmapLayerAllocationInfo>& BorderWeightmapLayerAllocations = BorderComponent[3]->GetWeightmapLayerAllocations(GetEditLayer());
								const TArray<UTexture2D*>& BorderWeightmapTextures = BorderComponent[3]->GetWeightmapTextures(GetEditLayer());

								for( int32 LayerIdx=0;LayerIdx<BorderWeightmapLayerAllocations.Num();LayerIdx++ )
								{
									if(BorderWeightmapLayerAllocations[LayerIdx].LayerInfo == LayerInfo )
									{
										NeighborWeightmapTexture[3] = BorderWeightmapTextures[BorderWeightmapLayerAllocations[LayerIdx].WeightmapTextureIndex];
										NeighborTexDataInfo[3] = GetTextureDataInfo(NeighborWeightmapTexture[3]);
										NeighborWeightmapTextureData[3] = (uint8*)NeighborTexDataInfo[3]->GetMipData(/*InMipIndex = */0);
										NeighborWeightmapChannelOffset[3] = ChannelOffsets[BorderWeightmapLayerAllocations[LayerIdx].WeightmapTextureChannel];
										break;
									}
								}
							}
							break;
						}
					}
				}
				else
				{
					BorderComponent[3] = BorderComponentY2[ComponentIndexXX];
					if (BorderComponent[3])
					{
						if (LayerInfo != nullptr)
						{
							const TArray<FWeightmapLayerAllocationInfo>& BorderWeightmapLayerAllocations = BorderComponent[3]->GetWeightmapLayerAllocations(GetEditLayer());
							const TArray<UTexture2D*>& BorderWeightmapTextures = BorderComponent[3]->GetWeightmapTextures(GetEditLayer());

							for( int32 LayerIdx=0;LayerIdx<BorderWeightmapLayerAllocations.Num();LayerIdx++ )
							{
								if(BorderWeightmapLayerAllocations[LayerIdx].LayerInfo == LayerInfo )
								{
									NeighborWeightmapTexture[3] = BorderWeightmapTextures[BorderWeightmapLayerAllocations[LayerIdx].WeightmapTextureIndex];
									NeighborTexDataInfo[3] = GetTextureDataInfo(NeighborWeightmapTexture[3]);
									NeighborWeightmapTextureData[3] = (uint8*)NeighborTexDataInfo[3]->GetMipData(/*InMipIndex = */0);
									NeighborWeightmapChannelOffset[3] = ChannelOffsets[BorderWeightmapLayerAllocations[LayerIdx].WeightmapTextureChannel];
									break;
								}
							}
						}
					}
				}

				CornerComponent[0] = ComponentIndexX >= ComponentIndexX1 && ComponentIndexY >= ComponentIndexY1 ?
					LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint((ComponentIndexX-1),(ComponentIndexY-1))) : nullptr;
				CornerComponent[1] = ComponentIndexX <= ComponentIndexX2 && ComponentIndexY >= ComponentIndexY1 ?
					LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint((ComponentIndexX+1),(ComponentIndexY-1))) : nullptr;
				CornerComponent[2] = ComponentIndexX >= ComponentIndexX1 && ComponentIndexY <= ComponentIndexY2 ?
					LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint((ComponentIndexX-1),(ComponentIndexY+1))) : nullptr;
				CornerComponent[3] = ComponentIndexX <= ComponentIndexX2 && ComponentIndexY <= ComponentIndexY2 ?
					LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint((ComponentIndexX+1),(ComponentIndexY+1))) : nullptr;

				if (CornerComponent[0])
				{
					CornerSet |= 1;
					CornerValues[0] = GetWeightMapData(CornerComponent[0], LayerInfo, EdgeCoord, EdgeCoord);
				}
				else if ((ExistLeft || ExistUp) && X1 <= ComponentIndexX*ComponentSizeQuads && Y1 <= ComponentIndexY*ComponentSizeQuads  )
				{
					CornerSet |= 1;
					CornerValues[0] = StoreData.Load( ComponentIndexX*ComponentSizeQuads, ComponentIndexY*ComponentSizeQuads);
				}
				else if (BorderComponent[0])
				{
					CornerSet |= 1;
					CornerValues[0] = GetWeightMapData(BorderComponent[0], LayerInfo, EdgeCoord, 0, NeighborWeightmapChannelOffset[0], NeighborWeightmapTexture[0], NeighborWeightmapTextureData[0]);
				}
				else if (BorderComponent[2])
				{
					CornerSet |= 1;
					CornerValues[0] = GetWeightMapData(BorderComponent[2], LayerInfo, 0, EdgeCoord, NeighborWeightmapChannelOffset[2], NeighborWeightmapTexture[2], NeighborWeightmapTextureData[2]);
				}

				if (CornerComponent[1])
				{
					CornerSet |= 1 << 1;
					CornerValues[1] = GetWeightMapData(CornerComponent[1], LayerInfo, 0, EdgeCoord);
				}
				else if (ExistUp && X2 >= (ComponentIndexX+1)*ComponentSizeQuads)
				{
					CornerSet |= 1 << 1;
					CornerValues[1] = StoreData.Load( (ComponentIndexX+1)*ComponentSizeQuads, ComponentIndexY*ComponentSizeQuads);
				}
				else if (BorderComponent[1])
				{
					CornerSet |= 1 << 1;
					CornerValues[1] = GetWeightMapData(BorderComponent[1], LayerInfo, 0, 0, NeighborWeightmapChannelOffset[1], NeighborWeightmapTexture[1], NeighborWeightmapTextureData[1]);
				}
				else if (BorderComponent[2])
				{
					CornerSet |= 1 << 1;
					CornerValues[1] = GetWeightMapData(BorderComponent[2], LayerInfo, EdgeCoord, EdgeCoord, NeighborWeightmapChannelOffset[2], NeighborWeightmapTexture[2], NeighborWeightmapTextureData[2]);
				}

				if (CornerComponent[2])
				{
					CornerSet |= 1 << 2;
					CornerValues[2] = GetWeightMapData(CornerComponent[2], LayerInfo, EdgeCoord, 0);
				}
				else if (ExistLeft && Y2 >= (ComponentIndexY+1)*ComponentSizeQuads) // Use data already stored for 0, 2
				{
					CornerSet |= 1 << 2;
					CornerValues[2] = StoreData.Load( ComponentIndexX*ComponentSizeQuads, (ComponentIndexY+1)*ComponentSizeQuads);
				}
				else if (BorderComponent[0])
				{
					CornerSet |= 1 << 2;
					CornerValues[2] = GetWeightMapData(BorderComponent[0], LayerInfo, EdgeCoord, EdgeCoord, NeighborWeightmapChannelOffset[0], NeighborWeightmapTexture[0], NeighborWeightmapTextureData[0]);
				}
				else if (BorderComponent[3])
				{
					CornerSet |= 1 << 2;
					CornerValues[2] = GetWeightMapData(BorderComponent[3], LayerInfo, 0, 0, NeighborWeightmapChannelOffset[3], NeighborWeightmapTexture[3], NeighborWeightmapTextureData[3]);
				}

				if (CornerComponent[3])
				{
					CornerSet |= 1 << 3;
					CornerValues[3] = GetWeightMapData(CornerComponent[3], LayerInfo, 0, 0);
				}
				else if (BorderComponent[1])
				{
					CornerSet |= 1 << 3;
					CornerValues[3] = GetWeightMapData(BorderComponent[1], LayerInfo, 0, EdgeCoord, NeighborWeightmapChannelOffset[1], NeighborWeightmapTexture[1], NeighborWeightmapTextureData[1]);
				}
				else if (BorderComponent[3])
				{
					CornerSet |= 1 << 3;
					CornerValues[3] = GetWeightMapData(BorderComponent[3], LayerInfo, EdgeCoord, 0, NeighborWeightmapChannelOffset[3], NeighborWeightmapTexture[3], NeighborWeightmapTextureData[3]);
				}

				FillCornerValues(CornerSet, CornerValues);
				ComponentDataExist[ComponentIndexXY] = ExistLeft || ExistUp || (BorderComponent[0] || BorderComponent[1] || BorderComponent[2] || BorderComponent[3]) || CornerSet;
			}

			if (!ComponentDataExist[ComponentIndexXY])
			{
				continue;
			}

			// Find coordinates of box that lies inside component
			int32 ComponentX1 = FMath::Clamp<int32>(X1-ComponentIndexX*ComponentSizeQuads, 0, ComponentSizeQuads);
			int32 ComponentY1 = FMath::Clamp<int32>(Y1-ComponentIndexY*ComponentSizeQuads, 0, ComponentSizeQuads);
			int32 ComponentX2 = FMath::Clamp<int32>(X2-ComponentIndexX*ComponentSizeQuads, 0, ComponentSizeQuads);
			int32 ComponentY2 = FMath::Clamp<int32>(Y2-ComponentIndexY*ComponentSizeQuads, 0, ComponentSizeQuads);

			// Find subsection range for this box
			int32 SubIndexX1 = FMath::Clamp<int32>((ComponentX1-1) / SubsectionSizeQuads,0,ComponentNumSubsections-1);	// -1 because we need to pick up vertices shared between subsections
			int32 SubIndexY1 = FMath::Clamp<int32>((ComponentY1-1) / SubsectionSizeQuads,0,ComponentNumSubsections-1);
			int32 SubIndexX2 = FMath::Clamp<int32>(ComponentX2 / SubsectionSizeQuads,0,ComponentNumSubsections-1);
			int32 SubIndexY2 = FMath::Clamp<int32>(ComponentY2 / SubsectionSizeQuads,0,ComponentNumSubsections-1);

			for( int32 SubIndexY=SubIndexY1;SubIndexY<=SubIndexY2;SubIndexY++ )
			{
				for( int32 SubIndexX=SubIndexX1;SubIndexX<=SubIndexX2;SubIndexX++ )
				{
					// Find coordinates of box that lies inside subsection
					int32 SubX1 = FMath::Clamp<int32>(ComponentX1-SubsectionSizeQuads*SubIndexX, 0, SubsectionSizeQuads);
					int32 SubY1 = FMath::Clamp<int32>(ComponentY1-SubsectionSizeQuads*SubIndexY, 0, SubsectionSizeQuads);
					int32 SubX2 = FMath::Clamp<int32>(ComponentX2-SubsectionSizeQuads*SubIndexX, 0, SubsectionSizeQuads);
					int32 SubY2 = FMath::Clamp<int32>(ComponentY2-SubsectionSizeQuads*SubIndexY, 0, SubsectionSizeQuads);

					// Update texture data for the box that lies inside subsection
					for( int32 SubY=SubY1;SubY<=SubY2;SubY++ )
					{
						for( int32 SubX=SubX1;SubX<=SubX2;SubX++ )
						{
							int32 LandscapeX = SubIndexX*SubsectionSizeQuads + ComponentIndexX*ComponentSizeQuads + SubX;
							int32 LandscapeY = SubIndexY*SubsectionSizeQuads + ComponentIndexY*ComponentSizeQuads + SubY;

							if (LayerInfo != nullptr)
							{
								// Find the input data corresponding to this vertex
								uint8 Weight = 0;
								if( WeightmapTexture )
								{
									// Find the texture data corresponding to this vertex
									Weight = GetWeightMapData(Component, LayerInfo, (SubsectionSizeQuads+1) * SubIndexX + SubX, (SubsectionSizeQuads+1) * SubIndexY + SubY, WeightmapChannelOffset, WeightmapTexture, WeightmapTextureData );
								}
								// only try generating between component if we have no component, otherwise we might end up trying to add some layer info to some component that do not originally had the allocation.
								else if (Component == nullptr && !IsBorderPosition)
								{
									// Find the texture data corresponding to this vertex
									uint8 Value[4] = {0, 0, 0, 0};
									int32 Dist[4] = {INT_MAX, INT_MAX, INT_MAX, INT_MAX};
									float ValueX = 0.0f, ValueY = 0.0f;
									bool Exist[4] = {false, false, false, false};

									// Use data already stored for 0, 2
									if (ExistLeft && SubX == 0)
									{
										Value[0] = StoreData.Load( ComponentIndexX*ComponentSizeQuads, LandscapeY);
										Dist[0] = LandscapeX - (ComponentIndexX*ComponentSizeQuads);
										Exist[0] = true;
									}
									else if (BorderComponent[0] && NeighborWeightmapTexture[0])
									{
										Value[0] = GetWeightMapData(BorderComponent[0], LayerInfo, EdgeCoord, (SubsectionSizeQuads+1) * SubIndexY + SubY, NeighborWeightmapChannelOffset[0], NeighborWeightmapTexture[0], NeighborWeightmapTextureData[0]);
										Dist[0] = LandscapeX - (BorderComponent[0]->GetSectionBase().X + ComponentSizeQuads);
										Exist[0] = true;
									}
									else 
									{
										if ((CornerSet & 1) && (CornerSet & 1 << 2))
										{
											int32 Dist1 = LandscapeY - (ComponentIndexY*ComponentSizeQuads);
											int32 Dist2 = ((ComponentIndexY+1)*ComponentSizeQuads) - LandscapeY;
											Value[0] = static_cast<uint8>((float)(Dist2 * CornerValues[0] + Dist1 * CornerValues[2]) / (Dist1 + Dist2));
											Dist[0] = LandscapeX - (ComponentIndexX*ComponentSizeQuads);
											Exist[0] = true;
										}
									}

									if (BorderComponent[1] && NeighborWeightmapTexture[1])
									{
										Value[1] = GetWeightMapData(BorderComponent[1], LayerInfo, 0, (SubsectionSizeQuads+1) * SubIndexY + SubY, NeighborWeightmapChannelOffset[1], NeighborWeightmapTexture[1], NeighborWeightmapTextureData[1]);
										Dist[1] = (BorderComponent[1]->GetSectionBase().X) - LandscapeX;
										Exist[1] = true;
									}
									else
									{
										if ((CornerSet & 1 << 1) && (CornerSet & 1 << 3))
										{
											int32 Dist1 = LandscapeY - (ComponentIndexY*ComponentSizeQuads);
											int32 Dist2 = ((ComponentIndexY+1)*ComponentSizeQuads) - LandscapeY;
											Value[1] = static_cast<uint8>((float)(Dist2 * CornerValues[1] + Dist1 * CornerValues[3]) / (Dist1 + Dist2));
											Dist[1] = (ComponentIndexX+1)*ComponentSizeQuads - LandscapeX;
											Exist[1] = true;
										}
									}

									if (ExistUp && SubY == 0)
									{
										Value[2] = StoreData.Load( LandscapeX, ComponentIndexY*ComponentSizeQuads);
										Dist[2] = LandscapeY - (ComponentIndexY*ComponentSizeQuads);
										Exist[2] = true;
									}
									else if (BorderComponent[2] && NeighborWeightmapTexture[2])
									{
										Value[2] = GetWeightMapData(BorderComponent[2], LayerInfo, (SubsectionSizeQuads+1) * SubIndexX + SubX, EdgeCoord, NeighborWeightmapChannelOffset[2], NeighborWeightmapTexture[2], NeighborWeightmapTextureData[2]);
										Dist[2] = LandscapeY - (BorderComponent[2]->GetSectionBase().Y + ComponentSizeQuads);
										Exist[2] = true;
									}
									else
									{
										if ((CornerSet & 1) && (CornerSet & 1 << 1))
										{
											int32 Dist1 = LandscapeX - (ComponentIndexX*ComponentSizeQuads);
											int32 Dist2 = (ComponentIndexX+1)*ComponentSizeQuads - LandscapeX;
											Value[2] = static_cast<uint8>((float)(Dist2 * CornerValues[0] + Dist1 * CornerValues[1]) / (Dist1 + Dist2));
											Dist[2] = LandscapeY - (ComponentIndexY*ComponentSizeQuads);
											Exist[2] = true;
										}
									}

									if (BorderComponent[3] && NeighborWeightmapTexture[3])
									{
										Value[3] = GetWeightMapData(BorderComponent[3], LayerInfo, (SubsectionSizeQuads+1) * SubIndexX + SubX, 0, NeighborWeightmapChannelOffset[3], NeighborWeightmapTexture[3], NeighborWeightmapTextureData[3]);
										Dist[3] = (BorderComponent[3]->GetSectionBase().Y) - LandscapeY;
										Exist[3] = true;
									}
									else
									{
										if ((CornerSet & 1 << 2) && (CornerSet & 1 << 3))
										{
											int32 Dist1 = LandscapeX - (ComponentIndexX*ComponentSizeQuads);
											int32 Dist2 = (ComponentIndexX+1)*ComponentSizeQuads - LandscapeX;
											Value[3] = static_cast<uint8>((float)(Dist2 * CornerValues[2] + Dist1 * CornerValues[3]) / (Dist1 + Dist2));
											Dist[3] = (ComponentIndexY+1)*ComponentSizeQuads - LandscapeY;
											Exist[3] = true;
										}
									}

									CalcInterpValue<uint8>(Dist, Exist, Value, ValueX, ValueY);

									uint8 FinalValue = 0; // Default Value
									if ( (Exist[0] || Exist[1]) && (Exist[2] || Exist[3]) )
									{
										FinalValue = CalcValueFromValueXY<uint8>(Dist, ValueX, ValueY, CornerSet, CornerValues);
									}
									else if ( (Exist[0] || Exist[1]) )
									{
										FinalValue = static_cast<uint8>(ValueX);
									}
									else if ( (Exist[2] || Exist[3]) )
									{
										FinalValue = static_cast<uint8>(ValueY);
									}

									Weight = FinalValue;
								}

								StoreData.Store(LandscapeX, LandscapeY, Weight);
							}
							else // Whole weight map case... no interpolation now...
							{
								StoreData.PreInit(LandscapeInfo->Layers.Num());

								const TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = Component->GetWeightmapLayerAllocations(GetEditLayer());
								const TArray<UTexture2D*>& ComponentWeightmapTextures = Component->GetWeightmapTextures(GetEditLayer());

								for( int32 LayerIdx=0;LayerIdx<ComponentWeightmapLayerAllocations.Num();LayerIdx++ )
								{
									int32 Idx = ComponentWeightmapLayerAllocations[LayerIdx].WeightmapTextureIndex;
									UTexture2D* ComponentWeightmapTexture = ComponentWeightmapTextures[Idx];
									uint8* ComponentWeightmapTextureData = (uint8*)TexDataInfos[Idx]->GetMipData(/*InMipIndex = */0);
									uint8 ComponentWeightmapChannelOffset = ChannelOffsets[ComponentWeightmapLayerAllocations[LayerIdx].WeightmapTextureChannel];

									// Find the texture data corresponding to this vertex
									int32 SizeU = ComponentWeightmapTexture->Source.GetSizeX();
									int32 SizeV = ComponentWeightmapTexture->Source.GetSizeY();
									int32 WeightmapOffsetX = static_cast<int32>(Component->WeightmapScaleBias.Z * SizeU);
									int32 WeightmapOffsetY = static_cast<int32>(Component->WeightmapScaleBias.W * SizeV);

									int32 TexX = WeightmapOffsetX + (SubsectionSizeQuads+1) * SubIndexX + SubX;
									int32 TexY = WeightmapOffsetY + (SubsectionSizeQuads+1) * SubIndexY + SubY;

									uint8 Weight = ComponentWeightmapTextureData[ 4 * (TexX + TexY * SizeU) + ComponentWeightmapChannelOffset ];

									// Find index in LayerInfos
									{
										int32 LayerInfoIdx = LandscapeInfo->GetLayerInfoIndex(ComponentWeightmapLayerAllocations[LayerIdx].LayerInfo);
										if (LayerInfoIdx != INDEX_NONE)
										{
											StoreData.Store(LandscapeX, LandscapeY, Weight, LayerInfoIdx);
										}
									}
								}
							}
						}
					}
				}
			}
		}
	}

	if (bHasMissingValue)
	{
		// generate something to fill in any missing data within the requested region
		CalcMissingValues<uint8, TStoreData, float>( X1, X2, Y1, Y2,
			ComponentIndexX1, ComponentIndexX2, ComponentIndexY1, ComponentIndexY2, 
			ComponentSizeX, ComponentSizeY, CornerValues,
			NoBorderY1, NoBorderY2, ComponentDataExist, StoreData );

		// clamp the returned bounds to the original request bounds
		// this means the returned bounds are allowed to shrink (if there is only a smaller part of the request that actually exists)
		// but it is not allowed to expand beyond the original request
		ValidX1 = FMath::Max<int32>(X1, ValidX1);
		ValidX2 = FMath::Min<int32>(X2, ValidX2);
		ValidY1 = FMath::Max<int32>(Y1, ValidY1);
		ValidY2 = FMath::Min<int32>(Y2, ValidY2);
	}
	else
	{
		// if there were no missing values, then the entire requested bounds is returned
		ValidX1 = X1;
		ValidX2 = X2;
		ValidY1 = Y1;
		ValidY2 = Y2;
	}
}

void FLandscapeEditDataInterface::GetWeightData(ULandscapeLayerInfoObject* LayerInfo, int32& X1, int32& Y1, int32& X2, int32& Y2, uint8* Data, int32 Stride)
{
	if( Stride==0 )
	{
		Stride = (1+X2-X1);
	}
	TArrayStoreData<uint8> ArrayStoreData(X1, Y1, Data, Stride);
	GetWeightDataTempl(LayerInfo, X1, Y1, X2, Y2, ArrayStoreData);
}

void FLandscapeEditDataInterface::GetWeightDataFast(ULandscapeLayerInfoObject* LayerInfo, const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, uint8* Data, int32 Stride)
{
	if( Stride==0 )
	{
		Stride = (1+X2-X1);
	}
	TArrayStoreData<uint8> ArrayStoreData(X1, Y1, Data, Stride);
	GetWeightDataTemplFast(LayerInfo, X1, Y1, X2, Y2, ArrayStoreData);
}

void FLandscapeEditDataInterface::GetWeightData(ULandscapeLayerInfoObject* LayerInfo, int32& X1, int32& Y1, int32& X2, int32& Y2, TMap<FIntPoint, uint8>& SparseData)
{
	TSparseStoreData<uint8> SparseStoreData(SparseData);
	GetWeightDataTempl(LayerInfo, X1, Y1, X2, Y2, SparseStoreData);
}

void FLandscapeEditDataInterface::GetWeightDataFast(ULandscapeLayerInfoObject* LayerInfo, const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TMap<FIntPoint, uint8>& SparseData)
{
	TSparseStoreData<uint8> SparseStoreData(SparseData);
	GetWeightDataTemplFast(LayerInfo, X1, Y1, X2, Y2, SparseStoreData);
}

void FLandscapeEditDataInterface::GetWeightDataFast(ULandscapeLayerInfoObject* LayerInfo, const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TArray<uint8>* Data, int32 Stride)
{
	if( Stride==0 )
	{
		Stride = (1+X2-X1);
	}
	TArrayStoreData<TArray<uint8>> ArrayStoreData(X1, Y1, Data, Stride);
	GetWeightDataTemplFast(LayerInfo, X1, Y1, X2, Y2, ArrayStoreData);
}

void FLandscapeEditDataInterface::GetWeightDataFast(ULandscapeLayerInfoObject* LayerInfo, const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TMap<FIntPoint, TArray<uint8>>& SparseData)
{
	TSparseStoreData<TArray<uint8>> SparseStoreData(SparseData);
	GetWeightDataTemplFast(LayerInfo, X1, Y1, X2, Y2, SparseStoreData);
}

FLandscapeTextureDataInfo* FLandscapeTextureDataInterface::GetTextureDataInfo(UTexture2D* Texture)
{
	FLandscapeTextureDataInfo* Result = TextureDataMap.FindRef(Texture);
	if (Result == nullptr)
	{
		Result = TextureDataMap.Add(Texture, new FLandscapeTextureDataInfo(Texture, GetShouldDirtyPackage()));
	}
	return Result;
}

void FLandscapeTextureDataInterface::CopyTextureChannel(UTexture2D* Dest, int32 DestChannel, UTexture2D* Src, int32 SrcChannel)
{
	FLandscapeTextureDataInfo* DestDataInfo = GetTextureDataInfo(Dest);
	FLandscapeTextureDataInfo* SrcDataInfo = GetTextureDataInfo(Src);
	int32 MipSize = Dest->Source.GetSizeX();
	check(Dest->Source.GetSizeX() == Dest->Source.GetSizeY() && Src->Source.GetSizeX() == Dest->Source.GetSizeX());

	int32 NumMips = DestDataInfo->NumMips();
	int32 SrcMips = SrcDataInfo->NumMips();
	if (SrcMips != NumMips)
	{
		// We could migrate data from an old landscape that still had multiple mips in its edit layer data so it's not worth displaying a warning then since no data loss will actually occur : 
		if (SrcMips < NumMips)
		{
			UE_LOGF(LogLandscape, Warning, "Unexpected mip count mismatch when copying landscape texture channels from '%ls' (%d mips) to '%ls' (%d mips) -- mip data may be lost, or left uninitialized",
				*Src->GetPathName(), SrcMips, *Dest->GetPathName(), NumMips);
		}
		NumMips = FMath::Min(NumMips, SrcMips);
	}

	for( int32 MipIdx=0;MipIdx< NumMips;MipIdx++ )
	{
		uint8* DestTextureData = (uint8*)DestDataInfo->GetMipData(MipIdx);
		uint8* SrcTextureData = (uint8*)SrcDataInfo->GetMipData(MipIdx);

		if (DestTextureData && SrcTextureData)
		{
			DestTextureData += ChannelOffsets[DestChannel];
			SrcTextureData += ChannelOffsets[SrcChannel];

			for (int32 i = 0; i < FMath::Square(MipSize); i++)
			{
				DestTextureData[i * 4] = SrcTextureData[i * 4];
			}
		}

		DestDataInfo->AddMipUpdateRegion(MipIdx, 0, 0, MipSize-1, MipSize-1);
		MipSize >>= 1;
	}
}

void FLandscapeTextureDataInterface::CopyTextureFromHeightmap(UTexture2D* Dest, int32 DestChannel, ULandscapeComponent* Comp, int32 SrcChannel)
{
	FLandscapeTextureDataInfo* DestDataInfo = GetTextureDataInfo(Dest);
	int32 MipSize = Dest->Source.GetSizeX();
	check(Dest->Source.GetSizeX() == Dest->Source.GetSizeY());

	for( int32 MipIdx=0;MipIdx<DestDataInfo->NumMips();MipIdx++ )
	{
		FLandscapeComponentDataInterface DataInterface(Comp, MipIdx);
		TArray<FColor> Heightmap;
		DataInterface.GetHeightmapTextureData(Heightmap);

		uint8* DestTextureData = (uint8*)DestDataInfo->GetMipData(MipIdx) + ChannelOffsets[DestChannel];
		uint8* SrcTextureData = (uint8*)Heightmap.GetData() + ChannelOffsets[SrcChannel];

		for( int32 i=0;i<FMath::Square(MipSize);i++ )
		{
			DestTextureData[i*4] = SrcTextureData[i*4];
		}

		DestDataInfo->AddMipUpdateRegion(MipIdx, 0, 0, MipSize-1, MipSize-1);
		MipSize >>= 1;
	}
}

void FLandscapeTextureDataInterface::CopyTextureFromHeightmap(UTexture2D* Dest, ULandscapeComponent* Comp, int32 MipIndex)
{
	FLandscapeTextureDataInfo* DestDataInfo = GetTextureDataInfo(Dest);
	const int32 MipSize = DestDataInfo->GetMipSizeX(MipIndex);
	check(MipSize == DestDataInfo->GetMipSizeY(MipIndex));
	
	const bool bEditingLayer = false;
	FLandscapeComponentDataInterface DataInterface(Comp, MipIndex, bEditingLayer);
	
	FColor* DestTextureData = (FColor*)DestDataInfo->GetMipData(MipIndex);
	FColor* SrcTextureData = DataInterface.GetRawHeightData();

	int32 MipSizeSquare = FMath::Square(MipSize);
	check(MipSizeSquare == DataInterface.GetHeightmapSizeX(MipIndex)*DataInterface.GetHeightmapSizeY(MipIndex));

	FMemory::Memcpy(DestTextureData, SrcTextureData, MipSizeSquare*sizeof(FColor));
	DestDataInfo->AddMipUpdateRegion(MipIndex, 0, 0, MipSize - 1, MipSize - 1);
}

void FLandscapeTextureDataInterface::CopyTextureFromWeightmap(UTexture2D* Dest, int32 DestChannel, ULandscapeComponent* Comp, ULandscapeLayerInfoObject* LayerInfo)
{
	FLandscapeTextureDataInfo* DestDataInfo = GetTextureDataInfo(Dest);
	for (int32 MipIdx = 0; MipIdx < DestDataInfo->NumMips(); MipIdx++)
	{
		CopyTextureFromWeightmap(DestDataInfo, DestChannel, Comp, LayerInfo, MipIdx);
	}
}

void FLandscapeTextureDataInterface::CopyTextureFromWeightmap(UTexture2D* Dest, int32 DestChannel, ULandscapeComponent* Comp, ULandscapeLayerInfoObject* LayerInfo, int32 MipIndex)
{
	FLandscapeTextureDataInfo* DestDataInfo = GetTextureDataInfo(Dest);
	CopyTextureFromWeightmap(DestDataInfo, DestChannel, Comp, LayerInfo, MipIndex);
}

void FLandscapeTextureDataInterface::CopyTextureFromWeightmap(FLandscapeTextureDataInfo* DestDataInfo, int32 DestChannel, ULandscapeComponent* Comp, ULandscapeLayerInfoObject* LayerInfo, int32 MipIndex)
{
	const int32 MipSize = DestDataInfo->GetMipSizeX(MipIndex);
	check(MipSize == DestDataInfo->GetMipSizeY(MipIndex));

	const bool bEditingLayer = false;
	FLandscapeComponentDataInterface DataInterface(Comp, MipIndex, bEditingLayer);
	TArray<uint8> WeightData;
	DataInterface.GetWeightmapTextureData(LayerInfo, WeightData, bEditingLayer);
	int32 MipSizeSquare = FMath::Square(MipSize);
	check(WeightData.Num() == MipSizeSquare);
	
	uint8* DestTextureData = (uint8*)DestDataInfo->GetMipData(MipIndex) + ChannelOffsets[DestChannel];
	for (int32 i = 0; i < MipSizeSquare; i++)
	{
		DestTextureData[i * 4] = WeightData[i];
	}

	DestDataInfo->AddMipUpdateRegion(MipIndex, 0, 0, MipSize - 1, MipSize - 1);
}

void FLandscapeTextureDataInterface::ZeroTextureChannel(UTexture2D* Dest, int32 DestChannel)
{
	FLandscapeTextureDataInfo* DestDataInfo = GetTextureDataInfo(Dest);
	int32 MipSize = Dest->Source.GetSizeX();
	check(Dest->Source.GetSizeX() == Dest->Source.GetSizeY());

	for( int32 MipIdx=0;MipIdx<DestDataInfo->NumMips();MipIdx++ )
	{
		uint8* DestTextureData = (uint8*)DestDataInfo->GetMipData(MipIdx) + ChannelOffsets[DestChannel];

		for( int32 i=0;i<FMath::Square(MipSize);i++ )
		{
			DestTextureData[i*4] = 0;
		}

		DestDataInfo->AddMipUpdateRegion(MipIdx, 0, 0, MipSize-1, MipSize-1);
		MipSize >>= 1;
	}
}

template<typename TData>
void FLandscapeTextureDataInterface::SetTextureValueTempl(UTexture2D* Dest, TData Value)
{
	FLandscapeTextureDataInfo* DestDataInfo = GetTextureDataInfo(Dest);
	int32 MipSize = Dest->Source.GetSizeX();
	check(Dest->Source.GetSizeX() == Dest->Source.GetSizeY());

	for( int32 MipIdx=0;MipIdx<DestDataInfo->NumMips();MipIdx++ )
	{
		TData* DestTextureData = (TData*)DestDataInfo->GetMipData(MipIdx);

		for( int32 i=0;i<FMath::Square(MipSize);i++ )
		{
			DestTextureData[i] = Value;
		}

		DestDataInfo->AddMipUpdateRegion(MipIdx, 0, 0, MipSize-1, MipSize-1);
		MipSize >>= 1;
	}
}

void FLandscapeTextureDataInterface::ZeroTexture(UTexture2D* Dest)
{
	SetTextureValueTempl<uint8>(Dest, 0);
}

void FLandscapeTextureDataInterface::SetTextureValue(UTexture2D* Dest, FColor Value)
{
	SetTextureValueTempl<FColor>(Dest, Value);
}

template<typename TData>
bool FLandscapeTextureDataInterface::EqualTextureValueTempl(UTexture2D* Src, TData Value)
{
	FLandscapeTextureDataInfo* DestDataInfo = GetTextureDataInfo(Src);
	TData* DestTextureData = (TData*)DestDataInfo->GetMipData(/*InMipIndex = */0);
	int32 Size = Src->Source.GetSizeX() * Src->Source.GetSizeY();

	for( int32 i = 0 ;i < Size; ++i )
	{
		if (DestTextureData[i] != Value)
		{
			return false;
		}
	}

	return true;
}

bool FLandscapeTextureDataInterface::EqualTextureValue(UTexture2D* Src, FColor Value)
{
	return EqualTextureValueTempl<FColor>(Src, Value);
}

template<typename TStoreData>
void FLandscapeEditDataInterface::GetSelectDataTempl(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TStoreData& StoreData)
{
	auto ReturnComponentTexture = [&](ULandscapeComponent* Component) -> UTexture2D*
	{
		check(Component);
		return Component->EditToolRenderData.DataTexture;
	};
	GetEditToolTextureData(X1, Y1, X2, Y2, StoreData, ReturnComponentTexture);
}

void FLandscapeEditDataInterface::GetSelectData(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TMap<FIntPoint, uint8>& SparseData)
{
	TSparseStoreData<uint8> SparseStoreData(SparseData);
	GetSelectDataTempl(X1, Y1, X2, Y2, SparseStoreData);
}

void FLandscapeEditDataInterface::GetSelectData(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, uint8* Data, int32 Stride)
{
	if( Stride==0 )
	{
		Stride = (1+X2-X1);
	}
	TArrayStoreData<uint8> ArrayStoreData(X1, Y1, Data, Stride);
	GetSelectDataTempl(X1, Y1, X2, Y2, ArrayStoreData);
}

void FLandscapeEditDataInterface::SetSelectData(int32 X1, int32 Y1, int32 X2, int32 Y2, const uint8* Data, int32 Stride)
{
	auto ReturnComponentTexture = [&](ULandscapeComponent* Component) -> UTexture2D*&
	{
		check(Component);
		return static_cast<UTexture2D*&>(Component->EditToolRenderData.DataTexture);
	};
	SetEditToolTextureData(X1, Y1, X2, Y2, Data, Stride, ReturnComponentTexture, TEXTUREGROUP_8BitData);
}

template<typename TStoreData>
void FLandscapeEditDataInterface::GetLayerContributionDataTempl(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TStoreData& StoreData)
{
	auto ReturnComponentTexture = [&](const ULandscapeComponent* Component) -> UTexture2D*
	{
		check(Component);
		return Component->EditToolRenderData.LayerContributionTexture;
	};
	GetEditToolTextureData(X1, Y1, X2, Y2, StoreData, ReturnComponentTexture);
}

void FLandscapeEditDataInterface::GetLayerContributionData(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TMap<FIntPoint, uint8>& SparseData)
{
	TSparseStoreData<uint8> SparseStoreData(SparseData);
	GetLayerContributionDataTempl(X1, Y1, X2, Y2, SparseStoreData);
}

void FLandscapeEditDataInterface::GetLayerContributionData(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, uint8* Data, int32 Stride)
{
	if (Stride == 0)
	{
		Stride = (1 + X2 - X1);
	}
	TArrayStoreData<uint8> ArrayStoreData(X1, Y1, Data, Stride);
	GetLayerContributionDataTempl(X1, Y1, X2, Y2, ArrayStoreData);
}


void FLandscapeEditDataInterface::SetLayerContributionData(int32 X1, int32 Y1, int32 X2, int32 Y2, const uint8* Data, int32 Stride)
{
	auto ReturnComponentTexture = [](ULandscapeComponent* Component) -> UTexture2D*&
	{
		check(Component);
		return static_cast<UTexture2D*&>(Component->EditToolRenderData.LayerContributionTexture);
	};
	SetEditToolTextureData(X1, Y1, X2, Y2, Data, Stride, ReturnComponentTexture, TEXTUREGROUP_8BitData);
}

template<typename TStoreData>
void FLandscapeEditDataInterface::GetDirtyDataTempl(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TStoreData& StoreData)
{
	auto ReturnComponentTexture = [&](const ULandscapeComponent* Component) -> UTexture2D*
	{
		check(Component);
		return Component->EditToolRenderData.DirtyTexture;
	};
	GetEditToolTextureData(X1, Y1, X2, Y2, StoreData, ReturnComponentTexture);
}

void FLandscapeEditDataInterface::GetDirtyData(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TMap<FIntPoint, uint8>& SparseData)
{
	TSparseStoreData<uint8> SparseStoreData(SparseData);
	GetDirtyDataTempl(X1, Y1, X2, Y2, SparseStoreData);
}

void FLandscapeEditDataInterface::GetDirtyData(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, uint8* Data, int32 Stride)
{
	if (Stride == 0)
	{
		Stride = (1 + X2 - X1);
	}
	TArrayStoreData<uint8> ArrayStoreData(X1, Y1, Data, Stride);
	GetDirtyDataTempl(X1, Y1, X2, Y2, ArrayStoreData);
}


void FLandscapeEditDataInterface::SetDirtyData(int32 X1, int32 Y1, int32 X2, int32 Y2, const uint8* Data, int32 Stride)
{
	auto ReturnComponentTexture = [](ULandscapeComponent* Component) -> UTexture2D*&
	{
		check(Component);
		return static_cast<UTexture2D*&>(Component->EditToolRenderData.DirtyTexture);
	};
	SetEditToolTextureData(X1, Y1, X2, Y2, Data, Stride, ReturnComponentTexture, TEXTUREGROUP_8BitData);
}

template<typename TStoreData>
void FLandscapeEditDataInterface::GetEditToolTextureData(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TStoreData& StoreData, TFunctionRef<UTexture2D*(ULandscapeComponent*)> GetComponentTexture)
{
	int32 ComponentIndexX1, ComponentIndexY1, ComponentIndexX2, ComponentIndexY2;
	ALandscape::CalcComponentIndicesNoOverlap(X1, Y1, X2, Y2, ComponentSizeQuads, ComponentIndexX1, ComponentIndexY1, ComponentIndexX2, ComponentIndexY2);

	for (int32 ComponentIndexY = ComponentIndexY1; ComponentIndexY <= ComponentIndexY2; ComponentIndexY++)
	{
		for (int32 ComponentIndexX = ComponentIndexX1; ComponentIndexX <= ComponentIndexX2; ComponentIndexX++)
		{
			ULandscapeComponent* Component = LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(ComponentIndexX, ComponentIndexY));

			FLandscapeTextureDataInfo* TexDataInfo = nullptr;
			uint8* SelectTextureData = nullptr;
			UTexture2D* EditToolTexture = nullptr;
			if (Component)
			{
				EditToolTexture = GetComponentTexture(Component);
			}
			if (EditToolTexture)
			{
				TexDataInfo = GetTextureDataInfo(EditToolTexture);
				SelectTextureData = (uint8*)TexDataInfo->GetMipData(/*InMipIndex = */0);
			}

			// Find coordinates of box that lies inside component
			int32 ComponentX1 = FMath::Clamp<int32>(X1 - ComponentIndexX * ComponentSizeQuads, 0, ComponentSizeQuads);
			int32 ComponentY1 = FMath::Clamp<int32>(Y1 - ComponentIndexY * ComponentSizeQuads, 0, ComponentSizeQuads);
			int32 ComponentX2 = FMath::Clamp<int32>(X2 - ComponentIndexX * ComponentSizeQuads, 0, ComponentSizeQuads);
			int32 ComponentY2 = FMath::Clamp<int32>(Y2 - ComponentIndexY * ComponentSizeQuads, 0, ComponentSizeQuads);

			// Find subsection range for this box
			int32 SubIndexX1 = FMath::Clamp<int32>((ComponentX1 - 1) / SubsectionSizeQuads, 0, ComponentNumSubsections - 1);	// -1 because we need to pick up vertices shared between subsections
			int32 SubIndexY1 = FMath::Clamp<int32>((ComponentY1 - 1) / SubsectionSizeQuads, 0, ComponentNumSubsections - 1);
			int32 SubIndexX2 = FMath::Clamp<int32>(ComponentX2 / SubsectionSizeQuads, 0, ComponentNumSubsections - 1);
			int32 SubIndexY2 = FMath::Clamp<int32>(ComponentY2 / SubsectionSizeQuads, 0, ComponentNumSubsections - 1);

			for (int32 SubIndexY = SubIndexY1; SubIndexY <= SubIndexY2; SubIndexY++)
			{
				for (int32 SubIndexX = SubIndexX1; SubIndexX <= SubIndexX2; SubIndexX++)
				{
					// Find coordinates of box that lies inside subsection
					int32 SubX1 = FMath::Clamp<int32>(ComponentX1 - SubsectionSizeQuads * SubIndexX, 0, SubsectionSizeQuads);
					int32 SubY1 = FMath::Clamp<int32>(ComponentY1 - SubsectionSizeQuads * SubIndexY, 0, SubsectionSizeQuads);
					int32 SubX2 = FMath::Clamp<int32>(ComponentX2 - SubsectionSizeQuads * SubIndexX, 0, SubsectionSizeQuads);
					int32 SubY2 = FMath::Clamp<int32>(ComponentY2 - SubsectionSizeQuads * SubIndexY, 0, SubsectionSizeQuads);

					// Update texture data for the box that lies inside subsection
					for (int32 SubY = SubY1; SubY <= SubY2; SubY++)
					{
						for (int32 SubX = SubX1; SubX <= SubX2; SubX++)
						{
							int32 LandscapeX = SubIndexX * SubsectionSizeQuads + ComponentIndexX * ComponentSizeQuads + SubX;
							int32 LandscapeY = SubIndexY * SubsectionSizeQuads + ComponentIndexY * ComponentSizeQuads + SubY;

							// Find the input data corresponding to this vertex
							if (Component && SelectTextureData)
							{
								// Find the texture data corresponding to this vertex
								int32 SizeU = EditToolTexture->Source.GetSizeX();
								int32 SizeV = EditToolTexture->Source.GetSizeY();
								int32 WeightmapOffsetX = static_cast<int32>(Component->WeightmapScaleBias.Z * (float)SizeU);
								int32 WeightmapOffsetY = static_cast<int32>(Component->WeightmapScaleBias.W * (float)SizeV);

								int32 TexX = WeightmapOffsetX + (SubsectionSizeQuads + 1) * SubIndexX + SubX;
								int32 TexY = WeightmapOffsetY + (SubsectionSizeQuads + 1) * SubIndexY + SubY;
								uint8& TexData = SelectTextureData[TexX + TexY * SizeU];

								StoreData.Store(LandscapeX, LandscapeY, TexData);
							}
							else
							{
								StoreData.Store(LandscapeX, LandscapeY, 0);
							}

						}
					}
				}
			}
		}
	}
}

void FLandscapeEditDataInterface::SetEditToolTextureData(int32 X1, int32 Y1, int32 X2, int32 Y2, const uint8* Data, int32 Stride, TFunctionRef<UTexture2D*&(ULandscapeComponent*)> GetComponentTexture, TextureGroup InTextureGroup)
{
	if( Stride==0 )
	{
		Stride = (1+X2-X1);
	}

	check(ComponentSizeQuads > 0);
	// Find component range for this block of data
	int32 ComponentIndexX1, ComponentIndexY1, ComponentIndexX2, ComponentIndexY2;
	ALandscape::CalcComponentIndicesNoOverlap(X1, Y1, X2, Y2, ComponentSizeQuads, ComponentIndexX1, ComponentIndexY1, ComponentIndexX2, ComponentIndexY2);

	for( int32 ComponentIndexY=ComponentIndexY1;ComponentIndexY<=ComponentIndexY2;ComponentIndexY++ )
	{
		for( int32 ComponentIndexX=ComponentIndexX1;ComponentIndexX<=ComponentIndexX2;ComponentIndexX++ )
		{	
			FIntPoint ComponentKey(ComponentIndexX,ComponentIndexY);
			ULandscapeComponent* Component = LandscapeInfo->XYtoComponentMap.FindRef(ComponentKey);
			if (Component == nullptr)
			{
				continue;
			}

			UTexture2D*& EditToolTexture = GetComponentTexture(Component);
			// if nullptr, it was painted away
			if (EditToolTexture == nullptr)
			{
				// Construct Texture...
				int32 WeightmapSize = (Component->SubsectionSizeQuads+1) * Component->NumSubsections;
				EditToolTexture = Component->GetLandscapeProxy()->CreateLandscapeToolTexture(WeightmapSize, WeightmapSize, InTextureGroup, TSF_G8);
				EditToolTexture->PostEditChange();
				
				Component->UpdateEditToolRenderData();
			}

			FLandscapeTextureDataInfo* TexDataInfo = GetTextureDataInfo(EditToolTexture);
			uint8* SelectTextureData = (uint8*)TexDataInfo->GetMipData(/*InMipIndex = */0);

			// Find the texture data corresponding to this vertex
			int32 SizeU = EditToolTexture->Source.GetSizeX();
			int32 SizeV = EditToolTexture->Source.GetSizeY();
			int32 WeightmapOffsetX = static_cast<int32>(Component->WeightmapScaleBias.Z * SizeU);
			int32 WeightmapOffsetY = static_cast<int32>(Component->WeightmapScaleBias.W * SizeV);

			// Find coordinates of box that lies inside component
			int32 ComponentX1 = FMath::Clamp<int32>(X1-ComponentIndexX*ComponentSizeQuads, 0, ComponentSizeQuads);
			int32 ComponentY1 = FMath::Clamp<int32>(Y1-ComponentIndexY*ComponentSizeQuads, 0, ComponentSizeQuads);
			int32 ComponentX2 = FMath::Clamp<int32>(X2-ComponentIndexX*ComponentSizeQuads, 0, ComponentSizeQuads);
			int32 ComponentY2 = FMath::Clamp<int32>(Y2-ComponentIndexY*ComponentSizeQuads, 0, ComponentSizeQuads);

			// Find subsection range for this box
			int32 SubIndexX1 = FMath::Clamp<int32>((ComponentX1-1) / SubsectionSizeQuads,0,ComponentNumSubsections-1);	// -1 because we need to pick up vertices shared between subsections
			int32 SubIndexY1 = FMath::Clamp<int32>((ComponentY1-1) / SubsectionSizeQuads,0,ComponentNumSubsections-1);
			int32 SubIndexX2 = FMath::Clamp<int32>(ComponentX2 / SubsectionSizeQuads,0,ComponentNumSubsections-1);
			int32 SubIndexY2 = FMath::Clamp<int32>(ComponentY2 / SubsectionSizeQuads,0,ComponentNumSubsections-1);

			for( int32 SubIndexY=SubIndexY1;SubIndexY<=SubIndexY2;SubIndexY++ )
			{
				for( int32 SubIndexX=SubIndexX1;SubIndexX<=SubIndexX2;SubIndexX++ )
				{
					// Find coordinates of box that lies inside subsection
					int32 SubX1 = FMath::Clamp<int32>(ComponentX1-SubsectionSizeQuads*SubIndexX, 0, SubsectionSizeQuads);
					int32 SubY1 = FMath::Clamp<int32>(ComponentY1-SubsectionSizeQuads*SubIndexY, 0, SubsectionSizeQuads);
					int32 SubX2 = FMath::Clamp<int32>(ComponentX2-SubsectionSizeQuads*SubIndexX, 0, SubsectionSizeQuads);
					int32 SubY2 = FMath::Clamp<int32>(ComponentY2-SubsectionSizeQuads*SubIndexY, 0, SubsectionSizeQuads);

					// Update texture data for the box that lies inside subsection
					for( int32 SubY=SubY1;SubY<=SubY2;SubY++ )
					{
						for( int32 SubX=SubX1;SubX<=SubX2;SubX++ )
						{
							int32 LandscapeX = SubIndexX*SubsectionSizeQuads + ComponentIndexX*ComponentSizeQuads + SubX;
							int32 LandscapeY = SubIndexY*SubsectionSizeQuads + ComponentIndexY*ComponentSizeQuads + SubY;
							checkSlow( LandscapeX >= X1 && LandscapeX <= X2 );
							checkSlow( LandscapeY >= Y1 && LandscapeY <= Y2 );

							// Find the input data corresponding to this vertex
							int32 DataIndex = (LandscapeX-X1) + Stride * (LandscapeY-Y1);
							const uint8& Value = Data[DataIndex];

							int32 TexX = WeightmapOffsetX + (SubsectionSizeQuads+1) * SubIndexX + SubX;
							int32 TexY = WeightmapOffsetY + (SubsectionSizeQuads+1) * SubIndexY + SubY;
							uint8& TexData = SelectTextureData[ TexX + TexY * SizeU ];

							TexData = Value;
						}
					}

					// Record the areas of the texture we need to re-upload
					int32 TexX1 = WeightmapOffsetX + (SubsectionSizeQuads+1) * SubIndexX + SubX1;
					int32 TexY1 = WeightmapOffsetY + (SubsectionSizeQuads+1) * SubIndexY + SubY1;
					int32 TexX2 = WeightmapOffsetX + (SubsectionSizeQuads+1) * SubIndexX + SubX2;
					int32 TexY2 = WeightmapOffsetY + (SubsectionSizeQuads+1) * SubIndexY + SubY2;
					TexDataInfo->AddMipUpdateRegion(0,TexX1,TexY1,TexX2,TexY2);
				}
			}
		}
	}
}

// Deprecated
void FLandscapeEditDataInterface::SetXYOffsetData(int32 X1, int32 Y1, int32 X2, int32 Y2, const FVector2D* Data, int32 Stride)
{
}

// Deprecated
void FLandscapeEditDataInterface::SetXYOffsetData(int32 X1, int32 Y1, int32 X2, int32 Y2, const FVector* Data, int32 Stride)
{
}

// Deprecated
FVector2D FLandscapeEditDataInterface::GetXYOffsetmapData(const ULandscapeComponent* Component, int32 TexU, int32 TexV, FColor* TextureData/* = nullptr*/)
{
	return FVector2D();
}

// Deprecated
void FLandscapeEditDataInterface::GetXYOffsetData(int32& X1, int32& Y1, int32& X2, int32& Y2, FVector2D* Data, int32 Stride)
{
}
// Deprecated
void FLandscapeEditDataInterface::GetXYOffsetData(int32& X1, int32& Y1, int32& X2, int32& Y2, TMap<FIntPoint, FVector2D>& SparseData)
{
}

// Deprecated
void FLandscapeEditDataInterface::GetXYOffsetData(int32& X1, int32& Y1, int32& X2, int32& Y2, FVector* Data, int32 Stride)
{
}

// Deprecated
void FLandscapeEditDataInterface::GetXYOffsetData(int32& X1, int32& Y1, int32& X2, int32& Y2, TMap<FIntPoint, FVector>& SparseData)
{
}

const ALandscape* FLandscapeEditDataInterface::GetTargetLandscape() const
{
	return LandscapeInfo ? LandscapeInfo->LandscapeActor.Get() : nullptr;
}

// Deprecated 
bool FLandscapeEditDataInterface::CanHaveLandscapeLayersContent() const
{
	return true;
}

// Deprecated 
bool FLandscapeEditDataInterface::HasLandscapeLayersContent() const
{
	return true;
}

// Deprecated
void FLandscapeEditDataInterface::GetXYOffsetDataFast(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, FVector2D* Data, int32 Stride)
{
}

// Deprecated
void FLandscapeEditDataInterface::GetXYOffsetDataFast(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TMap<FIntPoint, FVector2D>& SparseData)
{
}

// Deprecated
void FLandscapeEditDataInterface::GetXYOffsetDataFast(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, FVector* Data, int32 Stride)
{
}

// Deprecated
void FLandscapeEditDataInterface::GetXYOffsetDataFast(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TMap<FIntPoint, FVector>& SparseData)
{
}

//
// FLandscapeTextureDataInfo
//

FLandscapeTextureDataInfo::FLandscapeTextureDataInfo(UTexture2D* InTexture, bool bShouldDirtyPackage, ELandscapeTextureUsage InTextureUsage, ELandscapeTextureType InTextureType)
	: Texture(InTexture)
	, TextureUsage(InTextureUsage)
	, TextureType(InTextureType)
{
	MipInfo.AddDefaulted(Texture->Source.GetNumMips());
	Texture->SetFlags(RF_Transactional);
	Texture->TemporarilyDisableStreaming();

	// Call BlockOnAnyAsyncBuild here to make sure that Modify cannot cancel the potentially on-going async cache
	//  The better fix would be not to call Modify when we don't intend to actually modify the texture (e.g. when FLandscapeTextureDataInfo is used for reading the data only)
	//  but in practice, that would require rebuilding the entire TLandscapeEditCache system, that is used both as a read-only and/ read-write facility
	Texture->BlockOnAnyAsyncBuild();
	Texture->Modify(bShouldDirtyPackage);
	check(!Texture->IsDefaultTexture()); // If we're the default texture, then Modify likely has cancelled the async cache, which would be a disaster if nothing requests a new cache afterwards
}

void FLandscapeTextureDataInfo::AddMipUpdateRegion(int32 InMipIndex, int32 InX1, int32 InY1, int32 InX2, int32 InY2)
{
	if (MipInfo[InMipIndex].bFull)
	{
		return;
	}

	check(InMipIndex < MipInfo.Num());
	uint32 Width = 1 + InX2 - InX1;
	uint32 Height = 1 + InY2 - InY1;
	// Catch situation where we are updating the whole texture to avoid adding redundant regions once the whole region as been included.
	if (Width == GetMipSizeX(InMipIndex) && Height == GetMipSizeY(InMipIndex))
	{
		MipInfo[InMipIndex].bFull = true;
		MipInfo[InMipIndex].MipUpdateRegions.Reset();
		// Push a full region for UpdateTextureData() to process later
		MipInfo[InMipIndex].MipUpdateRegions.Emplace(0, 0, 0, 0, Width, Height);
		return;
	}

	MipInfo[InMipIndex].MipUpdateRegions.Emplace(InX1, InY1, InX1, InY1, Width, Height);
}

void* FLandscapeTextureDataInfo::GetMipData(int32 InMipIndex)
{
	check(InMipIndex < MipInfo.Num());

	if (!MipInfo[InMipIndex].MipData)
	{
		// will Unlock in destructor
		MipInfo[InMipIndex].MipData = Texture->Source.LockMip(InMipIndex);
		// probably should check that we got the lock :
		//check( MipInfo[InMipIndex].MipData != nullptr );
	}
	return MipInfo[InMipIndex].MipData;
}

bool FLandscapeTextureDataInfo::UpdateTextureData()
{
	const bool bCompressed = !Texture->CompressionNone;

	bool bNeedToWaitForUpdate = false;
	int32 DataSize = sizeof(FColor);
	if (Texture->GetPixelFormat() == PF_G8)
	{
		DataSize = sizeof(uint8);
	}

	// Only wait once
	bool bNeedToFinishCompilation = true;

	for (int32 MipIndex = 0; MipIndex < MipInfo.Num(); MipIndex++)
	{
		if (MipInfo[MipIndex].MipData && MipInfo[MipIndex].MipUpdateRegions.Num() > 0)
		{
			if (bCompressed)
			{
				bNeedToWaitForUpdate = true;
				// Cannot update regions on compressed textures so we will update the whole texture below.
				break;
			}

			if (bNeedToFinishCompilation)
			{
				// Need to make sure we have a valid Resource
				FTextureCompilingManager::Get().FinishCompilation({ Texture });
				bNeedToFinishCompilation = false;
			}

			const uint32 SrcSizeX = (Texture->Source.GetSizeX()) >> MipIndex;
			const uint32 SrcSizeY = (Texture->Source.GetSizeY()) >> MipIndex;
			const uint32 SrcPitch = (SrcSizeX * DataSize);
			const uint32 BufferSize = SrcSizeX * SrcSizeY * DataSize;

			// Copy Mip update data so we can avoid waiting for Render thread in calling method
			FMipInfo* CopyMipInfo = new FMipInfo();
			CopyMipInfo->MipUpdateRegions = MipInfo[MipIndex].MipUpdateRegions;
			CopyMipInfo->MipData = FMemory::Malloc(BufferSize);
			FMemory::Memcpy(CopyMipInfo->MipData, MipInfo[MipIndex].MipData, BufferSize);

			Texture->UpdateTextureRegions(MipIndex, CopyMipInfo->MipUpdateRegions.Num(), &CopyMipInfo->MipUpdateRegions[0], SrcPitch, DataSize, (uint8*)CopyMipInfo->MipData,
				[CopyMipInfo](uint8* SrcData, const FUpdateTextureRegion2D*) 
				{ 
					FMemory::Free(CopyMipInfo->MipData);
					delete CopyMipInfo; 
				});
		}
	}
	if (bNeedToWaitForUpdate)
	{
		Texture->UpdateResource();
	}

	return bNeedToWaitForUpdate;
}

FLandscapeTextureDataInfo::~FLandscapeTextureDataInfo()
{
	// Unlock any mips still locked.
	for( int32 MipIndex = 0; MipIndex < MipInfo.Num(); MipIndex++ )
	{
		if(MipInfo[MipIndex].MipData != nullptr)
		{
			Texture->Source.UnlockMip(MipIndex);
			MipInfo[MipIndex].MipData = nullptr;
		}
	}
	Texture->ClearFlags(RF_Transactional);
	ULandscapeTextureHash::UpdateHash(Texture, TextureUsage, TextureType);
}

#endif // WITH_EDITOR
