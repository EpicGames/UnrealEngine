// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeEditLayer.h"
#include "LandscapeEditTypes.h"
#include "Landscape.h"
#include "Algo/AllOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapeEditLayer)

#define LOCTEXT_NAMESPACE "LandscapeEditLayer"

// ----------------------------------------------------------------------------------

#if WITH_EDITOR
bool ULandscapeEditLayerBase::SupportsAlphaForTargetType(ELandscapeToolTargetType InType) const
{
	return (InType == ELandscapeToolTargetType::Heightmap) || (InType == ELandscapeToolTargetType::Weightmap);
}

void ULandscapeEditLayerBase::SetAlphaForTargetType(ELandscapeToolTargetType InType, float InNewValue, bool bInModify, EPropertyChangeType::Type InChangeType)
{
	check(SupportsAlphaForTargetType(InType));

	const FFloatInterval AlphaInterval = GetAlphaRangeForTargetType(InType);
	const float ClampedNewValue = FMath::Clamp(InNewValue, AlphaInterval.Min, AlphaInterval.Max);
	float& AlphaValueRef = GetAlphaForTargetTypeRef(InType);
	const bool bHasValueChanged = (AlphaValueRef != ClampedNewValue);
	if (bHasValueChanged)
	{
		if (bInModify)
		{
			Modify();
		}

		AlphaValueRef = ClampedNewValue;
	}

	FProperty* AlphaProperty = GetAlphaPropertyForTargetType(InType);
	check(AlphaProperty != nullptr);
	BroadcastOnLayerDataChanged(FPropertyChangedEvent(AlphaProperty, InChangeType), /*bInUserTriggered = */true, /*bInRequiresLandscapeUpdate =*/ true, bHasValueChanged);
}

float ULandscapeEditLayerBase::GetAlphaForTargetType(ELandscapeToolTargetType InType) const
{
	return const_cast<ULandscapeEditLayerBase*>(this)->GetAlphaForTargetTypeRef(InType);
}

float& ULandscapeEditLayerBase::GetAlphaForTargetTypeRef(ELandscapeToolTargetType InType) 
{
	switch (InType)
	{
	case ELandscapeToolTargetType::Heightmap: 
	{
		return HeightmapAlpha;
	}
	case ELandscapeToolTargetType::Weightmap: 
	{
		return WeightmapAlpha;
	}
	default:
	{
		static float DefaultValue = 1.0f;
		return DefaultValue;
	}
	}
}

FProperty* ULandscapeEditLayerBase::GetAlphaPropertyForTargetType(ELandscapeToolTargetType InType) const
{
	switch (InType)
	{
	case ELandscapeToolTargetType::Heightmap:
	{
		return FindFProperty<FProperty>(StaticClass(), GET_MEMBER_NAME_CHECKED(ULandscapeEditLayerBase, HeightmapAlpha));
	}
	case ELandscapeToolTargetType::Weightmap:
	{
		return FindFProperty<FProperty>(StaticClass(), GET_MEMBER_NAME_CHECKED(ULandscapeEditLayerBase, WeightmapAlpha));
	}
	default:
	{
		return nullptr;
	}
	}
}

FFloatInterval ULandscapeEditLayerBase::GetAlphaRangeForTargetType(ELandscapeToolTargetType InType) const
{
	switch (InType)
	{
	case ELandscapeToolTargetType::Heightmap:
	{
		return FFloatInterval(-1.0f, 1.0f);
	}
	case ELandscapeToolTargetType::Weightmap:
	{
		return FFloatInterval(0.0f, 1.0f);
	}
	default:
	{
		return FFloatInterval(0.0f, 1.0f);
	}
	}
}

void ULandscapeEditLayerBase::SetGuid(const FGuid& InGuid, bool bInModify)
{
	const bool bHasValueChanged = (InGuid != Guid);
	if (bHasValueChanged)
	{
		if (bInModify)
		{
			Modify();
		}

		Guid = InGuid;
	}

	FProperty* Property = FindFProperty<FProperty>(StaticClass(), GET_MEMBER_NAME_CHECKED(ULandscapeEditLayerBase, Guid));
	check(Property != nullptr);
	BroadcastOnLayerDataChanged(FPropertyChangedEvent(Property, EPropertyChangeType::ValueSet), /*bInUserTriggered = */true, /*bInRequiresLandscapeUpdate =*/ true, bHasValueChanged);
}

const FGuid& ULandscapeEditLayerBase::GetGuid() const
{
	return Guid;
}

void ULandscapeEditLayerBase::SetName(FName InName, bool bInModify)
{
	check(OwningLandscape.IsValid())
	if (!OwningLandscape->IsLayerNameUnique(InName) || LayerName == InName)
	{
		return;
	}

	const bool bHasValueChanged = (InName != LayerName);
	if (bHasValueChanged)
	{
		if (bInModify)
		{
			Modify();
		}

		LayerName = InName;
	}

	FProperty* Property = FindFProperty<FProperty>(StaticClass(), GET_MEMBER_NAME_CHECKED(ULandscapeEditLayerBase, LayerName));
	check(Property != nullptr);
	BroadcastOnLayerDataChanged(FPropertyChangedEvent(Property, EPropertyChangeType::ValueSet), /*bInUserTriggered = */true, /*bInRequiresLandscapeUpdate =*/ true, bHasValueChanged);
}

FName ULandscapeEditLayerBase::GetName() const
{
	return LayerName;
}

void ULandscapeEditLayerBase::SetVisible(bool bInVisible, bool bInModify)
{
	const bool bHasValueChanged = (bInVisible != bVisible);
	if (bHasValueChanged)
	{
		if (bInModify)
		{
			Modify();
		}

		bVisible = bInVisible;
	}

	FProperty* Property = FindFProperty<FProperty>(StaticClass(), GET_MEMBER_NAME_CHECKED(ULandscapeEditLayerBase, bVisible));
	check(Property != nullptr);
	BroadcastOnLayerDataChanged(FPropertyChangedEvent(Property, EPropertyChangeType::ValueSet), /*bInUserTriggered = */true, /*bInRequiresLandscapeUpdate =*/ true, bHasValueChanged);
}

bool ULandscapeEditLayerBase::IsVisible() const
{
	return bVisible;
}

void ULandscapeEditLayerBase::SetLocked(bool bInLocked, bool bInModify)
{
	const bool bHasValueChanged = (bInLocked != bLocked);
	if (bHasValueChanged)
	{
		if (bInModify)
		{
			Modify();
		}

		bLocked = bInLocked;
	}
	
	FProperty* Property = FindFProperty<FProperty>(StaticClass(), GET_MEMBER_NAME_CHECKED(ULandscapeEditLayerBase, bLocked));
	check(Property != nullptr);
	BroadcastOnLayerDataChanged(FPropertyChangedEvent(Property, EPropertyChangeType::ValueSet), /*bInUserTriggered = */true, /*bInRequiresLandscapeUpdate =*/ true, bHasValueChanged);
}

bool ULandscapeEditLayerBase::IsLocked() const
{
	return bLocked;
}

ELandscapeBlendMode ULandscapeEditLayerBase::GetBlendMode() const
{
	return ELandscapeBlendMode::LSBM_AdditiveBlend;
}

bool ULandscapeEditLayerBase::RemoveAndCopyWeightmapAllocationLayerBlend(TObjectPtr<ULandscapeLayerInfoObject> InKey, bool& bOutValue, bool bInModify)
{
	bool bHasValueChanged = false;

	if (WeightmapLayerAllocationBlend.RemoveAndCopyValue(InKey, bOutValue))
	{
		Modify(bInModify);

		bHasValueChanged = true;
	}

	FProperty* Property = FindFProperty<FProperty>(StaticClass(), GET_MEMBER_NAME_CHECKED(ULandscapeEditLayerBase, WeightmapLayerAllocationBlend));
	check(Property != nullptr);
	BroadcastOnLayerDataChanged(FPropertyChangedEvent(Property, EPropertyChangeType::ValueSet), /*bInUserTriggered = */true, /*bInRequiresLandscapeUpdate =*/ true, bHasValueChanged);

	// return true if and only if RemoveAndCopy returns true
	return bHasValueChanged;
}

void ULandscapeEditLayerBase::AddOrUpdateWeightmapAllocationLayerBlend(TObjectPtr<ULandscapeLayerInfoObject> InKey, bool bInValue, bool bInModify)
{ 
	const bool* bValueExists = WeightmapLayerAllocationBlend.Find(InKey);
	bool& bFoundValue = WeightmapLayerAllocationBlend.FindOrAdd(InKey, bInValue);

	// changed if existing value has been toggled or a new entry is added to the map
	const bool bHasValueChanged = ((bValueExists == nullptr) || (bFoundValue != bInValue));

	bFoundValue = bInValue;
	
	if (bHasValueChanged)
	{
		if (bInModify)
		{
			Modify();
		}
	}

	FProperty* Property = FindFProperty<FProperty>(StaticClass(), GET_MEMBER_NAME_CHECKED(ULandscapeEditLayerBase, WeightmapLayerAllocationBlend));
	check(Property != nullptr);
	BroadcastOnLayerDataChanged(FPropertyChangedEvent(Property, EPropertyChangeType::ValueSet), /*bInUserTriggered = */true, /*bInRequiresLandscapeUpdate =*/ true, bHasValueChanged);
}

const TMap<TObjectPtr<ULandscapeLayerInfoObject>, bool>& ULandscapeEditLayerBase::GetWeightmapLayerAllocationBlend() const
{
	return WeightmapLayerAllocationBlend;
}

void ULandscapeEditLayerBase::SetWeightmapLayerAllocationBlend(const TMap<TObjectPtr<ULandscapeLayerInfoObject>, bool>& InWeightmapLayerAllocationBlend, bool bInModify)
{
	const bool bHasValueChanged = !(WeightmapLayerAllocationBlend.Num() == InWeightmapLayerAllocationBlend.Num() &&
		Algo::AllOf(WeightmapLayerAllocationBlend, [&InWeightmapLayerAllocationBlend](const TPair<TObjectPtr<ULandscapeLayerInfoObject>, bool>& ExistingPair)
			{
				const bool* BlendValue = InWeightmapLayerAllocationBlend.Find(ExistingPair.Key);
				return BlendValue != nullptr && *BlendValue == ExistingPair.Value;
			}));

	WeightmapLayerAllocationBlend = InWeightmapLayerAllocationBlend;

	if (bHasValueChanged)
	{
		if (bInModify)
		{
			Modify();
		}
	}

	FProperty* Property = FindFProperty<FProperty>(StaticClass(), GET_MEMBER_NAME_CHECKED(ULandscapeEditLayerBase, WeightmapLayerAllocationBlend));
	check(Property != nullptr);
	BroadcastOnLayerDataChanged(FPropertyChangedEvent(Property, EPropertyChangeType::ValueSet), /*bInUserTriggered = */true, /*bInRequiresLandscapeUpdate =*/ true, bHasValueChanged);
}

void ULandscapeEditLayerBase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Currently, every property requires a landscape update but this could be specialized per property to avoid unnecessary updates when new properties are added
	BroadcastOnLayerDataChanged(PropertyChangedEvent, /*bInUserTriggered = */true, /* bRequiresLandscapeUpdate =*/ true, /*bHasValueChanged =*/true);
}

void ULandscapeEditLayerBase::PostEditUndo()
{
	Super::PostEditUndo();

	BroadcastOnLayerDataChanged(FPropertyChangedEvent(/*InProperty = */nullptr), /*bInUserTriggered = */false, /*bInRequiresLandscapeUpdate =*/ true, /*bInHasValueChanged = */true);
}

bool ULandscapeEditLayerBase::CanEditChange(const FProperty* InProperty) const
{
	bool bCanEdit = Super::CanEditChange(InProperty);

	const FName MemberPropertyName = InProperty ? InProperty->GetFName() : NAME_None;

	// Always able to update locked property
	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ULandscapeEditLayerBase, bLocked))
	{
		return true;
	}

	// All other properties disabled when layer is locked
	if (IsLocked())
	{
		return false;
	}

	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ULandscapeEditLayerBase, HeightmapAlpha))
	{
		bCanEdit &= SupportsAlphaForTargetType(ELandscapeToolTargetType::Heightmap);
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(ULandscapeEditLayerBase, WeightmapAlpha))
	{
		bCanEdit &= SupportsAlphaForTargetType(ELandscapeToolTargetType::Weightmap);
	}

	return bCanEdit;
}

void ULandscapeEditLayerBase::PostLoad()
{
	Super::PostLoad();

	// Needed because we might have saved some layers before we realized we were missing this flag
	SetFlags(RF_Transactional);
}

void ULandscapeEditLayerBase::BroadcastOnLayerDataChanged(const FPropertyChangedEvent& InPropertyChangeEvent, bool bInUserTriggered, bool bInRequiresLandscapeUpdate, bool bInHasValueChanged)
{
	FOnLandscapeEditLayerDataChangedParams OnLayerDataChangedParams(InPropertyChangeEvent);
	OnLayerDataChangedParams.bUserTriggered = bInUserTriggered;
	OnLayerDataChangedParams.bRequiresLandscapeUpdate = bInRequiresLandscapeUpdate;
	OnLayerDataChangedParams.bHasValueChanged = bInHasValueChanged;
	OnLayerDataChangedDelegate.Broadcast(OnLayerDataChangedParams);
}

ELandscapeToolTargetTypeFlags ULandscapeEditLayerBase::GetEnabledTargetTypeMask() const
{
	// Compute the default state of each target type : 
	ELandscapeToolTargetTypeFlags EnabledTargetTypeMask = ELandscapeToolTargetTypeFlags::None;
	if (IsVisible())
	{
		//  HeightmapAlpha might still be set to a different value)
		if (GetAlphaForTargetType(ELandscapeToolTargetType::Heightmap) != 0.0f)
		{
			EnabledTargetTypeMask |= ELandscapeToolTargetTypeFlags::Heightmap;
		}
		if (GetAlphaForTargetType(ELandscapeToolTargetType::Weightmap) > 0.0f)
		{
			EnabledTargetTypeMask |= ELandscapeToolTargetTypeFlags::Weightmap;
		}
		// If the layer is visible, visibility is always considered enabled since it does not depend on WeightmapAlpha 
		EnabledTargetTypeMask |= ELandscapeToolTargetTypeFlags::Visibility;
	}
	return EnabledTargetTypeMask;
}

void ULandscapeEditLayerBase::SetBackPointer(ALandscape* Landscape)
{
	OwningLandscape = Landscape;
}
#endif //WITH_EDITOR

#if WITH_EDITORONLY_DATA
void ULandscapeEditLayerBase::SetHeightmapAlphaInternal(float InNewValue)
{
	SetAlphaForTargetType(ELandscapeToolTargetType::Heightmap, InNewValue, /*bInModify = */true, /*InChangeType = */EPropertyChangeType::ValueSet);
}

void ULandscapeEditLayerBase::SetWeightmapAlphaInternal(float InNewValue)
{
	SetAlphaForTargetType(ELandscapeToolTargetType::Weightmap, InNewValue, /*bInModify = */true, /*InChangeType = */EPropertyChangeType::ValueSet);
}

void ULandscapeEditLayerBase::SetGuidInternal(const FGuid& InGuid)
{
	SetGuid(InGuid, /*bInModify = */true);
}

void ULandscapeEditLayerBase::SetNameInternal(FName InName)
{
	SetName(InName, /*bInModify = */true);
}

void ULandscapeEditLayerBase::SetVisibleInternal(bool bInVisible)
{
	SetVisible(bInVisible, /*bInModify = */true);
}

void ULandscapeEditLayerBase::SetLockedInternal(bool bInLocked)
{
	SetLocked(bInLocked, /*bInModify = */true);
}

void ULandscapeEditLayerBase::SetWeightmapLayerAllocationBlendInternal(const TMap<TObjectPtr<ULandscapeLayerInfoObject>, bool>& InWeightmapLayerAllocationBlend)
{
	SetWeightmapLayerAllocationBlend(InWeightmapLayerAllocationBlend, /*bInModify = */true);
}

#endif // WITH_EDITORONLY_DATA

FName ULandscapeEditLayerBase::GetNameBP() const
{
#if WITH_EDITOR
	return GetName();
#else // WITH_EDITOR
	return NAME_None;
#endif // !WITH_EDITOR
}


// ----------------------------------------------------------------------------------

bool ULandscapeEditLayer::SupportsTargetType(ELandscapeToolTargetType InType) const
{
	return (InType == ELandscapeToolTargetType::Heightmap) || (InType == ELandscapeToolTargetType::Weightmap) || (InType == ELandscapeToolTargetType::Visibility);
}

// ----------------------------------------------------------------------------------

bool ULandscapeEditLayerSplines::SupportsTargetType(ELandscapeToolTargetType InType) const
{
	return (InType == ELandscapeToolTargetType::Heightmap) || (InType == ELandscapeToolTargetType::Weightmap) || (InType == ELandscapeToolTargetType::Visibility);
}

#undef LOCTEXT_NAMESPACE
