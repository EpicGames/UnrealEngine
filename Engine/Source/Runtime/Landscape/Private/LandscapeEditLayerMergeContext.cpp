// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeEditLayerMergeContext.h"
#include "Algo/AllOf.h"
#include "Algo/Transform.h"
#include "Landscape.h"
#include "LandscapeInfo.h"
#include "LandscapeLayerInfoObject.h"
#include "LandscapeEditLayerTargetLayerGroup.h"
#include "Materials/MaterialExpressionLandscapeVisibilityMask.h"

namespace UE::Landscape::EditLayers
{

#if WITH_EDITOR

FMergeContext::FMergeContext(ALandscape* InLandscape, bool bInIsHeightmapMerge, bool bInSkipProceduralRenderers)
	: bIsHeightmapMerge(bInIsHeightmapMerge)
	, bSkipProceduralRenderers(bInSkipProceduralRenderers)
	, Landscape(InLandscape)
	, LandscapeInfo(InLandscape->GetLandscapeInfo())
{
	check(LandscapeInfo != nullptr);

	// Start by gathering all possible target layer names on this landscape: 
	//  This list of all unique target layer names will help accelerate the gathering of output layers on each component (using bit arrays) as well as the target layers intersection tests:
	if (bInIsHeightmapMerge)
	{
		// Only one target layer in the case of heightmap: 
		const FName TargetLayerName = FName("Height");
		AllTargetLayerNames = { TargetLayerName };
		// And it's valid :
		ValidTargetLayerBitIndices = TBitArray<>(true, 1);
		VisibilityTargetLayerMask = TBitArray<>(false, 1);
		AllWeightmapLayerInfos = { nullptr };
	}
	else
	{
		// Gather all target layer names and mark those that are valid layers :
		for (const FLandscapeInfoLayerSettings& LayerSettings : LandscapeInfo->Layers)
		{
			check(!LayerSettings.LayerName.IsNone());
			ULandscapeLayerInfoObject*& LayerInfo = AllWeightmapLayerInfos.Add_GetRef(LayerSettings.LayerInfoObj);
			AllTargetLayerNames.Add(LayerSettings.LayerName);
			ValidTargetLayerBitIndices.Add(LayerInfo != nullptr);
		}

		// Visibility is always a valid layer : 
		VisibilityTargetLayerIndex = AllTargetLayerNames.Find(UMaterialExpressionLandscapeVisibilityMask::ParameterName);
		if (VisibilityTargetLayerIndex == INDEX_NONE)
		{
			VisibilityTargetLayerIndex = AllTargetLayerNames.Num();
			AllTargetLayerNames.Add(UMaterialExpressionLandscapeVisibilityMask::ParameterName);
			check(!AllWeightmapLayerInfos.Contains(ALandscape::VisibilityLayer));
			AllWeightmapLayerInfos.Add(ALandscape::VisibilityLayer);
			check(ALandscape::VisibilityLayer != nullptr);
			ValidTargetLayerBitIndices.Add(true);
		}
		check(VisibilityTargetLayerIndex != INDEX_NONE);

		// Now that we have all valid target layers ready, analyze them to prepare the target layer groups as they are declared within the layer infos : 
		FinalizeTargetLayerGroupsPerBlendingMethod();

		VisibilityTargetLayerMask = TBitArray<>(false, AllTargetLayerNames.Num());
		VisibilityTargetLayerMask[VisibilityTargetLayerIndex] = true;
	}

	NegatedVisibilityTargetLayerMask = VisibilityTargetLayerMask;
	NegatedVisibilityTargetLayerMask.BitwiseNOT();
}

TArray<FName> FMergeContext::GetValidTargetLayerNames() const
{
	return bIsHeightmapMerge ? AllTargetLayerNames : ConvertTargetLayerBitIndicesToNames(ValidTargetLayerBitIndices);
}

int32 FMergeContext::IsValidTargetLayerName(const FName& InName) const
{
	int32 Index = GetTargetLayerIndexForName(InName);
	return (Index != INDEX_NONE) ? ValidTargetLayerBitIndices[Index] : false;
}

int32 FMergeContext::IsValidTargetLayerNameChecked(const FName& InName) const
{
	int32 Index = AllTargetLayerNames.Find(InName);
	check(Index != INDEX_NONE);
	return ValidTargetLayerBitIndices[Index];
}

int32 FMergeContext::IsTargetLayerIndexValid(int32 InIndex) const
{
	return AllTargetLayerNames.IsValidIndex(InIndex);
}

int32 FMergeContext::GetTargetLayerIndexForName(const FName& InName) const
{
	return AllTargetLayerNames.Find(InName);
}

int32 FMergeContext::GetTargetLayerIndexForNameChecked(const FName& InName) const
{
	int32 Index = AllTargetLayerNames.Find(InName);
	check(Index != INDEX_NONE);
	return Index;
}

FName FMergeContext::GetTargetLayerNameForIndex(int32 InIndex) const
{
	return AllTargetLayerNames.IsValidIndex(InIndex) ? AllTargetLayerNames[InIndex] : NAME_None;
}

FName FMergeContext::GetTargetLayerNameForIndexChecked(int32 InIndex) const
{
	check(AllTargetLayerNames.IsValidIndex(InIndex));
	return AllTargetLayerNames[InIndex];
}

int32 FMergeContext::GetTargetLayerIndexForLayerInfo(ULandscapeLayerInfoObject* InLayerInfo) const
{
	return AllWeightmapLayerInfos.Find(InLayerInfo);
}

int32 FMergeContext::GetTargetLayerIndexForLayerInfoChecked(ULandscapeLayerInfoObject* InLayerInfo) const
{
	int32 Index = AllWeightmapLayerInfos.Find(InLayerInfo);;
	check(Index != INDEX_NONE);
	return Index;
}

ULandscapeLayerInfoObject* FMergeContext::GetTargetLayerInfoForName(const FName& InName) const
{
	int32 Index = GetTargetLayerIndexForName(InName);
	return (Index != INDEX_NONE) ? AllWeightmapLayerInfos[Index] : nullptr;
}

ULandscapeLayerInfoObject* FMergeContext::GetTargetLayerInfoForNameChecked(const FName& InName) const
{
	int32 Index = GetTargetLayerIndexForNameChecked(InName);
	return AllWeightmapLayerInfos[Index];
}

ULandscapeLayerInfoObject* FMergeContext::GetTargetLayerInfoForIndex(int32 InIndex) const
{
	check(AllTargetLayerNames.IsValidIndex(InIndex));
	return AllWeightmapLayerInfos[InIndex];
}

TBitArray<> FMergeContext::ConvertTargetLayerNamesToBitIndices(TConstArrayView<FName> InTargetLayerNames) const
{
	TBitArray<> Result(false, AllTargetLayerNames.Num());
	for (FName Name : InTargetLayerNames)
	{
		if (int32 Index = GetTargetLayerIndexForName(Name); Index != INDEX_NONE)
		{
			Result[Index] = true;
		}
	}
	return Result;
}

TBitArray<> FMergeContext::ConvertTargetLayerNamesToBitIndicesChecked(TConstArrayView<FName> InTargetLayerNames) const
{
	TBitArray<> Result(false, AllTargetLayerNames.Num());
	for (FName Name : InTargetLayerNames)
	{
		int32 Index = GetTargetLayerIndexForNameChecked(Name);
		Result[Index] = true;
	}
	return Result;
}

TArray<FName> FMergeContext::ConvertTargetLayerBitIndicesToNames(const TBitArray<>& InTargetLayerBitIndices) const
{
	const int32 NumNames = AllTargetLayerNames.Num();
	check(InTargetLayerBitIndices.Num() == NumNames);
	TArray<FName> Names;
	Names.Reserve(NumNames);
	for (TConstSetBitIterator It(InTargetLayerBitIndices); It; ++It)
	{
		Names.Add(AllTargetLayerNames[It.GetIndex()]);
	}
	return Names;
}

TArray<ULandscapeLayerInfoObject*> FMergeContext::ConvertTargetLayerBitIndicesToLayerInfos(const TBitArray<>& InTargetLayerBitIndices) const
{
	const int32 NumTargetLayerInfos = AllTargetLayerNames.Num();
	check(InTargetLayerBitIndices.Num() == NumTargetLayerInfos);
	TArray<ULandscapeLayerInfoObject*> LayerInfos;
	LayerInfos.Reserve(NumTargetLayerInfos);
	for (TConstSetBitIterator It(InTargetLayerBitIndices); It; ++It)
	{
		LayerInfos.Add(AllWeightmapLayerInfos[It.GetIndex()]);
	}
	return LayerInfos;
}

void FMergeContext::ForEachTargetLayer(const TBitArray<>& InTargetLayerBitIndices, TFunctionRef<bool(int32 /*InTargetLayerIndex*/, const FName& /*InTargetLayerName*/, ULandscapeLayerInfoObject* /*InWeightmapLayerInfo*/)> Fn) const
{
	check(InTargetLayerBitIndices.Num() == AllTargetLayerNames.Num());
	for (TConstSetBitIterator It(InTargetLayerBitIndices); It; ++It)
	{
		const int32 TargetLayerIndex = It.GetIndex();
		if (!AllTargetLayerNames.IsValidIndex(TargetLayerIndex))
		{
			return;
		}

		if (!Fn(TargetLayerIndex, AllTargetLayerNames[TargetLayerIndex], AllWeightmapLayerInfos[TargetLayerIndex]))
		{
			return;
		}
	}
}

void FMergeContext::ForEachValidTargetLayer(TFunctionRef<bool(int32 /*InTargetLayerIndex*/, const FName& /*InTargetLayerName*/, ULandscapeLayerInfoObject* /*InWeightmapLayerInfo*/)> Fn) const
{
	return ForEachTargetLayer(ValidTargetLayerBitIndices, Fn);
}

TBitArray<> FMergeContext::BuildTargetLayerBitIndices(bool bInBitValue) const
{
	return TBitArray<>(bInBitValue, AllTargetLayerNames.Num());
}

TArray<FTargetLayerGroup> FMergeContext::BuildGenericBlendTargetLayerGroups(const TBitArray<>& InTargetLayerBitIndices) const
{
	// Only ELandscapeTargetLayerBlendMethod::PremultipliedAlphaBlending is applied per renderer, so we only need to retrieve the target layer groups
	//  for the specified target layers and trim them of all other target layers
	const TArray<FTargetLayerGroup>& SourceTargetLayerGroups = TargetLayerGroupsPerBlendingMethod[static_cast<int32>(ELandscapeTargetLayerBlendMethod::PremultipliedAlphaBlending)];

	// First, intersect our layer groups with the specified target layers 
	TArray<FTargetLayerGroup> FilteredTargetLayerGroups;
	FilteredTargetLayerGroups.Reserve(SourceTargetLayerGroups.Num());
	Algo::Transform(SourceTargetLayerGroups, FilteredTargetLayerGroups, [&InTargetLayerBitIndices](const FTargetLayerGroup& InTargetLayerGroup)
		{
			return FTargetLayerGroup(InTargetLayerGroup.GetName(), TBitArray<>::BitwiseAND(InTargetLayerGroup.GetWeightmapTargetLayerBitIndices(), InTargetLayerBitIndices, EBitwiseOperatorFlags::MinSize));
		});
	
	// Then remove all those that are now empty : 
	FilteredTargetLayerGroups.RemoveAllSwap([](const FTargetLayerGroup& InTargetLayerGroup) { return !InTargetLayerGroup.GetWeightmapTargetLayerBitIndices().Contains(true); });

	return FilteredTargetLayerGroups;
}

void FMergeContext::FinalizeTargetLayerGroupsPerBlendingMethod()
{
	for (int32 Index = 0; Index < static_cast<int32>(ELandscapeTargetLayerBlendMethod::Count); ++Index)
	{
		TargetLayerGroupsPerBlendingMethod[Index] = GatherTargetLayerGroupsForBlendMethod(static_cast<ELandscapeTargetLayerBlendMethod>(Index));
	}
}

TArray<FTargetLayerGroup> FMergeContext::GatherTargetLayerGroupsForBlendMethod(ELandscapeTargetLayerBlendMethod InBlendMethod) const
{
	TArray<FTargetLayerGroup> TargetLayerGroups;
	switch (InBlendMethod)
	{
	case ELandscapeTargetLayerBlendMethod::None:
	{
		// Returning an empty layer group list for the no weight-blending case is fine because it will be treated as N independent layer groups (each with a single target layer)
	}
	break;
	case ELandscapeTargetLayerBlendMethod::FinalWeightBlending:
	{
		TBitArray<> WeightBlendedWeightmapLayerBitIndices = BuildTargetLayerBitIndices(/*bInBitValue = */false);
		ForEachValidTargetLayer(
			[&WeightBlendedWeightmapLayerBitIndices](int32 InTargetLayerIndex, const FName& InTargetLayerName, ULandscapeLayerInfoObject* InWeightmapLayerInfo)
			{
				check(InWeightmapLayerInfo != nullptr);
				WeightBlendedWeightmapLayerBitIndices[InTargetLayerIndex] = (InWeightmapLayerInfo->GetBlendMethod() == ELandscapeTargetLayerBlendMethod::FinalWeightBlending);
				return true;
			});
		static const FName FinalWeightBlendingLayerGroupName("FinalWeightBlendingLayerGroup");
		TargetLayerGroups.Emplace(FinalWeightBlendingLayerGroupName, WeightBlendedWeightmapLayerBitIndices);
	}
	break;
	case ELandscapeTargetLayerBlendMethod::PremultipliedAlphaBlending:
	{
		// Find all ELandscapeTargetLayerBlendMethod::PremultipliedAlphaBlending layers and segregate them by blend group :
		TMap<FName, TBitArray<>> TargetLayerGroupsMap;
		const TBitArray<> DefaultTargetLayerBitIndices = BuildTargetLayerBitIndices(/*bInBitValue = */false);
		ForEachValidTargetLayer(
			[&TargetLayerGroupsMap, &DefaultTargetLayerBitIndices](int32 InTargetLayerIndex, const FName& InTargetLayerName, ULandscapeLayerInfoObject* InWeightmapLayerInfo)
			{
				check(InWeightmapLayerInfo != nullptr);
				if (InWeightmapLayerInfo->GetBlendMethod() == ELandscapeTargetLayerBlendMethod::PremultipliedAlphaBlending)
				{
					TBitArray<>& TargetLayerGroup = TargetLayerGroupsMap.FindOrAdd(InWeightmapLayerInfo->GetBlendGroup(), DefaultTargetLayerBitIndices);
					TargetLayerGroup[InTargetLayerIndex] = true;
				}
				return true;
			});

		Algo::Transform(TargetLayerGroupsMap, TargetLayerGroups, [](const TPair<FName, TBitArray<>>& InTargetLayerGroupPair) { return FTargetLayerGroup(InTargetLayerGroupPair.Key, InTargetLayerGroupPair.Value); });
	}
	break;
	default:
		check(false);
	}

	return TargetLayerGroups;
}

#endif // WITH_EDITOR

} // namespace UE::Landscape::EditLayers
