// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeEditLayerMergeRenderContext.h"
#include "Algo/AllOf.h"
#include "Algo/Transform.h"
#include "Algo/Count.h"
#include "Algo/ForEach.h"
#include "Algo/Sort.h"
#include "Engine/Engine.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "EngineModule.h"
#include "Landscape.h"
#include "LandscapeComponent.h"
#include "LandscapeEditLayerMergeRenderBlackboardItem.h"
#include "LandscapeEditLayerRenderer.h"
#include "LandscapeEditResourcesSubsystem.h"
#include "LandscapeLayerInfoObject.h"
#include "LandscapePrivate.h"
#include "LandscapeUtils.h"
#include "Materials/MaterialExpressionLandscapeVisibilityMask.h"
#include "PixelShaderUtils.h"
#include "RenderGraph.h"
#include "RenderingThread.h"
#include "RHIAccess.h"
#include "SceneView.h"
#include "TextureResource.h"
#include "VisualLogger/VisualLogger.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"

extern TAutoConsoleVariable<float> CVarLandscapeBatchedMergeVisualLogOffsetIncrement;
extern TAutoConsoleVariable<int32> CVarLandscapeBatchedMergeClearBeforeEachWriteToScratch;
extern TAutoConsoleVariable<int32> CVarLandscapeBatchedMergeVisualLogShowMergeType;
extern TAutoConsoleVariable<int32> CVarLandscapeBatchedMergeVisualLogShowMergeProcess;
extern TAutoConsoleVariable<float> CVarLandscapeBatchedMergeVisualLogAlpha;

namespace UE::Landscape::EditLayers
{

// ----------------------------------------------------------------------------------
// /Engine/Private/Landscape/LandscapeEditLayersUtils.usf shaders : 

class FRenderComponentIdPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FRenderComponentIdPS);
	SHADER_USE_PARAMETER_STRUCT(FRenderComponentIdPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, InQuadComponentIds)
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return UE::Landscape::DoesPlatformSupportEditLayers(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		UE::Landscape::ModifyShaderCompilerEnvironmentForDebug(OutEnvironment);

		OutEnvironment.SetDefine(TEXT("RENDER_COMPONENTID"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FRenderComponentIdPS, "/Engine/Private/Landscape/LandscapeEditLayersUtils.usf", "RenderComponentIdPS", SF_Pixel);

BEGIN_SHADER_PARAMETER_STRUCT(FRenderComponentIdPSParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FPixelShaderUtils::FRasterizeToRectsVS::FParameters, VS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FRenderComponentIdPS::FParameters, PS)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()


// ----------------------------------------------------------------------------------

class FCopyQuadsPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FCopyQuadsPS);
	SHADER_USE_PARAMETER_STRUCT(FCopyQuadsPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, InSourceTexture)
	END_SHADER_PARAMETER_STRUCT()

	using FPermutationDomain = TShaderPermutationDomain<>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return UE::Landscape::DoesPlatformSupportEditLayers(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		UE::Landscape::ModifyShaderCompilerEnvironmentForDebug(OutEnvironment);

		OutEnvironment.SetDefine(TEXT("COPY_QUADS"), 1);
	}
};

IMPLEMENT_GLOBAL_SHADER(FCopyQuadsPS, "/Engine/Private/Landscape/LandscapeEditLayersUtils.usf", "CopyQuadsPS", SF_Pixel);

BEGIN_SHADER_PARAMETER_STRUCT(FCopyQuadsPSParameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FPixelShaderUtils::FRasterizeToRectsVS::FParameters, VS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FCopyQuadsPS::FParameters, PS)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()


// ----------------------------------------------------------------------------------
// /Engine/Private/Landscape/LandscapeEditLayersHeightmaps.usf shaders :

class FLandscapeEditLayersHeightmapsMergeEditLayerPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FLandscapeEditLayersHeightmapsMergeEditLayerPS);
	SHADER_USE_PARAMETER_STRUCT(FLandscapeEditLayersHeightmapsMergeEditLayerPS, FGlobalShader);

public:
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, InEditLayerBlendMode)
		SHADER_PARAMETER(float, InEditLayerAlpha)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, InCurrentEditLayerHeightmap)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, InPreviousEditLayersHeightmap)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& InParameters)
	{
		return UE::Landscape::DoesPlatformSupportEditLayers(InParameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		UE::Landscape::ModifyShaderCompilerEnvironmentForDebug(OutEnvironment);

		OutEnvironment.SetDefine(TEXT("MERGE_EDIT_LAYER"), 1);
	}

	static void MergeEditLayerPS(FRDGEventName&& InRDGEventName, FRDGBuilder& GraphBuilder, FParameters* InParameters, const FIntPoint& InTextureSize)
	{
		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FLandscapeEditLayersHeightmapsMergeEditLayerPS> PixelShader(ShaderMap);

		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			ShaderMap,
			MoveTemp(InRDGEventName),
			PixelShader,
			InParameters,
			FIntRect(0, 0, InTextureSize.X, InTextureSize.Y),
			TStaticBlendStateWriteMask<CW_RG>::GetRHI());
	}
};

IMPLEMENT_GLOBAL_SHADER(FLandscapeEditLayersHeightmapsMergeEditLayerPS, "/Engine/Private/Landscape/LandscapeEditLayersHeightmaps.usf", "MergeEditLayerPS", SF_Pixel);


// ----------------------------------------------------------------------------------
// /Engine/Private/Landscape/LandscapeEditLayersWeightmaps.usf shaders :

class FLandscapeEditLayersWeightmapsMergeEditLayerPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FLandscapeEditLayersWeightmapsMergeEditLayerPS);
	SHADER_USE_PARAMETER_STRUCT(FLandscapeEditLayersWeightmapsMergeEditLayerPS, FGlobalShader);

public:
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, InTargetLayerIndex)
		SHADER_PARAMETER(uint32, InNumTargetLayers)
		SHADER_PARAMETER(uint32, InEditLayerTargetLayerBlendMode)
		SHADER_PARAMETER(float, InEditLayerAlpha)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<UE::Landscape::FMergeEditLayerTargetLayerInfo>, InMergeEditLayerTargetLayerInfos)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2DArray<float4>, InCurrentEditLayerWeightmaps)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2DArray<float4>, InPreviousEditLayersWeightmaps)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& InParameters)
	{
		return UE::Landscape::DoesPlatformSupportEditLayers(InParameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		UE::Landscape::ModifyShaderCompilerEnvironmentForDebug(OutEnvironment);

		OutEnvironment.SetDefine(TEXT("MERGE_EDIT_LAYER"), 1);
	}

	static void MergeEditLayerPS(FRDGEventName&& InRDGEventName, FRDGBuilder& GraphBuilder, FParameters* InParameters, const FIntPoint& InTextureSize)
	{
		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FLandscapeEditLayersWeightmapsMergeEditLayerPS> PixelShader(ShaderMap);

		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			ShaderMap,
			MoveTemp(InRDGEventName),
			PixelShader,
			InParameters,
			FIntRect(0, 0, InTextureSize.X, InTextureSize.Y),
			TStaticBlendStateWriteMask<CW_RG>::GetRHI());
	}
};

IMPLEMENT_GLOBAL_SHADER(FLandscapeEditLayersWeightmapsMergeEditLayerPS, "/Engine/Private/Landscape/LandscapeEditLayersWeightmaps.usf", "MergeEditLayerPS", SF_Pixel);


// ----------------------------------------------------------------------------------

#if WITH_EDITOR


FRenderParams::FRenderParams(
	FMergeRenderContext* InMergeRenderContext,
	const TArrayView<FName>& InTargetLayerGroupLayerNames,
	const TArrayView<ULandscapeLayerInfoObject*>& InTargetLayerGroupLayerInfos,
	const FEditLayerRendererState& InRendererState,
	const TArrayView<FComponentMergeRenderInfo>& InSortedComponentMergeRenderInfos,
	const FTransform& InRenderAreaWorldTransform,
	const FIntRect& InRenderAreaSectionRect,
	int32 InNumSuccessfulRenderLayerStepsUntilBlendLayerStep)
	: MergeRenderContext(InMergeRenderContext)
	, TargetLayerGroupLayerNames(InTargetLayerGroupLayerNames)
	, TargetLayerGroupLayerInfos(InTargetLayerGroupLayerInfos)
	, RendererState(InRendererState)
	, SortedComponentMergeRenderInfos(InSortedComponentMergeRenderInfos)
	, RenderAreaWorldTransform(InRenderAreaWorldTransform)
	, RenderAreaSectionRect(InRenderAreaSectionRect)
	, NumSuccessfulRenderLayerStepsUntilBlendLayerStep(InNumSuccessfulRenderLayerStepsUntilBlendLayerStep)
{
}


// ----------------------------------------------------------------------------------

bool FMergeRenderBatch::operator<(const FMergeRenderBatch& InOther) const
{
	// Sort batches in a deterministic order (upper-left -> upper-right -> lower-left- -> lower-right) to ensure that components with a conflict on their
	//  borders get rendered in a deterministic way : 
	auto ComputeSortIndex = [this](const FIntPoint& InSectionBase) -> int64
	{
		int64 AbsoluteLinearIndex = ((int64)InSectionBase.X - LandscapeExtent.Min.X) + ((int64)InSectionBase.Y - LandscapeExtent.Min.Y) * ((int64)LandscapeExtent.Max.X - LandscapeExtent.Min.X + 1);
		check(AbsoluteLinearIndex >= 0);
		return AbsoluteLinearIndex;
	};
	
	int64 LinearIndex = ComputeSortIndex(SectionRect.Min);
	int64 OtherLinearIndex = ComputeSortIndex(InOther.SectionRect.Min);
	if (LinearIndex < OtherLinearIndex)
	{
		return true;
	}
	else if (LinearIndex == OtherLinearIndex)
	{
		LinearIndex = ComputeSortIndex(SectionRect.Max);
		OtherLinearIndex = ComputeSortIndex(InOther.SectionRect.Max);
		return LinearIndex < OtherLinearIndex;
	}
	return false;
}


int32 FMergeRenderBatch::ComputeSubsectionRects(ULandscapeComponent* InComponent, TArray<FIntRect, TInlineAllocator<4>>& OutSubsectionRects, TArray<FIntRect, TInlineAllocator<4>>& OutSubsectionRectsWithDuplicateBorders) const
{
	check(ComponentsToRender.Contains(InComponent));
	const int32 NumSubsections = Landscape->NumSubsections;
	const int32 ComponentSizeQuads = Landscape->ComponentSizeQuads;
	const int32 SubsectionSizeQuads = Landscape->SubsectionSizeQuads;
	const int32 SubsectionVerts = SubsectionSizeQuads + 1;
	const int32 TotalNumSubsections = NumSubsections * NumSubsections;
	OutSubsectionRects.Reserve(TotalNumSubsections);
	OutSubsectionRectsWithDuplicateBorders.Reserve(TotalNumSubsections);

	const FIntPoint ComponentSectionBase = InComponent->GetSectionBase();
	checkf((ComponentSectionBase.X >= SectionRect.Min.X) && (ComponentSectionBase.Y >= SectionRect.Min.Y)
		&& ((ComponentSectionBase.X + ComponentSizeQuads + 1) <= SectionRect.Max.X) && ((ComponentSectionBase.Y + ComponentSizeQuads + 1) <= SectionRect.Max.Y), 
		TEXT("The requested component is not included in the render batch"));

	const FIntPoint ComponentLocalKey = (ComponentSectionBase - SectionRect.Min) / ComponentSizeQuads;
	for (int32 SubY = 0; SubY < NumSubsections; ++SubY)
	{
		for (int32 SubX = 0; SubX < NumSubsections; ++SubX)
		{
			{
				FIntPoint SubSectionMin = ComponentSectionBase - SectionRect.Min + FIntPoint(SubX * SubsectionSizeQuads, SubY * SubsectionSizeQuads);
				FIntPoint SubSectionMax = SubSectionMin + FIntPoint(SubsectionVerts, SubsectionVerts);
				OutSubsectionRects.Add(FIntRect(SubSectionMin, SubSectionMax));
			}
			{
				FIntPoint SubSectionMin = (ComponentLocalKey * NumSubsections + FIntPoint(SubX, SubY)) * SubsectionVerts;
				FIntPoint SubSectionMax = SubSectionMin + SubsectionVerts;
				OutSubsectionRectsWithDuplicateBorders.Add(FIntRect(SubSectionMin, SubSectionMax));
			}
		}
	}

	return TotalNumSubsections;
}

FIntRect FMergeRenderBatch::ComputeSectionRect(ULandscapeComponent* InComponent, bool bInWithDuplicateBorders) const
{
	check(ComponentsToRender.Contains(InComponent));

	const FIntPoint ComponentSectionBase = InComponent->GetSectionBase();
	checkf((ComponentSectionBase.X >= SectionRect.Min.X) && (ComponentSectionBase.Y >= SectionRect.Min.Y)
		&& ((ComponentSectionBase.X + InComponent->ComponentSizeQuads + 1) <= SectionRect.Max.X) && ((ComponentSectionBase.Y + InComponent->ComponentSizeQuads + 1) <= SectionRect.Max.Y),
		TEXT("The requested component is not included in the render batch"));

	const FIntPoint ComponentLocalKey = (ComponentSectionBase - SectionRect.Min) / InComponent->ComponentSizeQuads;
	const int32 ComponentSubsectionVerts = InComponent->SubsectionSizeQuads + 1;
	
	const int32 ComponentSize = InComponent->NumSubsections * (bInWithDuplicateBorders ? ComponentSubsectionVerts : InComponent->SubsectionSizeQuads);
	FIntPoint SectionMin = ComponentLocalKey * ComponentSize;
	FIntPoint SectionMax = SectionMin + ComponentSize;

	return FIntRect(SectionMin, SectionMax);
}

TArray<FIntRect> FMergeRenderBatch::ComputeAllComponentSectionRects(bool bInWithDuplicateBorders) const
{
	TArray<FIntRect> ComponentRects;
	ComponentRects.Reserve(ComponentsToRender.Num());
	for (ULandscapeComponent* Component : ComponentsToRender)
	{
		ComponentRects.Add(ComputeSectionRect(Component, bInWithDuplicateBorders));
	}
	return ComponentRects;
}

void FMergeRenderBatch::ComputeAllSubsectionRects(TArray<FIntRect>& OutSubsectionRects, TArray<FIntRect>& OutSubsectionRectsWithDuplicateBorders) const
{
	const int32 NumSubsections = Landscape->NumSubsections;
	const int32 ComponentSizeQuads = Landscape->ComponentSizeQuads;
	const int32 SubsectionSizeQuads = Landscape->SubsectionSizeQuads;
	const int32 SubsectionVerts = SubsectionSizeQuads + 1;
	const int32 TotalNumSubsectionRects = ComponentsToRender.Num() * NumSubsections * NumSubsections;
	OutSubsectionRects.Reserve(TotalNumSubsectionRects);
	OutSubsectionRectsWithDuplicateBorders.Reserve(TotalNumSubsectionRects);

	for (ULandscapeComponent* Component : ComponentsToRender)
	{
		const FIntPoint ComponentSectionBase = Component->GetSectionBase();
		checkf((ComponentSectionBase.X >= SectionRect.Min.X) && (ComponentSectionBase.Y >= SectionRect.Min.Y)
			&& ((ComponentSectionBase.X + ComponentSizeQuads + 1) <= SectionRect.Max.X) && ((ComponentSectionBase.Y + ComponentSizeQuads + 1) <= SectionRect.Max.Y),
			TEXT("The requested component is not included in the render batch"));

		const FIntPoint ComponentLocalKey = (ComponentSectionBase - SectionRect.Min) / ComponentSizeQuads;
		TArray<FIntRect, TInlineAllocator<4>> SubSectionRects;
		for (int32 SubY = 0; SubY < NumSubsections; ++SubY)
		{
			for (int32 SubX = 0; SubX < NumSubsections; ++SubX)
			{
				{
					FIntPoint SubSectionMin = ComponentSectionBase - SectionRect.Min + FIntPoint(SubX * SubsectionSizeQuads, SubY * SubsectionSizeQuads);
					FIntPoint SubSectionMax = SubSectionMin + FIntPoint(SubsectionVerts, SubsectionVerts);
					OutSubsectionRects.Add(FIntRect(SubSectionMin, SubSectionMax));
				}
				{
					FIntPoint SubSectionMin = (ComponentLocalKey * NumSubsections + FIntPoint(SubX, SubY)) * SubsectionVerts;
					FIntPoint SubSectionMax = SubSectionMin + SubsectionVerts;
					OutSubsectionRectsWithDuplicateBorders.Add(FIntRect(SubSectionMin, SubSectionMax));
				}
			}
		}
	}
}

uint32 FMergeRenderBatch::ComputeComponentLinearIndex(const FIntPoint& InComponentKey) const
{
	check((InComponentKey.X >= MinComponentKey.X) && (InComponentKey.Y >= MinComponentKey.Y));
	check((InComponentKey.X <= MaxComponentKey.X) && (InComponentKey.Y <= MaxComponentKey.Y));
	return (InComponentKey.X - MinComponentKey.X) + (InComponentKey.Y - MinComponentKey.Y) * (MaxComponentKey.X - MinComponentKey.X + 1);
}

FIntPoint FMergeRenderBatch::GetRenderTargetResolution(bool bInWithDuplicateBorders) const
{
	return bInWithDuplicateBorders ? Resolution : SectionRect.Size();
}


// ----------------------------------------------------------------------------------

FMergeRenderContext::FMergeRenderContext(const FMergeContext& InMergeContext)
	: FMergeContext(InMergeContext)
{
	for (ULandscapeScratchRenderTarget*& BlendRenderTarget : BlendRenderTargets)
	{
		BlendRenderTarget = nullptr;
	}
}

FMergeRenderContext::~FMergeRenderContext()
{
	FreeResources();

	checkf(Algo::AllOf(BlendRenderTargets, [](ULandscapeScratchRenderTarget* InRT) { return (InRT == nullptr); }) 
		&& (ComponentIdRenderTarget == nullptr), TEXT("Every scratch render target should have been freed at this point."));
}

void FMergeRenderContext::AllocateResources()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMergeRenderContext::AllocateResources);

	using namespace UE::Landscape;

	// Prepare the transient render resources we'll need throughout the merge: 
	const int32 NumSlices = IsHeightmapMerge() ? 0 : MaxNeededNumSlices;
	FLinearColor RenderTargetClearColor(ForceInitToZero);
	ETextureRenderTargetFormat RenderTargetFormat = ETextureRenderTargetFormat::RTF_R8;
	if (IsHeightmapMerge())
	{
		// Convert the height value 0.0f to how it's stored in the texture : 
		const uint16 HeightValue = LandscapeDataAccess::GetTexHeight(0.0f);
		RenderTargetClearColor = FLinearColor((float)((HeightValue - (HeightValue & 255)) >> 8) / 255.0f, (float)(HeightValue & 255) / 255.0f, 0.0f, 0.0f);

		RenderTargetFormat = ETextureRenderTargetFormat::RTF_RGBA8;
	}
	// When rendering weightmaps, we should have at least 1 slice (if == 1, we can use a UTextureRenderTarget2D, otherwise, we'll need to use a UTextureRenderTarget2DArray) : 
	else
	{
		checkf(MaxNeededNumSlices > 0, TEXT("Weightmaps should have at least 1 slice"));
		// We use extra channels for weightmaps for storing alpha / alpha flags :
		RenderTargetFormat = ETextureRenderTargetFormat::RTF_RGBA8;
	}

	ULandscapeEditResourcesSubsystem* LandscapeEditResourcesSubsystem = GEngine->GetEngineSubsystem<ULandscapeEditResourcesSubsystem>();
	check(LandscapeEditResourcesSubsystem != nullptr);
	checkf(Algo::AllOf(BlendRenderTargets, [](ULandscapeScratchRenderTarget* InRT) { return (InRT == nullptr); }), TEXT("We shouldn't allocate without having freed first."));
	check(CurrentBlendRenderTargetWriteIndex == -1);

	{
		// We need N render targets large enough to fit all batches : 
		// Write : 
		FScratchRenderTargetParams ScratchRenderTargetParams(TEXT("ScratchRT0"), /*bInExactDimensions = */false, /*bInUseUAV = */false, /*bInTargetArraySlicesIndependently = */(NumSlices > 0),
			MaxNeededResolution, NumSlices, RenderTargetFormat, RenderTargetClearColor, ERHIAccess::RTV);
		BlendRenderTargets[0] = LandscapeEditResourcesSubsystem->RequestScratchRenderTarget(ScratchRenderTargetParams);
		// Read and ReadPrevious : 
		ScratchRenderTargetParams.DebugName = TEXT("ScratchRT1");
		ScratchRenderTargetParams.InitialState = ERHIAccess::SRVMask;
		BlendRenderTargets[1] = LandscapeEditResourcesSubsystem->RequestScratchRenderTarget(ScratchRenderTargetParams);
		ScratchRenderTargetParams.DebugName = TEXT("ScratchRT2");
		BlendRenderTargets[2] = LandscapeEditResourcesSubsystem->RequestScratchRenderTarget(ScratchRenderTargetParams);
	}

	{
		// We need a render target to store the component linear Ids within a given batch. i.e. for a batch dealing with N x M components : 
		//  (0, 0) ---------- (N - 1, 0)  --> linear Ids --> 0 -------------------------- (N - 1)
		//  |                          |                     |                                  |
		//  |                          |                     |                                  |
		//  (0, M - 1) -- (N - 1, M - 1)                     (M - 1) * N -- (N - 1 + (M - 1) * N)
		check(ComponentIdRenderTarget == nullptr);
		check((MaxNumComponents.X > 0) && (MaxNumComponents.Y > 0));
		checkf(static_cast<uint64>(MaxNumComponents.X) * static_cast<uint64>(MaxNumComponents.Y) < static_cast<uint64>(MAX_uint32 - 1), TEXT("Cannot render %d x %d components in a given batch : the component index wouldn't fit on 32 bits"), MaxNumComponents.X, MaxNumComponents.Y);

		static const uint32 NoComponentId = 0xffffffff;
		static const FLinearColor ClearValue(*reinterpret_cast<const float*>(&NoComponentId), 0.0f, 0.0f, 0.0f);

		// Using a uint32 for component ids is a tad overkill, but the cost is minimal and this avoids issues with very large batches : 
		FScratchRenderTargetParams ScratchRenderTargetParams(TEXT("ComponentIdRT"), /*bInExactDimensions = */false, /*bInUseUAV = */false, /*bInTargetArraySlicesIndependently = */false,
			MaxNeededResolution, 0, ETextureRenderTargetFormat::RTF_R32f, ClearValue, ERHIAccess::RTV);
		ComponentIdRenderTarget = LandscapeEditResourcesSubsystem->RequestScratchRenderTarget(ScratchRenderTargetParams);
	}
}

void FMergeRenderContext::FreeResources()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMergeRenderContext::FreeResources);

	ULandscapeEditResourcesSubsystem* LandscapeEditResourcesSubsystem = GEngine->GetEngineSubsystem<ULandscapeEditResourcesSubsystem>();
	check(LandscapeEditResourcesSubsystem != nullptr);

	// We can now return those scratch render targets to the pool : 
	for (int32 Index = 0; Index < BlendRenderTargets.Num(); ++Index)
	{
		if (BlendRenderTargets[Index] != nullptr)
		{
			LandscapeEditResourcesSubsystem->ReleaseScratchRenderTarget(BlendRenderTargets[Index]);
			BlendRenderTargets[Index] = nullptr;
		}
	}
	CurrentBlendRenderTargetWriteIndex = -1;

	if (ComponentIdRenderTarget != nullptr)
	{
		// We can now return the scratch render targets to the pool : 
		LandscapeEditResourcesSubsystem->ReleaseScratchRenderTarget(ComponentIdRenderTarget);
		ComponentIdRenderTarget = nullptr;
	}
}

void FMergeRenderContext::AllocateBatchResources(const FMergeRenderBatch& InRenderBatch)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMergeRenderContext::AllocateBatchResources);
}

void FMergeRenderContext::FreeBatchResources(const FMergeRenderBatch& InRenderBatch)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMergeRenderContext::FreeBatchResources);
}

void FMergeRenderContext::CycleBlendRenderTargets(UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder)
{
	const bool bFirstWrite = (CurrentBlendRenderTargetWriteIndex < 0);
	CurrentBlendRenderTargetWriteIndex = (CurrentBlendRenderTargetWriteIndex + 1) % BlendRenderTargets.Num();

	if (!bFirstWrite)
	{
		// Optionally clear the write render target for debug purposes : 
		if (CVarLandscapeBatchedMergeClearBeforeEachWriteToScratch.GetValueOnGameThread())
		{
			GetBlendRenderTargetWrite()->Clear(RDGBuilderRecorder);
		}
	}
}

ULandscapeScratchRenderTarget* EditLayers::FMergeRenderContext::GetBlendRenderTargetWrite() const
{
	checkf(CurrentBlendRenderTargetWriteIndex >= 0, TEXT("CycleBlendRenderTargets must be called at least once prior to accessing the blend render targets"));
	return BlendRenderTargets[CurrentBlendRenderTargetWriteIndex % BlendRenderTargets.Num()];
}

ULandscapeScratchRenderTarget* FMergeRenderContext::GetBlendRenderTargetRead() const
{
	checkf(CurrentBlendRenderTargetWriteIndex >= 0, TEXT("CycleBlendRenderTargets must be called at least once prior to accessing the blend render targets")); 
	return BlendRenderTargets[(CurrentBlendRenderTargetWriteIndex + BlendRenderTargets.Num() - 1) % BlendRenderTargets.Num()];
}

ULandscapeScratchRenderTarget* FMergeRenderContext::GetBlendRenderTargetReadPrevious() const
{
	checkf(CurrentBlendRenderTargetWriteIndex >= 0, TEXT("CycleBlendRenderTargets must be called at least once prior to accessing the blend render targets"));
	return BlendRenderTargets[(CurrentBlendRenderTargetWriteIndex + BlendRenderTargets.Num() - 2) % BlendRenderTargets.Num()];
}

ULandscapeScratchRenderTarget* FMergeRenderContext::GetComponentIdRenderTarget() const
{
	check(ComponentIdRenderTarget != nullptr);
	return ComponentIdRenderTarget;
}

FTransform FMergeRenderContext::ComputeVisualLogTransform(const FTransform& InTransform) const
{
	FTransform ZTransform(CurrentVisualLogOffset / InTransform.GetScale3D()); // the Offset is given in world space so unapply the scale before applying the transform
	return ZTransform * InTransform;
}

void FMergeRenderContext::IncrementVisualLogOffset()
{
	double VisualLogOffsetIncrement = CVarLandscapeBatchedMergeVisualLogOffsetIncrement.GetValueOnGameThread();
	CurrentVisualLogOffset.Z += VisualLogOffsetIncrement;
}

void FMergeRenderContext::ResetVisualLogOffset()
{
	CurrentVisualLogOffset = FVector(ForceInitToZero);
}

void FMergeRenderContext::RenderComponentIds(UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder)
{
	using namespace UE::Landscape;

	const FMergeRenderBatch* RenderBatch = GetCurrentRenderBatch();
	check(RenderBatch != nullptr);
	check(ComponentIdRenderTarget != nullptr);
	FSceneInterface* SceneInterface = Landscape->GetWorld()->Scene;

	struct FQuadParams
	{
		// Sort key for rendering overlapping quads in a deterministic way :
		uint32 ComponentLinearIndex = INDEX_NONE;
		// Texture region to write to :
		FIntRect DestinationRect;
	};
	TArray<FQuadParams> QuadParams;
	QuadParams.Reserve(RenderBatch->ComponentsToRender.Num() * Landscape->NumSubsections * Landscape->NumSubsections);
	for (ULandscapeComponent* Component : RenderBatch->ComponentsToRender)
	{
		FQuadParams& Quad = QuadParams.Emplace_GetRef();
		Quad.ComponentLinearIndex = RenderBatch->ComputeComponentLinearIndex(Component->GetComponentKey());
		check(Quad.ComponentLinearIndex < static_cast<uint64>(MaxNumComponents.X * MaxNumComponents.Y));
		Quad.DestinationRect = RenderBatch->ComputeSectionRect(Component, /*bInWithDuplicateBorders = */false);
	}
	// We want to render quads for each component and we want to render them in a deterministic order : 
	Algo::SortBy(QuadParams, &FQuadParams::ComponentLinearIndex);

	TArray<FUintVector4> QuadRects;
	TArray<uint32> QuadComponentLinearIds;
	QuadRects.Reserve(QuadParams.Num());
	QuadComponentLinearIds.Reserve(QuadParams.Num());
	Algo::ForEach(QuadParams, [&QuadRects, &QuadComponentLinearIds](const FQuadParams& InQuadParams)
		{
			QuadRects.Emplace(InQuadParams.DestinationRect.Min.X, InQuadParams.DestinationRect.Min.Y, InQuadParams.DestinationRect.Max.X + 1, InQuadParams.DestinationRect.Max.Y + 1);
			QuadComponentLinearIds.Emplace(InQuadParams.ComponentLinearIndex);
		});

	ComponentIdRenderTarget->Clear(RDGBuilderRecorder);
	ComponentIdRenderTarget->TransitionTo(ERHIAccess::RTV, RDGBuilderRecorder);

	auto RDGCommand =
		[ComponentIdRenderTargetResource = ComponentIdRenderTarget->GetRenderTarget2D()->GetResource(), 
		SceneInterface,
		QuadRects = MoveTemp(QuadRects), 
		QuadComponentLinearIds = MoveTemp(QuadComponentLinearIds)]
		(FRDGBuilder& GraphBuilder)
	{
			FRDGBufferRef RectBuffer = CreateUploadBuffer(GraphBuilder, TEXT("ComponentIdRects"), MakeArrayView(QuadRects));
			FRDGBufferSRVRef RectBufferSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(RectBuffer, PF_R32G32B32A32_UINT));
			FRDGBufferRef ComponentIdBuffer = CreateUploadBuffer(GraphBuilder, TEXT("ComponentIds"), MakeArrayView(QuadComponentLinearIds));
			FRDGBufferSRVRef ComponentIdBufferSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(ComponentIdBuffer, PF_R32_UINT));
			FRDGTextureRef OutputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(ComponentIdRenderTargetResource->GetTexture2DRHI(), TEXT("ComponentIds")));

			// We need a depth texture to ensure overlapping quads render in a deterministic order :
			FRDGTextureDesc DepthTextureDesc = FRDGTextureDesc::Create2D(OutputTexture->Desc.Extent, PF_DepthStencil, FClearValueBinding::DepthOne, TexCreate_DepthStencilTargetable);
			FRDGTextureRef DepthTexture = GraphBuilder.CreateTexture(DepthTextureDesc, TEXT("ComponentIdRectsDepth"));

			// TODO [jonathan.bard] this is just to avoid a RHI validation error for unoptimized shaders... once validation is made to not issue those errors, we can remove this
			// Create a SceneView to please the shader bindings, but it's unused in practice 
			FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(nullptr, SceneInterface, FEngineShowFlags(ESFIM_Game)).SetTime(FGameTime::GetTimeSinceAppStart()));
			FSceneViewInitOptions ViewInitOptions;
			ViewInitOptions.ViewFamily = &ViewFamily;
			ViewInitOptions.SetViewRectangle(FIntRect(0, 0, 1, 1)); // Use a dummy rect to avoid a check(slow)
			GetRendererModule().CreateAndInitSingleView(GraphBuilder.RHICmdList, &ViewFamily, &ViewInitOptions);
			const FSceneView* View = ViewFamily.Views[0];

			FRenderComponentIdPSParameters* PassParameters = GraphBuilder.AllocParameters<FRenderComponentIdPSParameters>();
			PassParameters->RenderTargets.DepthStencil = FDepthStencilBinding(DepthTexture,
				/*InDepthLoadAction = */ ERenderTargetLoadAction::EClear,
				/*InStencilLoadAction = */ERenderTargetLoadAction::ENoAction,
				/*InDepthStencilAccess = */FExclusiveDepthStencil::DepthWrite_StencilNop);
			PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputTexture, ERenderTargetLoadAction::ELoad);
			PassParameters->PS.View = View->ViewUniformBuffer;
			PassParameters->PS.InQuadComponentIds = ComponentIdBufferSRV;
			FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
			TShaderRef<FRenderComponentIdPS> PixelShader = ShaderMap->GetShader<FRenderComponentIdPS>();

			// There is some overlap between quads of 2 neighboring components at this step (e.g. the last column of component (X, Y) is supposed to correspond to the first of component (X+1, Y))
			//  Normally, this is fine but if both components don't agree on the values of those overlapped lines/columns (e.g. one component was submitted with updated values on the edge but
			//  not the other), then we still need to ensure a deterministic value to be output. Since quads are sorted by coordinates and we can get AddRasterizeToRectsPass to render quads in 
			//  a consistent near to far order (if DepthIncrement is passed), we can ensure a consistent ordering by enabling the depth test :
			FRHIDepthStencilState* DepthStencilState = TStaticDepthStencilState</*bEnableDepthWrite = */true, CF_LessEqual>::GetRHI();

			// Spread out the rects across the [0, 1[ range in case this is used with the depth test enabled (e.g. to ensure a deterministic result on overlapping quads) : 
			//  Divide the Z space in NumQuads increments, so that the first rect stands at Z == 0 and the last one at Z == 1.0 - DepthIncrement (avoids standing exactly on the far clipping plane,
			//  which could be problematic for precision reasons)
			float DepthIncrement = 1.0f / QuadRects.Num();

			FPixelShaderUtils::AddRasterizeToRectsPass<FRenderComponentIdPS>(GraphBuilder,
				ShaderMap,
				RDG_EVENT_NAME("RenderComponentId"),
				PixelShader,
				PassParameters,
				/*ViewportSize = */OutputTexture->Desc.Extent,
				RectBufferSRV,
				QuadRects.Num(),
				/*BlendState = */ nullptr,
				/*RasterizerState = */ nullptr,
				DepthStencilState,
				/*StencilRef = */ 0,
				/*TextureSize = */ OutputTexture->Desc.Extent,
				/*RectUVBufferSRV = */nullptr,
				/*DownsampleFactor = */1,
				/*AdditionalPassFlags = */ERDGPassFlags::None,
				DepthIncrement);
		};

	// We need to specify the final state of the external textures to prevent the graph builder from transitioning them to SRVMask :
	RDGBuilderRecorder.EnqueueRDGCommand(RDGCommand, { { ComponentIdRenderTarget->GetRenderTarget2D()->GetResource(), ERHIAccess::RTV } });
}

void FMergeRenderContext::RenderExpandedRenderTarget(UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder)
{
	using namespace UE::Landscape;

	const FMergeRenderBatch* RenderBatch = GetCurrentRenderBatch();
	check(RenderBatch != nullptr);

	TArray<FUintVector4> SourceRects, DestinationRects;
	{
		TArray<FIntRect> SourceInclusiveRects, DestinationInclusiveRects;
		RenderBatch->ComputeAllSubsectionRects(SourceInclusiveRects, DestinationInclusiveRects);
		// ComputeAllSubsectionRects returns inclusive bounds while AddRasterizeToRectsPass requires exclusive bounds : 
		Algo::Transform(SourceInclusiveRects, SourceRects, [](const FIntRect& InRect) { return FUintVector4(InRect.Min.X, InRect.Min.Y, InRect.Max.X + 1, InRect.Max.Y + 1); });
		Algo::Transform(DestinationInclusiveRects, DestinationRects, [](const FIntRect& InRect) { return FUintVector4(InRect.Min.X, InRect.Min.Y, InRect.Max.X + 1, InRect.Max.Y + 1); });
	}

	ULandscapeScratchRenderTarget* WriteRT = GetBlendRenderTargetWrite();
	ULandscapeScratchRenderTarget* ReadRT = GetBlendRenderTargetRead();
	WriteRT->TransitionTo(ERHIAccess::RTV, RDGBuilderRecorder);
	ReadRT->TransitionTo(ERHIAccess::SRVMask, RDGBuilderRecorder);

	FSceneInterface* SceneInterface = GetLandscape()->GetWorld()->Scene;

	auto RDGCommand =
		[ SourceRects = MoveTemp(SourceRects)
		, DestinationRects = MoveTemp(DestinationRects)
		, OutputResource = WriteRT->GetRenderTarget2D()->GetResource()
		, SourceResource = ReadRT->GetRenderTarget2D()->GetResource()
		, SceneInterface] (FRDGBuilder& GraphBuilder)
	{
		FRDGBufferRef RectBuffer = CreateUploadBuffer(GraphBuilder, TEXT("ExpandRects"), MakeArrayView(DestinationRects));
		FRDGBufferSRVRef RectBufferSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(RectBuffer, PF_R32G32B32A32_UINT));

		FRDGBufferRef RectUVBuffer = CreateUploadBuffer(GraphBuilder, TEXT("ExpandRectsUVs"), MakeArrayView(SourceRects));
		FRDGBufferSRVRef RectUVBufferSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(RectUVBuffer, PF_R32G32B32A32_UINT));

		FRDGTextureRef OutputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(OutputResource->GetTexture2DRHI(), TEXT("OutputTexture")));
		FRDGTextureRef SourceTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(SourceResource->GetTexture2DRHI(), TEXT("SourceTexture")));

		// TODO [jonathan.bard] this is just a rhi validation error for unoptimized shaders... once validation is made to not issue those errors, we can remove this
		// Create a SceneView to please the shader bindings, but it's unused in practice 
		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(nullptr, SceneInterface, FEngineShowFlags(ESFIM_Game)).SetTime(FGameTime::GetTimeSinceAppStart()));
		FSceneViewInitOptions ViewInitOptions;
		ViewInitOptions.ViewFamily = &ViewFamily;
		ViewInitOptions.SetViewRectangle(FIntRect(0, 0, 1, 1)); // Use a dummy rect to avoid a check(slow)
			GetRendererModule().CreateAndInitSingleView(GraphBuilder.RHICmdList, &ViewFamily, &ViewInitOptions);
		const FSceneView* View = ViewFamily.Views[0];

		FCopyQuadsPSParameters* PassParameters = GraphBuilder.AllocParameters<FCopyQuadsPSParameters>();
		PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputTexture, ERenderTargetLoadAction::ELoad);
		PassParameters->PS.View = View->ViewUniformBuffer;
		PassParameters->PS.InSourceTexture = SourceTexture;

		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderRef<FCopyQuadsPS> PixelShader = ShaderMap->GetShader<FCopyQuadsPS>();

		FPixelShaderUtils::AddRasterizeToRectsPass<FCopyQuadsPS>(GraphBuilder,
			ShaderMap,
			RDG_EVENT_NAME("CopyQuadsPS"),
			PixelShader,
			PassParameters,
			/*ViewportSize = */OutputTexture->Desc.Extent,
			RectBufferSRV,
			DestinationRects.Num(),
			/*BlendState = */nullptr,
			/*RasterizerState = */nullptr,
			/*DepthStencilState = */nullptr,
			/*StencilRef = */0,
			/*TextureSize = */SourceTexture->Desc.Extent,
			RectUVBufferSRV);
		};

	// We need to specify the final state of the external textures to prevent the graph builder from transitioning them to SRVMask (even those that end up as SRVMask at the end of this command,
	//  because they will likely be part of another RDGCommand down the line so we need to maintain an accurate picture of every external texture ever involved in the recorded command so that 
	//  we can set a proper access when the recorder is flushed (and the FRDGBuilder, executed) :
	TArray<FRDGBuilderRecorder::FRDGExternalTextureAccessFinal> RDGExternalTextureAccessFinalList =
	{
		{ WriteRT->GetRenderTarget()->GetResource(), ERHIAccess::RTV },
		{ ReadRT->GetRenderTarget()->GetResource(), ERHIAccess::SRVMask }
	};
	RDGBuilderRecorder.EnqueueRDGCommand(RDGCommand, RDGExternalTextureAccessFinalList);
}

void FMergeRenderContext::GenericBlendLayer(const FBlendParams& InBlendParams, FRenderParams& RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder)
{
	using namespace UE::Landscape;

	const FMergeRenderBatch* RenderBatch = GetCurrentRenderBatch();
	check(RenderBatch != nullptr);

	// In this step, we perform the blend : merge the edit layer with the previous layers in the stack
	CycleBlendRenderTargets(RDGBuilderRecorder);
	ULandscapeScratchRenderTarget* WriteRT = GetBlendRenderTargetWrite();
	ULandscapeScratchRenderTarget* CurrentLayerReadRT = GetBlendRenderTargetRead();
	ULandscapeScratchRenderTarget* PreviousLayersReadRT = GetBlendRenderTargetReadPrevious();

	WriteRT->TransitionTo(ERHIAccess::RTV, RDGBuilderRecorder);
	CurrentLayerReadRT->TransitionTo(ERHIAccess::SRVMask, RDGBuilderRecorder);
	PreviousLayersReadRT->TransitionTo(ERHIAccess::SRVMask, RDGBuilderRecorder);

	if (IsHeightmapMerge())
	{
		auto RDGCommand =
			[ HeightmapBlendParams = InBlendParams.HeightmapBlendParams
			, OutputResource = WriteRT->GetRenderTarget2D()->GetResource()
			, OutputResourceName = WriteRT->GetDebugName()
			, CurrentEditLayerResource = CurrentLayerReadRT->GetRenderTarget2D()->GetResource()
			, PreviousEditLayersResource = PreviousLayersReadRT->GetRenderTarget2D()->GetResource()
			, EffectiveTextureSize = RenderBatch->GetRenderTargetResolution(/*bInWithDuplicateBorders = */false)]
			(FRDGBuilder& GraphBuilder)
			{
				FRDGTextureRef OutputTextureRef = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(OutputResource->TextureRHI, TEXT("OutputTexture")));
				FRDGTextureRef CurrentEditLayerTextureRef = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(CurrentEditLayerResource->TextureRHI, TEXT("CurrentEditLayerTexture")));
				FRDGTextureRef PreviousEditLayersTextureRef = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(PreviousEditLayersResource->TextureRHI, TEXT("PreviousEditLayersTexture")));

				FLandscapeEditLayersHeightmapsMergeEditLayerPS::FParameters* PSParams = GraphBuilder.AllocParameters<FLandscapeEditLayersHeightmapsMergeEditLayerPS::FParameters>();
				PSParams->RenderTargets[0] = FRenderTargetBinding(OutputTextureRef, ERenderTargetLoadAction::ENoAction);
				PSParams->InEditLayerBlendMode = static_cast<uint32>(HeightmapBlendParams.BlendMode);
				PSParams->InEditLayerAlpha = HeightmapBlendParams.Alpha;
				PSParams->InCurrentEditLayerHeightmap = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(CurrentEditLayerTextureRef));
				PSParams->InPreviousEditLayersHeightmap = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(PreviousEditLayersTextureRef));

				FLandscapeEditLayersHeightmapsMergeEditLayerPS::MergeEditLayerPS(
					RDG_EVENT_NAME("MergeEditLayer(Height) -> %s", *OutputResourceName), GraphBuilder, PSParams, EffectiveTextureSize);
			};


		// We need to specify the final state of the external textures to prevent the graph builder from transitioning them to SRVMask (even those that end up as SRVMask at the end of this command,
		//  because they will likely be part of another RDGCommand down the line so we need to maintain an accurate picture of every external texture ever involved in the recorded command so that 
		//  we can set a proper access when the recorder is flushed (and the FRDGBuilder, executed) :
		TArray<FRDGBuilderRecorder::FRDGExternalTextureAccessFinal> RDGExternalTextureAccessFinalList =
		{
			{ WriteRT->GetRenderTarget()->GetResource(), ERHIAccess::RTV },
			{ CurrentLayerReadRT->GetRenderTarget()->GetResource(), ERHIAccess::SRVMask },
			{ PreviousLayersReadRT->GetRenderTarget()->GetResource(), ERHIAccess::SRVMask },
		};
		RDGBuilderRecorder.EnqueueRDGCommand(RDGCommand, RDGExternalTextureAccessFinalList);
	}
	else
	{
		const TArray<FTargetLayerGroup> TargetLayerGroups(BuildGenericBlendTargetLayerGroups(ConvertTargetLayerNamesToBitIndices(RenderParams.TargetLayerGroupLayerNames)));

		// Prepare layer info for the target layers that will need it for merging :
		TArray<FMergeEditLayerTargetLayerInfo> WeightmapTargetLayerInfos;
		WeightmapTargetLayerInfos.Reserve(RenderParams.TargetLayerGroupLayerInfos.Num());
		Algo::Transform(RenderParams.TargetLayerGroupLayerInfos, WeightmapTargetLayerInfos, [this, &TargetLayerGroups, RenderBatch](ULandscapeLayerInfoObject* InLayerInfo)
			{
				check(InLayerInfo != nullptr); // There should only be valid layer infos at this point
				const int32 TargetLayerIndex = GetTargetLayerIndexForLayerInfoChecked(InLayerInfo);
				// Skip weight-blended layers that are not rendered in the batch, because their texture slice might contain garbage :
				const bool bRenderedInBatch = RenderBatch->TargetLayerBitIndices[TargetLayerIndex];

				FMergeEditLayerTargetLayerInfo WeightmapTargetLayerInfo;
				WeightmapTargetLayerInfo.BlendGroupIndex = INDEX_NONE;
				if (!bRenderedInBatch)
				{
					WeightmapTargetLayerInfo.Flags |= EWeightmapTargetLayerFlags::Skip;
				}
				if (InLayerInfo->GetBlendMethod() == ELandscapeTargetLayerBlendMethod::PremultipliedAlphaBlending)
				{
					WeightmapTargetLayerInfo.Flags |= EWeightmapTargetLayerFlags::IsPremultipliedAlphaWeightBlended;
					// Find the target layer group that corresponds to this target layer. There must be one
					const int32 TargetLayerGroupIndex = TargetLayerGroups.IndexOfByPredicate([TargetLayerIndex](const FTargetLayerGroup& InTargetLayerGroup)
						{ 
							return InTargetLayerGroup.GetWeightmapTargetLayerBitIndices()[TargetLayerIndex];
						});
					checkf(TargetLayerGroupIndex != INDEX_NONE, TEXT("Couldn't find a target layer group for layer %s. All premultiplied alpha blending must belong to a group (and one only)"), *InLayerInfo->GetLayerName().ToString());
					WeightmapTargetLayerInfo.BlendGroupIndex = TargetLayerGroupIndex;
				}
				if (InLayerInfo == ALandscapeProxy::VisibilityLayer)
				{
					WeightmapTargetLayerInfo.Flags |= EWeightmapTargetLayerFlags::IsVisibilityLayer;
				}
				
				return WeightmapTargetLayerInfo;
			});

		ForEachTargetLayer(RenderBatch->TargetLayerBitIndices,
			[WriteRT, CurrentLayerReadRT, PreviousLayersReadRT, &WeightmapTargetLayerInfos, &RenderBatch, &RenderParams, &InBlendParams, &RDGBuilderRecorder](int32 InTargetLayerIndex, const FName& InTargetLayerName, ULandscapeLayerInfoObject* InWeightmapLayerInfo)
			{
				const int32 TargetLayerIndexInGroup = RenderParams.TargetLayerGroupLayerNames.Find(InTargetLayerName);
				check(TargetLayerIndexInGroup != INDEX_NONE);
				const int32 NumTargetLayersInGroup = RenderParams.TargetLayerGroupLayerNames.Num();

				// By default, use passthrough mode so that each layer gets at least copied into the WriteRT :
				const FWeightmapBlendParams* TargetLayerBlendParams = &FWeightmapBlendParams::GetDefaultPassthroughBlendParams();
				// Special case for visibility which is always "Additive" :
				if (InTargetLayerName == UMaterialExpressionLandscapeVisibilityMask::ParameterName)
				{
					TargetLayerBlendParams = &FWeightmapBlendParams::GetDefaultAdditiveBlendParams();
				}
				else if (const FWeightmapBlendParams* FoundTargetLayerBlendParams = InBlendParams.WeightmapBlendParams.Find(InTargetLayerName))
				{
					TargetLayerBlendParams = FoundTargetLayerBlendParams;
				}

				// TODO [jonathan.bard] : we could render several layers at once via MRT (up to MaxSimultaneousRenderTargets)
				auto RDGCommand =
					[ TargetLayerBlendParams = *TargetLayerBlendParams
					, TargetLayerIndexInGroup
					, NumTargetLayersInGroup
					, WeightmapTargetLayerInfos
					, InTargetLayerName
					, OutputResource = WriteRT->GetRenderTarget2DArray()->GetResource()
					, OutputResourceName = WriteRT->GetDebugName()
					, CurrentEditLayerResource = CurrentLayerReadRT->GetRenderTarget2DArray()->GetResource()
					, PreviousEditLayersResource = PreviousLayersReadRT->GetRenderTarget2DArray()->GetResource()
					, EffectiveTextureSize = RenderBatch->GetRenderTargetResolution(/*bInWithDuplicateBorders = */false)]
					(FRDGBuilder& GraphBuilder)
					{
						FRDGTextureRef OutputTextureRef = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(OutputResource->TextureRHI, TEXT("OutputTexture")));
						FRDGTextureRef CurrentEditLayerTextureRef = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(CurrentEditLayerResource->TextureRHI, TEXT("CurrentEditLayerTexture")));
						FRDGTextureSRVRef CurrentEditLayerTextureSRVRef = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(CurrentEditLayerTextureRef));
						FRDGTextureRef PreviousEditLayersTextureRef = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(PreviousEditLayersResource->TextureRHI, TEXT("PreviousEditLayersTexture")));
						FRDGTextureSRVRef PreviousEditLayersTextureSRVRef = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(PreviousEditLayersTextureRef));

						// TODO [jonathan.bard] : we could find a way to create this once for all target layers instead of re-creating it for each RDGCommand
						FRDGBufferRef TargetLayerInfosBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("LandscapeMergeEditLayerTargetLayerInfosBuffer"), WeightmapTargetLayerInfos);
						FRDGBufferSRVRef TargetLayerInfosBufferSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(TargetLayerInfosBuffer));

						FLandscapeEditLayersWeightmapsMergeEditLayerPS::FParameters* PSParams = GraphBuilder.AllocParameters<FLandscapeEditLayersWeightmapsMergeEditLayerPS::FParameters>();
						PSParams->RenderTargets[0] = FRenderTargetBinding(OutputTextureRef, ERenderTargetLoadAction::ENoAction, /*InMipIndex = */0, /*InArraySlice = */TargetLayerIndexInGroup);
						PSParams->InTargetLayerIndex = TargetLayerIndexInGroup;
						PSParams->InNumTargetLayers = NumTargetLayersInGroup;
						PSParams->InEditLayerTargetLayerBlendMode = static_cast<uint32>(TargetLayerBlendParams.BlendMode);
						PSParams->InEditLayerAlpha = TargetLayerBlendParams.Alpha;
						PSParams->InMergeEditLayerTargetLayerInfos = TargetLayerInfosBufferSRV;
						PSParams->InCurrentEditLayerWeightmaps = CurrentEditLayerTextureSRVRef;
						PSParams->InPreviousEditLayersWeightmaps = PreviousEditLayersTextureSRVRef;

						FLandscapeEditLayersWeightmapsMergeEditLayerPS::MergeEditLayerPS(
							RDG_EVENT_NAME("MergeEditLayer(%s) -> %s", *InTargetLayerName.ToString(), *OutputResourceName), GraphBuilder, PSParams, EffectiveTextureSize);
					};

				// We need to specify the final state of the external textures to prevent the graph builder from transitioning them to SRVMask (even those that end up as SRVMask at the end of this command,
				//  because they will likely be part of another RDGCommand down the line so we need to maintain an accurate picture of every external texture ever involved in the recorded command so that 
				//  we can set a proper access when the recorder is flushed (and the FRDGBuilder, executed) :
				TArray<FRDGBuilderRecorder::FRDGExternalTextureAccessFinal> RDGExternalTextureAccessFinalList =
				{
					{ WriteRT->GetRenderTarget()->GetResource(), ERHIAccess::RTV },
					{ CurrentLayerReadRT->GetRenderTarget()->GetResource(), ERHIAccess::SRVMask },
					{ PreviousLayersReadRT->GetRenderTarget()->GetResource(), ERHIAccess::SRVMask },
				};
				RDGBuilderRecorder.EnqueueRDGCommand(RDGCommand, RDGExternalTextureAccessFinalList);

				return true;
			});

	}
}

void FMergeRenderContext::Render(TFunction<void(const FOnRenderBatchTargetGroupDoneParams& /*InParams*/, UE::Landscape::FRDGBuilderRecorder& /*RDGBuilderRecorder*/)> OnBatchTargetGroupDone)
{
	using namespace UE::Landscape;

	TRACE_CPUPROFILER_EVENT_SCOPE(FMergeRenderContext::Render);

	check(CurrentRenderBatchIndex == INDEX_NONE);

	const int32 ShowMergeProcess = CVarLandscapeBatchedMergeVisualLogShowMergeProcess.GetValueOnGameThread();

	const FTransform& LandscapeTransform = Landscape->GetTransform();
	// For visual logging, start at the top of the landscape's bounding box :
	FVector LandscapeTopPosition(0.0, 0.0, MaxLocalHeight);
	FTransform LandscapeWorldTransformForVisLog = FTransform(LandscapeTopPosition) * LandscapeTransform;

	AllocateResources();

	// Command recorder when accumulating render commands for several consecutive RenderLayer calls when ERenderFlags::RenderMode_Recorded is used 
	FRDGBuilderRecorder RDGBuilderRecorder;

	// Kick start the blend render targets : 
	CycleBlendRenderTargets(RDGBuilderRecorder);

	const int32 NumBatches = RenderBatches.Num();
	for (CurrentRenderBatchIndex = 0; CurrentRenderBatchIndex < NumBatches; ++CurrentRenderBatchIndex)
	{
		const FMergeRenderBatch& RenderBatch = RenderBatches[CurrentRenderBatchIndex];
		FString RenderBatchDebugName = FString::Format(TEXT("Render Batch [{0}] : ({1},{2})->({3},{4})"), { CurrentRenderBatchIndex, RenderBatch.SectionRect.Min.X, RenderBatch.SectionRect.Min.Y, RenderBatch.SectionRect.Max.X, RenderBatch.SectionRect.Max.Y });
		RHI_BREADCRUMB_EVENT_GAMETHREAD_F("Render Batch", "%s", RenderBatchDebugName);
		TRACE_CPUPROFILER_EVENT_SCOPE(RenderBatch);

		checkf((RenderBatch.RenderSteps.Num() >= 1) && (RenderBatch.RenderSteps.Last().Type == FMergeRenderStep::EType::SignalBatchMergeGroupDone),
			TEXT("Any batch should end with a SignalBatchMergeGroupDone step and there should be at least another step prior to that, otherwise, the batch is just useless."));

		AllocateBatchResources(RenderBatch);

		IncrementVisualLogOffset();

		// Drop a visual log showing the area covered by this batch : 
		UE_IFVLOG(
			if (IsVisualLogEnabled() && ShowMergeProcess != 0)
			{
				// Pick a new color for each batch : 
				uint32 Hash = PointerHash(&RenderBatch);
				uint8 * HashElement = reinterpret_cast<uint8*>(&Hash);
				FColor Color(HashElement[0], HashElement[1], HashElement[2]);

				UE_VLOG_OBOX(Landscape, LogLandscape, Log, FBox(FVector(RenderBatch.SectionRect.Min) - FVector(0.5, 0.5, 0.0), FVector(RenderBatch.SectionRect.Max) - FVector(0.5, 0.5, 0.0)),
					ComputeVisualLogTransform(LandscapeWorldTransformForVisLog).ToMatrixWithScale(), Color.WithAlpha(GetVisualLogAlpha()),
					TEXT("%s"), *FString::Format(TEXT("{0}\nBatch.SectionRect=([{1},{2}],[{3},{4}])"), { *RenderBatchDebugName, RenderBatch.SectionRect.Min.X, RenderBatch.SectionRect.Min.Y, RenderBatch.SectionRect.Max.X, RenderBatch.SectionRect.Max.Y }));

				// Draw each component's bounds rendered by this renderer : 
				for (const ULandscapeComponent* Component : RenderBatch.ComponentsToRender)
				{
					UE_VLOG_WIREOBOX(Landscape, LogLandscape, Log, FBox(FVector(Component->GetSectionBase()), FVector(Component->GetSectionBase() + Component->ComponentSizeQuads)),
						ComputeVisualLogTransform(LandscapeWorldTransformForVisLog).ToMatrixWithScale(), FColor::White, TEXT(""));
				}
			});

		const int32 NumRenderSteps = RenderBatch.RenderSteps.Num();
		// Current index of RenderLayer (for debugging purposes)
		int32 RenderLayerStepIndex = 0;
		// Index of RenderLayer at which we started recording the current render command sequence
		int32 RenderCommandStartLayerStepIndex = INDEX_NONE;
		// Indicates how many successful and consecutive RenderLayer steps (i.e. something has been rendered) have occurred (valid until the next BlendLayer step). 
		//  It's useful for BlendLayer steps as it allows to skip the separate blend step if nothing was rendered prior to it :
		int32 NumSuccessfulRenderLayerStepsUntilBlendLayerStep = 0;
		for (int32 RenderStepIndex = 0; RenderStepIndex < NumRenderSteps; ++RenderStepIndex)
		{
			const FMergeRenderStep& RenderStep = RenderBatch.RenderSteps[RenderStepIndex];
			TScriptInterface<ILandscapeEditLayerRenderer> Renderer = RenderStep.RendererState.GetRenderer();

			TArray<FName> TargetLayerGroupLayerNames;
			TArray<ULandscapeLayerInfoObject*> TargetLayerGroupLayerInfos;
			TArray<FComponentMergeRenderInfo> SortedComponentMergeRenderInfos;

			FTransform RenderAreaWorldTransform;
			FIntRect RenderAreaSectionRect;

			// Compute some data for the actual render steps : 
			if ((RenderStep.Type == FMergeRenderStep::EType::RenderLayer)
				|| (RenderStep.Type == FMergeRenderStep::EType::BlendLayer)
				|| (RenderStep.Type == FMergeRenderStep::EType::SignalBatchMergeGroupDone))
			{
				TargetLayerGroupLayerNames = ConvertTargetLayerBitIndicesToNames(RenderStep.TargetLayerGroupBitIndices);
				TargetLayerGroupLayerInfos = bIsHeightmapMerge ? TArray<ULandscapeLayerInfoObject*> { nullptr } : ConvertTargetLayerBitIndicesToLayerInfos(RenderStep.TargetLayerGroupBitIndices);

			    // Compute all necessary info about the components affected by this renderer at this step
			    SortedComponentMergeRenderInfos.Reserve(RenderStep.ComponentsToRender.Num());
			    Algo::Transform(RenderStep.ComponentsToRender, SortedComponentMergeRenderInfos, [MinComponentKey = RenderBatch.MinComponentKey](ULandscapeComponent* InComponent)
			    {
				    FComponentMergeRenderInfo ComponentMergeRenderInfo;
				    ComponentMergeRenderInfo.Component = InComponent;
    
				    const FIntPoint ComponentKey = InComponent->GetComponentKey();
				    const FIntPoint LocalComponentKey = ComponentKey - MinComponentKey;
				    check((LocalComponentKey.X >= 0) && (LocalComponentKey.Y >= 0));
				    ComponentMergeRenderInfo.ComponentKeyInRenderArea = LocalComponentKey;
				    // Area in the render target for this component : 
				    ComponentMergeRenderInfo.ComponentRegionInRenderArea = FIntRect(LocalComponentKey * InComponent->ComponentSizeQuads, (LocalComponentKey + 1) * InComponent->ComponentSizeQuads);
    
				    return ComponentMergeRenderInfo;
			    });
			    SortedComponentMergeRenderInfos.Sort();
			}

			// Compute some additional data for the actual render steps : 
			if ((RenderStep.Type == FMergeRenderStep::EType::BeginRenderLayerGroup)
				|| (RenderStep.Type == FMergeRenderStep::EType::EndRenderLayerGroup)
				|| (RenderStep.Type == FMergeRenderStep::EType::RenderLayer)
				|| (RenderStep.Type == FMergeRenderStep::EType::BlendLayer)
				|| (RenderStep.Type == FMergeRenderStep::EType::SignalBatchMergeGroupDone))
			{
				// TODO[jonathan.bard] offset the world transform to account for the half-pixel offset?
				//RenderParams.RenderAreaWorldTransform = FTransform(LandscapeTransform.GetRotation(), LandscapeTransform.GetTranslation() + FVector(RenderBatch.SectionRect.Min) /** (double)ComponentSizeQuads - FVector(0.5, 0.5, 0)*/, LandscapeTransform.GetScale3D());

				// TODO [jonathan.bard] : this is more of a Batch world transform/section rect at the moment. Shall we have a RenderAreaWorldTransform/RenderAreaSectionRect in FRenderParams and a BatchRenderAreaWorldTransform in FMergeRenderBatch?
				//  because currently the old BP brushes work with FMergeRenderBatch data (i.e. 1 transform for the batch and a section rect for the entire batch) but eventually, renderers might be interested in just their Render step context, 
				//  that is : 1 matrix corresponding to the bottom left corner of their list of components to render?
				RenderAreaWorldTransform = FTransform(FVector(RenderBatch.SectionRect.Min)) * LandscapeTransform;
				RenderAreaSectionRect = RenderBatch.SectionRect;
			}

			switch (RenderStep.Type)
			{
			case FMergeRenderStep::EType::BeginRenderCommandRecorder:
			{
				// Start recording a new sequence of RDG commands in order to let the upcoming FMergeRenderStep::EType::RenderLayer_Recorded steps push their render thread-based "operations" (lambdas)
				checkf(RDGBuilderRecorder.IsEmpty() && !RDGBuilderRecorder.IsRecording(), TEXT("There shouldn't be any pending command being recorded when starting a new render command"));
				checkf(RenderCommandStartLayerStepIndex == INDEX_NONE, TEXT("RenderCommandStartLayerStepIndex should be invalid as no render command recording should be active"));

				// Remember the render layer step index at which we started recording the RDG render command : 
				RenderCommandStartLayerStepIndex = RenderLayerStepIndex;
				check(RenderCommandStartLayerStepIndex >= 0);
				RDGBuilderRecorder.StartRecording();
				break;
			}
			case FMergeRenderStep::EType::EndRenderCommandRecorder:
			{
				checkf(RDGBuilderRecorder.IsRecording(), TEXT("There should be a pending command being recorded when starting a ending a render command"));
				checkf(RenderCommandStartLayerStepIndex != INDEX_NONE, TEXT("We should have initiated the render command recording with a begin operation, which should set a valid RenderCommandStartLayerStepIndex"));
				check(RenderCommandStartLayerStepIndex <= RenderLayerStepIndex);

				// This is where we actually push the current render command with all the render thread operations that have been accumulated on the render command context : 
				FRDGEventName RDGEventName = (RenderCommandStartLayerStepIndex == RenderLayerStepIndex - 1) 
					? RDG_EVENT_NAME("Step [%i]", RenderCommandStartLayerStepIndex)
					: RDG_EVENT_NAME("Steps [%i-%i]", RenderCommandStartLayerStepIndex, RenderLayerStepIndex - 1);
				RDGBuilderRecorder.StopRecordingAndFlush(MoveTemp(RDGEventName));

				// We've flushed the render command, we can reset the starting render layer step index : 
				RenderCommandStartLayerStepIndex = INDEX_NONE;
				break;
			}
			case FMergeRenderStep::EType::BeginRenderLayerGroup:
			case FMergeRenderStep::EType::EndRenderLayerGroup:
			{
				const ERenderFlags RenderFlags = Renderer->GetRenderFlags(this);
				check((Renderer != nullptr) 
					&& EnumHasAnyFlags(RenderFlags, ERenderFlags::RenderMode_Mask) 
					&& EnumHasAllFlags(RenderFlags, ERenderFlags::BlendMode_SeparateBlend | ERenderFlags::RenderLayerGroup_SupportsGrouping));

				if (RenderStep.Type == FMergeRenderStep::EType::BeginRenderLayerGroup)
				{
					// We start a new render layer group, so let's start tracking the number of successful RenderLayer steps within this group :
					NumSuccessfulRenderLayerStepsUntilBlendLayerStep = 0;
					FRenderParams RenderParams(this, TargetLayerGroupLayerNames, TargetLayerGroupLayerInfos, RenderStep.RendererState, SortedComponentMergeRenderInfos,
						RenderAreaWorldTransform, RenderAreaSectionRect, NumSuccessfulRenderLayerStepsUntilBlendLayerStep);
					Renderer->BeginRenderLayerGroup(RenderParams, RDGBuilderRecorder);
				}
                else
                {
					FRenderParams RenderParams(this, TargetLayerGroupLayerNames, TargetLayerGroupLayerInfos, RenderStep.RendererState, SortedComponentMergeRenderInfos,
						RenderAreaWorldTransform, RenderAreaSectionRect, NumSuccessfulRenderLayerStepsUntilBlendLayerStep);
					Renderer->EndRenderLayerGroup(RenderParams, RDGBuilderRecorder);
                }
				break;
			}
			case FMergeRenderStep::EType::RenderLayer:
			case FMergeRenderStep::EType::BlendLayer:
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(RenderAndBlendLayer);

				check((Renderer != nullptr) && EnumHasAnyFlags(Renderer->GetRenderFlags(this), ERenderFlags::RenderMode_Mask));

				const bool bIsRecordedStep = EnumHasAnyFlags(RenderStep.RenderFlags, ERenderFlags::RenderMode_Recorded);
				const bool bIsSeparateBlendStep = (RenderStep.Type == FMergeRenderStep::EType::BlendLayer);

				FString RenderStepProfilingEventName = FString::Format(TEXT("Step [{0}] ({1}): {2} {3}"), { RenderLayerStepIndex, *ConvertTargetLayerNamesToString(TargetLayerGroupLayerNames), bIsSeparateBlendStep ? TEXT("Blend") : TEXT("Render"), *Renderer->GetEditLayerRendererDebugName() });

				// Drop some visual cues to help understand how each renderer is applied :
				UE_IFVLOG(
				if (IsVisualLogEnabled() && !bIsSeparateBlendStep && (ShowMergeProcess == 2))
					{
						FTransform RenderAreaWorldTransformForVisLog = FTransform(FVector(RenderBatch.SectionRect.Min)) * LandscapeWorldTransformForVisLog;
						IncrementVisualLogOffset();
						UE_VLOG_LOCATION(Landscape, LogLandscape, Log, ComputeVisualLogTransform(RenderAreaWorldTransformForVisLog).GetTranslation(), 10.0f, FColor::Red, TEXT("%s"), *RenderStepProfilingEventName);
						UE_VLOG_WIREOBOX(Landscape, LogLandscape, Log, FBox(FVector(RenderBatch.SectionRect.Min) - FVector(0.5, 0.5, 0.0), FVector(RenderBatch.SectionRect.Max) - FVector(0.5, 0.5, 0.0)),
							ComputeVisualLogTransform(LandscapeWorldTransformForVisLog).ToMatrixWithScale(), FColor::White, TEXT(""));

						// Draw each component's bounds rendered by this renderer : 
						for (const FComponentMergeRenderInfo& ComponentMergeRenderInfo : SortedComponentMergeRenderInfos)
						{
							UE_VLOG_WIREOBOX(Landscape, LogLandscape, Log, FBox(FVector(ComponentMergeRenderInfo.ComponentRegionInRenderArea.Min), FVector(ComponentMergeRenderInfo.ComponentRegionInRenderArea.Max)),
								ComputeVisualLogTransform(RenderAreaWorldTransformForVisLog).ToMatrixWithScale(), FColor::White, TEXT(""));
						}
					});
		
				FRenderParams RenderParams(this, TargetLayerGroupLayerNames, TargetLayerGroupLayerInfos, RenderStep.RendererState, SortedComponentMergeRenderInfos, 
					RenderAreaWorldTransform, RenderAreaSectionRect, NumSuccessfulRenderLayerStepsUntilBlendLayerStep);
				auto RenderOrBlend = [&RDGBuilderRecorder, &RenderStepProfilingEventName, &NumSuccessfulRenderLayerStepsUntilBlendLayerStep, &RenderParams, bIsSeparateBlendStep, Renderer]()
					{
						if (bIsSeparateBlendStep)
						{
							// Skip the blend if nothing was ever rendered : 
							if (NumSuccessfulRenderLayerStepsUntilBlendLayerStep > 0)
							{
								Renderer->BlendLayer(RenderParams, RDGBuilderRecorder);
							}
							// The blend has occurred, we can now stop tracking the number of successful RenderLayer steps :
							NumSuccessfulRenderLayerStepsUntilBlendLayerStep = 0;
						}
						else
						{
							const bool bHasStepRenderedSomething = Renderer->RenderLayer(RenderParams, RDGBuilderRecorder);
							NumSuccessfulRenderLayerStepsUntilBlendLayerStep += bHasStepRenderedSomething ? 1 : 0;
						}
					};
    
				    if (bIsRecordedStep)
				    {
					    checkf(RDGBuilderRecorder.IsRecording(), TEXT("(Render/Blend)Layer_Recorded must be preceded by a BeginRenderCommandRecorder which should create a command recorder"));
					    RDG_RENDER_COMMAND_RECORDER_BREADCRUMB_EVENT(RDGBuilderRecorder, "%s", *RenderStepProfilingEventName);
						RenderOrBlend();
				    }
				    else
				    {
					    checkf(RDGBuilderRecorder.IsEmpty() && !RDGBuilderRecorder.IsRecording(), TEXT("(Render/Blend)Layer_Immediate should be preceded by a EndRenderCommandRecorder which should finalize a command recorder and destroy it"));
					        RHI_BREADCRUMB_EVENT_GAMETHREAD_F("Step", "%s", RenderStepProfilingEventName);
					    RenderOrBlend();
				    }
    
				    ++RenderLayerStepIndex;
				    break;
			    }
			    case FMergeRenderStep::EType::SignalBatchMergeGroupDone:
			    {
				    TRACE_CPUPROFILER_EVENT_SCOPE(MergeGroupDone);
				    RHI_BREADCRUMB_EVENT_GAMETHREAD_F("Step", "Step [%i] (%s) : Render Group Done", RenderLayerStepIndex, ConvertTargetLayerNamesToString(TargetLayerGroupLayerNames));
    
				    checkf(RDGBuilderRecorder.IsEmpty() && !RDGBuilderRecorder.IsRecording(), TEXT("SignalBatchMergeGroupDone should be preceded by a EndRenderCommandRecorder which should finalize a command recorder and destroy it"));

				    // The last render target we wrote to is the one containing the batch group's merge result : 
				    FOnRenderBatchTargetGroupDoneParams Params(this, TargetLayerGroupLayerNames, TargetLayerGroupLayerInfos, SortedComponentMergeRenderInfos);
				    OnBatchTargetGroupDone(Params, RDGBuilderRecorder);
				    break;
			    }
			    default:
				    checkf(false, TEXT("Invalid render step type"));
			}
		}
		
		checkf(RDGBuilderRecorder.IsEmpty() && !RDGBuilderRecorder.IsRecording(), TEXT("We should not have any pending render command recorder at the end of render"));

		FreeBatchResources(RenderBatch);
	}

	FreeResources();
}

const FMergeRenderBatch* FMergeRenderContext::GetCurrentRenderBatch() const
{
	return RenderBatches.IsValidIndex(CurrentRenderBatchIndex) ? &RenderBatches[CurrentRenderBatchIndex] : nullptr;
}

const int32 FMergeRenderContext::GetCurrentRenderBatchIdx() const
{
	return RenderBatches.IsValidIndex(CurrentRenderBatchIndex) ? CurrentRenderBatchIndex : INDEX_NONE;
}

bool FMergeRenderContext::IsValid() const
{
	return !RenderBatches.IsEmpty();
}

#if ENABLE_VISUAL_LOG

int32 FMergeRenderContext::GetVisualLogAlpha()
{
	return FMath::Clamp(CVarLandscapeBatchedMergeVisualLogAlpha.GetValueOnGameThread(), 0.0f, 1.0f) * 255;
}

bool FMergeRenderContext::IsVisualLogEnabled() const
{
	switch (CVarLandscapeBatchedMergeVisualLogShowMergeType.GetValueOnGameThread())
	{
	case 0: // Disabled
		return false;
	case 1: // Heightmaps only
		return bIsHeightmapMerge;
	case 2: // Weightmaps only
		return !bIsHeightmapMerge;
	case 3: // Both
		return true;
	default:
		return false;
	}
}

#endif // ENABLE_VISUAL_LOG


// ----------------------------------------------------------------------------------

FBox FOOBox2D::BuildAABB() const
{
	return FBox(
		{
			Transform.TransformPosition(FVector(+Extents.X, +Extents.Y, 0.0)),
			Transform.TransformPosition(FVector(+Extents.X, -Extents.Y, 0.0)),
			Transform.TransformPosition(FVector(-Extents.X, +Extents.Y, 0.0)),
			Transform.TransformPosition(FVector(-Extents.X, -Extents.Y, 0.0)),
		});
}


// ----------------------------------------------------------------------------------

FIntRect FInputWorldArea::GetLocalComponentKeys(const FIntPoint& InComponentKey) const
{
	check(Type == EType::LocalComponent); 
	return LocalArea + InComponentKey;
}

FIntRect FInputWorldArea::GetSpecificComponentKeys() const
{
	check(Type == EType::SpecificComponent);
	return LocalArea + SpecificComponentKey;
}

FBox FInputWorldArea::ComputeWorldAreaAABB(const FTransform& InLandscapeTransform, const FBox& InLandscapeLocalBounds, const FTransform& InComponentTransform, const FBox& InComponentLocalBounds) const
{
	switch (Type)
	{
		case EType::Infinite:
		{
			return InLandscapeLocalBounds.TransformBy(InLandscapeTransform);
		}
		case EType::LocalComponent:
		{
			return InComponentLocalBounds.TransformBy(InComponentTransform);
		}
		case EType::SpecificComponent:
		{
			FVector ComponentLocalSize = InComponentLocalBounds.GetSize();
			FIntRect LocalAreaCoordinates(SpecificComponentKey + LocalArea.Min, SpecificComponentKey + LocalArea.Max);
			FBox LocalAreaBounds = FBox(
				FVector(LocalAreaCoordinates.Min.X * ComponentLocalSize.X, LocalAreaCoordinates.Min.Y * ComponentLocalSize.Y, InComponentLocalBounds.Min.Z),
				FVector(LocalAreaCoordinates.Max.X * ComponentLocalSize.X, LocalAreaCoordinates.Max.Y * ComponentLocalSize.Y, InComponentLocalBounds.Max.Z));
			return LocalAreaBounds.TransformBy(InComponentTransform);
		}
		case EType::OOBox:
		{
			return OOBox2D.BuildAABB();
		}
		default:
			check(false);
	}

	return FBox();
}

FOOBox2D FInputWorldArea::ComputeWorldAreaOOBB(const FTransform& InLandscapeTransform, const FBox& InLandscapeLocalBounds, const FTransform& InComponentTransform, const FBox& InComponentLocalBounds) const
{
	switch (Type)
	{
	case EType::Infinite:
	{
		FVector Center, Extents;
		InLandscapeLocalBounds.GetCenterAndExtents(Center, Extents);
		FTransform LandscapeTransformAtCenter = InLandscapeTransform;
		LandscapeTransformAtCenter.SetTranslation(InLandscapeTransform.TransformVector(Center));
		return FOOBox2D(LandscapeTransformAtCenter, FVector2D(Extents));
	}
	case EType::LocalComponent:
	{
		FVector Center, Extents;
		InComponentLocalBounds.GetCenterAndExtents(Center, Extents);
		FTransform ComponentTransformAtCenter = InComponentTransform;
		ComponentTransformAtCenter.SetTranslation(InComponentTransform.TransformVector(Center));
		return FOOBox2D(ComponentTransformAtCenter, FVector2D(Extents));
	}
	case EType::SpecificComponent:
	{
		FVector ComponentLocalSize = InComponentLocalBounds.GetSize();
		FIntRect LocalAreaCoordinates(SpecificComponentKey + LocalArea.Min, SpecificComponentKey + LocalArea.Max);
		FBox LocalAreaBounds = FBox(
			FVector(LocalAreaCoordinates.Min.X * ComponentLocalSize.X, LocalAreaCoordinates.Min.Y * ComponentLocalSize.Y, InComponentLocalBounds.Min.Z),
			FVector(LocalAreaCoordinates.Max.X * ComponentLocalSize.X, LocalAreaCoordinates.Max.Y * ComponentLocalSize.Y, InComponentLocalBounds.Max.Z));
		FVector Center, Extents;
		LocalAreaBounds.GetCenterAndExtents(Center, Extents);
		FTransform ComponentTransformAtCenter = InComponentTransform;
		ComponentTransformAtCenter.SetTranslation(InComponentTransform.TransformVector(Center));
		return FOOBox2D(ComponentTransformAtCenter, FVector2D(Extents));
	}
	case EType::OOBox:
	{
		return OOBox2D;
	}
	default:
		check(false);
	}

	return FOOBox2D();
}


// ----------------------------------------------------------------------------------

FBox FOutputWorldArea::ComputeWorldAreaAABB(const FTransform& InComponentTransform, const FBox& InComponentLocalBounds) const
{
	switch (Type)
	{
	case EType::LocalComponent:
	{
		return InComponentLocalBounds.TransformBy(InComponentTransform);
	}
	case EType::SpecificComponent:
	{
		FVector ComponentLocalSize = InComponentLocalBounds.GetSize();
		FBox LocalAreaBounds = FBox(
			FVector(SpecificComponentKey.X * ComponentLocalSize.X, SpecificComponentKey.Y * ComponentLocalSize.Y, InComponentLocalBounds.Min.Z),
			FVector((SpecificComponentKey.X + 1) * ComponentLocalSize.X, (SpecificComponentKey.Y + 1) * ComponentLocalSize.Y, InComponentLocalBounds.Max.Z));
		return LocalAreaBounds.TransformBy(InComponentTransform);
	}
	case EType::OOBox:
	{
		return OOBox2D.BuildAABB();
	}
	default:
		check(false);
	}

	return FBox();
}

FOOBox2D FOutputWorldArea::ComputeWorldAreaOOBB(const FTransform& InComponentTransform, const FBox& InComponentLocalBounds) const
{
	switch (Type)
	{
	case EType::LocalComponent:
	{
		FVector Center, Extents;
		InComponentLocalBounds.GetCenterAndExtents(Center, Extents);
		FTransform ComponentTransformAtCenter = InComponentTransform;
		ComponentTransformAtCenter.SetTranslation(InComponentTransform.TransformVector(Center));
		return FOOBox2D(ComponentTransformAtCenter, FVector2D(Extents));
	}
	case EType::SpecificComponent:
	{
		FVector ComponentLocalSize = InComponentLocalBounds.GetSize();
		FBox LocalAreaBounds = FBox(
			FVector(SpecificComponentKey.X * ComponentLocalSize.X, SpecificComponentKey.Y * ComponentLocalSize.Y, InComponentLocalBounds.Min.Z),
			FVector((SpecificComponentKey.X + 1) * ComponentLocalSize.X, (SpecificComponentKey.Y + 1) * ComponentLocalSize.Y, InComponentLocalBounds.Max.Z));
		FVector Center, Extents;
		LocalAreaBounds.GetCenterAndExtents(Center, Extents);
		FTransform ComponentTransformAtCenter = InComponentTransform;
		ComponentTransformAtCenter.SetTranslation(InComponentTransform.TransformVector(Center));
		return FOOBox2D(ComponentTransformAtCenter, FVector2D(Extents));
	}
	case EType::OOBox:
	{
		return OOBox2D;
	}
	default:
		check(false);
	}

	return FOOBox2D();
}


// ----------------------------------------------------------------------------------

bool FComponentMergeRenderInfo::operator<(const FComponentMergeRenderInfo& InOther) const
{
	// Sort by X / Y so that the order in which we render them is always consistent : 
	if (ComponentRegionInRenderArea.Min.Y < InOther.ComponentRegionInRenderArea.Min.Y)
	{
		return true;
	}
	else if (ComponentRegionInRenderArea.Min.Y == InOther.ComponentRegionInRenderArea.Min.Y)
	{
		return (ComponentRegionInRenderArea.Min.X < InOther.ComponentRegionInRenderArea.Min.X);
	}
	return false;
}


// ----------------------------------------------------------------------------------

FMergeRenderParams::FMergeRenderParams(TArray<ULandscapeComponent*> InComponentsToMerge, const TArrayView<FEditLayerRendererState>& InEditLayerRendererStates, const TSet<FName>& InWeightmapLayerNames, bool bInRequestAllLayers)
	: ComponentsToMerge(InComponentsToMerge)
	, EditLayerRendererStates(InEditLayerRendererStates)
	, WeightmapLayerNames(InWeightmapLayerNames)
	, bRequestAllLayers(bInRequestAllLayers)
{
}

#endif // WITH_EDITOR

} // namespace UE::Landscape::EditLayers
