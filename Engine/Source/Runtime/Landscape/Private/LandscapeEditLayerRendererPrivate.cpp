// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeEditLayerRendererPrivate.h"

#include "Algo/AllOf.h"
#include "Algo/ForEach.h"
#include "Algo/Transform.h"
#include "LandscapeEditResourcesSubsystem.h"
#include "LandscapeEditLayerMergeRenderContext.h"
#include "LandscapeEditLayerTargetLayerGroup.h"
#include "LandscapeLayerInfoObject.h"
#include "LandscapeInfo.h"
#include "RHIAccess.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapeEditLayerRendererPrivate)


#if WITH_EDITOR

// ----------------------------------------------------------------------------------

void ULandscapeDefaultEditLayerRenderer::GetRendererStateInfo(const UE::Landscape::EditLayers::FMergeContext* InMergeContext,
	UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutSupportedTargetTypeState, UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutEnabledTargetTypeState, 
	TArray<UE::Landscape::EditLayers::FTargetLayerGroup>& OutTargetLayerGroups) const
{
	using namespace UE::Landscape::EditLayers;

	// Supports all heightmaps and weightmaps:
	OutSupportedTargetTypeState = FEditLayerTargetTypeState(InMergeContext, ELandscapeToolTargetTypeFlags::All, InMergeContext->GetValidTargetLayerBitIndices());
	OutEnabledTargetTypeState = OutSupportedTargetTypeState;
}

TArray<UE::Landscape::EditLayers::FEditLayerRenderItem> ULandscapeDefaultEditLayerRenderer::GetRenderItems(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const
{
	using namespace UE::Landscape::EditLayers;

	FEditLayerTargetTypeState OutputTargetTypeState(InMergeContext, ELandscapeToolTargetTypeFlags::All, InMergeContext->GetValidTargetLayerBitIndices());
	// Standard renderer : we don't need more than the component itself to render properly:
	FInputWorldArea InputWorldArea(FInputWorldArea::CreateLocalComponent());
	// The renderer only writes into the component itself (i.e. it renders to the area that it's currently being asked to render to):
	FOutputWorldArea OutputWorldArea(FOutputWorldArea::CreateLocalComponent());
	// The renderer is only providing default data for existing weightmaps so it doesn't generate new ones, hence we pass bModifyExistingWeightmapsOnly to true : 
	return { FEditLayerRenderItem(OutputTargetTypeState, InputWorldArea, OutputWorldArea, /*bModifyExistingWeightmapsOnly = */true) };
}

FString ULandscapeDefaultEditLayerRenderer::GetEditLayerRendererDebugName() const 
{
	return TEXT("Default");
}

bool ULandscapeDefaultEditLayerRenderer::RenderLayer(UE::Landscape::EditLayers::FRenderParams& RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder)
{
	using namespace UE::Landscape::EditLayers;

	checkf(RDGBuilderRecorder.IsRecording(), TEXT("ERenderFlags::RenderMode_Recorded means the command recorder should be recording at this point"));

	FMergeRenderContext* RenderContext = RenderParams.MergeRenderContext;

	RenderContext->CycleBlendRenderTargets(RDGBuilderRecorder);
	ULandscapeScratchRenderTarget* WriteRT = RenderContext->GetBlendRenderTargetWrite();

	// Start from a blank canvas so that the first layer is blended with nothing underneath :
	WriteRT->Clear(RDGBuilderRecorder);
	check(WriteRT->GetCurrentState() == ERHIAccess::RTV);

	// Render the components of the batch for each target layer into the "pseudo-stencil" buffer, so that it can be sampled by users as a UTextures in UMaterials and such : 
	RenderContext->RenderComponentIds(RDGBuilderRecorder);

	return true; 
}


// ----------------------------------------------------------------------------------

void ULandscapeHeightmapNormalsEditLayerRenderer::GetRendererStateInfo(const UE::Landscape::EditLayers::FMergeContext* InMergeContext,
	UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutSupportedTargetTypeState, UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutEnabledTargetTypeState, 
	TArray<UE::Landscape::EditLayers::FTargetLayerGroup>& OutTargetLayerGroups) const
{
	// Only relevant for heightmaps :
	OutSupportedTargetTypeState.SetTargetTypeMask(ELandscapeToolTargetTypeFlags::Heightmap);
	OutEnabledTargetTypeState.SetTargetTypeMask(ELandscapeToolTargetTypeFlags::Heightmap);
}

TArray<UE::Landscape::EditLayers::FEditLayerRenderItem> ULandscapeHeightmapNormalsEditLayerRenderer::GetRenderItems(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const
{
	using namespace UE::Landscape::EditLayers;

	// Only relevant for heightmaps :
	if (!InMergeContext->IsHeightmapMerge())
	{
		return {};
	}
	
	FEditLayerTargetTypeState OutputTargetTypeState(InMergeContext, ELandscapeToolTargetTypeFlags::Heightmap);
	// The input is relative and its size is equal to the size of 3x3 landscape components so that we gather all neighbor landscape components around each component : 
	FInputWorldArea InputWorldArea(FInputWorldArea::CreateLocalComponent(FIntRect(-1, -1, 1, 1)));
	// The renderer only writes into the component itself (i.e. it renders to the area that it's currently being asked to render to):
	FOutputWorldArea OutputWorldArea(FOutputWorldArea::CreateLocalComponent());
	return { FEditLayerRenderItem(OutputTargetTypeState, InputWorldArea, OutputWorldArea, /*bModifyExistingWeightmapsOnly = */false) };
}

FString ULandscapeHeightmapNormalsEditLayerRenderer::GetEditLayerRendererDebugName() const
{
	return TEXT("Normals");
}


// ----------------------------------------------------------------------------------

void ULandscapeWeightmapWeightBlendedLayersRenderer::GetRendererStateInfo(const UE::Landscape::EditLayers::FMergeContext* InMergeContext,
	UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutSupportedTargetTypeState, UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutEnabledTargetTypeState, 
	TArray<UE::Landscape::EditLayers::FTargetLayerGroup>& OutTargetLayerGroups) const
{
	using namespace UE::Landscape::EditLayers;

	TBitArray<> WeightBlendedWeightmapLayerBitIndices = GatherWeightBlendedWeightmapLayerBitIndices(InMergeContext);

	// Only relevant for weightmaps :
	FEditLayerTargetTypeState OutputTargetTypeState(InMergeContext, ELandscapeToolTargetTypeFlags::Weightmap, WeightBlendedWeightmapLayerBitIndices);
	OutEnabledTargetTypeState = OutSupportedTargetTypeState = OutputTargetTypeState;

	// Now fill in the target layer groups : one group for all weight-blended layers : 
	if (WeightBlendedWeightmapLayerBitIndices.Contains(true))
	{
		static const FName FinalWeightBlendedTargetLayerGroupName("FinalWeightBlendedLayerGroup");
		OutTargetLayerGroups.Add(FTargetLayerGroup(FinalWeightBlendedTargetLayerGroupName, WeightBlendedWeightmapLayerBitIndices));
	}
}

TArray<UE::Landscape::EditLayers::FEditLayerRenderItem> ULandscapeWeightmapWeightBlendedLayersRenderer::GetRenderItems(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const
{
	using namespace UE::Landscape::EditLayers;

	// Only relevant for weightmaps :
	if (InMergeContext->IsHeightmapMerge())
	{
		return {};
	}

	TBitArray<> WeightBlendedWeightmapLayerBitIndices = GatherWeightBlendedWeightmapLayerBitIndices(InMergeContext);
	FEditLayerTargetTypeState OutputTargetTypeState(InMergeContext, ELandscapeToolTargetTypeFlags::Weightmap, WeightBlendedWeightmapLayerBitIndices);
	// Standard renderer : we don't need more than the component itself to render properly:
	FInputWorldArea InputWorldArea(FInputWorldArea::CreateLocalComponent());
	// The renderer only writes into the component itself (i.e. it renders to the area that it's currently being asked to render to):
	FOutputWorldArea OutputWorldArea(FOutputWorldArea::CreateLocalComponent());
	// The renderer is only blending existing weightmaps so it doesn't generate new ones, hence we pass bModifyExistingWeightmapsOnly to true : 
	return { FEditLayerRenderItem(OutputTargetTypeState, InputWorldArea, OutputWorldArea, /*bModifyExistingWeightmapsOnly = */true) };
}

FString ULandscapeWeightmapWeightBlendedLayersRenderer::GetEditLayerRendererDebugName() const 
{
	return TEXT("Final Weight Blend");
}

TBitArray<> ULandscapeWeightmapWeightBlendedLayersRenderer::GatherWeightBlendedWeightmapLayerBitIndices(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const
{
	using namespace UE::Landscape::EditLayers;

	// Retrieve only the target layers that use ELandscapeTargetLayerBlendMethod::FinalWeightBlending
	TArray<FTargetLayerGroup> TargetLayerGroups = InMergeContext->GetTargetLayerGroupsPerBlendingMethod()[static_cast<uint8>(ELandscapeTargetLayerBlendMethod::FinalWeightBlending)];
	// There should be 0 if no target layer uses ELandscapeTargetLayerBlendMethod::FinalWeightBlending, or a single one since it doesn't have the notion of "blend group" :
	check(TargetLayerGroups.Num() <= 1);

	return TargetLayerGroups.IsEmpty()
		? InMergeContext->BuildTargetLayerBitIndices(/*bInBitValue = */false)
		: TargetLayerGroups[0].GetWeightmapTargetLayerBitIndices();
}

#endif // WITH_EDITOR
