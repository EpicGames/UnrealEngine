// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LandscapeEditTypes.h"
#include "LandscapeEditLayerRenderer.h"

#include "LandscapeEditLayerRendererPrivate.generated.h"


/**
* Edit layer renderer added at the bottom of the stack to provide the default value for every requested target layer (heightmaps and weightmaps)
*  It could have been handled with a simple clear as the first operation when performing the merge, but doing it through a renderer turns out to be an elegant way to resolve the
*  situation where the first actual edit layer's render item declares dependencies between each component and others (e.g. say you have only a ULandscapeHeightmapNormalsEditLayerRenderer
*  in the renderer stack, which requires each component's immediate neighbors). In such a situation, the component dependencies would be skipped because dependencies are
*  between a renderer and its previous one in the stack and since in the case described above (a single renderer in the stack), there's no previous renderer, then the dependencies
*  would simply not be registered, and the render batches would end up being incorrect as a result.
*/
UCLASS()
class ULandscapeDefaultEditLayerRenderer : public UObject
#if CPP && WITH_EDITOR
	, public ILandscapeEditLayerRenderer
#endif // CPP && WITH_EDITOR
{
	GENERATED_BODY()

#if WITH_EDITOR
public:
	//~ Begin ILandscapeEditLayerRenderer implementation
	virtual void GetRendererStateInfo(const UE::Landscape::EditLayers::FMergeContext* InMergeContext,
		UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutSupportedTargetTypeState, UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutEnabledTargetTypeState,
		TArray<UE::Landscape::EditLayers::FTargetLayerGroup>& OutTargetLayerGroups) const override;
	virtual TArray<UE::Landscape::EditLayers::FEditLayerRenderItem> GetRenderItems(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const override;
	virtual FString GetEditLayerRendererDebugName() const override;
	virtual UE::Landscape::EditLayers::ERenderFlags GetRenderFlags(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const override { return UE::Landscape::EditLayers::ERenderFlags::RenderMode_Recorded; }
	virtual bool RenderLayer(UE::Landscape::EditLayers::FRenderParams& RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder) override;
	//~ End ILandscapeEditLayerRenderer implementation
#endif // WITH_EDITOR
};

/**
* Edit layer renderer added at the top of the stack to generate the normals, right before resolving the textures.
*  For now, the rendered components require (up to) their 8 immediate neighbors to generate adequate normals on the border so the renderer inserts that strong dependency, so that the components
*  that are needed in the batch are guaranteed to have their neighbors present in the same batch :
*/
UCLASS()
class ULandscapeHeightmapNormalsEditLayerRenderer : public UObject
#if CPP && WITH_EDITOR
	, public ILandscapeEditLayerRenderer
#endif //CPP &&  WITH_EDITOR
{
	GENERATED_BODY()

#if WITH_EDITOR
public:
	//~ Begin ILandscapeEditLayerRenderer implementation
	virtual void GetRendererStateInfo(const UE::Landscape::EditLayers::FMergeContext* InMergeContext,
		UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutSupportedTargetTypeState, UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutEnabledTargetTypeState, 
		TArray<UE::Landscape::EditLayers::FTargetLayerGroup>& OutTargetLayerGroups) const override;
	virtual TArray<UE::Landscape::EditLayers::FEditLayerRenderItem> GetRenderItems(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const override;
	virtual FString GetEditLayerRendererDebugName() const override;
	virtual UE::Landscape::EditLayers::ERenderFlags GetRenderFlags(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const override { return UE::Landscape::EditLayers::ERenderFlags::RenderMode_Recorded; }
	virtual bool RenderLayer(UE::Landscape::EditLayers::FRenderParams& RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder) override;
	//~ End ILandscapeEditLayerRenderer implementation
#endif // WITH_EDITOR
};

/** 
* Edit layer renderer inserted at the end of the edit layers stack merge to normalize the weights of the rendered weightmaps 
*/
UCLASS()
class ULandscapeWeightmapWeightBlendedLayersRenderer : public UObject
#if CPP && WITH_EDITOR
	, public ILandscapeEditLayerRenderer
#endif // CPP && WITH_EDITOR
{
	GENERATED_BODY()

#if WITH_EDITOR
public:
	//~ Begin ILandscapeEditLayerRenderer implementation
	virtual void GetRendererStateInfo(const UE::Landscape::EditLayers::FMergeContext* InMergeContext,
		UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutSupportedTargetTypeState, UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutEnabledTargetTypeState, 
		TArray<UE::Landscape::EditLayers::FTargetLayerGroup>& OutTargetLayerGroups) const override;
	virtual TArray<UE::Landscape::EditLayers::FEditLayerRenderItem> GetRenderItems(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const override;
	virtual FString GetEditLayerRendererDebugName() const override;
	virtual UE::Landscape::EditLayers::ERenderFlags GetRenderFlags(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const override { return UE::Landscape::EditLayers::ERenderFlags::RenderMode_Recorded; }
	virtual bool RenderLayer(UE::Landscape::EditLayers::FRenderParams& RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder) override;
	//~ End ILandscapeEditLayerRenderer implementation

	TBitArray<> GatherWeightBlendedWeightmapLayerBitIndices(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const;
#endif // WITH_EDITOR
};

