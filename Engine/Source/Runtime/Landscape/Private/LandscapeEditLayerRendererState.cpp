// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeEditLayerRendererState.h"

#include "Algo/Count.h"
#include "LandscapeEditLayerRenderer.h"
#include "LandscapeEditLayerMergeContext.h"

namespace UE::Landscape::EditLayers
{

#if WITH_EDITOR

const FEditLayerRendererState& FEditLayerRendererState::GetDummyRendererState()
{
	static FEditLayerRendererState DummyRendererState;
	return DummyRendererState;
}

FEditLayerRendererState::FEditLayerRendererState(const FMergeContext* InMergeContext)
	: MergeContext(InMergeContext)
	, SupportedTargetTypeState(InMergeContext)
	, EnabledTargetTypeState(InMergeContext)
	, ActiveTargetTypeState(InMergeContext)
{
	UpdateActiveTargetTypeState();
}

FEditLayerRendererState::FEditLayerRendererState(const FMergeContext* InMergeContext, TScriptInterface<ILandscapeEditLayerRenderer> InRenderer)
	: MergeContext(InMergeContext)
	, Renderer(InRenderer)
	, DebugName(InRenderer->GetEditLayerRendererDebugName())
	, SupportedTargetTypeState(InMergeContext)
	, EnabledTargetTypeState(InMergeContext)
	, ActiveTargetTypeState(InMergeContext)
{
	Renderer->GetRendererStateInfo(InMergeContext, SupportedTargetTypeState, EnabledTargetTypeState, TargetLayerGroups);

	// Make sure that each supported weightmap belongs to one target layer group and one only. For those that are in no target layer group, put them in their own group, that simply means this renderer 
	//  can render them without requesting the presence of other target layers (e.g. no weight-blending)
	InMergeContext->ForEachTargetLayer(SupportedTargetTypeState.GetActiveWeightmapBitIndices(),
		[InMergeContext, InRenderer, this](int32 InTargetLayerIndex, const FName& InTargetLayerName, ULandscapeLayerInfoObject* InWeightmapLayerInfo)
		{
			const int32 TargetLayerGroupCount = Algo::CountIf(TargetLayerGroups, [InTargetLayerIndex](const FTargetLayerGroup& InTargetLayerGroup) { return InTargetLayerGroup.GetWeightmapTargetLayerBitIndices()[InTargetLayerIndex]; });
			checkf(TargetLayerGroupCount < 2, TEXT("Target layer %s belongs to more than 1 target layer group in edit layer renderer %s. This is forbidden: in the end, it must belong to 1 and 1 only."),
				*InTargetLayerName.ToString(), *InRenderer->GetEditLayerRendererDebugName());
			if (TargetLayerGroupCount == 0)
			{
				static const FName SoloTargetLayerGroupName("SoloTargetLayerGroup");
				TBitArray<> TargetLayerGroupBitIndices = InMergeContext->BuildTargetLayerBitIndices(false);
				TargetLayerGroupBitIndices[InTargetLayerIndex] = true;
				TargetLayerGroups.Add(FTargetLayerGroup(SoloTargetLayerGroupName, TargetLayerGroupBitIndices));
			}
			return true;
		});

	UpdateActiveTargetTypeState();
}

void FEditLayerRendererState::EnableTargetType(ELandscapeToolTargetType InTargetType)
{
	checkf(EnumHasAllFlags(SupportedTargetTypeState.GetTargetTypeMask(), GetLandscapeToolTargetTypeAsFlags(InTargetType)),
		TEXT("Target type %s cannot be enabled on this renderer state because it is not supported. Make sure that target types are supported before enabling them"), *UEnum::GetValueAsString(InTargetType));
	EnabledTargetTypeState.AddTargetType(InTargetType);
	UpdateActiveTargetTypeState();
}

void FEditLayerRendererState::EnableTargetTypeMask(ELandscapeToolTargetTypeFlags InTargetTypeMask)
{
	for (ELandscapeToolTargetTypeFlags TargetTypeFlag : MakeFlagsRange(InTargetTypeMask))
	{
		EnableTargetType(UE::Landscape::GetLandscapeToolTargetTypeSingleFlagAsType(TargetTypeFlag));
	}
}

void FEditLayerRendererState::DisableTargetType(ELandscapeToolTargetType InTargetType)
{
	EnabledTargetTypeState.RemoveTargetType(InTargetType);
	UpdateActiveTargetTypeState();
}

void FEditLayerRendererState::DisableTargetTypeMask(ELandscapeToolTargetTypeFlags InTargetTypeMask)
{
	EnabledTargetTypeState.RemoveTargetTypeMask(InTargetTypeMask);
	UpdateActiveTargetTypeState();
}

void FEditLayerRendererState::EnableWeightmap(ELandscapeToolTargetType InTargetType, const FName& InWeightmapLayerName)
{
	EnableTargetType(InTargetType);
	if (int32 TargetLayerIndex = MergeContext->GetTargetLayerIndexForName(InWeightmapLayerName); TargetLayerIndex != INDEX_NONE)
	{
		EnabledTargetTypeState.AddWeightmap(TargetLayerIndex);
		UpdateActiveTargetTypeState();
	}
}

void FEditLayerRendererState::EnableWeightmapChecked(ELandscapeToolTargetType InTargetType, const FName& InWeightmapLayerName)
{
	EnableWeightmap(InTargetType, (InWeightmapLayerName != NAME_None) ? MergeContext->GetTargetLayerIndexForNameChecked(InWeightmapLayerName) : INDEX_NONE);
}

void FEditLayerRendererState::EnableWeightmap(ELandscapeToolTargetType InTargetType, int32 InWeightmapLayerIndex)
{
	EnableTargetType(InTargetType);
	EnabledTargetTypeState.AddWeightmap(InWeightmapLayerIndex);
	UpdateActiveTargetTypeState();
}

void FEditLayerRendererState::DisableWeightmap(const FName& InWeightmapLayerName)
{
	if (int32 TargetLayerIndex = MergeContext->GetTargetLayerIndexForName(InWeightmapLayerName); TargetLayerIndex != INDEX_NONE)
	{
		DisableWeightmap(TargetLayerIndex);
	}
}

void FEditLayerRendererState::DisableWeightmapChecked(const FName& InWeightmapLayerName)
{
	DisableWeightmap(MergeContext->GetTargetLayerIndexForNameChecked(InWeightmapLayerName));
}

void FEditLayerRendererState::DisableWeightmap(int32 InWeightmapLayerIndex)
{
	EnabledTargetTypeState.RemoveWeightmap(InWeightmapLayerIndex);
	UpdateActiveTargetTypeState();
}

ELandscapeToolTargetTypeFlags FEditLayerRendererState::GetActiveTargetTypeMask() const
{
	return ActiveTargetTypeState.GetTargetTypeMask();
}

bool FEditLayerRendererState::IsTargetActive(ELandscapeToolTargetType InTargetType, const FName& InWeightmapLayerName) const
{
	return ActiveTargetTypeState.IsActive(InTargetType, InWeightmapLayerName);
}

bool FEditLayerRendererState::IsTargetActiveChecked(ELandscapeToolTargetType InTargetType, const FName& InWeightmapLayerName) const
{
	return ActiveTargetTypeState.IsActiveChecked(InTargetType, InWeightmapLayerName);
}

bool FEditLayerRendererState::IsTargetActive(ELandscapeToolTargetType InTargetType, int32 InWeightmapLayerIndex) const
{
	return ActiveTargetTypeState.IsActive(InTargetType, InWeightmapLayerIndex);
}

TArray<FName> FEditLayerRendererState::GetActiveTargetWeightmaps() const
{
	return ActiveTargetTypeState.GetActiveWeightmaps();
}

TBitArray<> FEditLayerRendererState::GetActiveTargetWeightmapBitIndices() const
{
	return ActiveTargetTypeState.GetActiveWeightmapBitIndices();
}

void FEditLayerRendererState::UpdateActiveTargetTypeState()
{
	ActiveTargetTypeState = SupportedTargetTypeState.Intersect(EnabledTargetTypeState);
}

#endif // WITH_EDITOR

} // namespace UE::Landscape::EditLayers
