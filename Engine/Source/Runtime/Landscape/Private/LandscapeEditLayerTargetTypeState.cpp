// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeEditLayerTargetTypeState.h"

#include "Algo/Count.h"
#include "LandscapeEditLayerMergeContext.h"
#include "LandscapeLayerInfoObject.h"
#include "LandscapeUtils.h"

namespace UE::Landscape::EditLayers
{

#if WITH_EDITOR

const FEditLayerTargetTypeState& FEditLayerTargetTypeState::GetDummyTargetTypeState()
{
	static FEditLayerTargetTypeState DummyTargetTypeState;
	return DummyTargetTypeState;
}

FEditLayerTargetTypeState::FEditLayerTargetTypeState(const FMergeContext* InMergeContext)
	: FEditLayerTargetTypeState(InMergeContext, ELandscapeToolTargetTypeFlags::None)
{
}

FEditLayerTargetTypeState::FEditLayerTargetTypeState(const FMergeContext* InMergeContext, ELandscapeToolTargetTypeFlags InTargetTypeMask)
	: FEditLayerTargetTypeState(InMergeContext, InTargetTypeMask, TBitArray<>())
{
}

FEditLayerTargetTypeState::FEditLayerTargetTypeState(const FMergeContext* InMergeContext, ELandscapeToolTargetTypeFlags InTargetTypeMask, const TArrayView<const FName>& InSupportedWeightmaps, bool bInChecked)
	: FEditLayerTargetTypeState(InMergeContext, InTargetTypeMask,
		bInChecked ? InMergeContext->ConvertTargetLayerNamesToBitIndicesChecked(InSupportedWeightmaps) : InMergeContext->ConvertTargetLayerNamesToBitIndices(InSupportedWeightmaps))
{
}

FEditLayerTargetTypeState::FEditLayerTargetTypeState(const FMergeContext* InMergeContext, ELandscapeToolTargetTypeFlags InTargetTypeMask, const TBitArray<>& InSupportedWeightmapLayerIndices)
	: MergeContext(InMergeContext)
{
	if (InSupportedWeightmapLayerIndices.IsEmpty())
	{
		// Even if all weightmaps are turned off in this constructor, make sure to build a bit array that is dealing with as many layers as the merge context : 
		WeightmapTargetLayerBitIndices = MergeContext->BuildTargetLayerBitIndices(false);
	}
	else
	{
		checkf(InSupportedWeightmapLayerIndices.Num() == MergeContext->GetAllTargetLayerNames().Num(),
			TEXT("Make sure that the target type state is dealing with the same amount of target layers as the merge context"));
		WeightmapTargetLayerBitIndices = InSupportedWeightmapLayerIndices;
	}
	SetTargetTypeMask(InTargetTypeMask);
}

bool FEditLayerTargetTypeState::IsActive(ELandscapeToolTargetType InTargetType, const FName& InWeightmapLayerName) const
{
	if (InWeightmapLayerName != NAME_None)
	{
		if (int32 TargetLayerIndex = MergeContext->GetTargetLayerIndexForName(InWeightmapLayerName); TargetLayerIndex != INDEX_NONE)
		{
			return IsActive(InTargetType, TargetLayerIndex);
		}

		return false;
	}

	return IsActive(InTargetType, INDEX_NONE);
}

bool FEditLayerTargetTypeState::IsActiveChecked(ELandscapeToolTargetType InTargetType, const FName& InWeightmapLayerName) const
{
	if (InWeightmapLayerName != NAME_None)
	{
		return IsActive(InTargetType, MergeContext->GetTargetLayerIndexForNameChecked(InWeightmapLayerName));
	}

	return IsActive(InTargetType, INDEX_NONE);
}

bool FEditLayerTargetTypeState::IsActive(ELandscapeToolTargetType InTargetType, int32 InWeightmapLayerIndex) const
{
	check((InWeightmapLayerIndex == INDEX_NONE) || MergeContext->IsTargetLayerIndexValid(InWeightmapLayerIndex));

	if (EnumHasAnyFlags(TargetTypeMask, GetLandscapeToolTargetTypeAsFlags(InTargetType)))
	{
		if (InTargetType != ELandscapeToolTargetType::Heightmap)
		{
			return GetActiveWeightmapBitIndices()[InWeightmapLayerIndex];
		}

		return true;
	}

	return false;
}

TArray<FName> FEditLayerTargetTypeState::GetActiveWeightmaps() const
{
	return MergeContext->ConvertTargetLayerBitIndicesToNames(GetActiveWeightmapBitIndices());
}

TBitArray<> FEditLayerTargetTypeState::GetActiveWeightmapBitIndices() const
{
	TBitArray<> Result = WeightmapTargetLayerBitIndices;
	if (!EnumHasAnyFlags(TargetTypeMask, ELandscapeToolTargetTypeFlags::Visibility))
	{
		Result.CombineWithBitwiseAND(MergeContext->GetNegatedVisibilityTargetLayerMask(), EBitwiseOperatorFlags::MinSize);
	}
	if (!EnumHasAnyFlags(TargetTypeMask, ELandscapeToolTargetTypeFlags::Weightmap))
	{
		Result.CombineWithBitwiseAND(MergeContext->GetVisibilityTargetLayerMask(), EBitwiseOperatorFlags::MinSize);
	}
	return Result;
}

void FEditLayerTargetTypeState::SetTargetTypeMask(ELandscapeToolTargetTypeFlags InTargetTypeMask)
{
	if (InTargetTypeMask != TargetTypeMask)
	{
		TargetTypeMask = InTargetTypeMask;

		// Special case for the visibility weightmap, where we want to make sure the weightmap layer name is specified if visibility is supported (and vice versa) : 
		const int32 VisibilityTargetLayerIndex = MergeContext->GetVisibilityTargetLayerIndex();
		if (VisibilityTargetLayerIndex != INDEX_NONE)
		{
			if (EnumHasAnyFlags(TargetTypeMask, ELandscapeToolTargetTypeFlags::Visibility))
			{
				AddWeightmap(VisibilityTargetLayerIndex);
			}
			else if (WeightmapTargetLayerBitIndices[VisibilityTargetLayerIndex])
			{
				RemoveWeightmap(VisibilityTargetLayerIndex);
			}
		}
	}
}

void FEditLayerTargetTypeState::AddTargetType(ELandscapeToolTargetType InTargetType)
{
	SetTargetTypeMask(TargetTypeMask | GetLandscapeToolTargetTypeAsFlags(InTargetType));
}

void FEditLayerTargetTypeState::AddTargetTypeMask(ELandscapeToolTargetTypeFlags InTargetTypeMask)
{
	SetTargetTypeMask(TargetTypeMask | InTargetTypeMask);
}

void FEditLayerTargetTypeState::RemoveTargetType(ELandscapeToolTargetType InTargetType)
{
	SetTargetTypeMask(TargetTypeMask & ~GetLandscapeToolTargetTypeAsFlags(InTargetType));
}

void FEditLayerTargetTypeState::RemoveTargetTypeMask(ELandscapeToolTargetTypeFlags InTargetTypeMask)
{
	SetTargetTypeMask(TargetTypeMask & ~InTargetTypeMask);
}

FEditLayerTargetTypeState FEditLayerTargetTypeState::Intersect(const FEditLayerTargetTypeState& InOther) const
{
	checkf(WeightmapTargetLayerBitIndices.Num() == InOther.WeightmapTargetLayerBitIndices.Num(),
		TEXT("It is assumed that the 2 target type states to intersect are from the same context and are therefore dealing with the same amount of target layers"));
	return FEditLayerTargetTypeState(MergeContext, InOther.GetTargetTypeMask() & TargetTypeMask, TBitArray<>::BitwiseAND(InOther.WeightmapTargetLayerBitIndices, WeightmapTargetLayerBitIndices, EBitwiseOperatorFlags::MinSize));
}

void FEditLayerTargetTypeState::AddWeightmap(const FName& InWeightmapLayerName)
{
	if (int32 TargetLayerIndex = MergeContext->GetTargetLayerIndexForName(InWeightmapLayerName); TargetLayerIndex != INDEX_NONE)
	{
		AddWeightmap(TargetLayerIndex);
	}
}

void FEditLayerTargetTypeState::AddWeightmapChecked(const FName& InWeightmapLayerName)
{
	AddWeightmap(MergeContext->GetTargetLayerIndexForNameChecked(InWeightmapLayerName));
}

void FEditLayerTargetTypeState::AddWeightmap(int32 InWeightmapLayerIndex)
{
	check(MergeContext->IsTargetLayerIndexValid(InWeightmapLayerIndex));

	checkf((InWeightmapLayerIndex != MergeContext->GetVisibilityTargetLayerIndex()) || EnumHasAnyFlags(TargetTypeMask, ELandscapeToolTargetTypeFlags::Visibility),
		TEXT("The visibility layer may only be used for target type states that support visibility"));

	checkf((InWeightmapLayerIndex == MergeContext->GetVisibilityTargetLayerIndex()) || EnumHasAnyFlags(TargetTypeMask, ELandscapeToolTargetTypeFlags::Weightmap),
		TEXT("A weightmap layer (here, %s) may only be used for target type states that support weightmaps"), *MergeContext->GetTargetLayerNameForIndexChecked(InWeightmapLayerIndex).ToString());

	WeightmapTargetLayerBitIndices[InWeightmapLayerIndex] = true;
}

void FEditLayerTargetTypeState::RemoveWeightmap(const FName& InWeightmapLayerName)
{
	if (int32 TargetLayerIndex = MergeContext->GetTargetLayerIndexForName(InWeightmapLayerName); TargetLayerIndex != INDEX_NONE)
	{
		RemoveWeightmap(TargetLayerIndex);
	}
}

void FEditLayerTargetTypeState::RemoveWeightmapChecked(const FName& InWeightmapLayerName)
{
	RemoveWeightmap(MergeContext->GetTargetLayerIndexForNameChecked(InWeightmapLayerName));
}

void FEditLayerTargetTypeState::RemoveWeightmap(int32 InWeightmapLayerIndex)
{
	check(MergeContext->IsTargetLayerIndexValid(InWeightmapLayerIndex));

	checkf(!EnumHasAnyFlags(TargetTypeMask, ELandscapeToolTargetTypeFlags::Visibility) || (InWeightmapLayerIndex != MergeContext->GetVisibilityTargetLayerIndex()),
		TEXT("Cannot remove weightmap %s from a target type state that supports visibility"), *MergeContext->GetTargetLayerNameForIndexChecked(InWeightmapLayerIndex).ToString());

	WeightmapTargetLayerBitIndices[InWeightmapLayerIndex] = false;
}

bool FEditLayerTargetTypeState::operator==(const FEditLayerTargetTypeState& InOther) const
{
	return (TargetTypeMask == InOther.TargetTypeMask)
		&& (WeightmapTargetLayerBitIndices == InOther.WeightmapTargetLayerBitIndices);
}

FString FEditLayerTargetTypeState::ToString() const
{
	FString Result = FString::Printf(TEXT("Target types: %s"), *UE::Landscape::GetLandscapeToolTargetTypeFlagsAsString(TargetTypeMask));
	if (EnumHasAnyFlags(TargetTypeMask, ELandscapeToolTargetTypeFlags::Weightmap | ELandscapeToolTargetTypeFlags::Visibility))
	{
		Result += FString::Printf(TEXT("\nWeightmaps: %s"), *UE::Landscape::ConvertTargetLayerNamesToString(MergeContext->ConvertTargetLayerBitIndicesToNames(WeightmapTargetLayerBitIndices)));
	}
	return Result;
}

#endif // WITH_EDITOR

} // namespace UE::Landscape::EditLayers