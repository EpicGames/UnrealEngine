// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
LandscapeEditLayers.cpp: Landscape editing layers mode
=============================================================================*/

#include "LandscapeEdit.h"
#include "Landscape.h"
#include "LandscapeEditLayer.h"
#include "LandscapeProxy.h"
#include "LandscapeStreamingProxy.h"
#include "LandscapeInfo.h"
#include "LandscapeComponent.h"
#include "LandscapeLayerInfoObject.h"
#include "LandscapeDataAccess.h"
#include "LandscapePrivate.h"
#include "LandscapeEditReadback.h"
#include "LandscapeEditResourcesSubsystem.h"
#include "LandscapeEditLayerRenderer.h"
#include "LandscapeEditLayerMergeRenderContext.h"
#include "LandscapeEditLayerRendererPrivate.h"
#include "LandscapeEditTypes.h"
#include "LandscapeUtils.h"
#include "LandscapeUtilsPrivate.h"
#include "LandscapeSubsystem.h"
#include "LandscapeTextureStreamingManager.h"
#include "LandscapeEdgeFixup.h"
#include "LandscapeGroup.h"

#include "Application/SlateApplicationBase.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "ShaderCompilerCore.h"
#include "ShaderParameterUtils.h"
#include "EngineModule.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "Logging/MessageLog.h"
#include "RenderCaptureInterface.h"
#include "RenderGraph.h"
#include "RenderGraphUtils.h"
#include "PixelShaderUtils.h"
#include "SystemTextures.h"
#include "Rendering/Texture2DResource.h"
#include "SceneView.h"
#include "MaterialCachedData.h"
#include "ContentStreaming.h"
#include "Templates/TypeHash.h"
#include "RHIResourceUtils.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"

#if WITH_EDITOR
#include "AssetCompilingManager.h"
#include "LandscapeEditorModule.h"
#include "LandscapeToolInterface.h"
#include "ComponentRecreateRenderStateContext.h"
#include "LandscapeBlueprintBrushBase.h"
#include "Materials/MaterialInstanceConstant.h"
#include "LandscapeMaterialInstanceConstant.h"
#include "Materials/MaterialExpressionLandscapeVisibilityMask.h"
#include "ShaderCompiler.h"
#include "Algo/Accumulate.h"
#include "Algo/Count.h"
#include "Algo/AnyOf.h"
#include "Algo/AllOf.h"
#include "Algo/Unique.h"
#include "Algo/Transform.h"
#include "Algo/ForEach.h"
#include "Algo/RemoveIf.h"
#include "LandscapeSettings.h"
#include "LandscapeRender.h"
#include "LandscapeInfoMap.h"
#include "Misc/MessageDialog.h"
#include "GameFramework/WorldSettings.h"
#include "UObject/UObjectThreadContext.h"
#include "LandscapeSplinesComponent.h"
#include "Misc/FileHelper.h"
#include "Misc/MapErrors.h"
#include "Misc/UObjectToken.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopedSlowTask.h"
#include "TextureCompiler.h"
#include "Editor.h"
#include "LandscapeNotification.h"
#include "ObjectCacheContext.h"
#include "Components/RuntimeVirtualTextureComponent.h"
#include "UnrealEdGlobals.h" // GUnrealEd
#include "Editor/UnrealEdEngine.h"
#include "VisualLogger/VisualLogger.h"
#include "LandscapeTextureStorageProvider.h"
#include "UObject/Script.h"
#include "RHIGPUReadback.h"
#include "EngineUtils.h"
#include "ImageUtils.h"
#endif

#define LOCTEXT_NAMESPACE "Landscape"

// TODO [jonathan.bard] this define is for when(/if) we implement the uber landscape material in the editor, where weightmaps are *not* RGBA packed but stored in a plain texture array. 
//  This will allow several simplifications and optimizations to edit layers 
#define SUPPORTS_LANDSCAPE_EDITORONLY_UBER_MATERIAL 0

// Channel remapping
extern const size_t ChannelOffsets[4];

ENGINE_API extern bool GDisableAutomaticTextureMaterialUpdateDependencies;

// GPU profiling stats
DECLARE_GPU_STAT_NAMED(LandscapeLayers_Clear, TEXT("Landscape Layer Clear"));
DECLARE_GPU_STAT_NAMED(LandscapeLayers_Render, TEXT("Landscape Layer Render"));
DECLARE_GPU_STAT_NAMED(LandscapeLayers_CopyTexture, TEXT("Landscape Layer Copy Texture"));
DECLARE_GPU_STAT_NAMED(LandscapeLayers_CopyTexturePS, TEXT("Landscape Layer Copy Texture PS"));
DECLARE_GPU_STAT_NAMED(LandscapeLayers_ExtractLayers, TEXT("Landscape Extract Layers"));
DECLARE_GPU_STAT_NAMED(LandscapeLayers_PackLayers, TEXT("Landscape Pack Layers"));

#if WITH_EDITOR
UE_DEPRECATED(5.8, "The CVar has been fully deprecated.")
TAutoConsoleVariableDeprecated_NoReplace<int32> CVarForceLayersUpdate_Deprecated(
	/*Name = */TEXT("landscape.ForceLayersUpdate"),
	/*DefaultValue = */0, 
	/*DeprecatedAtVersion = */TEXT("5.8"), 
	/*UsageBehavior = */EShadowCVarBehavior::Warn, 
	/*DeprecationMessage = */TEXT("landscape.ForceLayersUpdate is deprecated. Use landscape.ForceLayersFullUpdate to trigger a single global edit layers update."));

int32 RenderCaptureLayersNextHeightmapDraws = 0;
static FAutoConsoleVariableRef CVarRenderCaptureLayersNextHeightmapDraws(
	TEXT("landscape.RenderCaptureLayersNextHeightmapDraws"),
	RenderCaptureLayersNextHeightmapDraws,
	TEXT("Trigger N render captures during the next heightmap draw calls."));

int32 RenderCaptureLayersNextWeightmapDraws = 0;
static FAutoConsoleVariableRef CVarRenderCaptureLayersNextWeightmapDraws(
	TEXT("landscape.RenderCaptureLayersNextWeightmapDraws"),
	RenderCaptureLayersNextWeightmapDraws,
	TEXT("Trigger N render captures during the next weightmap draw calls."));

UE_DEPRECATED(5.7, "The CVar has been fully deprecated.")
TAutoConsoleVariableDeprecated_NoReplace<int32> CVarOutputLayersRTContent_Deprecated(
	/*Name = */TEXT("landscape.OutputLayersRTContent"),
	/*DefaultValue = */0,
	/*DeprecatedAtVersion = */TEXT("5.7"),
	/*UsageBehavior = */EShadowCVarBehavior::Warn,
	/*DeprecationMessage = */TEXT("landscape.OutputLayersRTContent is deprecated. This was only used in Global merge. Batched merge (landscape.EditLayersLocalMerge.Enable 2) is the only merge technique available now."));

UE_DEPRECATED(5.7, "The CVar has been fully deprecated.")
TAutoConsoleVariableDeprecated_NoReplace<int32> CVarOutputLayersWeightmapsRTContent_Deprecated(
	/*Name = */TEXT("landscape.OutputLayersWeightmapsRTContent"),
	/*DefaultValue = */0,
	/*DeprecatedAtVersion = */TEXT("5.7"),
	/*UsageBehavior = */EShadowCVarBehavior::Warn,
	/*DeprecationMessage = */TEXT("landscape.OutputLayersWeightmapsRTContent is deprecated. This was only used in Global merge. Batched merge (landscape.EditLayersLocalMerge.Enable 2) is the only merge technique available now."));

static TAutoConsoleVariable<int32> CVarLandscapeSimulatePhysics(
	TEXT("landscape.SimulatePhysics"),
	0,
	TEXT("This will enable physic simulation on worlds containing landscape."));

static TAutoConsoleVariable<int32> CVarLandscapeLayerOptim(
	TEXT("landscape.Optim"),
	1,
	TEXT("This will enable landscape layers optim."));

static TAutoConsoleVariable<int32> CVarLandscapeLayerBrushOptim(
	TEXT("landscape.BrushOptim"),
	0,
	TEXT("This will enable landscape layers optim."));

static TAutoConsoleVariable<int32> CVarLandscapeDumpHeightmapDiff(
	TEXT("landscape.DumpHeightmapDiff"),
	0,
	TEXT("This will save images for readback heightmap textures that have changed in the last edit layer blend phase. (= 0 No Diff, 1 = Mip 0 Diff, 2 = All Mips Diff"));

static TAutoConsoleVariable<int32> CVarLandscapeDumpWeightmapDiff(
	TEXT("landscape.DumpWeightmapDiff"),
	0,
	TEXT("This will save images for readback weightmap textures that have changed in the last edit layer blend phase. (= 0 No Diff, 1 = Mip 0 Diff, 2 = All Mips Diff"));

static TAutoConsoleVariable<bool> CVarLandscapeDumpDiffDetails(
	TEXT("landscape.DumpDiffDetails"),
	false,
	TEXT("When dumping diffs for heightmap (landscape.DumpHeightmapDiff) or weightmap (landscape.DumpWeightmapDiff), dumps additional details about the pixels being different"));

TAutoConsoleVariable<int32> CVarLandscapeDirtyHeightmapHeightThreshold(
	TEXT("landscape.DirtyHeightmapHeightThreshold"),
	0,
	TEXT("Threshold to avoid imprecision issues on certain GPUs when detecting when a heightmap height changes, i.e. only a height difference > than this threshold (N over 16-bits uint height) will be detected as a change."));

TAutoConsoleVariable<int32> CVarLandscapeDirtyHeightmapNormalThreshold(
	TEXT("landscape.DirtyHeightmapNormalThreshold"),
	0,
	TEXT("Threshold to avoid imprecision issues on certain GPUs when detecting when a heightmap normal changes, i.e. only a normal channel difference > than this threshold (N over each 8-bits uint B & A channels independently) will be detected as a change."));

TAutoConsoleVariable<int32> CVarLandscapeDirtyWeightmapThreshold(
	TEXT("landscape.DirtyWeightmapThreshold"),
	0,
	TEXT("Threshold to avoid imprecision issues on certain GPUs when detecting when a weightmap changes, i.e. only a difference > than this threshold (N over each 8-bits uint weightmap channel)."));

TAutoConsoleVariable<int32> CVarLandscapeShowDirty(
	TEXT("landscape.ShowDirty"),
	0,
	TEXT("This will highlight the data that has changed during the layer blend phase."));

TAutoConsoleVariable<int32> CVarLandscapeTrackDirty(
	TEXT("landscape.TrackDirty"),
	0,
	TEXT("This will track the accumulation of data changes during the layer blend phase."));

TAutoConsoleVariable<int32> CVarLandscapeForceFlush(
	TEXT("landscape.ForceFlush"),
	0,
	TEXT("This will force a render flush every frame when landscape editing."));

TAutoConsoleVariable<int32> CVarLandscapeValidateProxyWeightmapUsages(
	TEXT("landscape.ValidateProxyWeightmapUsages"),
	1,
	TEXT("This will validate that weightmap usages in landscape proxies and their components don't get desynchronized with the landscape component layer allocations."));

TAutoConsoleVariable<int32> CVarLandscapeRemoveEmptyPaintLayersOnEdit(
	TEXT("landscape.RemoveEmptyPaintLayersOnEdit"),
	// TODO [jonathan.bard] : this has been disabled for now, since it can lead to a permanent dirty-on-load state for landscape, where the edit layers will do a new weightmap allocation for the missing layer
	//  (e.g. if a BP brush writes to it), only to remove it after readback, which will lead to the actor to be marked dirty. We need to separate the final from the source weightmap data to avoid this issue : 
	0, 
	TEXT("This will analyze weightmaps on readback and remove unneeded allocations (for unpainted layers)."));

TAutoConsoleVariable<float> CVarLandscapeBatchedMergeVisualLogOffsetIncrement(
	TEXT("landscape.BatchedMerge.VisualLog.OffsetIncrement"),
	5000.0f,
	TEXT("Offset (in unreal units) for visualizing each operation of the batched merge in the viewport via the visual logger."));

TAutoConsoleVariable<float> CVarLandscapeBatchedMergeVisualLogAlpha(
	TEXT("landscape.BatchedMerge.VisualLog.Alpha"),
	0.5f,
	TEXT("Alpha value to use when visualizing batched merge info in the viewport via the visual logger ([0.0, 1.0] range)"));

TAutoConsoleVariable<int32> CVarLandscapeBatchedMergeVisualLogShowMergeType(
	TEXT("landscape.BatchedMerge.VisualLog.ShowMergeType"),
	3,
	TEXT("Filter what to visualize in the visual logger when using batched merge (0 = no visual log, 1 = show heightmaps only, 2 = show weightmaps only, 3 = show all"));

TAutoConsoleVariable<int32> CVarLandscapeBatchedMergeVisualLogShowMergeProcess(
	TEXT("landscape.BatchedMerge.VisualLog.ShowMergeProcess"),
	0,
	TEXT("Allows to visualize the merge process in the visual logger (0 = no visual log, 1 = show batches only, 2 = show batches and affected components per edit layer renderer)"));

TAutoConsoleVariable<int32> CVarLandscapeBatchedMergeVisualLogShowRenderItemsType(
	TEXT("landscape.BatchedMerge.VisualLog.ShowRenderItemsType"),
	0,
	TEXT("Allows to visualize the edit layer renderers' render items in the visual logger when using batched merge (0 = no visual log, 1 = show input areas, 2 = show output areas, 3 = show all)"));

TAutoConsoleVariable<FString> CVarLandscapeBatchedMergeVisualLogShowRenderItemsEditLayerRendererFilter(
	TEXT("landscape.BatchedMerge.VisualLog.ShowRenderItemsEditLayerRendererFilter"),
	"",
	TEXT("Allows to filter the elements added to the visual log to only those pertaining to a given edit layer renderer : use in conjunction with landscape.BatchedMerge.VisualLog.ShowRenderItemsType (empty : display all elements, otherwise, only display the items related to the edit layer renderer if its name matches (partial match)"));

TAutoConsoleVariable<bool> CVarLandscapeBatchedMergeVisualLogShowAllRenderItems(
	TEXT("landscape.BatchedMerge.VisualLog.ShowAllRenderItems"),
	false,
	TEXT("Allows to visualize all render items : use in conjunction with landscape.BatchedMerge.VisualLog.ShowRenderItemsType (if true, all render items will be displayed. If false, only those that participate to the render will be"));

TAutoConsoleVariable<int32> CVarLandscapeBatchedMergeVisualLogShowComponentDependencies(
	TEXT("landscape.BatchedMerge.VisualLog.ShowComponentDependencies"),
	0,
	TEXT("Allows to visualize the dependencies between landscape components when using batched merge (0 = no visual log, 1 = show component coordinates + area affecting component, 2 = show component coordinates + renderer name + area affecting component)"));

TAutoConsoleVariable<FString> CVarLandscapeBatchedMergeVisualLogShowComponentDependenciesFilter(
	TEXT("landscape.BatchedMerge.VisualLog.ShowComponentDependenciesFilter"),
	"",
	TEXT("Allows to visualize all the dependency graph for the component specified : use the \"X= Y=\" format to specify the component for which to show the dependencies"));

TAutoConsoleVariable<bool> CVarLandscapeBatchedMergeEnableRenderLayerGroup(
	TEXT("landscape.BatchedMerge.EnableRenderLayerGroup"),
	true,
	TEXT("Allows to batch several non-overlapping successive edit layer renderers (if they support render layer grouping), such that blending is only performed once at the end of the group instead of after each renderer"));

TAutoConsoleVariable<bool> CVarSilenceMergeBatchResolutionWarning(
	TEXT("landscape.BatchedMerge.SilenceResolutionWarning"),
	false,
	TEXT("When true, don't warn about about exceeding batch merge resolution from landscape.BatchedMerge.MaxResolutionPerRenderBatch"));

UE_DEPRECATED(5.7, "The CVar has been fully deprecated.")
TAutoConsoleVariableDeprecated_NoReplace CVarLandscapeEditLayersLocalMerge_Deprecated(
	/*Name = */TEXT("landscape.EditLayersLocalMerge.Enable"), 
	/*DefaultValue = */0,
	/*DeprecatedAtVersion = */TEXT("5.7"), 
	/*UsageBehavior = */EShadowCVarBehavior::Warn, 
	/*DeprecationMessage = */TEXT("landscape.EditLayersLocalMerge.Enable is deprecated. Batched merge (landscape.EditLayersLocalMerge.Enable 2) is the only merge technique available now."));

UE_DEPRECATED(5.7, "The CVar has been fully deprecated.")
TAutoConsoleVariableDeprecated_NoReplace CVarLandscapeEditLayersMaxComponentsPerHeightmapResolveBatch_Deprecated(
	/*Name = */TEXT("landscape.EditLayersLocalMerge.MaxComponentsPerHeightmapResolveBatch"),
	/*DefaultValue = */0,
	/*DeprecatedAtVersion = */TEXT("5.7"),
	/*UsageBehavior = */EShadowCVarBehavior::Warn,
	/*DeprecationMessage = */TEXT("landscape.EditLayersLocalMerge.MaxComponentsPerHeightmapResolveBatch is deprecated. This was only used in Local merge. Batched merge (landscape.EditLayersLocalMerge.Enable 2) is the only merge technique available now."));

UE_DEPRECATED(5.7, "The CVar has been fully deprecated.")
TAutoConsoleVariableDeprecated_NoReplace CVarLandscapeEditLayersMaxComponentsPerWeightmapResolveBatch_Deprecated(
	/*Name = */TEXT("landscape.EditLayersLocalMerge.MaxComponentsPerWeightmapResolveBatch"),
	/*DefaultValue = */0,
	/*DeprecatedAtVersion = */TEXT("5.7"),
	/*UsageBehavior = */EShadowCVarBehavior::Warn,
	/*DeprecationMessage = */TEXT("landscape.EditLayersLocalMerge.MaxComponentsPerWeightmapResolveBatch is deprecated. This was only used in Local merge. Batched merge (landscape.EditLayersLocalMerge.Enable 2) is the only merge technique available now."));

TAutoConsoleVariable<int32> CVarLandscapeBatchedMergeMaxResolutionPerRenderBatch(
	TEXT("landscape.BatchedMerge.MaxResolutionPerRenderBatch"),
	1024,
	TEXT("Maximum supported resolution for merging edit layers in a single batch. The higher the number, the more components can be resolved in a single batch (and the higher the GPU memory consumption since merging requires several temporary textures)"),
	ECVF_RenderThreadSafe);
UE_DEPRECATED(5.7, "The CVar has been renamed CVarLandscapeBatchedMergeMaxResolutionPerRenderBatch.")
FAutoConsoleVariableDeprecated CVarLandscapeEditLayersMaxResolutionPerRenderBatch_Deprecated(
	/*Name = */TEXT("landscape.EditLayersLocalMerge.MaxResolutionPerRenderBatch"),
	/*ShadowName = */TEXT("landscape.BatchedMerge.MaxResolutionPerRenderBatch"),
	/*DeprecatedAtVersion = */TEXT("5.7"),
	/*UsageBehavior = */EShadowCVarBehavior::Warn,
	/*LookupFailureBehavior = */EShadowCVarBehavior::Warn,
	TEXT("The CVar has been renamed landscape.BatchedMerge.MaxResolutionPerRenderBatch."));

TAutoConsoleVariable<int32> CVarLandscapeBatchedMergeClearBeforeEachWriteToScratch(
	TEXT("landscape.BatchedMerge.ClearBeforeEachWriteToScratch"),
	0,
	TEXT("Debug to help with RenderDoc debugging : clear each time we're about to write on a scratch render target (since those are reused and can be used to write RTs of different resolutions"),
	ECVF_RenderThreadSafe);
UE_DEPRECATED(5.7, "The CVar has been renamed CVarLandscapeBatchedMergeClearBeforeEachWriteToScratch.")
FAutoConsoleVariableDeprecated CVarLandscapeEditLayersClearBeforeEachWriteToScratch_Deprecated(
	/*Name = */TEXT("landscape.EditLayersLocalMerge.ClearBeforeEachWriteToScratch"),
	/*ShadowName = */TEXT("landscape.BatchedMerge.ClearBeforeEachWriteToScratch"),
	/*DeprecatedAtVersion = */TEXT("5.7"),
	/*UsageBehavior = */EShadowCVarBehavior::Warn,
	/*LookupFailureBehavior = */EShadowCVarBehavior::Warn,
	TEXT("The CVar has been renamed landscape.BatchedMerge.ClearBeforeEachWriteToScratch."));

static void LandscapeDumpSelectiveLayerRender(const TArray<FString>& Args);
FAutoConsoleCommand CmdLandscapeDumpSelectiveLayerRender(
	TEXT("landscape.DumpSelectiveLayerRender"),
	TEXT("Perform a selective layer render of a landscape and write the output to an image file."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&LandscapeDumpSelectiveLayerRender));

static void LandscapeForceLayersFullUpdate(const TArray<FString>& Args);
FAutoConsoleCommand CmdLandscapeForceLayersFullUpdate(
	TEXT("landscape.ForceLayersFullUpdate"),
	TEXT("Trigger an edit layers update for all (no argument) or a specific set of landscapes (partial name match based on the strings passed in argument).\n"),
	FConsoleCommandWithArgsDelegate::CreateStatic(&LandscapeForceLayersFullUpdate));


ENUM_CLASS_FLAGS(ALandscape::EPrepareTextureResourcesLimitedMode);

struct FLandscapeDirty
{
	FLandscapeDirty()
		: ClearDiffConsoleCommand(
			TEXT("Landscape.ClearDirty"),
			TEXT("Clears all Landscape Dirty Debug Data"),
			FConsoleCommandDelegate::CreateRaw(this, &FLandscapeDirty::ClearDirty))
	{
	}

private:
	FAutoConsoleCommand ClearDiffConsoleCommand;

	void ClearDirty()
	{
		bool bCleared = false;
		for (TObjectIterator<UWorld> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
		{
			UWorld* CurrentWorld = *It;
			if (!CurrentWorld->IsGameWorld())
			{
				ULandscapeInfoMap& LandscapeInfoMap = ULandscapeInfoMap::GetLandscapeInfoMap(CurrentWorld);
				for (auto& Pair : LandscapeInfoMap.Map)
				{
					if (Pair.Value && Pair.Value->SupportsLandscapeEditing())
					{
						Pair.Value->ClearDirtyData();
						bCleared = true;
					}
				}
			}
		}

		UE_LOGF(LogLandscape, Verbose, "Landscape.Dirty: %ls", bCleared ? TEXT("Cleared") : TEXT("Landscape.Dirty: Nothing to clear"));
	}
};

FLandscapeDirty GLandscapeDebugDirty;

/**
 * Mapping between heightmaps/weightmaps and components.
 * It's not safe to persist this across frames, so we recalculate at the start of each update.
 */
struct FTextureToComponentHelper
{
	// Partial refresh flags : allows to recompute only a subset of the helper information :
	enum class ERefreshFlags
	{
		None = 0,
		RefreshComponents = (1 << 0),
		RefreshHeightmaps = (1 << 1),
		RefreshWeightmaps = (1 << 2),
		RefreshAll = ~0,
	};
	FRIEND_ENUM_CLASS_FLAGS(ERefreshFlags);

	FTextureToComponentHelper(const ULandscapeInfo& InLandscapeInfo)
		: LandscapeInfo(&InLandscapeInfo)
	{
		Refresh(ERefreshFlags::RefreshAll);
	}

	FTextureToComponentHelper(const ULandscapeInfo& InLandscapeInfo, TArrayView<ULandscapeComponent*> Components, ERefreshFlags Flags)
		: LandscapeInfo(&InLandscapeInfo),
		LandscapeComponents(Components)
	{
		check(!EnumHasAnyFlags(Flags, ERefreshFlags::RefreshComponents));  // Expect Height and/or Weight
		Refresh(Flags);
	}

	void Refresh(ERefreshFlags InRefreshFlags)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TextureToComponentHelper_Refresh);
		// Compute the list of components in this landscape : 
		if (EnumHasAnyFlags(InRefreshFlags, ERefreshFlags::RefreshComponents))
		{
			// When components are refreshed, all other info has to be :
			check(EnumHasAllFlags(InRefreshFlags, ERefreshFlags::RefreshAll));

			LandscapeComponents.Reset();
			LandscapeInfo->ForAllLandscapeComponents([&](ULandscapeComponent* Component)
			{
				LandscapeComponents.Add(Component);
			});
		}

		if (EnumHasAnyFlags(InRefreshFlags, ERefreshFlags::RefreshHeightmaps | ERefreshFlags::RefreshWeightmaps))
		{
			// Cleanup our heightmap/weightmap info:
			if (EnumHasAnyFlags(InRefreshFlags, ERefreshFlags::RefreshHeightmaps))
			{
				Heightmaps.Reset();
				HeightmapToComponents.Reset();
			}

			if (EnumHasAnyFlags(InRefreshFlags, ERefreshFlags::RefreshWeightmaps))
			{
				Weightmaps.Reset();
				WeightmapToComponents.Reset();
				WeightmapToChannelMask.Reset();
			}

			// Iterate on all tracked landscape components and keep track of components/heightmaps/weightmaps relationship :
			for (ULandscapeComponent* Component : LandscapeComponents)
			{
				if (EnumHasAnyFlags(InRefreshFlags, ERefreshFlags::RefreshHeightmaps))
				{
					UTexture2D* Heightmap = Component->GetHeightmap();
					check(Heightmap != nullptr);

					Heightmaps.Add(Heightmap);
					HeightmapToComponents.FindOrAdd(Heightmap).Add(Component);
				}

				if (EnumHasAnyFlags(InRefreshFlags, ERefreshFlags::RefreshWeightmaps))
				{
					const TArray<UTexture2D*>& WeightmapTextures = Component->GetWeightmapTextures();
					const TArray<FWeightmapLayerAllocationInfo>& AllocInfos = Component->GetWeightmapLayerAllocations();

					for (FWeightmapLayerAllocationInfo const& AllocInfo : AllocInfos)
					{
						if (AllocInfo.IsAllocated() && AllocInfo.WeightmapTextureIndex < WeightmapTextures.Num())
						{
							UTexture2D* Weightmap = WeightmapTextures[AllocInfo.WeightmapTextureIndex];
							check(Weightmap != nullptr);

							Weightmaps.Add(Weightmap);

							WeightmapToComponents.FindOrAdd(Weightmap).AddUnique(Component);
							uint8& WeightmapChannelMask = WeightmapToChannelMask.FindOrAdd(Weightmap, 0);
							WeightmapChannelMask |= (1 << AllocInfo.WeightmapTextureChannel);
						}
					}
				}
			}
		}
	}

	const ULandscapeInfo* LandscapeInfo = nullptr;
	TArray< ULandscapeComponent* > LandscapeComponents;
	TSet< UTexture2D* > Heightmaps;
	TMap< UTexture2D*, TArray<ULandscapeComponent*> > HeightmapToComponents;
	TSet< UTexture2D* > Weightmaps;
	// Key = texture, Value = list of components that use this texture in their weightmap layer allocations 
	TMap< UTexture2D*, TArray<ULandscapeComponent*> > WeightmapToComponents;
	// Key = texture, Value = final channel mask for this texture (i.e. which channel is actually allocated in any component)
	TMap<UTexture2D*, uint8> WeightmapToChannelMask;
};
ENUM_CLASS_FLAGS(FTextureToComponentHelper::ERefreshFlags);

// Must match EEditLayerHeightmapBlendMode in LandscapeLayersHeightmapsPS.usf
enum class ELandscapeEditLayerHeightmapBlendMode : uint32
{
	Additive = 0,
	AlphaBlend,

	Num,
};

// Must match EEditLayerWeightmapBlendMode in LandscapeLayersWeightmapsPS.usf
enum class ELandscapeEditLayerWeightmapBlendMode : uint32
{
	Additive = 0,
	Subtractive,

	Num,
};

// Must match EWeightmapPaintLayerFlags in LandscapeLayersWeightmapsPS.usf
enum class ELandscapeEditLayerWeightmapPaintLayerFlags : uint32
{
	IsVisibilityLayer = (1 << 0), // This paint layer is the visibility layer
	IsWeightBlended = (1 << 1), // Blend the paint layer's value with all the other paint layers weights

	None = 0
};
ENUM_CLASS_FLAGS(ELandscapeEditLayerWeightmapPaintLayerFlags);

// Must match FEditLayerWeightmapPaintLayerInfo in LandscapeLayersWeightmapsPS.usf
struct FLandscapeEditLayerWeightmapPaintLayerInfo
{
	ELandscapeEditLayerWeightmapPaintLayerFlags Flags = ELandscapeEditLayerWeightmapPaintLayerFlags::None; // Additional info about this paint layer
};

#endif // WITH_EDITOR

namespace UE::Landscape::Private
{
#if WITH_EDITOR
	static FFileHelper::EColorChannel GetWeightmapColorChannel(const FWeightmapLayerAllocationInfo& AllocInfo)
	{
		FFileHelper::EColorChannel ColorChannelMapping[] = { FFileHelper::EColorChannel::R, FFileHelper::EColorChannel::G, FFileHelper::EColorChannel::B, FFileHelper::EColorChannel::A };
		FFileHelper::EColorChannel ColorChannel = FFileHelper::EColorChannel::All;
	
		if (ensure(AllocInfo.WeightmapTextureChannel < 4))
		{
			ColorChannel = ColorChannelMapping[AllocInfo.WeightmapTextureChannel];
		}

		return ColorChannel;
	}

	static ELandscapeEditLayerHeightmapBlendMode LandscapeBlendModeToEditLayerBlendMode(ELandscapeBlendMode InLandscapeBlendMode)
	{
		switch (InLandscapeBlendMode)
		{
		case LSBM_AdditiveBlend:
			return ELandscapeEditLayerHeightmapBlendMode::Additive;
		case LSBM_AlphaBlend:
			return ELandscapeEditLayerHeightmapBlendMode::AlphaBlend;
		default:
			check(false);
		}

		return ELandscapeEditLayerHeightmapBlendMode::Num;
	}

	static UE::Landscape::EditLayers::EHeightmapBlendMode LandscapeBlendModeToHeightmapBlendMode(ELandscapeBlendMode InLandscapeBlendMode)
	{
		using namespace UE::Landscape::EditLayers;
		switch (InLandscapeBlendMode)
		{
		case LSBM_AdditiveBlend:
			return EHeightmapBlendMode::Additive;
		case LSBM_AlphaBlend:
			return EHeightmapBlendMode::LegacyAlphaBlend; // LSBM_AlphaBlend corresponds to the landscape spline case, i.e. "legacy alpha blend"
		default:
			check(false);
		}

		return EHeightmapBlendMode::Num;
	}
#endif // WITH_EDITOR
	

	// ----------------------------------------------------------------------------------
	// Texture channel swizzling : 
	enum class ERGBAChannel : uint8 { R, G, B, A };

	static constexpr uint8 BuildChannelSwizzleMask(ERGBAChannel InChannelR = ERGBAChannel::R, ERGBAChannel InChannelG = ERGBAChannel::G, ERGBAChannel InChannelB = ERGBAChannel::B, ERGBAChannel InChannelA = ERGBAChannel::A)
	{
		return (static_cast<uint8>(InChannelR) << 0)
			| (static_cast<uint8>(InChannelG) << 2)
			| (static_cast<uint8>(InChannelB) << 4)
			| (static_cast<uint8>(InChannelA) << 6);
	}

	static ERGBAChannel ExtractDestinationChannelFromSwizzleMask(ERGBAChannel InSourceChannel, uint8 InSwizzleMask)
	{
		uint8 SourceChannelIndex = static_cast<uint8>(InSourceChannel);
		return static_cast<ERGBAChannel>((((uint8)3 << (SourceChannelIndex * 2)) & InSwizzleMask) >> (SourceChannelIndex * 2));
	}

	FString GetChannelSwizzleMaskDescription(uint8 InSwizzleMask, int32 InNumChannels = 4)
	{
		static constexpr auto ChannelToChar = [](ERGBAChannel InChannel) { return (InChannel == ERGBAChannel::R) ? TCHAR('R') : (InChannel == ERGBAChannel::G) ? TCHAR('G') : (InChannel == ERGBAChannel::B) ? TCHAR('B') : TCHAR('A'); };
		check(InNumChannels <= 4);
		const TCHAR ChannelsChar[4] =
		{
			ChannelToChar(ExtractDestinationChannelFromSwizzleMask(ERGBAChannel::R, InSwizzleMask)),
			ChannelToChar(ExtractDestinationChannelFromSwizzleMask(ERGBAChannel::G, InSwizzleMask)),
			ChannelToChar(ExtractDestinationChannelFromSwizzleMask(ERGBAChannel::B, InSwizzleMask)),
			ChannelToChar(ExtractDestinationChannelFromSwizzleMask(ERGBAChannel::A, InSwizzleMask))
		};
		FString Result;
		Result.AppendChars(ChannelsChar, InNumChannels);
		return Result;
	}

	static constexpr uint8 RGBAToRGBASwizzleMask = BuildChannelSwizzleMask(ERGBAChannel::R, ERGBAChannel::G, ERGBAChannel::B, ERGBAChannel::A);

	static bool InBPCallstack()
	{
#if DO_BLUEPRINT_GUARD
		const FBlueprintContextTracker* Tracker = FBlueprintContextTracker::TryGet();
		return Tracker && Tracker->GetScriptEntryTag() > 0;
#else
		return false;
#endif
	}
}

class FLandscapeCopyTextureVS : public FGlobalShader
{
public:
	DECLARE_GLOBAL_SHADER(FLandscapeCopyTextureVS);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& InParameters)
	{
		return UE::Landscape::DoesPlatformSupportEditLayers(InParameters.Platform);
	}

	FLandscapeCopyTextureVS()
	{};

	FLandscapeCopyTextureVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{}
};

class FLandscapeCopyTexturePS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FLandscapeCopyTexturePS);
public:

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return UE::Landscape::DoesPlatformSupportEditLayers(Parameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		UE::Landscape::ModifyShaderCompilerEnvironmentForDebug(OutEnvironment);
	}

	FLandscapeCopyTexturePS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		ReadTexture1Param.Bind(Initializer.ParameterMap, TEXT("ReadTexture1"));
		ReadTexture1SamplerParam.Bind(Initializer.ParameterMap, TEXT("ReadTexture1Sampler"));
		SourceOffsetAndSizeUVParam.Bind(Initializer.ParameterMap, TEXT("SourceOffsetAndSizeUV"));
		ChannelSwizzleMaskParam.Bind(Initializer.ParameterMap, TEXT("ChannelSwizzleMask"));
	}

	FLandscapeCopyTexturePS()
	{}

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, FRHITexture* InSourceTextureRHI,
		const FIntPoint& InSourcePosition = FIntPoint::ZeroValue, const FIntPoint& InCopySizePixels = FIntPoint::ZeroValue, uint8 InChannelSwizzleMask = UE::Landscape::Private::RGBAToRGBASwizzleMask)
	{
		FVector2f SourceSize(InSourceTextureRHI->GetSizeXY());
		FVector2f SourceOffsetUV = FVector2f(InSourcePosition) / SourceSize;
		FVector2f FinalCopySizePixels;
		FinalCopySizePixels.X = (InCopySizePixels.X > 0) ? InCopySizePixels.X : SourceSize.X;
		FinalCopySizePixels.Y = (InCopySizePixels.Y > 0) ? InCopySizePixels.Y : SourceSize.Y;
		FVector2f CopySizeUV = FinalCopySizePixels / SourceSize;
		SetTextureParameter(BatchedParameters, ReadTexture1Param, ReadTexture1SamplerParam, TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI(), InSourceTextureRHI);
		SetShaderValue(BatchedParameters, SourceOffsetAndSizeUVParam, FVector4f(SourceOffsetUV.X, SourceOffsetUV.Y, CopySizeUV.X, CopySizeUV.Y));
		SetShaderValue(BatchedParameters, ChannelSwizzleMaskParam, static_cast<uint32>(InChannelSwizzleMask));
	}

private:
	LAYOUT_FIELD(FShaderResourceParameter, ReadTexture1Param);
	LAYOUT_FIELD(FShaderResourceParameter, ReadTexture1SamplerParam);
	LAYOUT_FIELD(FShaderParameter, SourceOffsetAndSizeUVParam);
	LAYOUT_FIELD(FShaderParameter, ChannelSwizzleMaskParam);
};

IMPLEMENT_GLOBAL_SHADER(FLandscapeCopyTextureVS, "/Engine/Private/LandscapeLayersPS.usf", "CopyTextureVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FLandscapeCopyTexturePS, "/Engine/Private/LandscapeLayersPS.usf", "CopyTexturePS", SF_Pixel);

// ----------------------------------------------------------------------------------
// /Engine/Private/Landscape/LandscapeEditLayersHeightmaps.usf shaders :

class FLandscapeEditLayersHeightmapsGenerateNormalsPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FLandscapeEditLayersHeightmapsGenerateNormalsPS);
	SHADER_USE_PARAMETER_STRUCT(FLandscapeEditLayersHeightmapsGenerateNormalsPS, FGlobalShader);

public:
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FUintVector4, InTextureSize)
		SHADER_PARAMETER(FVector3f, InLandscapeGridScale)
		SHADER_PARAMETER(uint32, InComponentSizeQuads)
		SHADER_PARAMETER(FUintVector2, InNumComponents)
		SHADER_PARAMETER_SAMPLER(SamplerState, InSourceHeightmapSampler)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, InSourceHeightmap)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float>, InComponentIdTexture)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& InParameters)
	{
		return UE::Landscape::DoesPlatformSupportEditLayers(InParameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("GENERATE_NORMALS"), 1);
		OutEnvironment.CompilerFlags.Remove(CFLAG_HLSL2021 | CFLAG_PrecompileWithDXC);
		UE::Landscape::ModifyShaderCompilerEnvironmentForDebug(OutEnvironment);
	}

	static void GenerateNormalsPS(FRDGEventName&& InRDGEventName, FRDGBuilder& GraphBuilder, FParameters* InParameters, const FIntPoint& InTextureSize)
	{
		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FLandscapeEditLayersHeightmapsGenerateNormalsPS> PixelShader(ShaderMap);

		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			ShaderMap,
			MoveTemp(InRDGEventName),
			PixelShader,
			InParameters,
			FIntRect(0, 0, InTextureSize.X, InTextureSize.Y));
	}
};

IMPLEMENT_GLOBAL_SHADER(FLandscapeEditLayersHeightmapsGenerateNormalsPS, "/Engine/Private/Landscape/LandscapeEditLayersHeightmaps.usf", "GenerateNormalsPS", SF_Pixel);

class FLandscapeEditLayersHeightmapsGenerateMipsPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FLandscapeEditLayersHeightmapsGenerateMipsPS);
	SHADER_USE_PARAMETER_STRUCT(FLandscapeEditLayersHeightmapsGenerateMipsPS, FGlobalShader);

public:
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FUintVector2, InCurrentMipSubsectionSize)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, InSourceHeightmap)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& InParameters)
	{
		return UE::Landscape::DoesPlatformSupportEditLayers(InParameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("GENERATE_MIPS"), 1);
		UE::Landscape::ModifyShaderCompilerEnvironmentForDebug(OutEnvironment);
	}

	static void GenerateMipsPS(FRDGBuilder& GraphBuilder, FParameters* InParameters, const FIntPoint& InTextureSize)
	{
		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FLandscapeEditLayersHeightmapsGenerateMipsPS> PixelShader(ShaderMap);

		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			ShaderMap,
			RDG_EVENT_NAME("LandscapeLayers_GenerateMipsPS"),
			PixelShader,
			InParameters,
			FIntRect(0, 0, InTextureSize.X, InTextureSize.Y));
	}
};

IMPLEMENT_GLOBAL_SHADER(FLandscapeEditLayersHeightmapsGenerateMipsPS, "/Engine/Private/Landscape/LandscapeEditLayersHeightmaps.usf", "GenerateMipsPS", SF_Pixel);


// ----------------------------------------------------------------------------------
// /Engine/Private/Landscape/LandscapeEditLayersWeightmaps.usf shaders :

class FLandscapeEditLayersWeightmapsPerformFinalWeightBlendingPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FLandscapeEditLayersWeightmapsPerformFinalWeightBlendingPS);
	SHADER_USE_PARAMETER_STRUCT(FLandscapeEditLayersWeightmapsPerformFinalWeightBlendingPS, FGlobalShader);

public:
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(uint32, InTargetLayerIndex)
		SHADER_PARAMETER(uint32, InNumTargetLayers)
		SHADER_PARAMETER_RDG_BUFFER_SRV(StructuredBuffer<UE::Landscape::FFinalWeightBlendingTargetLayerInfo>, InFinalWeightBlendingTargetLayerInfos)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2DArray<float4>, InCurrentEditLayerWeightmaps)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& InParameters)
	{
		return UE::Landscape::DoesPlatformSupportEditLayers(InParameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("PERFORM_FINAL_WEIGHT_BLENDING"), 1);
		UE::Landscape::ModifyShaderCompilerEnvironmentForDebug(OutEnvironment);
	}

	static void PerformFinalWeightBlendingPS(FRDGEventName&& InRDGEventName, FRDGBuilder& GraphBuilder, FParameters* InParameters, const FIntPoint& InTextureSize)
	{
		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FLandscapeEditLayersWeightmapsPerformFinalWeightBlendingPS> PixelShader(ShaderMap);

		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			ShaderMap,
			MoveTemp(InRDGEventName),
			PixelShader,
			InParameters,
			FIntRect(0, 0, InTextureSize.X, InTextureSize.Y), 
			TStaticBlendStateWriteMask<CW_RG>::GetRHI());
	}
};

IMPLEMENT_GLOBAL_SHADER(FLandscapeEditLayersWeightmapsPerformFinalWeightBlendingPS, "/Engine/Private/Landscape/LandscapeEditLayersWeightmaps.usf", "PerformFinalWeightBlendingPS", SF_Pixel);

class FLandscapeEditLayersWeightmapsPackWeightmapPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FLandscapeEditLayersWeightmapsPackWeightmapPS);
	SHADER_USE_PARAMETER_STRUCT(FLandscapeEditLayersWeightmapsPackWeightmapPS, FGlobalShader);

public:
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FIntVector4, InSourceSliceIndices)
		SHADER_PARAMETER_ARRAY(FUintVector4, InSourcePixelOffsets, [4])
		SHADER_PARAMETER(FUintVector2, InSubsectionPixelOffset)
		SHADER_PARAMETER(uint32, InIsAdditive)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2DArray<float4>, InSourceWeightmaps)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, InWeightmapBeingPacked)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& InParameters)
	{
		return UE::Landscape::DoesPlatformSupportEditLayers(InParameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("PACK_WEIGHTMAP"), 1);
		UE::Landscape::ModifyShaderCompilerEnvironmentForDebug(OutEnvironment);
	}

	static void PackWeightmapPS(FRDGBuilder& GraphBuilder, FParameters* InParameters, const FIntRect& InTextureRect)
	{
		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FLandscapeEditLayersWeightmapsPackWeightmapPS> PixelShader(ShaderMap);

		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			ShaderMap,
			RDG_EVENT_NAME("LandscapeLayers_PackWeightmapPS"),
			PixelShader,
			InParameters,
			InTextureRect);
	}
};

IMPLEMENT_GLOBAL_SHADER(FLandscapeEditLayersWeightmapsPackWeightmapPS, "/Engine/Private/Landscape/LandscapeEditLayersWeightmaps.usf", "PackWeightmapPS", SF_Pixel);

class FLandscapeEditLayersWeightmapsGenerateMipsPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FLandscapeEditLayersWeightmapsGenerateMipsPS);
	SHADER_USE_PARAMETER_STRUCT(FLandscapeEditLayersWeightmapsGenerateMipsPS, FGlobalShader);

public:
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FUintVector2, InCurrentMipSubsectionSize)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, InSourceWeightmap)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& InParameters)
	{
		return UE::Landscape::DoesPlatformSupportEditLayers(InParameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("GENERATE_MIPS"), 1);
		UE::Landscape::ModifyShaderCompilerEnvironmentForDebug(OutEnvironment);
	}

	static void GenerateMipsPS(FRDGBuilder& GraphBuilder, FParameters* InParameters, const FIntPoint& InTextureSize)
	{
		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef<FLandscapeEditLayersWeightmapsGenerateMipsPS> PixelShader(ShaderMap);

		FPixelShaderUtils::AddFullscreenPass(
			GraphBuilder,
			ShaderMap,
			RDG_EVENT_NAME("LandscapeLayers_GenerateMipsPS"),
			PixelShader,
			InParameters,
			FIntRect(0, 0, InTextureSize.X, InTextureSize.Y));
	}
};

IMPLEMENT_GLOBAL_SHADER(FLandscapeEditLayersWeightmapsGenerateMipsPS, "/Engine/Private/Landscape/LandscapeEditLayersWeightmaps.usf", "GenerateMipsPS", SF_Pixel);


// ----------------------------------------------------------------------------------
// /Engine/Private/Landscape/LandscapeEditLayersUtils.usf shaders : 
class FCopyQuadsMultiSourcePSDefault : public FGlobalShader
{
public:
	static constexpr int32 NumMultiSources = 62;

	DECLARE_GLOBAL_SHADER(FCopyQuadsMultiSourcePSDefault);
	SHADER_USE_PARAMETER_STRUCT(FCopyQuadsMultiSourcePSDefault, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint4>, InQuadInfos)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D, InComponentIdTexture)
		SHADER_PARAMETER_RDG_TEXTURE_ARRAY(Texture2D, InSourceTexture, [NumMultiSources])
	END_SHADER_PARAMETER_STRUCT()

	class FCopyWeightmap : SHADER_PERMUTATION_BOOL("COPY_WEIGHTMAP");
	using FPermutationDomain = TShaderPermutationDomain<FCopyWeightmap>;

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& InParameters)
{
		return UE::Landscape::DoesPlatformSupportEditLayers(InParameters.Platform);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.SetDefine(TEXT("COPY_QUADS_MULTISOURCE"), 1);
		UE::Landscape::ModifyShaderCompilerEnvironmentForDebug(OutEnvironment);
	}
};

// Helper class to trigger the appropriate PS regardless of the platform : 
class FCopyQuadsMultiSourceBase
{
public:
	virtual ~FCopyQuadsMultiSourceBase() = default;
	virtual int32 GetNumMultiSources() const PURE_VIRTUAL(FCopyQuadsMultiSourceBase::GetNumMultiSources, return 0; );
	virtual void CopyQuads(FRDGBuilder& GraphBuilder,
		FRDGTextureRef InOutputTexture,
		TConstArrayView<FRDGTextureRef> InSourceTextures,
		FRDGBufferSRVRef InRectBufferSRV,
		FRDGBufferSRVRef InQuadInfosBufferSRV,
		FRDGBufferSRVRef InRectUVBufferSRV,
		FRDGTextureSRVRef InComponentIdTextureSRV,
		const FSceneView* InView,
		int32 InArrayIndex,
		int32 InNumRects,
		const FIntPoint& InTextureSize,
		bool bIsWeightmapMerge) PURE_VIRTUAL(FCopyQuadsMultiSourceBase::CopyQuads, );
};

template <typename FCopyQuadsMultiSourcePSType, typename FCopyQuadsMultiSourcePSParametersType>
class TCopyQuadsMultiSource : public FCopyQuadsMultiSourceBase
{
public:
	static constexpr int32 NumMultiSources = FCopyQuadsMultiSourcePSType::NumMultiSources;

	virtual int32 GetNumMultiSources() const override
	{
		return NumMultiSources;
	}

	virtual void CopyQuads(FRDGBuilder& GraphBuilder, 
		FRDGTextureRef InOutputTexture, 
		TConstArrayView<FRDGTextureRef> InSourceTextures, 
		FRDGBufferSRVRef InRectBufferSRV, 
		FRDGBufferSRVRef InQuadInfosBufferSRV, 
		FRDGBufferSRVRef InRectUVBufferSRV,
		FRDGTextureSRVRef InComponentIdTextureSRV,
		const FSceneView* InView, 
		int32 InArrayIndex,
		int32 InNumRects, 
		const FIntPoint& InTextureSize, 
		bool bIsWeightmapMerge) override
	{
		typename FCopyQuadsMultiSourcePSType::FPermutationDomain PermutationVector;
		PermutationVector.template Set<typename FCopyQuadsMultiSourcePSType::FCopyWeightmap>(bIsWeightmapMerge);
		FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderRef<FCopyQuadsMultiSourcePSType> PixelShader = ShaderMap->GetShader<FCopyQuadsMultiSourcePSType>(PermutationVector);

		FCopyQuadsMultiSourcePSParametersType* PassParameters = GraphBuilder.AllocParameters<FCopyQuadsMultiSourcePSParametersType>();
		PassParameters->RenderTargets[0] = FRenderTargetBinding(InOutputTexture, ERenderTargetLoadAction::ELoad, /*InMipIndex = */0, InArrayIndex);
		PassParameters->PS.View = InView->ViewUniformBuffer;
		PassParameters->PS.InQuadInfos = InQuadInfosBufferSRV;
		PassParameters->PS.InComponentIdTexture = InComponentIdTextureSRV;
		check(InSourceTextures.Num() == NumMultiSources);
		for (int32 TextureIndex = 0; TextureIndex < NumMultiSources; ++TextureIndex)
		{
			PassParameters->PS.InSourceTexture[TextureIndex] = InSourceTextures[TextureIndex];
		}

		FPixelShaderUtils::AddRasterizeToRectsPass<FCopyQuadsMultiSourcePSType>(GraphBuilder,
			ShaderMap,
			RDG_EVENT_NAME("CopyQuadsMultiSourcePS"),
			PixelShader,
			PassParameters,
			/*ViewportSize = */InOutputTexture->Desc.Extent,
			InRectBufferSRV,
			InNumRects,
			/*BlendState = */nullptr,
			/*RasterizerState = */nullptr,
			/*DepthStencilState = */nullptr,
			/*StencilRef = */0,
			InTextureSize,
			InRectUVBufferSRV);
	}
};

BEGIN_SHADER_PARAMETER_STRUCT(FCopyQuadsMultiSourcePSParametersDefault, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FPixelShaderUtils::FRasterizeToRectsVS::FParameters, VS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FCopyQuadsMultiSourcePSDefault::FParameters, PS)
	RENDER_TARGET_BINDING_SLOTS() 
END_SHADER_PARAMETER_STRUCT() 
IMPLEMENT_GLOBAL_SHADER(FCopyQuadsMultiSourcePSDefault, "/Engine/Private/Landscape/LandscapeEditLayersUtils.usf", "CopyQuadsMultiSourcePS", SF_Pixel);

using FCopyQuadsMultiSourceDefault = TCopyQuadsMultiSource<FCopyQuadsMultiSourcePSDefault, FCopyQuadsMultiSourcePSParametersDefault>;

// ----------------------------------------------------------------------------------
// Copy texture render command

struct FLandscapeLayersCopyTextureParams
{
	FLandscapeLayersCopyTextureParams(UTexture* InSourceTexture, UTexture* InDestTexture)
	{
		if (InSourceTexture != nullptr)
		{
			SourceResourceDebugName = InSourceTexture->GetName();
			SourceResource = InSourceTexture->GetResource();
		}
		if (InDestTexture != nullptr)
		{
			DestResourceDebugName = InDestTexture->GetName();
			DestResource = InDestTexture->GetResource();
		}
	}

	FLandscapeLayersCopyTextureParams(const FString& InSourceResourceDebugName, FTextureResource* InSourceResource, const FString& InDestResourceDebugName, FTextureResource* InDestResource)
		: SourceResourceDebugName(InSourceResourceDebugName)
		, SourceResource(InSourceResource)
		, DestResourceDebugName(InDestResourceDebugName)
		, DestResource(InDestResource)
	{}

	FLandscapeLayersCopyTextureParams(const FLandscapeLayersCopyTextureParams&) = default;
	FLandscapeLayersCopyTextureParams(FLandscapeLayersCopyTextureParams&&) = default;
	FLandscapeLayersCopyTextureParams& operator=(FLandscapeLayersCopyTextureParams&&) = default;

	FString SourceResourceDebugName;
	FTextureResource* SourceResource = nullptr;
	FString DestResourceDebugName;
	FTextureResource* DestResource = nullptr;
	FIntPoint CopySize = FIntPoint(0, 0);
	FIntPoint SourcePosition = FIntPoint(0, 0);
	FIntPoint DestPosition = FIntPoint(0, 0);
	uint8 SourceMip = 0;
	uint8 DestMip = 0;
	uint32 SourceArrayIndex = 0;
	uint32 DestArrayIndex = 0;
	ERHIAccess SourceAccess = ERHIAccess::SRVMask;
	ERHIAccess DestAccess = ERHIAccess::SRVMask;
	// There's a shader-version of the copy that is able to swizzle RGBA channels : this mask allows to specify how : 
	uint8 ChannelSwizzleMask = UE::Landscape::Private::RGBAToRGBASwizzleMask;
};

class FLandscapeLayersCopyTexture_RenderThread
{
public:
	FLandscapeLayersCopyTexture_RenderThread(const FLandscapeLayersCopyTextureParams& InParams)
		: Params(InParams)
	{}

	const FLandscapeLayersCopyTextureParams& GetParams() const { return Params; }
	void Copy(FRHICommandListImmediate& InRHICmdList)
	{
		// We must use the PS version if swizzling channels or if the format is different (e.g. R8G8B8A8 to R8)
		if ((Params.SourceResource->TextureRHI->GetFormat() != Params.DestResource->TextureRHI->GetFormat())
			|| (Params.ChannelSwizzleMask != UE::Landscape::Private::RGBAToRGBASwizzleMask))
		{
			checkf(EnumHasAllFlags(Params.DestResource->TextureRHI->GetFlags(), ETextureCreateFlags::RenderTargetable), TEXT("Cannot request swizzling if the texture is not render-targetable"));
			checkf(!Params.DestResource->TextureRHI->GetDesc().IsTextureArray() || EnumHasAllFlags(Params.DestResource->TextureRHI->GetFlags(), ETextureCreateFlags::TargetArraySlicesIndependently), TEXT("Cannot request swizzling on a texture array if the slices are not individually render-targetable"));
			CopyInternalPS(InRHICmdList);
		}
		else
		{
			CopyInternal(InRHICmdList);
		}
	}

private:
	void CopyInternal(FRHICommandListImmediate& InRHICmdList)
	{
		// TODO [jonathan.bard] : make those perf tags optional : with the amount of textures we copy, it slows down texture copies quite a bit : 
		RHI_BREADCRUMB_EVENT_STAT_F(InRHICmdList, LandscapeLayers_CopyTexture
			, "LandscapeLayers_Copy"
			, "LandscapeLayers_Copy %s -> %s, Mip (%d -> %d), Array Index (%d -> %d)"
			, Params.SourceResourceDebugName
			, Params.DestResourceDebugName
			, Params.SourceMip
			, Params.DestMip
			, Params.SourceArrayIndex
			, Params.DestArrayIndex
		);

		FIntPoint SourceSize(Params.SourceResource->GetSizeX() >> Params.SourceMip, Params.SourceResource->GetSizeY() >> Params.SourceMip);
		FIntPoint DestSize(Params.DestResource->GetSizeX() >> Params.DestMip, Params.DestResource->GetSizeY() >> Params.DestMip);

		FRHICopyTextureInfo Info;
		Info.NumSlices = 1;
		// If CopySize is passed, used that as the size (and don't adjust with the mip level : consider that the user has computed it properly) : 
		Info.Size.X = (Params.CopySize.X > 0) ? Params.CopySize.X : SourceSize.X;
		Info.Size.Y = (Params.CopySize.Y > 0) ? Params.CopySize.Y : SourceSize.Y;
		Info.Size.Z = 1;
		Info.SourcePosition.X = Params.SourcePosition.X;
		Info.SourcePosition.Y = Params.SourcePosition.Y;
		Info.DestPosition.X = Params.DestPosition.X;
		Info.DestPosition.Y = Params.DestPosition.Y;
		Info.SourceSliceIndex = Params.SourceArrayIndex;
		Info.DestSliceIndex = Params.DestArrayIndex;
		Info.SourceMipIndex = Params.SourceMip;
		Info.DestMipIndex = Params.DestMip;

		check((Info.SourcePosition.X >= 0) && (Info.SourcePosition.Y >= 0) && (Info.DestPosition.X >= 0) && (Info.DestPosition.Y >= 0));
		check(Info.SourcePosition.X + Info.Size.X <= SourceSize.X);
		check(Info.SourcePosition.Y + Info.Size.Y <= SourceSize.Y);
		check(Info.DestPosition.X + Info.Size.X <= DestSize.X);
		check(Info.DestPosition.Y + Info.Size.Y <= DestSize.Y);

		InRHICmdList.Transition(FRHITransitionInfo(Params.SourceResource->TextureRHI, Params.SourceAccess, ERHIAccess::CopySrc));
		InRHICmdList.Transition(FRHITransitionInfo(Params.DestResource->TextureRHI, Params.DestAccess, ERHIAccess::CopyDest));
		InRHICmdList.CopyTexture(Params.SourceResource->TextureRHI, Params.DestResource->TextureRHI, Info);
		InRHICmdList.Transition(FRHITransitionInfo(Params.SourceResource->TextureRHI, ERHIAccess::CopySrc, Params.SourceAccess));
		InRHICmdList.Transition(FRHITransitionInfo(Params.DestResource->TextureRHI, ERHIAccess::CopyDest, Params.DestAccess));
	}

	void CopyInternalPS(FRHICommandListImmediate& InRHICmdList)
	{
		using namespace UE::Landscape::Private;

		const int32 NumChannelsDest = GPixelFormats[Params.DestResource->TextureRHI->GetDesc().Format].NumComponents;

		// TODO [jonathan.bard] : make those perf tags optional : with the amount of textures we copy, it slows down texture copies quite a bit : 
		RHI_BREADCRUMB_EVENT_STAT_F(InRHICmdList, LandscapeLayers_CopyTexturePS
			, "LandscapeLayers_CopyPS"
			, "LandscapeLayers_CopyPS %s -> %s, Mip (%d -> %d), Array Index (%d -> %d), [%s]"
			, Params.SourceResourceDebugName
			, Params.DestResourceDebugName
			, Params.SourceMip
			, Params.DestMip
			, Params.SourceArrayIndex
			, Params.DestArrayIndex
			, GetChannelSwizzleMaskDescription(Params.ChannelSwizzleMask, NumChannelsDest)
		);

		FIntPoint SourceSize(Params.SourceResource->GetSizeX() >> Params.SourceMip, Params.SourceResource->GetSizeY() >> Params.SourceMip);
		FIntPoint DestSize(Params.DestResource->GetSizeX() >> Params.DestMip, Params.DestResource->GetSizeY() >> Params.DestMip);

		// If CopySize is passed, used that as the size (and don't adjust with the mip level : consider that the user has computed it properly) : 
		FIntPoint Size;
		Size.X = (Params.CopySize.X > 0) ? Params.CopySize.X : SourceSize.X;
		Size.Y = (Params.CopySize.Y > 0) ? Params.CopySize.Y : SourceSize.Y;
		check((Params.SourceArrayIndex == 0) && (Params.SourceMip == 0) && (Params.DestMip == 0)); // The PS version of copy is not supported on texture arrays and mips for now

		InRHICmdList.Transition(FRHITransitionInfo(Params.SourceResource->TextureRHI, Params.SourceAccess, ERHIAccess::SRVGraphics));
		InRHICmdList.Transition(FRHITransitionInfo(Params.DestResource->TextureRHI, Params.DestAccess, ERHIAccess::RTV));

		int32 PassArraySlice = Params.DestResource->TextureRHI->GetDesc().IsTextureArray() ? Params.DestArrayIndex : -1; // Little hack to make sure we pass -1 to FRHIRenderPassInfo for a non-texture array resource as that's what it expects :
		FRHIRenderPassInfo RPInfo(Params.DestResource->TextureRHI, ERenderTargetActions::DontLoad_Store, /*ResolveRT = */nullptr, /*InMipIndex = */0, PassArraySlice);
		InRHICmdList.BeginRenderPass(RPInfo, TEXT("CopyTexture"));

		FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
		TShaderMapRef< FLandscapeCopyTextureVS > VertexShader(GlobalShaderMap);
		TShaderMapRef< FLandscapeCopyTexturePS > PixelShader(GlobalShaderMap);

		FGraphicsPipelineStateInitializer GraphicsPSOInit;
		InRHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);
		GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
		GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
		GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();
		GraphicsPSOInit.PrimitiveType = PT_TriangleList;
		GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GetVertexDeclarationFVector4();
		GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
		GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
		SetGraphicsPipelineState(InRHICmdList, GraphicsPSOInit, 0);

		SetShaderParametersLegacyPS(InRHICmdList, PixelShader, Params.SourceResource->TextureRHI, Params.SourcePosition, Size, Params.ChannelSwizzleMask);

		InRHICmdList.SetViewport((float)Params.DestPosition.X, (float)Params.DestPosition.Y, 0.0f, (float)(Params.DestPosition.X + Size.X), (float)(Params.DestPosition.Y + Size.Y), 1.0f);
		InRHICmdList.DrawIndexedPrimitive(GTwoTrianglesIndexBuffer.IndexBufferRHI, 0, 0, 4, 0, 2, 1);

		InRHICmdList.EndRenderPass();

		InRHICmdList.Transition(FRHITransitionInfo(Params.SourceResource->TextureRHI, ERHIAccess::SRVGraphics, Params.SourceAccess));
		InRHICmdList.Transition(FRHITransitionInfo(Params.DestResource->TextureRHI, ERHIAccess::RTV, Params.DestAccess));
	}

private:
	FLandscapeLayersCopyTextureParams Params;
};

// ----------------------------------------------------------------------------------
// Clear command

class LandscapeLayersWeightmapClear_RenderThread
{
public:
	LandscapeLayersWeightmapClear_RenderThread(const FString& InDebugName, FTextureRenderTargetResource* InTextureResourceToClear)
		: DebugName(InDebugName)
		, RenderTargetResource(InTextureResourceToClear)
	{}

	virtual ~LandscapeLayersWeightmapClear_RenderThread()
	{}

	void Clear(FRHICommandListImmediate& InRHICmdList)
	{
		RHI_BREADCRUMB_EVENT_STAT_F(InRHICmdList, LandscapeLayers_Clear, "LandscapeLayers_Clear", "LandscapeLayers_Clear %s", DebugName);

		TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeLayersWeightmapClear_RenderThread::Clear);

		check(IsInRenderingThread());

		InRHICmdList.Transition(FRHITransitionInfo(RenderTargetResource->TextureRHI, ERHIAccess::SRVMask, ERHIAccess::RTV));
		FRHIRenderPassInfo RPInfo(RenderTargetResource->TextureRHI, ERenderTargetActions::Clear_Store);
		InRHICmdList.BeginRenderPass(RPInfo, TEXT("Clear"));
		InRHICmdList.EndRenderPass();
		InRHICmdList.Transition(FRHITransitionInfo(RenderTargetResource->TextureRHI, ERHIAccess::RTV, ERHIAccess::SRVMask));
	}

	FString DebugName;
	FTextureRenderTargetResource* RenderTargetResource;
};

#if WITH_EDITOR

bool ALandscape::IsMaterialResourceCompiled(FMaterialResource* InMaterialResource, bool bInWaitForCompilation)
{
	check(InMaterialResource);
	if (InMaterialResource->IsGameThreadShaderMapComplete())
	{
		return true;
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Landscape_WaitForMaterialCompilation);
		InMaterialResource->SubmitCompileJobs_GameThread(EShaderCompileJobPriority::High);
		if (bInWaitForCompilation)
		{
			InMaterialResource->FinishCompilation();
		}
	}
	return InMaterialResource->IsGameThreadShaderMapComplete();
}

bool ALandscape::ComputeLandscapeLayerBrushInfo(FTransform& OutLandscapeTransform, FIntPoint& OutLandscapeSize, FIntPoint& OutLandscapeRenderTargetSize)
{
	ULandscapeInfo* Info = GetLandscapeInfo();
	if (Info == nullptr)
	{
		return false;
	}

	FIntRect LandscapeExtent;
	if (!Info->GetLandscapeExtent(LandscapeExtent.Min.X, LandscapeExtent.Min.Y, LandscapeExtent.Max.X, LandscapeExtent.Max.Y))
	{
		return false;
	}

	ALandscape* Landscape = GetLandscapeActor();
	if (Landscape == nullptr)
	{
		return false;
	}

	OutLandscapeTransform = Landscape->GetTransform();
	FVector OffsetVector(LandscapeExtent.Min.X, LandscapeExtent.Min.Y, 0.f);
	FVector Translation = OutLandscapeTransform.TransformFVector4(OffsetVector);
	OutLandscapeTransform.SetTranslation(Translation);
	OutLandscapeSize = LandscapeExtent.Max - LandscapeExtent.Min;

	const FIntPoint ComponentCounts = ComputeComponentCounts();
	OutLandscapeRenderTargetSize.X = FMath::RoundUpToPowerOfTwo(((SubsectionSizeQuads + 1) * NumSubsections) * ComponentCounts.X);
	OutLandscapeRenderTargetSize.Y = FMath::RoundUpToPowerOfTwo(((SubsectionSizeQuads + 1) * NumSubsections) * ComponentCounts.Y);

	return true;
}

void ALandscape::ConvertNonEditLayerLandscape()
{
	// Detect any attempt to call this in the middle of UpdateLayersContent.  If called from blueprint, log an error and return early instead of asserting.
	if (LayerUpdateCount > 0 && UE::Landscape::Private::InBPCallstack())
	{
		UE_LOGF(LogLandscape, Error, "Attempting to make illegal call to ConvertNonEditLayerLandscape during UpdateLayersContent.");
		return;
	}
	check(LayerUpdateCount == 0);

	// If this landscape already has edit-layers it cannot be converted again
	if (!LandscapeEditLayers.IsEmpty())
	{
		UE_LOGF(LogLandscape, Warning, "Attempting to ConvertNonEditLayerLandscape on a landscape with edit-layer data.");
		return;
	}

	CreateDefaultLayer();
	check(SelectedEditLayerIndex == 0);

	// This will only copy data from proxies that have completed the registration process
	CopyOldDataToDefaultLayer();
}

// Deprecated
void ALandscape::ToggleCanHaveLayersContent()
{
}

FIntPoint ALandscape::ComputeComponentCounts() const
{
	ULandscapeInfo* Info = GetLandscapeInfo();

	if (Info == nullptr)
	{
		return FIntPoint(INDEX_NONE, INDEX_NONE);
	}

	FIntPoint NumComponents(0, 0);
	FIntPoint MaxSectionBase(TNumericLimits<int32>::Min(), TNumericLimits<int32>::Min());
	FIntPoint MinSectionBase(TNumericLimits<int32>::Max(), TNumericLimits<int32>::Max());

	Info->ForEachLandscapeProxy([&MaxSectionBase, &MinSectionBase](ALandscapeProxy* Proxy)
	{
		for (ULandscapeComponent* Component : Proxy->LandscapeComponents)
		{
			MaxSectionBase.X = FMath::Max(MaxSectionBase.X, Component->SectionBaseX);
			MaxSectionBase.Y = FMath::Max(MaxSectionBase.Y, Component->SectionBaseY);

			MinSectionBase.X = FMath::Min(MinSectionBase.X, Component->SectionBaseX);
			MinSectionBase.Y = FMath::Min(MinSectionBase.Y, Component->SectionBaseY);
		}
		return true;
	});

	if ((MaxSectionBase.X >= MinSectionBase.X) && (MaxSectionBase.Y >= MinSectionBase.Y))
	{
		NumComponents.X = ((MaxSectionBase.X - MinSectionBase.X) / ComponentSizeQuads) + 1;
		NumComponents.Y = ((MaxSectionBase.Y - MinSectionBase.Y) / ComponentSizeQuads) + 1;
	}

	return NumComponents;
}

void ALandscape::CopyOldDataToDefaultLayer()
{
	ULandscapeInfo* Info = GetLandscapeInfo();

	if (Info == nullptr)
	{
		return;
	}

	// This will only copy data from proxies that have completed the registration process
	Info->ForEachLandscapeProxy([&](ALandscapeProxy* Proxy)
	{
		CopyOldDataToDefaultLayer(Proxy);
		return true;
	});
}

void ALandscape::CopyOldDataToDefaultLayer(ALandscapeProxy* InProxy)
{
	// "Old" data represents the last generated final merged weightmap/heightmap
	CopyDataToEditLayer(InProxy);
}

void ALandscape::CopyDataToEditLayer(ALandscapeProxy* InProxy, const FGuid& InSourceEditLayerGuid, const FGuid& InDestEditLayerGuid, bool bInUseObsoleteLayerData)
{
	ULandscapeInfo* Info = GetLandscapeInfo();

	if (Info == nullptr)
	{
		return;
	}

	// No data to migrate or proxy has not finished registration
	if (InProxy->LandscapeComponents.IsEmpty() || !InProxy->bIsLandscapeActorRegisteredWithLandscapeInfo)
	{
		return;
	}

	InProxy->Modify();

	const ULandscapeEditLayerBase* FinalDestEditLayer = InDestEditLayerGuid.IsValid() ? GetEditLayerConst(InDestEditLayerGuid) : GetEditLayerConst(0);
	if (FinalDestEditLayer == nullptr || !Cast<ULandscapeEditLayerPersistent>(FinalDestEditLayer))
	{
		UE_LOGF(LogLandscape, Error, "CopyDataToEditLayer: Failed to copy data to destination layer for proxy %ls. Destination edit layer is not found in ALandscape or does not support persistent edit layer data", *InProxy->GetName());
		return;
	}

	TSet<UTexture2D*> ProcessedWeightmaps;
	TSet<UTexture2D*> ProcessedHeightmaps;

	for (ULandscapeComponent* Component : InProxy->LandscapeComponents)
	{
		// During an auto-conversion to edit layers all registered components should have synchronized the ALandscape edit layers property by now
		// It appears some legacy maps may load improperly (see MapCheck_Message_LandscapeLazyObjectPtrDeprecation_Warning) resulting in edit layers not synching before this point
		// To avoid crashing, we defensively (and safely) create the edit layer here before copying data
		FLandscapeLayerComponentData* DestLayerComponentData = Component->GetLayerData(FinalDestEditLayer->GetGuid());
		if (DestLayerComponentData == nullptr || !DestLayerComponentData->IsInitialized())
		{
			InProxy->AddLayer(FinalDestEditLayer->GetGuid());
			UE_LOGF(LogLandscape, Warning, "CopyDataToEditLayer: Defensively creating edit layer before data copy. It's expected this layer should have already been created via FixupSharedData. Other shared properties may not have upgraded properly!");
		}

		Component->CopyComponentDataToEditLayer(InSourceEditLayerGuid, FinalDestEditLayer->GetGuid(), bInUseObsoleteLayerData, ProcessedHeightmaps, ProcessedWeightmaps);
	}

	RequestLayersContentUpdateForceAll();
}

void ALandscape::UpdateProxyLayersWeightmapUsage()
{
	ULandscapeInfo* Info = GetLandscapeInfo();

	if (Info == nullptr)
	{
		return;
	}

	Info->ForEachLandscapeProxy([&](ALandscapeProxy* Proxy)
	{
		Proxy->UpdateProxyLayersWeightmapUsage();
		return true;
	});
}

void ALandscapeProxy::UpdateProxyLayersWeightmapUsage()
{
	if (bNeedsWeightmapUsagesUpdate)
	{
		InitializeProxyLayersWeightmapUsage();
	}
	check(!bNeedsWeightmapUsagesUpdate);
}

void ALandscapeProxy::PostEditUndo()
{
	check(ULandscapeComponent::UndoRedoModifiedComponentCount == 0);

	Super::PostEditUndo();
}

void ALandscape::InitializeLandscapeLayersWeightmapUsage()
{
	ULandscapeInfo* Info = GetLandscapeInfo();

	if (Info == nullptr)
	{
		return;
	}

	Info->ForEachLandscapeProxy([&](ALandscapeProxy* Proxy)
	{
		Proxy->InitializeProxyLayersWeightmapUsage();
		return true;
	});
}

void ALandscapeProxy::InitializeProxyLayersWeightmapUsage()
{
	if (ALandscape* Landscape = GetLandscapeActor())
	{
		// Reset the entire proxy's usage map and then request all components to repopulate it :
		WeightmapUsageMap.Reset();
		for (ULandscapeComponent* Component : LandscapeComponents)
		{
			// Reinitialize the weightmap usages for the base (final) paint layers allocations :
			Component->InitializeLayersWeightmapUsage(FGuid());

			for (const ULandscapeEditLayerBase* EditLayer : Landscape->GetEditLayersConst())
			{
				// Reinitialize each edit layer's weightmap usages list :
				Component->InitializeLayersWeightmapUsage(EditLayer->GetGuid());
			}
		}
	}

	bNeedsWeightmapUsagesUpdate = false;
	ValidateProxyLayersWeightmapUsage();
}
                                         
void ULandscapeComponent::InitializeLayersWeightmapUsage(const FGuid& InLayerGuid)
{
	ALandscapeProxy* Proxy = GetLandscapeProxy();
	check(Proxy);
	ALandscape* Landscape = GetLandscapeActor();
	check(Landscape);
	const ULandscapeEditLayerBase* SplinesEditLayer = Landscape->FindEditLayerOfTypeConst(ULandscapeEditLayerSplines::StaticClass());
	FGuid SplinesEditLayerGuid = SplinesEditLayer ? SplinesEditLayer->GetGuid() : FGuid();

	// Don't consider invalid edit layers : 
	if (InLayerGuid.IsValid())
	{
		FLandscapeLayerComponentData* LayerData = GetLayerData(InLayerGuid);
		if (LayerData == nullptr || !LayerData->IsInitialized())
		{
			return;
		}
	}

	const TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = GetWeightmapLayerAllocations(InLayerGuid);
	const TArray<UTexture2D*>& ComponentWeightmapTextures = GetWeightmapTextures(InLayerGuid);
	TArray<TObjectPtr<ULandscapeWeightmapUsage>>& ComponentWeightmapTexturesUsage = GetWeightmapTexturesUsage(InLayerGuid);

	ComponentWeightmapTexturesUsage.Reset();
	ComponentWeightmapTexturesUsage.AddDefaulted(ComponentWeightmapTextures.Num());

	for (int32 LayerIdx = 0; LayerIdx < ComponentWeightmapLayerAllocations.Num(); LayerIdx++)
	{
		const FWeightmapLayerAllocationInfo& Allocation = ComponentWeightmapLayerAllocations[LayerIdx];
		if (Allocation.IsAllocated())
		{
			check(ComponentWeightmapTextures.IsValidIndex(Allocation.WeightmapTextureIndex));
			UTexture2D* WeightmapTexture = ComponentWeightmapTextures[Allocation.WeightmapTextureIndex];
			TObjectPtr<ULandscapeWeightmapUsage>* TempUsage = Proxy->WeightmapUsageMap.Find(WeightmapTexture);
			if (TempUsage == nullptr)
			{
				TempUsage = &Proxy->WeightmapUsageMap.Add(WeightmapTexture, Proxy->CreateWeightmapUsage());
				(*TempUsage)->LayerGuid = InLayerGuid;
			}

			ULandscapeWeightmapUsage* Usage = *TempUsage;
			ComponentWeightmapTexturesUsage[Allocation.WeightmapTextureIndex] = Usage; // Keep a ref to it for faster access

			// Validate that there are no conflicting allocations (two allocations claiming the same texture channel)
			check(Usage->ChannelUsage[Allocation.WeightmapTextureChannel] == nullptr || Usage->ChannelUsage[Allocation.WeightmapTextureChannel] == this);

			// Validate that there are no duplicated allocation (except on the splines layer, since it's updated outside of a transaction and the transactor can later restore a duplicated 
			//  allocation in 2 different components, which will assert here but will be corrected in the next UpdateLandscapeSplines, which is called right after)
			check((SplinesEditLayerGuid.IsValid() && (InLayerGuid == SplinesEditLayerGuid))
				|| (Usage->ChannelUsage[Allocation.WeightmapTextureChannel] == nullptr)
				|| (Usage->ChannelUsage[Allocation.WeightmapTextureChannel] == this));

			Usage->ChannelUsage[Allocation.WeightmapTextureChannel] = this;
		}
	}

	// If there were some invalid allocations there, we will end up with null entries in ComponentWeightmapTexturesUsage, which is not desirable since we want 
	//  ComponentWeightmapTexturesUsage and ComponentWeightmapTextures to be in sync. Fix the situation by creating the missing usages here : 
	for (int32 Index = 0; Index < ComponentWeightmapTexturesUsage.Num(); ++Index)
	{
		if (UTexture2D* WeightmapTexture = ComponentWeightmapTextures[Index])
		{
			if (ComponentWeightmapTexturesUsage[Index] == nullptr)
			{
				TObjectPtr<ULandscapeWeightmapUsage>* TempUsage = Proxy->WeightmapUsageMap.Find(WeightmapTexture);
				if (TempUsage == nullptr)
				{
					TempUsage = &Proxy->WeightmapUsageMap.Add(WeightmapTexture, Proxy->CreateWeightmapUsage());
					(*TempUsage)->LayerGuid = InLayerGuid;
				}

				ULandscapeWeightmapUsage* Usage = *TempUsage;
				ComponentWeightmapTexturesUsage[Index] = Usage; // Keep a ref to it for faster access
			}
		}
	}
}

void ALandscape::ValidateProxyLayersWeightmapUsage() const
{
	ULandscapeInfo* Info = GetLandscapeInfo();
	if (Info == nullptr)
	{
		return;
	}

	Info->ForEachLandscapeProxy([=](const ALandscapeProxy* Proxy)
	{
		Proxy->ValidateProxyLayersWeightmapUsage();
		return true;
	});
}

void ALandscapeProxy::ValidateProxyLayersWeightmapUsage() const
{
	if ((CVarLandscapeValidateProxyWeightmapUsages.GetValueOnGameThread() == 0) || bTemporarilyDisableWeightmapUsagesValidation)
	{
		return;
	}

	// Fixup and usages should have been updated any time we run validation
	check(WeightmapFixupVersion == CurrentVersion);
	check(!bNeedsWeightmapUsagesUpdate);

	TRACE_CPUPROFILER_EVENT_SCOPE(Landscape_ValidateProxyLayersWeightmapUsage);
	TMap<UTexture2D*, TArray<FWeightmapLayerAllocationInfo>> PerTextureAllocations;
	if (const ALandscape* Landscape = GetLandscapeActor())
	{
		for (ULandscapeComponent* Component : LandscapeComponents)
		{
			auto ValidateWeightmapAllocationAndUsage = [&](UTexture2D* InWeightmapTexture, const FWeightmapLayerAllocationInfo& InAllocation, ULandscapeWeightmapUsage* InUsage, const FGuid &InLayerGuid)
			{
				if (InUsage)
				{
					// Each usage should also be stored in the proxy's map
					const TObjectPtr<ULandscapeWeightmapUsage>* ProxyMapUsage = WeightmapUsageMap.Find(InWeightmapTexture);
					check(ProxyMapUsage);
					check(InUsage == *ProxyMapUsage);

					// Our component should own the channel, and the LayerGuid should match
					check(InUsage->ChannelUsage[InAllocation.WeightmapTextureChannel] == Component);
					check(InUsage->LayerGuid == InLayerGuid);
				}

				// There should not be any other allocations pointing to this channel on this texture
				TArray<FWeightmapLayerAllocationInfo>& AllAllocationsForThisTexture = PerTextureAllocations.FindOrAdd(InWeightmapTexture);
				for (FWeightmapLayerAllocationInfo &Alloc : AllAllocationsForThisTexture)
				{
					check(Alloc.WeightmapTextureChannel != InAllocation.WeightmapTextureChannel);
				}
			};

			const TArray<UTexture2D*>& WeightmapTextures = Component->GetWeightmapTextures(/*InReturnEditingWeightmap = */false);
			const TArray<ULandscapeWeightmapUsage*>& WeightmapTextureUsages = Component->GetWeightmapTexturesUsage(/*InReturnEditingWeightmap = */false);

			// Validate weightmap allocations
			FGuid BaseGuid;
			for (const FWeightmapLayerAllocationInfo& Allocation : Component->GetWeightmapLayerAllocations(/*InReturnEditingWeightmap = */false))
			{
				if (Allocation.IsAllocated())
				{
					// The allocation texture index should point to a valid texture
					UTexture2D* WeightmapTexture = WeightmapTextures[Allocation.WeightmapTextureIndex];
					check(WeightmapTexture != nullptr);

					// Either it's out of bounds  i.e. not initialized yet,  or it is initialized and we validate that it is correct...
					ULandscapeWeightmapUsage* Usage = WeightmapTextureUsages.IsValidIndex(Allocation.WeightmapTextureIndex) ? WeightmapTextureUsages[Allocation.WeightmapTextureIndex] : nullptr;
					ValidateWeightmapAllocationAndUsage(WeightmapTexture, Allocation, Usage, BaseGuid);
				}
			}

			// Validate edit layers weightmap allocations : 
			{
				const ULandscapeEditLayerBase* SplinesEditLayer = Landscape->FindEditLayerOfTypeConst(ULandscapeEditLayerSplines::StaticClass());
				for (const ULandscapeEditLayerBase* EditLayer : Landscape->GetEditLayersConst())
				{
					FLandscapeLayerComponentData* LayerData = Component->GetLayerData(EditLayer->GetGuid());

					// Skip validation on SplinesLayer since it can momentarily contain duplicated layer allocations after undo (since it's updated outside of a transaction) :
					if (LayerData != nullptr && LayerData->IsInitialized() && (EditLayer != SplinesEditLayer))
					{
						for (int32 LayerIdx = 0; LayerIdx < LayerData->WeightmapData.LayerAllocations.Num(); LayerIdx++)
						{
							const FWeightmapLayerAllocationInfo& Allocation = LayerData->WeightmapData.LayerAllocations[LayerIdx];
							if (Allocation.IsAllocated())
							{
								UTexture2D* WeightmapTexture = LayerData->WeightmapData.Textures[Allocation.WeightmapTextureIndex];
								if (ULandscapeWeightmapUsage* Usage = LayerData->WeightmapData.TextureUsages.IsValidIndex(Allocation.WeightmapTextureIndex) ? LayerData->WeightmapData.TextureUsages[Allocation.WeightmapTextureIndex].Get() : nullptr)
								{
									ValidateWeightmapAllocationAndUsage(WeightmapTexture, Allocation, Usage, EditLayer->GetGuid());
								}
							}
						}
					}
				}
			}
		}
	}
}

void ALandscapeProxy::RequestProxyLayersWeightmapUsageUpdate()
{
	bNeedsWeightmapUsagesUpdate = true;
}

void ExecuteCopyLayersTexture(TArray<FLandscapeLayersCopyTextureParams>&& InCopyTextureParams)
{
	ENQUEUE_RENDER_COMMAND(LandscapeLayers_Cmd_CopyTexture)(
		[CopyTextureParams = MoveTemp(InCopyTextureParams)](FRHICommandListImmediate& RHICmdList) mutable
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeLayers_RT_CopyTextures);
		SCOPED_DRAW_EVENTF(RHICmdList, LandscapeLayers, TEXT("LandscapeLayers : Copy %d texture regions"), CopyTextureParams.Num());

		for (const FLandscapeLayersCopyTextureParams& Params : CopyTextureParams)
		{
			if ((Params.SourceResource != nullptr) && (Params.DestResource != nullptr))
			{
				FLandscapeLayersCopyTexture_RenderThread CopyTexture(Params);
				CopyTexture.Copy(RHICmdList);
			}
		}
	});
}

/** Per component information from read back results. */
struct FLandscapeEditLayerComponentReadbackResult
{
	ULandscapeComponent* LandscapeComponent = nullptr;
	/** ELandscapeLayerUpdateMode flags set on ULandscapeComponent at time when read back task was submitted. */
	uint32 UpdateModes = 0;
	/** Were the associated heightmap/weightmaps modified. */
	bool bModified = false;
	bool bCleared = false;
	/** Indicates which of the component's weightmaps is not needed anymore. */
	TArray<ULandscapeLayerInfoObject*> AllZeroLayers;

	FLandscapeEditLayerComponentReadbackResult(ULandscapeComponent* InLandscapeComponent, uint32 InUpdateModes)
		: LandscapeComponent(InLandscapeComponent)
		, UpdateModes(InUpdateModes)
	{}
};

/** Description for a single read back operation. */
struct FLandscapeLayersCopyReadbackTextureParams
{
	FLandscapeLayersCopyReadbackTextureParams(UTexture2D* InSource, FLandscapeEditLayerReadback* InDest)
		: Source(InSource)
		, Dest(InDest)
	{}

	UTexture2D* Source;
	FLandscapeEditLayerReadback* Dest;
	FLandscapeEditLayerReadback::FReadbackContext Context;
};

void ExecuteCopyToReadbackTexture(TArray<FLandscapeLayersCopyReadbackTextureParams>& InParams)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ExecuteCopyToReadbackTexture);
	RHI_BREADCRUMB_EVENT_GAMETHREAD_F("Copy to readback textures", "Copy to readback textures (%d copies)", InParams.Num());
	if (!FApp::CanEverRender())
	{
		return;
	}
	for (FLandscapeLayersCopyReadbackTextureParams& Params : InParams)
	{
		// stop any GPU texture edge patching on readback components,
		// until we can update the GPU edge hashes after the readback results are available
		// (otherwise we might might get incorrect GPU edge hashes in our tracking)
		if (TObjectPtr<ULandscapeComponent>* ComponentPtr = FLandscapeGroup::HeightmapTextureToActiveComponent.Find(Params.Source))
		{
			if (ULandscapeHeightmapTextureEdgeFixup* Fixup = (*ComponentPtr)->RegisteredEdgeFixup)
			{
				Fixup->PauseTextureEdgePatchingUntilGPUEdgeHashesUpdated();
			}
		}

		Params.Dest->Enqueue(Params.Source, MoveTemp(Params.Context));
	}
}

TArray<FLandscapeLayersCopyReadbackTextureParams> PrepareLandscapeLayersCopyReadbackTextureParams(const FTextureToComponentHelper& InMapHelper, TArray<UTexture2D*> InTextures, bool bWeightmaps)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeLayers_PrepareLandscapeLayersCopyReadbackTextureParams);
	TArray<FLandscapeLayersCopyReadbackTextureParams> Result;
	Result.Reserve(InTextures.Num());

	for (UTexture2D* Texture : InTextures)
	{
		const TMap<UTexture2D*, TArray<ULandscapeComponent*>>& TexturesToComponents = bWeightmaps ? InMapHelper.WeightmapToComponents : InMapHelper.HeightmapToComponents;
		const TArray<ULandscapeComponent*>* Components = TexturesToComponents.Find(Texture);
		check(Components && !Components->IsEmpty());
		ALandscapeProxy* Proxy = (*Components)[0]->GetLandscapeProxy();
		FLandscapeEditLayerReadback** CPUReadback = bWeightmaps ? Proxy->WeightmapsCPUReadback.Find(Texture) : Proxy->HeightmapsCPUReadback.Find(Texture);
		check(CPUReadback && *CPUReadback);

		FLandscapeLayersCopyReadbackTextureParams& CopyReadbackTextureParams = Result.Add_GetRef(FLandscapeLayersCopyReadbackTextureParams(Texture, *CPUReadback));
		// Init the CPU read back contexts for all components dependent on this texture. This includes a context containing the current component states : 
		for (ULandscapeComponent* ComponentToResolve : *Components)
		{
			const FIntPoint ComponentToResolveKey = ComponentToResolve->GetComponentKey();
			const int32 ComponentToResolveFlags = ComponentToResolve->GetLayerUpdateFlagPerMode();
			FLandscapeEditLayerReadback::FPerChannelLayerNames PerChannelLayerNames;

			// Weightmaps could be reallocated randomly before we actually perform the readback, so we need to keep a picture of which channel was affected to which paint layer at readback time:
			if (bWeightmaps)
			{
				const TArray<UTexture2D*>& WeightmapTextures = ComponentToResolve->GetWeightmapTextures(/*InReturnEditingWeightmap = */false);
				for (FWeightmapLayerAllocationInfo const& AllocInfo : ComponentToResolve->GetWeightmapLayerAllocations(/*InReturnEditingWeightmap = */false))
				{
					if (AllocInfo.IsAllocated())
					{
						UTexture2D* PaintLayerTexture = WeightmapTextures[AllocInfo.WeightmapTextureIndex];
						if (PaintLayerTexture == Texture)
						{
							PerChannelLayerNames[AllocInfo.WeightmapTextureChannel] = AllocInfo.LayerInfo->GetLayerName();
						}
					}
				}
			}
			CopyReadbackTextureParams.Context.Add(FLandscapeEditLayerReadback::FComponentReadbackContext { ComponentToResolveKey, ComponentToResolveFlags, PerChannelLayerNames });
		}
	}

	return Result;
}

bool ALandscape::PrepareTextureResourcesLimited(EPrepareTextureResourcesLimitedMode Mode)
{
	// Rate-limit to once per frame. Note that commandlets don't advance the frame counter.
	// Also, force it when bInWaitForStreaming is true, so that we don't prevent new textures from being prepared in case PrepareTextureResourcesLimited is called 
	// multiple times in the same frame with some new textures being created in-between (e.g. when creating a large WP landscape via multiple successive regions)
	if (EnumHasAnyFlags(Mode, EPrepareTextureResourcesLimitedMode::BypassRateLimit | EPrepareTextureResourcesLimitedMode::WaitForStreaming)
		|| (GFrameNumber != LastPrepareTextureResourcesCalled)
		|| IsRunningCommandlet())
	{
		LastPrepareTextureResourcesCalled = GFrameNumber;
		bLastPrepareTextureResourcesResult = PrepareTextureResources(EnumHasAnyFlags(Mode, EPrepareTextureResourcesLimitedMode::WaitForStreaming));
	}
	return bLastPrepareTextureResourcesResult;
}

bool ALandscape::PrepareTextureResources(bool bInWaitForStreaming)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeLayers_PrepareTextureResources);

	ULandscapeInfo* Info = GetLandscapeInfo();
	if (Info == nullptr || !FApp::CanEverRender())
	{
		return false;
	}

	// Only keep the textures that are still valid:
	TSet<UTexture2D*> StreamingInTexturesBefore;
	StreamingInTexturesBefore.Reserve(TrackedStreamingInTextures.Num());	
	Algo::TransformIf(TrackedStreamingInTextures, StreamingInTexturesBefore,
		[](const TWeakObjectPtr<UTexture2D>& Texture) { return Texture.IsValid(); },
		[](const TWeakObjectPtr<UTexture2D>& Texture) { return Texture.Get(); });
	TrackedStreamingInTextures.Empty();

	// Textures that are still streaming in (filled out below)
	TSet<UTexture2D*> StreamingInTexturesAfter;
	StreamingInTexturesAfter.Reserve(TrackedStreamingInTextures.Num());

	// Textures that have just completed streaming in (filled out below)
	TSet<UTexture2D*> StreamedInTextures;

	// All components containing heightmaps that have just completed streaming in (filled out below)
	TSet<ULandscapeComponent*> StreamedInHeightmapComponents;

	FLandscapeTextureStreamingManager* TextureStreamingManager = GetWorld()->GetSubsystem<ULandscapeSubsystem>()->GetTextureStreamingManager();

	bool bIsReady = true;
	Info->ForEachLandscapeProxy([&](ALandscapeProxy* Proxy)
	{
		for (ULandscapeComponent* Component : Proxy->LandscapeComponents)
		{
			UTexture2D* ComponentHeightmap = Component->GetHeightmap();
			check(ComponentHeightmap != nullptr);
			{
				bool bIsTextureReady = TextureStreamingManager->RequestTextureFullyStreamedInForever(ComponentHeightmap, bInWaitForStreaming);
				if (!bIsTextureReady)
				{
					StreamingInTexturesAfter.Add(ComponentHeightmap);
				}
				else
				{
					// If it was previously streaming in, then it has just completed.
					if (StreamingInTexturesBefore.Contains(ComponentHeightmap))
					{
						StreamedInTextures.Add(ComponentHeightmap);
						StreamedInHeightmapComponents.Add(Component);
					}
				}
				bIsReady &= bIsTextureReady;
			}

			for (UTexture2D* ComponentWeightmap : Component->GetWeightmapTextures())
			{
				check(ComponentWeightmap != nullptr);

				bool bIsTextureReady = TextureStreamingManager->RequestTextureFullyStreamedInForever(ComponentWeightmap, bInWaitForStreaming);
				// If the texture is not ready, start tracking its state changes to be notified when it's fully streamed in : 
				if (!bIsTextureReady)
				{
					StreamingInTexturesAfter.Add(ComponentWeightmap);
				}
				else
				{
					// If it was previously streaming in, then it has just completed.
					if (StreamingInTexturesBefore.Contains(ComponentWeightmap))
					{
						StreamedInTextures.Add(ComponentWeightmap);
					}
				}
				bIsReady &= bIsTextureReady;
			}
		}
		return true;
	});

	// The assets that were streaming in before and are not anymore can be considered streamed in: 
	InvalidateRVTForTextures(StreamedInTextures);

	// If we streamed in any heightmaps, notify interested parties (i.e. water)
	if (StreamedInHeightmapComponents.Num() > 0)
	{
		// Calculate update region.
		FBox2D HeightmapUpdateRegion(ForceInit);
		for (ULandscapeComponent* Component : StreamedInHeightmapComponents)
		{
			if (ALandscapeProxy* Proxy = Component->GetLandscapeProxy())
			{
				const FBox ProxyBox = Proxy->GetComponentsBoundingBox();
				HeightmapUpdateRegion += FBox2D(FVector2D(ProxyBox.Min), FVector2D(ProxyBox.Max));
			}
		}

		// Notify that heightmaps have been streamed.
		if (ULandscapeSubsystem* LandscapeSubsystem = GetWorld()->GetSubsystem<ULandscapeSubsystem>())
		{
			FOnHeightmapStreamedContext Context(this, HeightmapUpdateRegion, StreamedInHeightmapComponents);
			LandscapeSubsystem->GetDelegateAccess().OnHeightmapStreamedDelegate.Broadcast(Context);
		}
	}

	// Store as a list of TWeakObjectPtr<UTexture2D> so as not to keep references on the tracked textures :
	Algo::Transform(StreamingInTexturesAfter, TrackedStreamingInTextures, [](UTexture2D* Texture) { return TWeakObjectPtr<UTexture2D>(Texture); });

	return bIsReady;
}

void ALandscape::DeleteUnusedLayers()
{
	// TODO [jared.ritchie] Check there is not a layer update pending before deleting unused layers
	ULandscapeInfo* LandscapeInfo = GetLandscapeInfo();

	if (LandscapeInfo == nullptr)
	{
		return;
	}

	LandscapeInfo->ForEachLandscapeProxy([&](ALandscapeProxy* Proxy)
	{
		if (Proxy != this)
		{
			Proxy->DeleteUnusedLayers();
		}
		return true;
	});

	ALandscapeProxy::DeleteUnusedLayers();
}

// Note: this approach is generic, because FObjectCacheContextScope is a fast texture->material interface->primitive component lookup. 
// If FObjectCacheContextScope was available at runtime, it could become an efficient way to automatically invalidate RVT areas corresponding to primitive components that use textures that are being streamed in:
void ALandscape::InvalidateRVTForTextures(const TSet<UTexture2D*>& InTextures)
{
#if WITH_EDITOR
	TRACE_CPUPROFILER_EVENT_SCOPE(ALandscape_InvalidateRVTForTextures);

	if (!InTextures.IsEmpty())
	{
		// Retrieve all primitive components that use this texture through a RVT-writing material, using FObjectCacheContextScope, which is a fast texture->material interface->primitive component lookup
		FObjectCacheContextScope ObjectCacheScope;
		TSet<UPrimitiveComponent*> PrimitiveComponentsToInvalidate;

		for (UTexture2D* Texture : InTextures)
		{
			if (Texture != nullptr)
			{
				// First, find all the materials referencing this texture that are writing to the RVT in order to invalidate the primitive components referencing them when the texture 
				//  gets fully streamed in so that we're not left with low-res mips being rendered in the RVT tiles : 
				for (UMaterialInterface* MaterialInterface : ObjectCacheScope.GetContext().GetMaterialsAffectedByTexture(Texture))
				{
					if (MaterialInterface->WritesToRuntimeVirtualTexture())
					{
						for (IPrimitiveComponent* PrimitiveComponentInterface : ObjectCacheScope.GetContext().GetPrimitivesAffectedByMaterial(MaterialInterface))
						{
							// Landscape only supports UPrimitiveComponent for the moment
							if (UPrimitiveComponent* PrimitiveComponent = PrimitiveComponentInterface->GetUObject<UPrimitiveComponent>())
							{
								PrimitiveComponentsToInvalidate.Add(PrimitiveComponent);
							}
						}
					}
				}
			}
		}

		if (!PrimitiveComponentsToInvalidate.IsEmpty())
		{
			// Now invalidate the RVT regions that correspond to these components :
			for (TObjectIterator<URuntimeVirtualTextureComponent> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
			{
				for (UPrimitiveComponent* PrimitiveComponent : PrimitiveComponentsToInvalidate)
				{
					if (PrimitiveComponent->GetRuntimeVirtualTextures().Contains(It->GetVirtualTexture()))
					{
						It->Invalidate(FBoxSphereBounds(PrimitiveComponent->Bounds), EVTInvalidatePriority::Normal);
					}
				}
			}
		}
	}
#endif // WITH_EDITOR
}

bool ALandscape::PrepareLayersTextureResources(bool bInWaitForStreaming)
{
	return PrepareLayersTextureResources(LandscapeEditLayers, bInWaitForStreaming);
}

bool ALandscape::PrepareLayersTextureResources(const TArray<FLandscapeLayer>& InLayers, bool bInWaitForStreaming)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ALandscape::PrepareLayersTextureResources);

	ULandscapeInfo* Info = GetLandscapeInfo();
	if (Info == nullptr)
	{
		return false;
	}

	FLandscapeTextureStreamingManager* TextureStreamingManager = GetWorld()->GetSubsystem<ULandscapeSubsystem>()->GetTextureStreamingManager();

	bool bIsReady = true;
	Info->ForEachLandscapeProxy([&, TextureStreamingManager](ALandscapeProxy* Proxy)
	{
		for (const FLandscapeLayer& Layer : InLayers)
		{
			for (ULandscapeComponent* Component : Proxy->LandscapeComponents)
			{
				if (FLandscapeLayerComponentData* ComponentLayerData = Component->GetLayerData(Layer.EditLayer->GetGuid()))
				{
					check(ComponentLayerData->HeightmapData.Texture != nullptr);
					bIsReady &= TextureStreamingManager->RequestTextureFullyStreamedInForever(ComponentLayerData->HeightmapData.Texture, bInWaitForStreaming);

					for (UTexture2D* LayerWeightmap : ComponentLayerData->WeightmapData.Textures)
					{
						check(LayerWeightmap != nullptr);
						bIsReady &= TextureStreamingManager->RequestTextureFullyStreamedInForever(LayerWeightmap, bInWaitForStreaming);
					}
				}
			}
		}
		return true;
	});

	return bIsReady;
}

bool ALandscape::PrepareLayersResources(EShaderPlatform InShaderPlatform, bool bInWaitForStreaming)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ALandscape::PrepareLayersResources);
	TSet<UObject*> Dependencies;
	for (const FLandscapeLayer& Layer : LandscapeEditLayers)
	{
		ULandscapeEditLayerBase* EditLayer = Layer.EditLayer;
		check(EditLayer != nullptr);

		if (EditLayer->SupportsTargetType(ELandscapeToolTargetType::Heightmap) || EditLayer->SupportsTargetType(ELandscapeToolTargetType::Weightmap) || EditLayer->SupportsTargetType(ELandscapeToolTargetType::Visibility))
		{
			EditLayer->GetRenderDependencies(Dependencies);
		}

		for (const FLandscapeLayerBrush& Brush : Layer.Brushes)
		{
			if (ALandscapeBlueprintBrushBase* LandscapeBrush = Brush.GetBrush())
			{
				if (LandscapeBrush->AffectsWeightmap() || LandscapeBrush->AffectsHeightmap() || LandscapeBrush->AffectsVisibilityLayer())
				{
					LandscapeBrush->GetRenderDependencies(Dependencies);
				}
			}
		}
	}

	FLandscapeTextureStreamingManager* TextureStreamingManager = GetWorld()->GetSubsystem<ULandscapeSubsystem>()->GetTextureStreamingManager();

	bool bIsReady = true;
	for (UObject* Dependency : Dependencies)
	{
		// Streamable textures need to be fully streamed in : 
		if (UTexture* Texture = Cast<UTexture>(Dependency))
		{
			bIsReady &= TextureStreamingManager->RequestTextureFullyStreamedInForever(Texture, bInWaitForStreaming);
		}

		// Material shaders need to be fully compiled : 
		if (UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(Dependency))
		{
			if (FMaterialResource* MaterialResource = MaterialInterface->GetMaterialResource(InShaderPlatform))
			{
				// Don't early-out because checking for the material resource actually requests the shaders to be loaded so we want to make sure to request them all at once instead of one by one :
				bIsReady &= IsMaterialResourceCompiled(MaterialResource, bInWaitForStreaming);
			}
		}
	}

	return bIsReady;
}

namespace UE::Landscape::Private
{
// Find a matching entry in Array for the given Component.  If one does not exist, add one with the provided UpdateModes value.
FLandscapeEditLayerComponentReadbackResult* FindOrAddByComponent(TArray<FLandscapeEditLayerComponentReadbackResult>& Array, ULandscapeComponent* Component, uint32 DefaultUpdateModes)
{
	FLandscapeEditLayerComponentReadbackResult* ComponentReadbackResult = Algo::FindBy(Array, Component, &FLandscapeEditLayerComponentReadbackResult::LandscapeComponent);
	if (ComponentReadbackResult == nullptr)
	{
		ComponentReadbackResult = &Array.Emplace_GetRef(Component, DefaultUpdateModes);
	}
	return ComponentReadbackResult;
}
}  // UE::Landscape::Private


// Must match FEditLayerHeightmapMergeInfo in LandscapeLayersHeightmapsPS.usf
struct FLandscapeEditLayerHeightmapMergeInfo
{
	// COMMENT [jonathan.bard] : not used at the moment because we copy to a texture 2D array but if we didn't and had instead N statically bound textures, we could save that copy and sample the textures directly :
	FIntRect TextureSubregion; // Subregion of the source (edit layer) texture to use

	ELandscapeEditLayerHeightmapBlendMode BlendMode = ELandscapeEditLayerHeightmapBlendMode::Num; // How this layer blends with the previous ones in the layers stack
	float Alpha = 1.0f; // Alpha value to be used in the blend
	uint32 Padding0; // Align to next float4 
	uint32 Padding1;
};

// Must match FEditLayerWeightmapMergeInfo in LandscapeLayersWeightmapsPS.usf
struct FLandscapeEditLayerWeightmapMergeInfo
{
	uint32 SourceWeightmapTextureIndex = (uint32)INDEX_NONE; // The index in InPackedWeightmaps of the texture to read from for this layer
	uint32 SourceWeightmapTextureChannel = (uint32)INDEX_NONE; // The channel of the texture to read from for this layer
	ELandscapeEditLayerWeightmapBlendMode BlendMode = ELandscapeEditLayerWeightmapBlendMode::Num; // How this layer blends with the previous ones in the layers stack
	float Alpha = 1.0f; // Alpha value to be used in the blend
};

// Must match FPerEditLayerWeightmapPaintLayerInfo in LandscapeEditLayersWeightmaps.usf
// Additional info about this paint layer on this edit layer
struct FLandscapePerEditLayerWeightmapPaintLayerInfo
{
	ELandscapeEditLayerWeightmapBlendMode Flags = ELandscapeEditLayerWeightmapBlendMode::Num; // How this layer blends with the previous ones in the layers stack
};

// Struct that contains all the information relevant for the edit layers update operation (list of dirty components, heightmaps, weightmaps, etc.
//  Because this information can change during the course of the update (e.g. new weightmaps are added) it can be (partially or not) refreshed if necessary :
struct FUpdateLayersContentContext
{
	// Partial refresh flags : allows to recompute only a subset of the context information :
	enum class ERefreshFlags
	{
		None = 0,
		RefreshComponentInfos = (1 << 0),
		RefreshHeightmapInfos = (1 << 1),
		RefreshWeightmapInfos = (1 << 2),
		RefreshMapHelper = (1 << 3),
		RefreshAll = ~0,
	};
	FRIEND_ENUM_CLASS_FLAGS(ERefreshFlags);

	// Set up the layer content context for a selective render.  Rendering the components provided by InMapHelper based on their LayerUpdateFlags.
	FUpdateLayersContentContext(const FTextureToComponentHelper& InMapHelper, bool bInPartialUpdate)
		: bPartialUpdate(bInPartialUpdate)
		, bSelectiveRender(false)
		, MapHelper(InMapHelper)
	{
		// No need to update the map helper, it's assumed to be already ready in the constructor
		Refresh(ERefreshFlags::RefreshComponentInfos | ERefreshFlags::RefreshHeightmapInfos | ERefreshFlags::RefreshWeightmapInfos);
	}

	// Set up the layer content context for a selective render.  Rendering the components provided by InMapHelper, regardless of their LayerUpdateFlags.
	// Can specify RefreshHeightmapInfos and/or RefreshWeightmapInfos.
	struct FSelectiveRenderTag {};
	FUpdateLayersContentContext(const FTextureToComponentHelper& InMapHelper, FSelectiveRenderTag, ERefreshFlags HeightOrWeightFlags)
		: bPartialUpdate(false)
		, bSelectiveRender(true)
		, MapHelper(InMapHelper)
	{
		// expect only RefreshHeightmapInfos and/or RefreshWeightmapInfos
		check((HeightOrWeightFlags & (ERefreshFlags::RefreshHeightmapInfos | ERefreshFlags::RefreshWeightmapInfos)) != ERefreshFlags::None);
		check((HeightOrWeightFlags & ~(ERefreshFlags::RefreshHeightmapInfos | ERefreshFlags::RefreshWeightmapInfos)) == ERefreshFlags::None);  
		Refresh(ERefreshFlags::RefreshComponentInfos | HeightOrWeightFlags);
	}

	FTextureToComponentHelper::ERefreshFlags RefreshFlagsToMapHelperRefreshFlags(ERefreshFlags InRefreshFlags)
	{
		FTextureToComponentHelper::ERefreshFlags MapHelperRefreshFlags = FTextureToComponentHelper::ERefreshFlags::None;
		if (EnumHasAnyFlags(InRefreshFlags, ERefreshFlags::RefreshComponentInfos))
		{
			MapHelperRefreshFlags |= FTextureToComponentHelper::ERefreshFlags::RefreshComponents;
		}
		if (EnumHasAnyFlags(InRefreshFlags, ERefreshFlags::RefreshHeightmapInfos))
		{
			MapHelperRefreshFlags |= FTextureToComponentHelper::ERefreshFlags::RefreshHeightmaps;
		}
		if (EnumHasAnyFlags(InRefreshFlags, ERefreshFlags::RefreshWeightmapInfos))
		{
			MapHelperRefreshFlags |= FTextureToComponentHelper::ERefreshFlags::RefreshWeightmaps;
		}
		return MapHelperRefreshFlags;
	}

	void Refresh(ERefreshFlags InRefreshFlags)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UpdateLayersContentContext_Refresh);
		// Start by updating the map helper if necessary (keep track of components/heightmaps/weightmaps relationship) :
		if (EnumHasAnyFlags(InRefreshFlags, ERefreshFlags::RefreshMapHelper))
		{
			MapHelper.Refresh(RefreshFlagsToMapHelperRefreshFlags(InRefreshFlags));
		}

		// Then triage the dirty/non-dirty components  :
		if (EnumHasAnyFlags(InRefreshFlags, ERefreshFlags::RefreshComponentInfos))
		{
			// When components are refreshed, all other info has to be, except in selective renders :
			check(bSelectiveRender || EnumHasAllFlags(InRefreshFlags, ERefreshFlags::RefreshHeightmapInfos | ERefreshFlags::RefreshWeightmapInfos));

			DirtyLandscapeComponents.Reset();
			NonDirtyLandscapeComponents.Reset();
			for (ULandscapeComponent* Component : MapHelper.LandscapeComponents)
			{
				if (!bPartialUpdate || Component->GetLayerUpdateFlagPerMode() != 0)
				{
					DirtyLandscapeComponents.Add(Component);
				}
				else
				{
					NonDirtyLandscapeComponents.Add(Component);
				}
			}
		}

		if (EnumHasAnyFlags(InRefreshFlags, ERefreshFlags::RefreshHeightmapInfos | ERefreshFlags::RefreshWeightmapInfos))
		{
			TSet<UTexture2D*> HeightmapsToRender;
			TSet<ULandscapeComponent*> NeighborsComponents;
			TSet<ULandscapeComponent*> WeightmapsComponents;

			// Cleanup our heightmap/weightmap info:
			if (EnumHasAnyFlags(InRefreshFlags, ERefreshFlags::RefreshHeightmapInfos))
			{
				HeightmapsToResolve.Reset();
				LandscapeComponentsHeightmapsToRender.Reset();
				LandscapeComponentsHeightmapsToResolve.Reset();
			}
			if (EnumHasAnyFlags(InRefreshFlags, ERefreshFlags::RefreshWeightmapInfos))
			{
				WeightmapsToResolve.Reset();
				LandscapeComponentsWeightmapsToRender.Reset();
				LandscapeComponentsWeightmapsToResolve.Reset();
			}
			// Note that the AllLandscapeComponentsToResolve and AllLandscapeComponentReadbackResults are *not* reset here: they can only grow (we're assuming refresh only adds new components): 

			// Iterate on all dirty components and retrieve the components that need to be resolved or rendered for their heightmap or weightmaps :
			TArray<ULandscapeComponent*> AllLandscapeComponents;
			for (ULandscapeComponent* Component : DirtyLandscapeComponents)
			{
				AllLandscapeComponents.Add(Component);

				// If all components are dirty, we can take some shortcuts since all components will need to be rendered and resolved : 
				if (bPartialUpdate)
				{
					if (EnumHasAnyFlags(InRefreshFlags, ERefreshFlags::RefreshHeightmapInfos))
					{
						// Gather Neighbors (Neighbors need to be Rendered but not resolved so that the resolved Components have valid normals on edges)
						Component->GetLandscapeComponentNeighborsToRender(NeighborsComponents);
						Component->ForEachLayer([&](const FGuid&, FLandscapeLayerComponentData& LayerData)
						{
							HeightmapsToRender.Add(LayerData.HeightmapData.Texture);
						});
					}

					if (EnumHasAnyFlags(InRefreshFlags, ERefreshFlags::RefreshWeightmapInfos))
					{
						// Gather WeightmapUsages (Components sharing weightmap usages with the resolved Components need to be rendered so that the resolving is valid)
						Component->GetLandscapeComponentWeightmapsToRender(WeightmapsComponents);
					}
				}

				if (EnumHasAnyFlags(InRefreshFlags, ERefreshFlags::RefreshHeightmapInfos))
				{
					// Gather Heightmaps (All Components sharing Heightmap textures need to be rendered and resolved)
					HeightmapsToResolve.Add(Component->GetHeightmap(/*InReturnEditingHeightmap = */false));
				}

				if (EnumHasAnyFlags(InRefreshFlags, ERefreshFlags::RefreshWeightmapInfos))
				{
					// Gather Weightmaps
					const TArray<UTexture2D*>& WeightmapTextures = Component->GetWeightmapTextures(/*InReturnEditingWeightmap = */false);
					for (FWeightmapLayerAllocationInfo const& AllocInfo : Component->GetWeightmapLayerAllocations(/*InReturnEditingWeightmap = */false))
					{
						if (AllocInfo.IsAllocated() && AllocInfo.WeightmapTextureIndex < WeightmapTextures.Num())
						{
							WeightmapsToResolve.Add(WeightmapTextures[AllocInfo.WeightmapTextureIndex]);
						}
					}
				}
			}

			if (EnumHasAnyFlags(InRefreshFlags, ERefreshFlags::RefreshHeightmapInfos))
			{
				// Because of Heightmap Sharing anytime we render a heightmap we need to render all the components that use it
				for (ULandscapeComponent* NeighborsComponent : NeighborsComponents)
				{
					NeighborsComponent->ForEachLayer([&](const FGuid&, FLandscapeLayerComponentData& LayerData)
					{
						HeightmapsToRender.Add(LayerData.HeightmapData.Texture);
					});
				}

				// Copy first list into others
				LandscapeComponentsHeightmapsToResolve.Append(AllLandscapeComponents);
				LandscapeComponentsHeightmapsToRender.Append(AllLandscapeComponents);
			}

			if (EnumHasAnyFlags(InRefreshFlags, ERefreshFlags::RefreshWeightmapInfos))
			{
				LandscapeComponentsWeightmapsToResolve.Append(AllLandscapeComponents);
				LandscapeComponentsWeightmapsToRender.Append(AllLandscapeComponents);
			}

			if (bPartialUpdate)
			{
				for (ULandscapeComponent* Component : NonDirtyLandscapeComponents)
				{
					if (EnumHasAnyFlags(InRefreshFlags, ERefreshFlags::RefreshHeightmapInfos))
					{
						if (HeightmapsToResolve.Contains(Component->GetHeightmap(false)))
						{
							AllLandscapeComponents.Add(Component);
							LandscapeComponentsHeightmapsToRender.Add(Component);
							LandscapeComponentsHeightmapsToResolve.Add(Component);
						}
						else if (NeighborsComponents.Contains(Component))
						{
							LandscapeComponentsHeightmapsToRender.Add(Component);
						}
						else
						{
							bool bAdd = false;
							Component->ForEachLayer([&](const FGuid&, FLandscapeLayerComponentData& LayerData)
							{
								if (HeightmapsToRender.Contains(LayerData.HeightmapData.Texture))
								{
									bAdd = true;
								}
							});
							if (bAdd)
							{
								LandscapeComponentsHeightmapsToRender.Add(Component);
							}
						}
					}

					if (EnumHasAnyFlags(InRefreshFlags, ERefreshFlags::RefreshWeightmapInfos))
					{
						if (WeightmapsComponents.Contains(Component))
						{
							LandscapeComponentsWeightmapsToRender.Add(Component);
						}
					}
				}
			}

			// All selected components will have to be resolved : 
			AllLandscapeComponentsToResolve.Append(AllLandscapeComponents);

			if (!bSelectiveRender)
			{
				// Add components with deferred flag to update list
				for (ULandscapeComponent* Component : AllLandscapeComponents)
				{
					if (Component->GetLayerUpdateFlagPerMode() & ELandscapeLayerUpdateMode::Update_Client_Deferred)
					{
						UE::Landscape::Private::FindOrAddByComponent(AllLandscapeComponentReadbackResults, Component, ELandscapeLayerUpdateMode::Update_Client_Deferred);
					}
				}
			}
		}
	}

	// Indicates whether all components of the landscape are marked dirty :
	const bool bPartialUpdate = false;
	const bool bSelectiveRender = false;

	// Helper to gather mappings between heightmaps/weightmaps and components :
	FTextureToComponentHelper MapHelper;
	// List of landscape components that have been made dirty and need to be updated : 
	TArray<ULandscapeComponent*> DirtyLandscapeComponents;
	// List of landscape components that have not been made dirty : 
	TArray<ULandscapeComponent*> NonDirtyLandscapeComponents;
	// List of heightmap textures that might be affected by the update : 
	TSet<UTexture2D*> HeightmapsToResolve;
	// List of weightmap textures that might be affected by the update : 
	TSet<UTexture2D*> WeightmapsToResolve;
	// List of components that need to be rendered because they are either dirty or are neighbor to a component that is dirty or share a heightmap with a component that is dirty:
	TArray<ULandscapeComponent*> LandscapeComponentsHeightmapsToRender;
	// List of components whose heightmap needs to be resolved because they are either dirty or are neighbor to a component that is dirty:
	TArray<ULandscapeComponent*> LandscapeComponentsHeightmapsToResolve;
	// List of components that need to be rendered because they are either dirty or are neighbor to a component that is dirty or share a weightmap with a component that is dirty:
	TArray<ULandscapeComponent*> LandscapeComponentsWeightmapsToRender;
	// List of components whose weightmap needs to be resolved because they are either dirty or are neighbor to a component that is dirty:
	TArray<ULandscapeComponent*> LandscapeComponentsWeightmapsToResolve;
	// List of components whose heightmap or weightmaps needs to be resolved because they are either dirty or are neighbor to a component that is dirty:
	TSet<ULandscapeComponent*> AllLandscapeComponentsToResolve;
	// List of GPU readback results for heightmaps/weightmaps that need to be resolved, associated with their owning landscape component :
	TArray<FLandscapeEditLayerComponentReadbackResult> AllLandscapeComponentReadbackResults;
};
ENUM_CLASS_FLAGS(FUpdateLayersContentContext::ERefreshFlags);

struct FEditLayersHeightmapMergeParams
{
	int32 HeightmapUpdateModes = 0;
	bool bSkipBrush = false;
};

TArray<UE::Landscape::EditLayers::FEditLayerRendererState> ALandscape::GetEditLayerRendererStates(const UE::Landscape::EditLayers::FMergeContext* InMergeContext)
{
	return GetEditLayerRendererStatesEnableOverride(InMergeContext, {});
}

TArray<UE::Landscape::EditLayers::FEditLayerRendererState> ALandscape::GetEditLayerRendererStatesEnableOverride(const UE::Landscape::EditLayers::FMergeContext* InMergeContext, const TBitArray<>& InLayerEnableOverride)
{
	using namespace UE::Landscape::EditLayers;

	TArray<FEditLayerRendererState> RendererStates;
	int32 LayerIdx = 0;
	bool bUseEnableOverride = InLayerEnableOverride.Num() > 0;
	check(!bUseEnableOverride || InLayerEnableOverride.Num() == LandscapeEditLayers.Num());
	for (FLandscapeLayer& Layer : LandscapeEditLayers)
	{
		TArray<FEditLayerRendererState> LayerRendererStates = Layer.GetEditLayerRendererStates(InMergeContext);

		if (bUseEnableOverride)
		{
			ELandscapeToolTargetTypeFlags MergeTypeMask = InMergeContext->IsHeightmapMerge() ? ELandscapeToolTargetTypeFlags::Heightmap : (ELandscapeToolTargetTypeFlags::Weightmap | ELandscapeToolTargetTypeFlags::Visibility);
			bool bEnabled = InLayerEnableOverride[LayerIdx];
			Algo::ForEach(LayerRendererStates, [MergeTypeMask, bEnabled](FEditLayerRendererState& State)
				{
					// Don't try to enable/disable a state if it's not supported :
					if (EnumHasAnyFlags(State.GetSupportedTargetTypeState().GetTargetTypeMask(), MergeTypeMask))
					{
						if (bEnabled)
						{
							State.EnableTargetTypeMask(MergeTypeMask);
						}
						else
						{
							State.DisableTargetTypeMask(MergeTypeMask);
						}
					}
				});
		}

		RendererStates.Append(LayerRendererStates);

		++LayerIdx;
	}
	return RendererStates;
}

TArray<UE::Landscape::EditLayers::FEditLayerRendererState> FLandscapeLayer::GetEditLayerRendererStates(const UE::Landscape::EditLayers::FMergeContext* InMergeContext)
{
	using namespace UE::Landscape::EditLayers;

	// Gather all elements that can render some edit layers data, ordered: 
	TArray<FEditLayerRendererState> RendererStates;
	RendererStates.Reserve(1 + Brushes.Num());

	check(EditLayer != nullptr);
	// The edit layer itself might be a renderer:
	if (TScriptInterface<ILandscapeEditLayerRenderer> AsRenderer(EditLayer); AsRenderer != nullptr)
	{
		RendererStates.Emplace(InMergeContext, AsRenderer);
	}

	// The layer can also be a renderer provider
	RendererStates.Append(EditLayer->GetEditLayerRendererStates(InMergeContext));

	for (FLandscapeLayerBrush& Brush : Brushes)
	{
		RendererStates.Append(Brush.GetEditLayerRendererStates(InMergeContext));
	}

	// Renderer states generated from a layer inherit the layer's state so start by computing the layer's target type mask: 
	ELandscapeToolTargetTypeFlags LayerTargetTypeMask = EditLayer->GetEnabledTargetTypeMask();
	// Then disable all types that are not in the layer's mask :
	for (FEditLayerRendererState& LayerRendererState : RendererStates)
{
		LayerRendererState.DisableTargetTypeMask(~LayerTargetTypeMask);
		}

	return RendererStates;
}

TArray<UE::Landscape::EditLayers::FEditLayerRendererState> FLandscapeLayerBrush::GetEditLayerRendererStates(const UE::Landscape::EditLayers::FMergeContext* InMergeContext)
{
	using namespace UE::Landscape::EditLayers;

	// Gather all elements that can render some edit layers data, ordered: 
	if (BlueprintBrush == nullptr)
	{
		return {};
	}

	return BlueprintBrush->GetEditLayerRendererStates(InMergeContext);
}

void ULandscapeEditLayerPersistent::GetRendererStateInfo(const UE::Landscape::EditLayers::FMergeContext* InMergeContext,
	UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutSupportedTargetTypeState, UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutEnabledTargetTypeState,
	TArray<UE::Landscape::EditLayers::FTargetLayerGroup>& OutTargetLayerGroups) const
{
	using namespace UE::Landscape::EditLayers;

	// A layer can support all target types. 
	// Add an entry for each weightmap and consider them supported because there's nothing that prevents a given edit layer to write on a given weightmap layer :
	OutSupportedTargetTypeState = FEditLayerTargetTypeState(InMergeContext, ELandscapeToolTargetTypeFlags::All, InMergeContext->GetValidTargetLayerBitIndices());

	// Compute the default state of each target type : 
	OutEnabledTargetTypeState = FEditLayerTargetTypeState(InMergeContext, GetEnabledTargetTypeMask(), OutSupportedTargetTypeState.GetActiveWeightmapBitIndices());

	// Build the target layer groups for this renderer. Since we use the generic blend, we can use BuildGenericBlendTargetLayerGroups
	OutTargetLayerGroups = InMergeContext->BuildGenericBlendTargetLayerGroups(OutSupportedTargetTypeState.GetActiveWeightmapBitIndices());
}

UE::Landscape::EditLayers::ERenderFlags ULandscapeEditLayerPersistent::GetRenderFlags(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const
{
	using namespace UE::Landscape::EditLayers;
	return ERenderFlags::RenderMode_Recorded | ERenderFlags::BlendMode_SeparateBlend; // Supports the command recorder and has a separate BlendLayer function 
}

TArray<UE::Landscape::EditLayers::FEditLayerRenderItem> ULandscapeEditLayerPersistent::GetRenderItems(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const
{
	using namespace UE::Landscape::EditLayers;

	TArray<FEditLayerRenderItem> RenderItems;
	ULandscapeInfo* Info = InMergeContext->GetLandscapeInfo();
	RenderItems.Reserve(Info->XYtoComponentMap.Num() + 1);

	// Each component has an input local area of (-1, -1) -> (0, 0) in order to make sure that components on the left, top-left and top get included in the batch and rendered.
	//  This is to resolve potential non-determinism in case source data differs on the common edge between neighboring components : in such a case, components on the left and top 
	//  have authority (other edges are ignored), so we sort and render components in this order in order to enforce that behavior. 
	//  See FMergeRenderContext::RenderComponentIds for details
	const FIntRect InputArea(FIntPoint(-1, -1), FIntPoint::ZeroValue);

	// Heightmaps : We only need a single render item for heightmaps, because heightmaps are always present :
	if (InMergeContext->IsHeightmapMerge())
	{
		// The layer needs the component itself. Plus its 3 neighbors (see comment about InputArea definition), to render in a deterministic way
		FInputWorldArea InputWorldArea(FInputWorldArea::CreateLocalComponent(/*InLocalArea = */InputArea));
		// The layer only writes into the component itself (i.e. it renders to the area that it's currently being asked to render to):
		FOutputWorldArea OutputWorldArea(FOutputWorldArea::CreateLocalComponent());
		RenderItems.Emplace(FEditLayerTargetTypeState(InMergeContext, ELandscapeToolTargetTypeFlags::Heightmap), InputWorldArea, OutputWorldArea, /*bModifyExistingWeightmapsOnly = */false);
	}
	else
	// Weightmaps : add one render item per component in order to be able to indicate exactly which weightmap is needed for each of them. This avoids pre-allocating weightmaps on the merged result where we know we won't write a weightmap : 
	{
		TArray<ULandscapeComponent*> AllLandscapeComponents;
		Info->XYtoComponentMap.GenerateValueArray(AllLandscapeComponents);

		for (ULandscapeComponent* Component : AllLandscapeComponents)
		{
			const FIntPoint ComponentKey = Component->GetComponentKey();

			if (FLandscapeLayerComponentData* LayerData = Component->GetLayerData(GetGuid()))
			{
				TArray<FName, TInlineAllocator<16>> ComponentWeightmaps;
				FEditLayerTargetTypeState OutputTargetTypeState(InMergeContext);

				// Iterate through all allocated weightmaps in order to find which one we will really write to : 
				for (const FWeightmapLayerAllocationInfo& LayerAllocationInfo : LayerData->WeightmapData.LayerAllocations)
				{
					if (LayerAllocationInfo.IsAllocated() && InMergeContext->IsValidTargetLayerName(LayerAllocationInfo.GetLayerName()))
					{
						if (LayerAllocationInfo.LayerInfo == ALandscapeProxy::VisibilityLayer)
						{
							OutputTargetTypeState.AddTargetTypeMask(ELandscapeToolTargetTypeFlags::Visibility);
						}
						else
						{
							OutputTargetTypeState.AddTargetTypeMask(ELandscapeToolTargetTypeFlags::Weightmap);
						}
						OutputTargetTypeState.AddWeightmap(LayerAllocationInfo.GetLayerName());
					}
				}

				if (OutputTargetTypeState.GetTargetTypeMask() != ELandscapeToolTargetTypeFlags::None)
				{
					// The layer needs the component itself. Plus its 3 neighbors (see comment about InputArea definition), to render in a deterministic way
					FInputWorldArea InputWorldArea(FInputWorldArea::CreateSpecificComponent(ComponentKey, /*InLocalArea = */InputArea));
					// The layer only writes into the component itself (i.e. it renders to the area that it's currently being asked to render to):
					FOutputWorldArea OutputWorldArea(FOutputWorldArea::CreateSpecificComponent(ComponentKey));
					RenderItems.Emplace(OutputTargetTypeState, InputWorldArea, OutputWorldArea, /*bModifyExistingWeightmapsOnly = */false);
				}
			}
		}
	}
	return RenderItems;
}

bool ULandscapeEditLayerPersistent::RenderLayer(UE::Landscape::EditLayers::FRenderParams& RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder)
{
	using namespace UE::Landscape;
	using namespace UE::Landscape::Private;
	using namespace UE::Landscape::EditLayers;

	checkf(RDGBuilderRecorder.IsRecording(), TEXT("ERenderFlags::RenderMode_Recorded means the command recorder should be recording at this point"));
	check(!RenderParams.SortedComponentMergeRenderInfos.IsEmpty());

	ULandscapeInfo* Info = RenderParams.MergeRenderContext->GetLandscapeInfo();
	check(Info != nullptr);
	ALandscape* Landscape = RenderParams.MergeRenderContext->GetLandscape();
	FSceneInterface* SceneInterface = Landscape->GetWorld()->Scene;

	const int32 NumTargetLayersInGroup = RenderParams.TargetLayerGroupLayerNames.Num();

	// The first step is to copy all of the necessary components' textures to WriteRT and remove the duplicate borders when doing so (subsection by subsection). This is done with a 
	//  "copy from multiple sources" shader instead of several texture copies, in order to reduce the amount of copy texture commands, which can have a big impact on render-thread performance
	//  for large landscapes. Let's first build a list of quads to render and we'll render them all as efficiently as possible thereafter : 

	// For each subsection, we'll add a quad to render, using the following struct :
	struct FCopyQuadParams
	{
		// Texture to read from :
		FTextureResource* SourceTextureResource = nullptr;
		// Texture region to read from :
		FIntRect SourceRect;
		// Texture region to write to :
		FIntRect DestinationRect;
		// In case the destination is a texture array (weightmaps), this is the slice index to write to :
		int32 DestinationArrayIndex = 0;
		// Weightmaps are packed in the source texture. This specifies which channel to read from in the source texture for this quad :
		uint8 SourceChannelIndex = 0;
		// Linear index of this component within this batch. Allows to resolve conflicts in case 2 components render to the same pixel on a border, for example :
		uint32 ComponentId = INDEX_NONE;
	};
	TArray<FCopyQuadParams> CopyQuadsParams;
	CopyQuadsParams.Reserve(RenderParams.SortedComponentMergeRenderInfos.Num() * NumTargetLayersInGroup * Landscape->NumSubsections * Landscape->NumSubsections);

	// This is kinda hacky, but we since we reuse AddRasterizeToRectsPass and it passes the source texture's sample coordinates from VS to PS via the UVs, dividing the source rect by a single TextureSize, 
	//  we correct each quad's source rect by a common scale factor so that it samples at the right location in each source texture: 
	const FVector2D CommonSourceTextureSize(RenderParams.MergeRenderContext->GetMaxNeededResolution());
	const int32 SubsectionSizeVerts = (Landscape->SubsectionSizeQuads + 1);

	// Build the full list of quads to copy :
	for (const FComponentMergeRenderInfo& ComponentMergeRenderInfo : RenderParams.SortedComponentMergeRenderInfos)
	{
		const uint32 ComponentId = RenderParams.MergeRenderContext->GetCurrentRenderBatch()->ComputeComponentLinearIndex(ComponentMergeRenderInfo.Component->GetComponentKey());
		for (int32 TargetLayerIndexInGroup = 0; TargetLayerIndexInGroup < NumTargetLayersInGroup; ++TargetLayerIndexInGroup)
		{
			const FName TargetLayerName = RenderParams.TargetLayerGroupLayerNames[TargetLayerIndexInGroup];
			UTexture2D* SourceTexture = nullptr;
			FVector2D SourceTextureBias(ForceInit);
			uint8 SourceChannelIndex = 0;

			if (RenderParams.MergeRenderContext->IsHeightmapMerge())
			{
				SourceTexture = ComponentMergeRenderInfo.Component->GetHeightmap(GetGuid());
				SourceTextureBias = FVector2D(ComponentMergeRenderInfo.Component->HeightmapScaleBias.Z, ComponentMergeRenderInfo.Component->HeightmapScaleBias.W);
			}
			else
			{
				const TArray<UTexture2D*>& WeightmapTextures = ComponentMergeRenderInfo.Component->GetWeightmapTextures(GetGuid());
				const TArray<FWeightmapLayerAllocationInfo>& AllocInfos = ComponentMergeRenderInfo.Component->GetWeightmapLayerAllocations(GetGuid());
				const FWeightmapLayerAllocationInfo* AllocInfo = AllocInfos.FindByPredicate([TargetLayerName](const FWeightmapLayerAllocationInfo& InAllocInfo) { return InAllocInfo.IsAllocated() && (InAllocInfo.GetLayerName() == TargetLayerName); });
				if (AllocInfo != nullptr)
				{
					SourceTexture = WeightmapTextures[AllocInfo->WeightmapTextureIndex];
					check(SourceTexture != nullptr);
					// Note : don't use WeightmapScaleBias here, it has a different meaning than HeightmapScaleBias (very conveniently!) : this is compensated by the FloorToInt32 later on, 
					//  but still, let's set this to zero here and use the fact that there's no texture sharing on weightmaps : 
					SourceTextureBias = FVector2D::ZeroVector;
					// Copy from the appropriate source channel : 
					SourceChannelIndex = AllocInfo->WeightmapTextureChannel;
				}
			}

			if (SourceTexture != nullptr)
			{
				FTextureResource* SourceTextureResource = SourceTexture->GetResource();
				checkf(!SourceTexture->IsCompiling(), TEXT("All mips must have been loaded prior to using this function (%s)"), *SourceTexture->GetName());
				// We get the overall texture size via the resource instead of direct GetSizeX/Y calls because the latter is unreliable while the texture is being built.
				const FVector2D SourceTextureSize(SourceTextureResource->GetSizeX(), SourceTextureResource->GetSizeY());
				const FIntPoint SourceTextureOffset(FMath::FloorToInt32(SourceTextureBias.X * SourceTextureSize.X), FMath::FloorToInt32(SourceTextureBias.Y * SourceTextureSize.Y));

				auto SourceTexturePixelCoordinatesToQuadCoords = [&SourceTextureSize, &CommonSourceTextureSize](const FIntPoint& InSourceCoords)
					{
						FVector2D Result = FVector2D(InSourceCoords) / SourceTextureSize * CommonSourceTextureSize;
						return FIntPoint(FMath::FloorToInt32(Result.X), FMath::FloorToInt32(Result.Y));
					};

				// Fill that render target subsection by subsection, in order to bypass the redundant columns/lines on the subsection edges:
				for (int32 SubsectionY = 0; SubsectionY < Landscape->NumSubsections; ++SubsectionY)
				{
					for (int32 SubsectionX = 0; SubsectionX < Landscape->NumSubsections; ++SubsectionX)
					{
						const FIntPoint SubsectionKey(SubsectionX, SubsectionY);
						const FIntPoint SourcePosition = SourceTextureOffset + SubsectionKey * SubsectionSizeVerts;
						const FIntPoint DestinationPosition = ComponentMergeRenderInfo.ComponentRegionInRenderArea.Min + SubsectionKey * ComponentMergeRenderInfo.Component->SubsectionSizeQuads;

						FCopyQuadParams& QuadParams = CopyQuadsParams.Emplace_GetRef();
						QuadParams.SourceTextureResource = SourceTextureResource;
						QuadParams.SourceRect = FIntRect(SourceTexturePixelCoordinatesToQuadCoords(SourcePosition), SourceTexturePixelCoordinatesToQuadCoords(SourcePosition + FIntPoint(SubsectionSizeVerts, SubsectionSizeVerts)));
						QuadParams.DestinationRect = FIntRect(DestinationPosition, DestinationPosition + FIntPoint(SubsectionSizeVerts, SubsectionSizeVerts));
						QuadParams.DestinationArrayIndex = TargetLayerIndexInGroup;
						QuadParams.SourceChannelIndex = SourceChannelIndex;
						QuadParams.ComponentId = ComponentId;
					}
				}
			}
		}
	}

	if (CopyQuadsParams.IsEmpty())
	{
		// No need to do anything if there's nothing to be rendered (e.g. no weightmap on the rendered area), this layer will just be ineffective on this batch
		//  and we don't need to cycle the blend render targets, that will save some processing:
		return false;
	}

	RenderParams.MergeRenderContext->CycleBlendRenderTargets(RDGBuilderRecorder);
	ULandscapeScratchRenderTarget* WriteRT = RenderParams.MergeRenderContext->GetBlendRenderTargetWrite();

	WriteRT->Clear(RDGBuilderRecorder);
	// We will write to the RT using a (bunch of) PS : 
	check(WriteRT->GetCurrentState() == ERHIAccess::RTV);

	ULandscapeScratchRenderTarget* ComponentIdRT = RenderParams.MergeRenderContext->GetComponentIdRenderTarget();
	check(ComponentIdRT != nullptr);
	ComponentIdRT->TransitionTo(ERHIAccess::SRVMask, RDGBuilderRecorder);

	// In this render step, we'll simply render the edit layer's component quads

	// Sort the list of quads by output texture first, then by source texture, because we'll coalesce several texture copies into the same pass (one output, multiple inputs)
	CopyQuadsParams.Sort([](const FCopyQuadParams& InLHS, const FCopyQuadParams& InRHS)
		{
			if (InLHS.DestinationArrayIndex == InRHS.DestinationArrayIndex)
			{
				if (InLHS.SourceTextureResource == InRHS.SourceTextureResource)
				{
					return (InLHS.SourceRect.Min.Y == InRHS.SourceRect.Min.Y)
						? (InLHS.SourceRect.Min.X < InRHS.SourceRect.Min.X)
						: (InLHS.SourceRect.Min.Y < InRHS.SourceRect.Min.Y);
				}
				return (InLHS.SourceTextureResource < InRHS.SourceTextureResource);
			}

			return (InLHS.DestinationArrayIndex < InRHS.DestinationArrayIndex);
		});

	// Now process this list of quads and prepare as many passes as necessary for performing all the copies :
	auto RDGCommand =
		[CopyQuadsParams = MoveTemp(CopyQuadsParams)
		, OutputResource = WriteRT->GetRenderTarget()->GetResource()
		, ComponentIdResource = ComponentIdRT->GetRenderTarget()->GetResource()
		, OutputResourceName = WriteRT->GetDebugName()
		, SceneInterface
		, CommonSourceTextureSize
		, bIsWeightmapMerge = !RenderParams.MergeRenderContext->IsHeightmapMerge()] (FRDGBuilder& GraphBuilder)
		{
			RDG_EVENT_SCOPE(GraphBuilder, "CopyEditLayer -> %s", *OutputResourceName);

			FRDGTextureSRVRef BlackDummySRVRef = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(GSystemTextures.GetBlackDummy(GraphBuilder)));

			TUniquePtr<FCopyQuadsMultiSourceBase> CopyQuadsMultiSource;
			CopyQuadsMultiSource = MakeUnique<FCopyQuadsMultiSourceDefault>();

			// TODO [jonathan.bard] this is just to avoid a RHI validation error for unoptimized shaders... once validation is made to not issue those errors, we can remove this
			// Create a SceneView to please the shader bindings, but it's unused in practice 
			FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(nullptr, SceneInterface, FEngineShowFlags(ESFIM_Game)).SetTime(FGameTime::GetTimeSinceAppStart()));
			FSceneViewInitOptions ViewInitOptions;
			ViewInitOptions.ViewFamily = &ViewFamily;
			ViewInitOptions.SetViewRectangle(FIntRect(0, 0, 1, 1)); // Use a dummy rect to avoid a check(slow)
			GetRendererModule().CreateAndInitSingleView(GraphBuilder.RHICmdList, &ViewFamily, &ViewInitOptions);
			const FSceneView* View = ViewFamily.Views[0];

			// The following variables allow to accumulate quads for rendering in a single pass. Then, we flush them all out when the pass is full and we start again : 
			int32 CurrentDestinationArrayIndex = INDEX_NONE;
			TArray<FTextureResource*> CurrentSourceTextureResources;
			TArray<FUintVector4> CurrentQuadInfos;
			TArray<FUintVector4> CurrentSourceRects, CurrentDestinationRects;

			FRDGTextureRef OutputTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(OutputResource->GetTextureRHI(), TEXT("OutputTexture")));
			FRDGTextureRef ComponentIdTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(ComponentIdResource->GetTextureRHI(), TEXT("ComponentIdTexture")));
			FRDGTextureSRVRef ComponentIdTextureSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(ComponentIdTexture));

			// When we've reached the limit of what we can render in one pass, flush all and add a render pass : 
			auto FlushCopyQuads = [
				&GraphBuilder, 
				View, 
				OutputTexture, 
				ComponentIdTextureSRV,
				&CurrentDestinationArrayIndex, 
				&CurrentSourceTextureResources, 
				&CurrentSourceRects, 
				&CurrentDestinationRects,
				&CurrentQuadInfos, 
				&CommonSourceTextureSize, 
				BlackDummySRVRef, 
				CopyQuadsMultiSource = CopyQuadsMultiSource.Get(), 
				bIsWeightmapMerge]()
				{
					if (CurrentSourceTextureResources.IsEmpty())
					{
						// Nothing to flush :
						check(CurrentSourceRects.IsEmpty() && CurrentDestinationRects.IsEmpty() && CurrentQuadInfos.IsEmpty());
						return;
					}

					check(!CurrentSourceRects.IsEmpty() && (CurrentSourceRects.Num() == CurrentDestinationRects.Num()) && (CurrentSourceRects.Num() == CurrentQuadInfos.Num()));

					FRDGBufferRef RectBuffer = CreateUploadBuffer(GraphBuilder, TEXT("DestinationRects"), MakeConstArrayView(CurrentDestinationRects));
					FRDGBufferSRVRef RectBufferSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(RectBuffer, PF_R32G32B32A32_UINT));

					FRDGBufferRef RectUVBuffer = CreateUploadBuffer(GraphBuilder, TEXT("RectUVs"), MakeConstArrayView(CurrentSourceRects));
					FRDGBufferSRVRef RectUVBufferSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(RectUVBuffer, PF_R32G32B32A32_UINT));

					FRDGBufferRef QuadInfosBuffer = CreateUploadBuffer(GraphBuilder, TEXT("QuadInfos"), MakeConstArrayView(CurrentQuadInfos));
					FRDGBufferSRVRef QuadInfosBufferSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(QuadInfosBuffer, PF_R32G32B32A32_UINT)); 

					// We may target an array slice directly : 
					int32 ArrayIndex = -1;
					if (OutputTexture->Desc.IsTextureArray())
					{
						check(CurrentDestinationArrayIndex < OutputTexture->Desc.ArraySize);
						ArrayIndex = CurrentDestinationArrayIndex;
					}

					TArray<FRDGTextureRef> SourceTextures;
					check(CurrentSourceTextureResources.Num() <= CopyQuadsMultiSource->GetNumMultiSources());
					for (int32 TextureIndex = 0; TextureIndex < CopyQuadsMultiSource->GetNumMultiSources(); ++TextureIndex)
					{
						FRDGTextureRef SourceTexture = BlackDummySRVRef->GetParent();
						if (CurrentSourceTextureResources.IsValidIndex(TextureIndex))
						{
							SourceTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(CurrentSourceTextureResources[TextureIndex]->GetTexture2DRHI(), TEXT("SourceTexture")));
						}
						SourceTextures.Add(SourceTexture);
					}

					CopyQuadsMultiSource->CopyQuads(
						GraphBuilder,
						OutputTexture,
						MakeConstArrayView(SourceTextures),
						RectBufferSRV,
						QuadInfosBufferSRV,
						RectUVBufferSRV,
						ComponentIdTextureSRV,
						View,
						ArrayIndex,
						CurrentDestinationRects.Num(),
						FIntPoint(FMath::FloorToInt32(CommonSourceTextureSize.X), FMath::FloorToInt32(CommonSourceTextureSize.Y)),
						bIsWeightmapMerge);

					// Reset all for the next pass that comes along : 
					CurrentDestinationArrayIndex = INDEX_NONE;
					CurrentSourceTextureResources.Empty();
					CurrentSourceRects.Empty();
					CurrentDestinationRects.Empty();
					CurrentQuadInfos.Empty();
				};

			for (const FCopyQuadParams& InSingleQuadParams : CopyQuadsParams)
			{
				check(InSingleQuadParams.DestinationArrayIndex != INDEX_NONE);
				// If the output texture/slice has changed since the last iteration, it's time to flush and start a new pass :
				if (CurrentDestinationArrayIndex != InSingleQuadParams.DestinationArrayIndex)
				{
					if (CurrentDestinationArrayIndex != INDEX_NONE)
					{
						FlushCopyQuads();
					}
					CurrentDestinationArrayIndex = InSingleQuadParams.DestinationArrayIndex;
				}

				if (CurrentSourceTextureResources.IsEmpty())
				{
					CurrentSourceTextureResources.Add(InSingleQuadParams.SourceTextureResource);
				}
				else if (InSingleQuadParams.SourceTextureResource != CurrentSourceTextureResources.Last())
				{
					// If we've reached the amount of textures we can render in a single pass, we flush the pass and initiate a new one : 
					if (CurrentSourceTextureResources.Num() == CopyQuadsMultiSource->GetNumMultiSources())
					{
						FlushCopyQuads();
						check(CurrentSourceTextureResources.IsEmpty());

						CurrentDestinationArrayIndex = InSingleQuadParams.DestinationArrayIndex;
					}
					CurrentSourceTextureResources.Add(InSingleQuadParams.SourceTextureResource);
				}

				// If we are using the same texture as the last one, we can render it in the same pass, just append our quad : 
				if (InSingleQuadParams.SourceTextureResource == CurrentSourceTextureResources.Last())
				{
					const int32 SourceTextureIndex = CurrentSourceTextureResources.Num() - 1;
					CurrentQuadInfos.Add(FUintVector4(static_cast<uint32>(SourceTextureIndex), InSingleQuadParams.SourceChannelIndex, InSingleQuadParams.ComponentId, 0));
					CurrentSourceRects.Add(FUintVector4(InSingleQuadParams.SourceRect.Min.X, InSingleQuadParams.SourceRect.Min.Y, InSingleQuadParams.SourceRect.Max.X, InSingleQuadParams.SourceRect.Max.Y));
					CurrentDestinationRects.Add(FUintVector4(InSingleQuadParams.DestinationRect.Min.X, InSingleQuadParams.DestinationRect.Min.Y, InSingleQuadParams.DestinationRect.Max.X, InSingleQuadParams.DestinationRect.Max.Y));
				}
			}

			// Flush the remaining quads if any : 
			FlushCopyQuads();
		};

	// We need to specify the final state of the external textures to prevent the graph builder from transitioning them to SRVMask (even those that end up as SRVMask at the end of this command,
	//  because they will likely be part of another RDGCommand down the line so we need to maintain an accurate picture of every external texture ever involved in the recorded command so that 
	//  we can set a proper access when the recorder is flushed (and the FRDGBuilder, executed) :
	RDGBuilderRecorder.EnqueueRDGCommand(RDGCommand, 
	{ 
		{ WriteRT->GetRenderTarget()->GetResource(), ERHIAccess::RTV }, 
		{ ComponentIdRT->GetRenderTarget()->GetResource(), ERHIAccess::SRVMask } 
	});

	// We've rendered at least a quad : 
	return true;
}

void ULandscapeEditLayerPersistent::BlendLayer(UE::Landscape::EditLayers::FRenderParams& RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder)
{
	using namespace UE::Landscape::EditLayers;
	using namespace UE::Landscape::Private;

	ULandscapeInfo* Info = RenderParams.MergeRenderContext->GetLandscapeInfo();
	check(Info != nullptr);

	const int32 NumTargetLayersInGroup = RenderParams.TargetLayerGroupLayerNames.Num();

	// Prepare the generic blend params based on the layer's data : 
	FBlendParams BlendParams;
	if (RenderParams.MergeRenderContext->IsHeightmapMerge())
	{
		BlendParams.HeightmapBlendParams.BlendMode = LandscapeBlendModeToHeightmapBlendMode(GetBlendMode());
		BlendParams.HeightmapBlendParams.Alpha = GetAlphaForTargetType(ELandscapeToolTargetType::Heightmap);
	}
	else
	{
		check(NumTargetLayersInGroup == RenderParams.TargetLayerGroupLayerInfos.Num());
		BlendParams.WeightmapBlendParams.Reserve(NumTargetLayersInGroup);
		for (int32 i = 0; i < NumTargetLayersInGroup; ++i)
		{
			const FName& TargetLayerName = RenderParams.TargetLayerGroupLayerNames[i];
			const ULandscapeLayerInfoObject* LayerInfoObj = RenderParams.TargetLayerGroupLayerInfos[i];
			check(LayerInfoObj != nullptr);

			// only blend the layers involved in this step (the others are using EWeightmapBlendMode::Passthrough): 
			if (RenderParams.TargetLayerGroupLayerNames.Contains(TargetLayerName))
			{
				FWeightmapBlendParams& TargetLayerBlendParams = BlendParams.WeightmapBlendParams.Emplace(TargetLayerName, EWeightmapBlendMode::Additive);

				if (const bool* bSubstractiveInLayer = GetWeightmapLayerAllocationBlend().Find(LayerInfoObj); (bSubstractiveInLayer != nullptr) && *bSubstractiveInLayer)
				{
					TargetLayerBlendParams.BlendMode = EWeightmapBlendMode::Subtractive;
				}

				if (TargetLayerName != UMaterialExpressionLandscapeVisibilityMask::ParameterName)
				{
					TargetLayerBlendParams.Alpha = GetAlphaForTargetType(ELandscapeToolTargetType::Weightmap);
				}
			}
		}
	}

	// Then perform the generic blend : 
	RenderParams.MergeRenderContext->GenericBlendLayer(BlendParams, RenderParams, RDGBuilderRecorder);
}

FString ULandscapeEditLayerPersistent::GetEditLayerRendererDebugName() const
{
	return GetName().ToString();
}

namespace UE::Landscape::EditLayers::Private
{
	using namespace UE::Landscape::EditLayers;

	/** Struct that holds all the per-component information needed when preparing the batched merge context */
	struct FComponentToRenderInfo
	{
		FComponentToRenderInfo() = default;

		FComponentToRenderInfo(ULandscapeComponent* InComponent, int32 InComponentIndex, const int32 InNumAllComponents, const int32 NumTargetLayersToRender)
			: Component(InComponent)
			, ComponentIndex(InComponentIndex)
			, DependentComponentBitIndices(false, InNumAllComponents)
			, CombinedSectionRect(InComponent->GetSectionBase(), InComponent->GetSectionBase() + FIntPoint(InComponent->ComponentSizeQuads + 1, InComponent->ComponentSizeQuads + 1))
			, ComponentKey(InComponent->GetComponentKey())
			, MinDependentComponentKey(ComponentKey)
			, MaxDependentComponentKey(ComponentKey)
			, LocalBounds(InComponent->CachedLocalBox)
			, WorldBounds(InComponent->CachedLocalBox.TransformBy(InComponent->GetComponentTransform()))
			, TargetLayerBitIndices(false, NumTargetLayersToRender)
		{}

		void Finalize(const FIntRect& InDependentComponentInclusiveBounds, const FIntPoint& InComponentSizeQuads)
		{
			MinDependentComponentKey = InDependentComponentInclusiveBounds.Min;
			MaxDependentComponentKey = InDependentComponentInclusiveBounds.Max;
			CombinedSectionRect.Min = InDependentComponentInclusiveBounds.Min * InComponentSizeQuads;
			CombinedSectionRect.Max = (InDependentComponentInclusiveBounds.Max + 1) * InComponentSizeQuads + 1;
		}

		/** Component to render */
		ULandscapeComponent* Component = nullptr;

		/** Index of the component to render in AllComponentsToRenderInfos */
		int32 ComponentIndex = INDEX_NONE;

		/** List of components this component depends on to render appropriately. Each dependency is represented by a bit which corresponds to the index of the dependent component in AllComponentsToRenderInfos */
		TBitArray<> DependentComponentBitIndices;

		/** Section rect of all the components that this component depends on to render */
		FIntRect CombinedSectionRect;

		/** Coordinate of the component to render */
		FIntPoint ComponentKey = FIntPoint(ForceInit);

		/** Minimum coordinate of the components that this component depends on to render */
		FIntPoint MinDependentComponentKey = FIntPoint(MAX_int32, MAX_int32);

		/** Maximum coordinate of the components that this component depends on to render */
		FIntPoint MaxDependentComponentKey = FIntPoint(MIN_int32, MIN_int32);

		/** Bounding volume of this component in local space */
		FBox LocalBounds = FBox(ForceInit);

		/** Bounding volume of this component in world space */
		FBox WorldBounds = FBox(ForceInit);

		/** List of target layers being written by this component. Each target layer is represented by a bit which corresponds to the index of the target layer names in AllTargetLayerNames */
		TBitArray<> TargetLayerBitIndices;
	};

	/** Struct that holds all the per-render batch information needed when preparing the batched merge context */
	struct FRenderBatchInfo
	{
		FRenderBatchInfo(int32 InNumComponentsToRender, int32 InBatchIndex)
			: BatchIndex(InBatchIndex)
			, ComponentToRenderInfoBitIndices(false, InNumComponentsToRender)
		{
		}

		FIntRect GetProjectedSectionRect(const FComponentToRenderInfo& InComponentToRenderInfo) const
		{
			checkf(MinComponentKey.X != MAX_int32, TEXT("Shouldn't be called when the batch is empty"));
			FIntRect NewCombinedSectionRect(CombinedSectionRect);
			NewCombinedSectionRect.Union(InComponentToRenderInfo.CombinedSectionRect);
			return NewCombinedSectionRect;
		}

		void AddToBatch(const FComponentToRenderInfo& InComponentToRenderInfo)
		{
			ComponentToRenderInfoBitIndices.CombineWithBitwiseOR(InComponentToRenderInfo.DependentComponentBitIndices, EBitwiseOperatorFlags::MinSize);
			// Special case when it's the first addition to the batch : 
			if (MinComponentKey.X == MAX_int32)
			{
				CombinedSectionRect = InComponentToRenderInfo.CombinedSectionRect;
			}
			else
			{
				CombinedSectionRect.Union(InComponentToRenderInfo.CombinedSectionRect);
			}
			MinComponentKey = MinComponentKey.ComponentMin(InComponentToRenderInfo.MinDependentComponentKey);
			MaxComponentKey = MaxComponentKey.ComponentMax(InComponentToRenderInfo.MaxDependentComponentKey);
		}


	public:
		/** Index of the batch within the merge operation */
		int32 BatchIndex = INDEX_NONE;

		/** Indices (in AllComponentsToRenderInfos) of the components which we need to render within this batch
		  It's a bit array (1 bit per component to render info) to vastly optimize the batching operation, which is a O(N^2):  */
		TBitArray<> ComponentToRenderInfoBitIndices;

		/** Section rect of all the components that this batch will render*/
		FIntRect CombinedSectionRect;

		/** Minimum coordinate of the components that this batch will render */
		FIntPoint MinComponentKey = FIntPoint(MAX_int32, MAX_int32);

		/** Maximum coordinate of the components that this batch will render */
		FIntPoint MaxComponentKey = FIntPoint(MIN_int32, MIN_int32);
	};

	/** Struct that holds all the per-render item information on a given renderer, needed when preparing the batched merge context */
	struct FEditLayerRendererRenderItemRenderInfo
	{
		FEditLayerRendererRenderItemRenderInfo() = delete;
		FEditLayerRendererRenderItemRenderInfo(const FEditLayerRenderItem& InRenderItem, const TBitArray<>& InOutputLayerBitIndices, int32 InNumComponentsToRender)
			: RenderItem(InRenderItem)
			, OutputLayerBitIndices(InOutputLayerBitIndices)
			, RenderedComponentBitIndices(false, InNumComponentsToRender)
		{}

		FEditLayerRenderItem RenderItem;

		// Indices of the target layers affected by this render item on this renderer : 
		TBitArray<> OutputLayerBitIndices; 

		// Indices (in AllComponentsToRenderInfos) of the components which which are involved (as inputs or outputs) with this render item 
		TBitArray<> RenderedComponentBitIndices;
	};

	/** Struct that holds all the per-renderer information needed when preparing the batched merge context */
	struct FEditLayerRendererRenderInfo
	{
		FEditLayerRendererRenderInfo() = delete;
		FEditLayerRendererRenderInfo(const FEditLayerRendererState& InRendererState, int32 InRendererIndex, const FTransform& InLandscapeTransform, double InMaxLocalHeight, int32 InNumComponentsToRender)
			: RendererState(InRendererState)
			, RendererIndex(InRendererIndex)
			, ComponentToRenderInfoBitIndices(false, InNumComponentsToRender)
		{

#if ENABLE_VISUAL_LOG
			// Pick a random color for each renderer : 
			uint32 Hash = GetTypeHashHelper(InRendererIndex);
			uint8* HashElement = reinterpret_cast<uint8*>(&Hash);
			VisualLogColor = FColor(HashElement[0], HashElement[1], HashElement[2], FMergeRenderContext::GetVisualLogAlpha());

			VisualLogOffsetLocalSpace = InRendererIndex * InLandscapeTransform.InverseTransformVector(FVector(0.0, 0.0, CVarLandscapeBatchedMergeVisualLogOffsetIncrement.GetValueOnGameThread())).Z + InMaxLocalHeight;
#endif // ENABLE_VISUAL_LOG
		}
		
		FEditLayerRendererState RendererState;
		int32 RendererIndex = INDEX_NONE;

		TArray<FEditLayerRendererRenderItemRenderInfo> RenderItemRenderInfos;

		// Indices (in AllComponentsToRenderInfos) of the components which we need to render for this renderer
		TBitArray<> ComponentToRenderInfoBitIndices;

#if ENABLE_VISUAL_LOG
		FColor VisualLogColor = FColor(ForceInit);
		double VisualLogOffsetLocalSpace = 0.0;
#endif // ENABLE_VISUAL_LOG
	};

	/** Struct that holds all the per-renderer information for a given batch, needed when preparing the batched merge context */
	struct FPerBatchEditLayerRendererRenderInfo
	{
		// Indices (in AllComponentsToRenderInfos) of the components which we need to render for this renderer in this batch
		TBitArray<> ComponentsToRenderBitIndices;

		// Components which we need to render for this renderer in this batch
		TArray<ULandscapeComponent*> ComponentsToRender;
	};

	/** Given the list of components that actually need to be rendered, divide the work into batches, such that:
	 *   - When a component is being rendered by a given batch, all components needed for rendering this given component are present in the batch
	 *   - All components end up being rendered in at least one of the batches
	 *  @return an empty list if the render cannot actually be processed 
	 */
	TArray<FRenderBatchInfo> DivideIntoBatches(const TBitArray<>& InFinalComponentsToRenderInfoBitIndices, const TArray<FComponentToRenderInfo>& InAllComponentsToRenderInfos, bool &bInOutWarnedResolution, bool &bInOutWarnedMergeDimensionsExceeded)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DivideIntoBatches);

		const int32 NumFinalComponentsToRender = InFinalComponentsToRenderInfoBitIndices.CountSetBits();

		// Compute the final batch resolution we will use for this work : 
		const int32 DesiredMaxBatchResolution = CVarLandscapeBatchedMergeMaxResolutionPerRenderBatch.GetValueOnGameThread();

		// Compute the minimal batch resolution :
		FIntPoint MinBatchResolution(MIN_int32, MIN_int32);
		// Sort the components to render by MinDependentComponentKey in order to minimize the number of batches needed. 
		TArray<TPair<int32, FIntPoint>> RemainingComponentsToRenderInfoIndices;
		RemainingComponentsToRenderInfoIndices.Reserve(NumFinalComponentsToRender);
		for (TConstSetBitIterator It(InFinalComponentsToRenderInfoBitIndices); It; ++It)
		{
			const int32 ComponentIndex = It.GetIndex();
			const FComponentToRenderInfo& ComponentToRenderInfo = InAllComponentsToRenderInfos[ComponentIndex];
			RemainingComponentsToRenderInfoIndices.Add(MakeTuple(ComponentIndex, ComponentToRenderInfo.MinDependentComponentKey));
			// The component that has the largest CombinedSectionRect defines the minimum batch size, because it requires all components in that area to render appropriately : 
			MinBatchResolution = MinBatchResolution.ComponentMax(ComponentToRenderInfo.CombinedSectionRect.Size());
		}

		if (MinBatchResolution.X > GRHIGlobals.MaxTextureDimensions || MinBatchResolution.Y > GRHIGlobals.MaxTextureDimensions)
		{
			if (!bInOutWarnedMergeDimensionsExceeded)
			{
				UE_LOGF(LogLandscape, Error, "Cannot render landscape edit layers because the current device does not support render targets of the required size (required resolution: %ix%i, "
					"maximum resolution supported by the render device: %ix%i). Please reduce landscape size, use a different render device, or make sure that all edit layer renderers participating "
					"to this landscape don't require an input area that is larger than the device's max texture dimensions.", 
					MinBatchResolution.X, MinBatchResolution.Y, (int32)GRHIGlobals.MaxTextureDimensions, (int32)GRHIGlobals.MaxTextureDimensions);
				bInOutWarnedMergeDimensionsExceeded = true;
			}
			return {};
		}

		FIntPoint MaxBatchResolution = MinBatchResolution.ComponentMax(FIntPoint(DesiredMaxBatchResolution));
		if ((MinBatchResolution.X > DesiredMaxBatchResolution) || (MinBatchResolution.Y > DesiredMaxBatchResolution))
		{
			if (!bInOutWarnedResolution)
			{
				bInOutWarnedResolution = true;
				if (CVarSilenceMergeBatchResolutionWarning.GetValueOnGameThread())
				{
					// Reduce level to Display and slightly different message
					UE_LOGF(LogLandscape, Display, "Landscape edit layers merge requires a minimum batch size of resolution %ix%i, which is higher than the current desired maximum batch resolution %ix%i. Consider adjusting the maximum batch resolution (landscape.BatchedMerge.MaxResolutionPerRenderBatch) or make sure the landscape edit layers renderers in use require smaller work area.",
						MinBatchResolution.X, MinBatchResolution.Y, DesiredMaxBatchResolution, DesiredMaxBatchResolution);
				}
				else
				{
					UE_LOGF(LogLandscape, Warning, "Landscape edit layers merge requires a minimum batch size of resolution %ix%i, which is higher than the current desired maximum batch resolution %ix%i. Consider adjusting the maximum batch resolution (landscape.BatchedMerge.MaxResolutionPerRenderBatch) or make sure the landscape edit layers renderers in use require smaller work area. (Quiet this warning by setting landscape.BatchedMerge.SilenceResolutionWarning to 1).",
						MinBatchResolution.X, MinBatchResolution.Y, DesiredMaxBatchResolution, DesiredMaxBatchResolution);
				}
			}
			MaxBatchResolution = MinBatchResolution;
		}

		// Sort the components to render by MinDependentComponentKey in order to minimize the number of batches needed. 
		//  We actually use the inverse order because we want to process the elements from RemainingComponentsToRenderInfoIndices in inverse order 
		RemainingComponentsToRenderInfoIndices.Sort([](const TPair<int32, FIntPoint>& InLHS, const TPair<int32, FIntPoint>& InRHS)
		{
			const FIntPoint& InLHSMinDependentComponentKey = InLHS.Value;
			const FIntPoint& InRHSMinDependentComponentKey = InRHS.Value;
			if (InLHSMinDependentComponentKey.Y > InRHSMinDependentComponentKey.Y)
			{
				return true;
			}
			else if (InLHSMinDependentComponentKey.Y == InRHSMinDependentComponentKey.Y)
			{
				return (InLHSMinDependentComponentKey.X > InRHSMinDependentComponentKey.X);
			}
			return false;
		});

		// Iterate on all the work items and organize them into batches as large as possible (within the maximum allowed resolution)
		TArray<FRenderBatchInfo> AllBatchInfos;
		if (!RemainingComponentsToRenderInfoIndices.IsEmpty())
		{
			TBitArray<TInlineAllocator<1>> TempBitArray(false, InAllComponentsToRenderInfos.Num());
			while (!RemainingComponentsToRenderInfoIndices.IsEmpty())
			{
				const int32 ComponentIndex = RemainingComponentsToRenderInfoIndices.Pop(EAllowShrinking::No).Key;
				const FComponentToRenderInfo& ComponentToRenderInfo = InAllComponentsToRenderInfos[ComponentIndex];
				const FIntPoint NeededResolution = ComponentToRenderInfo.CombinedSectionRect.Size();
				check((NeededResolution.X <= MaxBatchResolution.X) && (NeededResolution.Y <= MaxBatchResolution.Y));

				int32 BestBatchIndex = INDEX_NONE;
				int32 MinBatchRenderArea = MAX_int32;
				int32 MaxNumComponentsInCommonWithBatch = -1;

				// Iterate through all batches and try to find which would be able to accept it and amongst those, which would have the minimal overall resolution: 
				const int32 NumBatches = AllBatchInfos.Num();
				for (int32 BatchIndex = 0; BatchIndex < NumBatches; ++BatchIndex)
				{
					const FRenderBatchInfo& BatchInfo = AllBatchInfos[BatchIndex];
					FIntRect ProjectedBatchSectionRect = BatchInfo.GetProjectedSectionRect(ComponentToRenderInfo);

					// If after adding this component and its dependent components, the batch still fits within the max allowed resolution, it can accept it : 
					if ((ProjectedBatchSectionRect.Width() <= MaxBatchResolution.X) && (ProjectedBatchSectionRect.Height() <= MaxBatchResolution.Y))
					{
						// Favor the batch that has the most components in common with what we're trying to render : 
						TempBitArray = TBitArray<>::BitwiseAND(BatchInfo.ComponentToRenderInfoBitIndices, ComponentToRenderInfo.DependentComponentBitIndices, EBitwiseOperatorFlags::MinSize);
						const int32 NumComponentsInCommonWithBatch = TempBitArray.CountSetBits();
						// If the batch already has all the components we need, it's a perfect match, we won't ever find a better batch so just stop the search there :
						if (NumComponentsInCommonWithBatch == ComponentToRenderInfo.DependentComponentBitIndices.CountSetBits())
						{
							BestBatchIndex = BatchIndex;
							break;
						}

						const int32 ProjectedBatchRenderArea = ProjectedBatchSectionRect.Size().X * ProjectedBatchSectionRect.Size().Y;
						if (NumComponentsInCommonWithBatch > MaxNumComponentsInCommonWithBatch)
						{
							MaxNumComponentsInCommonWithBatch = NumComponentsInCommonWithBatch;
							MinBatchRenderArea = ProjectedBatchRenderArea;
							BestBatchIndex = BatchIndex;
						}
						else if (NumComponentsInCommonWithBatch == MaxNumComponentsInCommonWithBatch)
						{
							// Favor the batch with the lesser area 
							if (ProjectedBatchRenderArea < MinBatchRenderArea)
							{
								MinBatchRenderArea = ProjectedBatchRenderArea;
								BestBatchIndex = BatchIndex;
							}
						}
					}
				}

				// If we have found a batch, just add the FRenderBatchInfo to it, otherwise, add a new batch:
				FRenderBatchInfo& SelectedBatchInfo = (BestBatchIndex != INDEX_NONE) ? AllBatchInfos[BestBatchIndex]
					: AllBatchInfos.Add_GetRef(FRenderBatchInfo(InAllComponentsToRenderInfos.Num(), /*InBatchIndex = */AllBatchInfos.Num()));

				SelectedBatchInfo.AddToBatch(ComponentToRenderInfo);
			}
		}
		return AllBatchInfos;
	}

#if ENABLE_VISUAL_LOG
	struct FComponentDependenciesVisLogHelper
	{
		enum class EShowNodeInfo : uint8
		{
			None = 0, 
			Minimal,
			Detailed
		};

		FComponentDependenciesVisLogHelper(const ALandscape* InLandscape, bool bInIsHeightmapMerge, EShowNodeInfo InShowNodeInfo, const FMergeRenderContext& InRenderContext)
			: Landscape(InLandscape)
			, LandscapeTransform(InLandscape->GetTransform())
			, bIsHeightmapMerge(bInIsHeightmapMerge)
			, ShowNodeInfo(InShowNodeInfo)
			, LandscapeComponentLocalSize(InLandscape->ComponentSizeQuads)
			, RenderContext(&InRenderContext)
		{}
		
		const FVector& AddNode(const FComponentToRenderInfo& InComponentRenderInfo, const FEditLayerRendererRenderInfo& InRendererRenderInfo)
		{
			const FIntPoint ComponentKey = InComponentRenderInfo.Component->GetComponentKey();

			TPair<int32, int32> Key(InComponentRenderInfo.ComponentIndex, InRendererRenderInfo.RendererIndex);
			const FVector* Center = KeyToCenter.Find(Key);
			if (Center == nullptr)
			{
				FTransform BaseTransform = FTransform(FVector(0.0, 0.0, InRendererRenderInfo.VisualLogOffsetLocalSpace)) * LandscapeTransform;
				FBox VisualBounds(
					FVector(ComponentKey.X * LandscapeComponentLocalSize, ComponentKey.Y * LandscapeComponentLocalSize, 0.0),
					FVector((ComponentKey.X + 1) * LandscapeComponentLocalSize, (ComponentKey.Y + 1) * LandscapeComponentLocalSize, 0.0));

				FString Message;
				if (ShowNodeInfo == EShowNodeInfo::Minimal)
				{
					FIntRect DependentComponentArea(InComponentRenderInfo.MinDependentComponentKey, InComponentRenderInfo.MaxDependentComponentKey + 1);
					Message = FString::Printf(TEXT("%s"), *ComponentKey.ToString());
				}
				else if (ShowNodeInfo == EShowNodeInfo::Detailed)
				{
					Message = FString::Printf(TEXT("%s\n(%s)"), *ComponentKey.ToString(), *InRendererRenderInfo.RendererState.GetRenderer()->GetEditLayerRendererDebugName());
				}
				// On the first renderer, show additional info: 
				if (InRendererRenderInfo.RendererIndex == 0)
				{
					FIntRect DependentComponentAreaRelative(InComponentRenderInfo.MinDependentComponentKey - ComponentKey, InComponentRenderInfo.MaxDependentComponentKey - ComponentKey);
					Message += FString::Printf(TEXT("\n{%s}"), *DependentComponentAreaRelative.ToString());
					if (!bIsHeightmapMerge)
					{
						TArray<FName> TargetLayerNames = RenderContext->ConvertTargetLayerBitIndicesToNames(InComponentRenderInfo.TargetLayerBitIndices);
						Message += FString::Printf(TEXT("\n%s"), *UE::Landscape::ConvertTargetLayerNamesToString(TargetLayerNames));
					}
				}

				FMatrix Transform = BaseTransform.ToMatrixWithScale();
				UE_VLOG_OBOX(Landscape, LogLandscape, Log, VisualBounds, Transform, InRendererRenderInfo.VisualLogColor, TEXT("%s"), *Message);
				Center = &KeyToCenter.Add(Key, Transform.TransformPosition(VisualBounds.GetCenter()));
			}
			return *Center;
		}

		void AddDependency(const FComponentToRenderInfo& InSourceComponentRenderInfo, const FEditLayerRendererRenderInfo& InSourceRendererRenderInfo,
			const FComponentToRenderInfo& InDestinationComponentRenderInfo, const FEditLayerRendererRenderInfo& InDestinationRendererRenderInfo)
		{
			const FVector SourceCenter = AddNode(InSourceComponentRenderInfo, InSourceRendererRenderInfo);
			const FVector DestinationCenter = AddNode(InDestinationComponentRenderInfo, InDestinationRendererRenderInfo);
			// TODO [jonathan.bard] : UE_VLOG_ARROW_MAG(Landscape, LogLandscape, Log, SourceCenter, DestinationCenter, InSourceRendererRenderInfo.VisualLogColor, TEXT(""), 80.0f); // TODO [jonathan.bard] : use proper mag here
			UE_VLOG_ARROW(Landscape, LogLandscape, Log, SourceCenter, DestinationCenter, InSourceRendererRenderInfo.VisualLogColor, TEXT(""));
		}

	private:
		const ALandscape* Landscape = nullptr;
		FTransform LandscapeTransform;
		bool bIsHeightmapMerge = false;
		EShowNodeInfo ShowNodeInfo = EShowNodeInfo::None;
		double LandscapeComponentLocalSize = 0.0;
		const FMergeRenderContext* RenderContext = nullptr;

		TMap<TPair<int32, int32>, FVector> KeyToCenter;
	};

	// Log the shapes of the render item output input render items if requested : 
	void VisLogRenderItemInput(const ALandscape* InLandscape, const FInputWorldArea& InInputWorldArea, const FEditLayerRendererRenderInfo& InRendererRenderInfo,
		const FTransform& InLandscapeTransform, const FBox& InLandscapeLoadedBounds, const TArrayView<ULandscapeComponent*>& InComponentsToRender)
	{
		// Display the shapes 2 unreal unit (in world space) under the requested offset (so that they're located under the output items) :
		static double OutputLocalOffsetLocalSpace = -2.0 / InLandscapeTransform.GetScale3D().Z;
		const double LandscapeComponentLocalSize = InLandscape->ComponentSizeQuads;

		FTransform BaseTransform = FTransform(FVector(0.0, 0.0, InRendererRenderInfo.VisualLogOffsetLocalSpace + OutputLocalOffsetLocalSpace)) * InLandscapeTransform;
		switch (InInputWorldArea.GetType())
		{
		case FInputWorldArea::EType::Infinite:
		{
			// Infinite input area means all loaded components: 
			FBox VisualBounds = InLandscapeLoadedBounds;
			VisualBounds.Min.Z = 0.0;
			VisualBounds.Max.Z = 0.0;
			UE_VLOG_WIREOBOX(InLandscape, LogLandscape, Log, VisualBounds, BaseTransform.ToMatrixWithScale(), InRendererRenderInfo.VisualLogColor, TEXT(""));
			break;
		}
		case FInputWorldArea::EType::LocalComponent:
		{
			// Local input area means each of the landscape components : 
			for (ULandscapeComponent* Component : InComponentsToRender)
			{
				FIntRect ComponentKeys = InInputWorldArea.GetLocalComponentKeys(Component->GetComponentKey());
				// Transform from inclusive to exclusive bounds : 
				ComponentKeys.Max += FIntPoint(1, 1);
				FBox VisualBounds(
					FVector(ComponentKeys.Min.X * LandscapeComponentLocalSize, ComponentKeys.Min.Y * LandscapeComponentLocalSize, 0.0),
					FVector(ComponentKeys.Max.X * LandscapeComponentLocalSize, ComponentKeys.Max.Y * LandscapeComponentLocalSize, 0.0));
				UE_VLOG_WIREOBOX(InLandscape, LogLandscape, Log, VisualBounds, BaseTransform.ToMatrixWithScale(), InRendererRenderInfo.VisualLogColor, TEXT(""));
			}
			break;
		}
		case FInputWorldArea::EType::SpecificComponent:
		{
			FIntRect ComponentKeys = InInputWorldArea.GetSpecificComponentKeys();
			// Transform from inclusive to exclusive bounds : 
			ComponentKeys.Max += FIntPoint(1, 1);
			FBox VisualBounds(
				FVector(ComponentKeys.Min.X * LandscapeComponentLocalSize, ComponentKeys.Min.Y * LandscapeComponentLocalSize, 0.0),
				FVector(ComponentKeys.Max.X * LandscapeComponentLocalSize, ComponentKeys.Max.Y * LandscapeComponentLocalSize, 0.0));
			UE_VLOG_WIREOBOX(InLandscape, LogLandscape, Log, VisualBounds, BaseTransform.ToMatrixWithScale(), InRendererRenderInfo.VisualLogColor, TEXT(""));
			break;
		}
		case FInputWorldArea::EType::OOBox:
		{
			FOOBox2D OOBox = InInputWorldArea.GetOOBox();
			FBox VisualBounds(-FVector(OOBox.Extents, 0.0), FVector(OOBox.Extents, 0.0));
			UE_VLOG_WIREOBOX(InLandscape, LogLandscape, Log, VisualBounds, OOBox.Transform.ToMatrixWithScale(), InRendererRenderInfo.VisualLogColor, TEXT(""));
			break;
		}
		default:
			check(false);
		}
	}

	// Log the shapes of the render item output input render items if requested : 
	void VisLogRenderItemOutput(const ALandscape* InLandscape, bool bInIsHeightmapMerge, bool bInAffectsOutputLayerBitIndices, const TArrayView<const FName>& InRenderItemTargetLayerNames,
		const FOutputWorldArea& InOutputWorldArea, const FEditLayerRendererRenderInfo& InRendererRenderInfo, const FTransform& InLandscapeTransform, const FBox& InLandsapeLoadedBounds, const TArrayView<ULandscapeComponent*>& InComponentsToRender)
	{
		// Display the shapes 1 unreal unit (in world space) under the requested offset :
		static double OutputLocalOffsetLocalSpace = -1.0 / InLandscapeTransform.GetScale3D().Z;
		const double LandscapeComponentLocalSize = InLandscape->ComponentSizeQuads;
		FString LogMessage = InRendererRenderInfo.RendererState.GetRenderer()->GetEditLayerRendererDebugName();
		if (!bInIsHeightmapMerge)
		{
			LogMessage += FString::Printf(TEXT("\n%s%s%s"), *UE::Landscape::ConvertTargetLayerNamesToString(InRenderItemTargetLayerNames), !bInAffectsOutputLayerBitIndices ? TEXT("(") : TEXT(""), !bInAffectsOutputLayerBitIndices ? TEXT(")") : TEXT(""));
		}

		FTransform BaseTransform = FTransform(FVector(0.0, 0.0, InRendererRenderInfo.VisualLogOffsetLocalSpace + OutputLocalOffsetLocalSpace)) * InLandscapeTransform;
		switch (InOutputWorldArea.GetType())
		{
		case FOutputWorldArea::EType::LocalComponent:
		{
			// Local input area means each of the landscape components : 
			for (ULandscapeComponent* Component : InComponentsToRender)
			{
				FIntPoint ComponentKey = Component->GetComponentKey();
				FBox VisualBounds(
					FVector(ComponentKey.X * LandscapeComponentLocalSize, ComponentKey.Y * LandscapeComponentLocalSize, 0.0),
					FVector((ComponentKey.X + 1) * LandscapeComponentLocalSize, (ComponentKey.Y + 1) * LandscapeComponentLocalSize, 0.0));
				UE_VLOG_OBOX(InLandscape, LogLandscape, Log, VisualBounds, BaseTransform.ToMatrixWithScale(), InRendererRenderInfo.VisualLogColor, TEXT("%s"), *LogMessage);
			}
			break;
		}
		case FOutputWorldArea::EType::SpecificComponent:
		{
			FIntPoint ComponentKey = InOutputWorldArea.GetSpecificComponentKey();
			FBox VisualBounds(
				FVector(ComponentKey.X * LandscapeComponentLocalSize, ComponentKey.Y * LandscapeComponentLocalSize, 0.0),
				FVector((ComponentKey.X + 1) * LandscapeComponentLocalSize, (ComponentKey.Y + 1) * LandscapeComponentLocalSize, 0.0));
			UE_VLOG_OBOX(InLandscape, LogLandscape, Log, VisualBounds, BaseTransform.ToMatrixWithScale(), InRendererRenderInfo.VisualLogColor, TEXT("%s"), *LogMessage);
			break;
		}
		case FOutputWorldArea::EType::OOBox:
		{
			FOOBox2D OOBox = InOutputWorldArea.GetOOBox();
			FBox VisualBounds(-FVector(OOBox.Extents, 0.0), FVector(OOBox.Extents, 0.0));
			UE_VLOG_OBOX(InLandscape, LogLandscape, Log, VisualBounds, OOBox.Transform.ToMatrixWithScale(), InRendererRenderInfo.VisualLogColor, TEXT("%s"), *LogMessage);
			break;
		}
		default:
			check(false);
		}
	}
#endif // ENABLE_VISUAL_LOG
} // UE::Landscape::EditLayers::Private

bool ULandscapeHeightmapNormalsEditLayerRenderer::RenderLayer(UE::Landscape::EditLayers::FRenderParams& RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder)
{
	using namespace UE::Landscape;
	using namespace UE::Landscape::EditLayers;
	using namespace UE::Landscape::EditLayers::Private;

	FMergeRenderContext* RenderContext = RenderParams.MergeRenderContext;
	const FMergeRenderBatch* RenderBatch = RenderContext->GetCurrentRenderBatch();
	ALandscape* Landscape = RenderContext->GetLandscape();

	checkf(RDGBuilderRecorder.IsRecording(), TEXT("ERenderFlags::RenderMode_Recorded means the command recorder should be recording at this point"));
	checkf(RenderParams.TargetLayerGroupLayerNames.Num() == 1, TEXT("Normals should only be generated on heightmap merge, which should have 1 and only target layer"));

	RenderContext->CycleBlendRenderTargets(RDGBuilderRecorder);
	ULandscapeScratchRenderTarget* WriteRT = RenderContext->GetBlendRenderTargetWrite();
	ULandscapeScratchRenderTarget* ReadRT = RenderContext->GetBlendRenderTargetRead();
	ULandscapeScratchRenderTarget* ComponentIdRT = RenderContext->GetComponentIdRenderTarget();
	check((WriteRT != nullptr) && (ReadRT != nullptr) && (ComponentIdRT != nullptr));

	WriteRT->TransitionTo(ERHIAccess::RTV, RDGBuilderRecorder);
	ReadRT->TransitionTo(ERHIAccess::SRVMask, RDGBuilderRecorder);
	ComponentIdRT->TransitionTo(ERHIAccess::SRVMask, RDGBuilderRecorder);

	FIntPoint NumComponentsInRenderArea = RenderBatch->MaxComponentKey - RenderBatch->MinComponentKey + 1;
	check((NumComponentsInRenderArea.X > 0) && (NumComponentsInRenderArea.Y > 0));

	auto RDGCommand =
		[OutputResource = WriteRT->GetRenderTarget2D()->GetResource()
		, OutputResourceName = WriteRT->GetDebugName()
		, SourceResource = ReadRT->GetRenderTarget2D()->GetResource()
		, ComponentIdResource = ComponentIdRT->GetRenderTarget2D()->GetResource()
		, EffectiveTextureSize = RenderBatch->GetRenderTargetResolution(/*bInWithDuplicateBorders = */false)
		, LandscapeGridScale = Landscape->GetRootComponent()->GetRelativeScale3D()
		, ComponentSizeQuads = Landscape->ComponentSizeQuads
		, NumComponentsInRenderArea](FRDGBuilder& GraphBuilder)
	{
		FRDGTextureRef OutputTextureRef = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(OutputResource->TextureRHI, TEXT("OutputTexture")));
		FRDGTextureRef SourceTextureRef = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(SourceResource->TextureRHI, TEXT("SourceTexture")));
		FRDGTextureRef ComponentIdTextureRef = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(ComponentIdResource->TextureRHI, TEXT("ComponentIdTexture")));

		FLandscapeEditLayersHeightmapsGenerateNormalsPS::FParameters* PSParams = GraphBuilder.AllocParameters<FLandscapeEditLayersHeightmapsGenerateNormalsPS::FParameters>();
		PSParams->RenderTargets[0] = FRenderTargetBinding(OutputTextureRef, ERenderTargetLoadAction::ENoAction);
		PSParams->InTextureSize = FUintVector4(EffectiveTextureSize.X, EffectiveTextureSize.Y, SourceResource->GetSizeX(), SourceResource->GetSizeY());
		PSParams->InLandscapeGridScale = static_cast<FVector3f>(LandscapeGridScale);
		PSParams->InComponentSizeQuads = ComponentSizeQuads;
		PSParams->InNumComponents = FUintVector2(NumComponentsInRenderArea.X, NumComponentsInRenderArea.Y);
		PSParams->InSourceHeightmapSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp>::GetRHI();
		PSParams->InSourceHeightmap = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(SourceTextureRef));
		PSParams->InComponentIdTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(ComponentIdTextureRef));
			FLandscapeEditLayersHeightmapsGenerateNormalsPS::GenerateNormalsPS(
				RDG_EVENT_NAME("GenerateNormals -> %s", *OutputResourceName), GraphBuilder, PSParams, EffectiveTextureSize);
		};

	// We need to specify the final state of the external textures to prevent the graph builder from transitioning them to SRVMask (even those that end up as SRVMask at the end of this command,
	//  because they will likely be part of another RDGCommand down the line so we need to maintain an accurate picture of every external texture ever involved in the recorded command so that 
	//  we can set a proper access when the recorder is flushed (and the FRDGBuilder, executed) :
	TArray<FRDGBuilderRecorder::FRDGExternalTextureAccessFinal> RDGExternalTextureAccessFinalList =
	{
		{ WriteRT->GetRenderTarget()->GetResource(), ERHIAccess::RTV },
		{ ReadRT->GetRenderTarget()->GetResource(), ERHIAccess::SRVMask },
		{ ComponentIdRT->GetRenderTarget()->GetResource(), ERHIAccess::SRVMask },
	};
	RDGBuilderRecorder.EnqueueRDGCommand(RDGCommand, RDGExternalTextureAccessFinalList);

	return true;
}

bool ULandscapeWeightmapWeightBlendedLayersRenderer::RenderLayer(UE::Landscape::EditLayers::FRenderParams& RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder)
{
	using namespace UE::Landscape;
	using namespace UE::Landscape::EditLayers;
	using namespace UE::Landscape::EditLayers::Private;

	checkf(RDGBuilderRecorder.IsRecording(), TEXT("ERenderFlags::RenderMode_Recorded means the command recorder should be recording at this point"));
	check(!RenderParams.MergeRenderContext->IsHeightmapMerge());

	const FMergeRenderBatch* RenderBatch = RenderParams.MergeRenderContext->GetCurrentRenderBatch();
	check(RenderBatch != nullptr);

	RenderParams.MergeRenderContext->CycleBlendRenderTargets(RDGBuilderRecorder);
	ULandscapeScratchRenderTarget* WriteRT = RenderParams.MergeRenderContext->GetBlendRenderTargetWrite();
	ULandscapeScratchRenderTarget* ReadRT = RenderParams.MergeRenderContext->GetBlendRenderTargetRead();

	WriteRT->TransitionTo(ERHIAccess::RTV, RDGBuilderRecorder);
	ReadRT->TransitionTo(ERHIAccess::SRVMask, RDGBuilderRecorder);

	TArray<FFinalWeightBlendingTargetLayerInfo> WeightmapTargetLayerInfos;
	WeightmapTargetLayerInfos.Reserve(RenderParams.TargetLayerGroupLayerInfos.Num());
	Algo::Transform(RenderParams.TargetLayerGroupLayerInfos, WeightmapTargetLayerInfos, [RenderBatch, MergeRenderContext = RenderParams.MergeRenderContext](ULandscapeLayerInfoObject* InLayerInfo)
		{
			check(InLayerInfo != nullptr); // There should only be valid layer infos at this point

			int32 TargetLayerIndex = MergeRenderContext->GetTargetLayerIndexForLayerInfoChecked(InLayerInfo);
			// Skip weight-blended layers that are not rendered in the batch, because their texture slice might contain garbage :
			const bool bRenderedInBatch = RenderBatch->TargetLayerBitIndices[TargetLayerIndex];

			FFinalWeightBlendingTargetLayerInfo WeightmapTargetLayerInfo;
			if (!bRenderedInBatch)
			{
				WeightmapTargetLayerInfo.Flags |= EWeightmapTargetLayerFlags::Skip;
			}
			if (InLayerInfo->GetBlendMethod() == ELandscapeTargetLayerBlendMethod::FinalWeightBlending)
			{
				WeightmapTargetLayerInfo.Flags |= EWeightmapTargetLayerFlags::IsWeightBlended;
			}
			if (InLayerInfo == ALandscapeProxy::VisibilityLayer)
			{
				WeightmapTargetLayerInfo.Flags |= EWeightmapTargetLayerFlags::IsVisibilityLayer;
			}
			return WeightmapTargetLayerInfo;
		});

	auto RDGCommand =
		[TargetLayerNames = RenderParams.TargetLayerGroupLayerNames
		, WeightmapTargetLayerInfos
		, OutputResource = WriteRT->GetRenderTarget2DArray()->GetResource()
		, OutputResourceName = WriteRT->GetDebugName()
		, CurrentEditLayerResource = ReadRT->GetRenderTarget2DArray()->GetResource()
		, EffectiveTextureSize = RenderParams.MergeRenderContext->GetCurrentRenderBatch()->GetRenderTargetResolution(/*bInWithDuplicateBorders = */false)](FRDGBuilder& GraphBuilder)
		{
			FRDGTextureRef OutputTextureRef = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(OutputResource->TextureRHI, TEXT("OutputTexture")));
			FRDGTextureRef CurrentEditLayerTextureRef = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(CurrentEditLayerResource->TextureRHI, TEXT("CurrentEditLayerTexture")));
			FRDGTextureSRVRef CurrentEditLayerTextureSRVRef = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(CurrentEditLayerTextureRef));

			FRDGBufferRef TargetLayerInfosBuffer = CreateStructuredBuffer(GraphBuilder, TEXT("LandscapeFinalWeightBlendingTargetLayerInfosBuffer"), WeightmapTargetLayerInfos);
			FRDGBufferSRVRef TargetLayerInfosBufferSRV = GraphBuilder.CreateSRV(FRDGBufferSRVDesc(TargetLayerInfosBuffer));

			const int32 NumTargetLayers = TargetLayerNames.Num();
			for (int32 TargetLayerIndex = 0; TargetLayerIndex < NumTargetLayers; ++TargetLayerIndex)
			{
				if (!EnumHasAnyFlags(WeightmapTargetLayerInfos[TargetLayerIndex].Flags, EWeightmapTargetLayerFlags::Skip))
				{ 
				    FLandscapeEditLayersWeightmapsPerformFinalWeightBlendingPS::FParameters* PSParams = GraphBuilder.AllocParameters<FLandscapeEditLayersWeightmapsPerformFinalWeightBlendingPS::FParameters>();
					PSParams->RenderTargets[0] = FRenderTargetBinding(OutputTextureRef, ERenderTargetLoadAction::ENoAction, /*InMipIndex = */0, /*InArraySlice = */TargetLayerIndex);
					PSParams->InTargetLayerIndex = TargetLayerIndex;
					PSParams->InNumTargetLayers = NumTargetLayers;
					PSParams->InFinalWeightBlendingTargetLayerInfos = TargetLayerInfosBufferSRV;
					PSParams->InCurrentEditLayerWeightmaps = CurrentEditLayerTextureSRVRef;

				    FLandscapeEditLayersWeightmapsPerformFinalWeightBlendingPS::PerformFinalWeightBlendingPS(
					    RDG_EVENT_NAME("FinalWeightBlending(%s) -> %s", *TargetLayerNames[TargetLayerIndex].ToString(), *OutputResourceName),
					GraphBuilder, PSParams, EffectiveTextureSize);
			}
			}
		};

	// We need to specify the final state of the external textures to prevent the graph builder from transitioning them to SRVMask (even those that end up as SRVMask at the end of this command,
	//  because they will likely be part of another RDGCommand down the line so we need to maintain an accurate picture of every external texture ever involved in the recorded command so that 
	//  we can set a proper access when the recorder is flushed (and the FRDGBuilder, executed) :
	TArray<FRDGBuilderRecorder::FRDGExternalTextureAccessFinal> RDGExternalTextureAccessFinalList =
	{
		{ WriteRT->GetRenderTarget()->GetResource(), ERHIAccess::RTV },
		{ ReadRT->GetRenderTarget()->GetResource(), ERHIAccess::SRVMask },
	};
	RDGBuilderRecorder.EnqueueRDGCommand(RDGCommand, RDGExternalTextureAccessFinalList);

	return true;
}

UE::Landscape::EditLayers::FMergeRenderContext ALandscape::PrepareEditLayersMergeRenderContext(const UE::Landscape::EditLayers::FMergeContext& InMergeContext, const UE::Landscape::EditLayers::FMergeRenderParams& InMergeRenderParams)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ALandscape::PrepareEditLayersMergeRenderContext);

	using namespace UE::Landscape::EditLayers;
	using namespace UE::Landscape::EditLayers::Private;
	using namespace UE::Landscape::Private;

	const int32 VisualLogShowRenderItems = CVarLandscapeBatchedMergeVisualLogShowRenderItemsType.GetValueOnGameThread();
	const bool bVisualLogShowRenderItemsInput = (VisualLogShowRenderItems == 1) || (VisualLogShowRenderItems == 3);
	const bool bVisualLogShowRenderItemsOutput = (VisualLogShowRenderItems == 2) || (VisualLogShowRenderItems == 3);
	const bool bVisualLogShowAllRenderItems = CVarLandscapeBatchedMergeVisualLogShowAllRenderItems.GetValueOnGameThread();
	const FString VisualLogShowRenderItemsEditLayerRendererFilter = CVarLandscapeBatchedMergeVisualLogShowRenderItemsEditLayerRendererFilter.GetValueOnGameThread();
	const int32 VisualLogShowComponentDependencies = CVarLandscapeBatchedMergeVisualLogShowComponentDependencies.GetValueOnGameThread();
	const FString VisualLogShowComponentDependenciesFilter = CVarLandscapeBatchedMergeVisualLogShowComponentDependenciesFilter.GetValueOnGameThread();
	const bool bEnableRenderLayerGrouping = CVarLandscapeBatchedMergeEnableRenderLayerGroup.GetValueOnGameThread();

	ULandscapeInfo* Info = GetLandscapeInfo();
	check(Info != nullptr);
	check(!LandscapeEditLayers.IsEmpty()); 

	// Warn if invalid layer names are requested : 
	if (!InMergeContext.IsHeightmapMerge())
	{
		for (FName TargetLayerName : InMergeRenderParams.WeightmapLayerNames)
		{
			if (!InMergeContext.IsValidTargetLayerName(TargetLayerName))
			{
				UE_LOGF(LogLandscape, Warning, "Target layer \"%ls\" was requested by the merge but is invalid (missing its layer info object asset). It will be ignored", *TargetLayerName.ToString());
			}
		}
	}

	const FTransform& LandscapeTransform = GetTransform();
	const FBox LandscapeLoadedBounds = Info->GetLoadedBounds();

	FMergeRenderContext MergeRenderContext(InMergeContext);

	// Prepare a dummy copy for early-outs :
	const FMergeRenderContext DummyMergeRenderContext(MergeRenderContext);
	check(!DummyMergeRenderContext.IsValid());

	const ELandscapeToolTargetTypeFlags MergeTypeMask = MergeRenderContext.IsHeightmapMerge() ? ELandscapeToolTargetTypeFlags::Heightmap : (ELandscapeToolTargetTypeFlags::Weightmap | ELandscapeToolTargetTypeFlags::Visibility);
	const int32 NumAllTargetLayerNames = MergeRenderContext.AllTargetLayerNames.Num();
	check(MergeRenderContext.ValidTargetLayerBitIndices.Num() == NumAllTargetLayerNames);

	// InMergeRenderParams.EditLayerRendererStates contains a list of renderers that is not quite final : ULandscapeDefaultEditLayerRenderer is always inserted at the beginning to make sure we always render at 
	//  least the default value and ULandscapeWeightmapWeightBlendedLayersRenderer can optionally be inserted at the end too :
	TArray<FEditLayerRendererState> CandidateEditLayerRendererStates;
	{
		CandidateEditLayerRendererStates.Reserve(InMergeRenderParams.EditLayerRendererStates.Num() + 2);

		// We always have at least 1 renderer at the start : the default one, whose job is to both provide the default value and act as the final "gatherer" of the component dependencies 
		//  from all the renderers above (see class comment for more details) :
		ULandscapeDefaultEditLayerRenderer* InitialEditLayerRenderer = ULandscapeDefaultEditLayerRenderer::StaticClass()->GetDefaultObject<ULandscapeDefaultEditLayerRenderer>();
		CandidateEditLayerRendererStates.Emplace(&MergeRenderContext, InitialEditLayerRenderer);

		// Then append all the provided renderer states
		CandidateEditLayerRendererStates.Append(InMergeRenderParams.EditLayerRendererStates);

		// Final weight-blending requires an additional renderer at the end of the stack, to weight-blend the weightmaps needing it :
		if (!MergeRenderContext.IsHeightmapMerge())
		{
			ULandscapeWeightmapWeightBlendedLayersRenderer* WeightmapWeightBlendedLayersRenderer = ULandscapeWeightmapWeightBlendedLayersRenderer::StaticClass()->GetDefaultObject<ULandscapeWeightmapWeightBlendedLayersRenderer>();
			if (WeightmapWeightBlendedLayersRenderer->GatherWeightBlendedWeightmapLayerBitIndices(&MergeRenderContext).Contains(true))
			{
				CandidateEditLayerRendererStates.Emplace(&MergeRenderContext, WeightmapWeightBlendedLayersRenderer);
			}
		}
	}

	// Only retain renderers that are relevant for this merge : 
	CandidateEditLayerRendererStates = CandidateEditLayerRendererStates.FilterByPredicate([&InMergeRenderParams, MergeTypeMask](const FEditLayerRendererState& InRendererState)
		{
			return EnumHasAnyFlags(InRendererState.GetActiveTargetTypeMask(), MergeTypeMask);
		});

	// FinalEditLayerRendererStates will contain the renderer states that are actually relevant to this merge :
	TArray<FEditLayerRendererState> FinalEditLayerRendererStates;
	FinalEditLayerRendererStates.Reserve(CandidateEditLayerRendererStates.Num());

	// Within each render batch, elements can be processed group by group. For heightmap/visibility, there's only one such group. For weightmaps, there's one group per list of weightmaps
	//  that need to be processed together for weight-blending. Each group is composed of a list of (weightmap layer) names (it's only a debug name in the case of heightmaps)
	TArray<TBitArray<>> FinalTargetLayerGroups;
	if (MergeRenderContext.IsHeightmapMerge())
	{
		// All candidates are valid in heightmap merge, the ones that don't affect heightmap have already been filtered from CandidateEditLayerRendererStates
		FinalEditLayerRendererStates = CandidateEditLayerRendererStates;

		// Only one group in the case of heightmap: 
		FinalTargetLayerGroups = { TBitArray<>(true, 1) };
	}
	else
	{
		// First, let's work out the weightmaps inter-dependencies (i.e. horizontal dependencies) : weight-blending requires some weightmaps to be processed together: 
		TRACE_CPUPROFILER_EVENT_SCOPE(PrepareTargetLayerGroups);

		TArray<TArray<TBitArray<>>> RelevantTargetLayerGroupsPerRenderer;
		TArray<TBitArray<>> RelevantTargetLayerBitIndicesPerRenderer;
		RelevantTargetLayerGroupsPerRenderer.Reserve(CandidateEditLayerRendererStates.Num());
		RelevantTargetLayerBitIndicesPerRenderer.Reserve(CandidateEditLayerRendererStates.Num());
		// Iterate through all renderers to find out what target layer group they render (only keep those that are relevant for the current state and request) : 
		for (const FEditLayerRendererState& EditLayerRendererState : CandidateEditLayerRendererStates)
		{
			TBitArray<> RendererStateEnabledTargetLayerBitIndices(EditLayerRendererState.GetActiveTargetWeightmapBitIndices());
			// Retain only the target layer names that are enabled and valid :
			RendererStateEnabledTargetLayerBitIndices.CombineWithBitwiseAND(MergeRenderContext.ValidTargetLayerBitIndices, EBitwiseOperatorFlags::MinSize);

			// List of all supported target layer groups for this renderer :
			TArray<TBitArray<>> RelevantTargetLayerGroupsForThisRenderer;
			for (const FTargetLayerGroup& TargetLayerGroup : EditLayerRendererState.GetTargetLayerGroups())
			{
				// Only retain the target layers that have passed the previous test (valid and enabled):
				TBitArray<> FilteredTargetLayerGroupBitIndices = TargetLayerGroup.GetWeightmapTargetLayerBitIndices();
				FilteredTargetLayerGroupBitIndices.CombineWithBitwiseAND(RendererStateEnabledTargetLayerBitIndices, EBitwiseOperatorFlags::MinSize);
				if (FilteredTargetLayerGroupBitIndices.Find(true) != INDEX_NONE)
				{
					checkf(Algo::NoneOf(RelevantTargetLayerGroupsForThisRenderer, [&FilteredTargetLayerGroupBitIndices](const TBitArray<>& InOtherGroupBitIndices)
						{ 
							return (TBitArray<>::BitwiseAND(InOtherGroupBitIndices, FilteredTargetLayerGroupBitIndices, EBitwiseOperatorFlags::MinSize).Find(true) != INDEX_NONE);
						}), 
						TEXT("All of the target layers returned by the renderer must belong to 1 target layer group of this renderer and 1 only"));
					RelevantTargetLayerGroupsForThisRenderer.Add(FilteredTargetLayerGroupBitIndices);
				}
			}

			RelevantTargetLayerGroupsPerRenderer.Add(RelevantTargetLayerGroupsForThisRenderer);
			RelevantTargetLayerBitIndicesPerRenderer.Add(RendererStateEnabledTargetLayerBitIndices);
		}
		check(CandidateEditLayerRendererStates.Num() == RelevantTargetLayerGroupsPerRenderer.Num());
		check(CandidateEditLayerRendererStates.Num() == RelevantTargetLayerBitIndicesPerRenderer.Num());

		const int32 CandidateNumRenderers = CandidateEditLayerRendererStates.Num();

		TBitArray<> RequestedTargetLayerBitIndices;
		if (InMergeRenderParams.bRequestAllLayers)
		{
			RequestedTargetLayerBitIndices = MergeRenderContext.ValidTargetLayerBitIndices;
		}
		else
		{
			RequestedTargetLayerBitIndices = MergeRenderContext.ConvertTargetLayerNamesToBitIndices(InMergeRenderParams.WeightmapLayerNames.Array());
		}
		// No need to retain the invalid target layers : 
		RequestedTargetLayerBitIndices.CombineWithBitwiseAND(MergeRenderContext.ValidTargetLayerBitIndices, EBitwiseOperatorFlags::MinSize);
		// Early-out when there's nothing to do :
		if (RequestedTargetLayerBitIndices.Find(true) == INDEX_NONE)
		{
			return DummyMergeRenderContext;
		}

		FinalTargetLayerGroups.Reserve(MergeRenderContext.ValidTargetLayerBitIndices.CountSetBits());
		// Start with minimal target layer groups : one per requested target layer : 
		for (TConstSetBitIterator It(RequestedTargetLayerBitIndices); It; ++It)
		{
			TBitArray<> TargetLayerGroup(false, NumAllTargetLayerNames);
			TargetLayerGroup[It.GetIndex()] = true;
			FinalTargetLayerGroups.Add(MoveTemp(TargetLayerGroup));
		}

		// Then iterate in reverse order on renderers to trace the dependency of each of their target layer groups towards one another and move target layers from one group to another as we discover new dependencies:
		MergeRenderContext.FinalTargetLayerBitIndices = RequestedTargetLayerBitIndices;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AnalyzeRenderersForTargetLayerGroups);

			for (int32 CurrentRendererIndex = CandidateNumRenderers - 1; CurrentRendererIndex >= 0; --CurrentRendererIndex)
			{
				const TArray<TBitArray<>>& RendererTargetLayerGroups = RelevantTargetLayerGroupsPerRenderer[CurrentRendererIndex];
				const int32 NumTargetLayerGroups = RendererTargetLayerGroups.Num();
				for (int32 TargetLayerGroupIndex = 0; TargetLayerGroupIndex < NumTargetLayerGroups; ++TargetLayerGroupIndex)
				{
					const TBitArray<>& TargetLayerGroup = RendererTargetLayerGroups[TargetLayerGroupIndex];

					TBitArray<> NewMergedTargetLayerGroup(false, NumAllTargetLayerNames);
					// In all of the final target layer groups, find the ones that have a layer in common with this target layer group and merge them all into a single new one
					FinalTargetLayerGroups.RemoveAllSwap([&NewMergedTargetLayerGroup, &TargetLayerGroup](const TBitArray<>& InFinalTargetLayerGroup)
						{
							const bool bShouldMerge = (InFinalTargetLayerGroup != TargetLayerGroup)
								&& (TBitArray<>::BitwiseAND(InFinalTargetLayerGroup, TargetLayerGroup, EBitwiseOperatorFlags::MinSize).Find(true) != INDEX_NONE);
							if (bShouldMerge)
							{
								NewMergedTargetLayerGroup.CombineWithBitwiseOR(InFinalTargetLayerGroup, EBitwiseOperatorFlags::MinSize);
							}
							return bShouldMerge;
						});
					// Now add it back to the list of final target layer groups if it's valid, so that it can be merged by the next renderer if required :
					if (NewMergedTargetLayerGroup.CountSetBits() > 0)
					{
						FinalTargetLayerGroups.Add(NewMergedTargetLayerGroup);
						MergeRenderContext.FinalTargetLayerBitIndices.CombineWithBitwiseOR(NewMergedTargetLayerGroup, EBitwiseOperatorFlags::MinSize);
					}
				}
			}
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FinalizeTargetLayerGroups);

			// Now that we have the list of all target layers required for rendering the requested ones, we trim the list of edit layer renderers that just don't do anything with those:
			for (int32 RendererIndex = 0; RendererIndex < CandidateNumRenderers; ++RendererIndex)
			{
				const TBitArray<> RelevantTargetLayerBitIndices = RelevantTargetLayerBitIndicesPerRenderer[RendererIndex];
				if (TBitArray<>::BitwiseAND(RelevantTargetLayerBitIndices, MergeRenderContext.FinalTargetLayerBitIndices, EBitwiseOperatorFlags::MinSize).Find(true) != INDEX_NONE)
				{
					FinalEditLayerRendererStates.Add(CandidateEditLayerRendererStates[RendererIndex]);
				}
			}

			// TODO [jonathan.bard] : revisit : this shouldn't be true now that we support additively resolving channels with blending : 
#if !SUPPORTS_LANDSCAPE_EDITORONLY_UBER_MATERIAL
			// When 4 weightmaps are packed in 1 RGBA channel, we cannot guarantee that weightmaps will be fully resolvable when a group is done, since there's no guarantee that a given component's 4 
			//  allocations will be contained in that group. So we create a single group instead, that contains all weightmap layers. This consumes more memory since we need 3 texture arrays for a batch and 
			//  the number of weightmaps in a group is the number of slices of the array : 
			FinalTargetLayerGroups = { MergeRenderContext.FinalTargetLayerBitIndices };
#endif // !SUPPORTS_LANDSCAPE_EDITORONLY_UBER_MATERIAL
		}
	}

	const int32 FinalNumRenderers = FinalEditLayerRendererStates.Num();
	
	// Early-out when there's nothing to do :
	if (FinalNumRenderers == 0)
	{
		return DummyMergeRenderContext;
	}

	FLandscapeComponent2DIndexer Component2DIndexer = CreateLandscapeComponent2DIndexer(Info);
	const TArray<ULandscapeComponent*> AllComponents = Component2DIndexer.GetAllValues();
	const TBitArray<> AllValidComponentBitIndices = Component2DIndexer.GetValidValueBitIndices();
	const int32 NumAllComponents = AllComponents.Num();

	// The list of all components that will end up being rendered across all renderers (one bit per component) :
	TBitArray<> FinalComponentsToRenderInfoBitIndices(false, NumAllComponents);

	// Pre-allocate a working list of all landscape components render info. Some of which we might not end up rendering, but at least, that allows to associate a component with an index, 
	//  which allows to turn intersection/union of components (which we do a lot in this function) into simple bit array bitwise AND/OR operations : 
	TArray<FComponentToRenderInfo> AllComponentsToRenderInfos;
	TArray<FEditLayerRendererRenderInfo> OrderedEditLayerRendererRenderInfos;

#if ENABLE_VISUAL_LOG
	int32 VisualLogShowComponentDependenciesIndex = INDEX_NONE;
	if (!VisualLogShowComponentDependenciesFilter.IsEmpty())
	{
		FIntPoint ComponentKey;
		if (ComponentKey.InitFromString(VisualLogShowComponentDependenciesFilter))
		{
			VisualLogShowComponentDependenciesIndex = Component2DIndexer.GetValueIndexForKeySafe(ComponentKey);
			if (VisualLogShowComponentDependenciesIndex == INDEX_NONE)
			{
				UE_LOGF(LogLandscape, Warning, "Component key \"%ls\" specified for dependencies filter does not correspond to a valid component. Ignoring show component dependencies filter", *VisualLogShowComponentDependenciesFilter);
			}
		}
		else
		{
			UE_LOGF(LogLandscape, Warning, "Cannot parse string \"%ls\". Ignoring show component dependencies filter", *VisualLogShowComponentDependenciesFilter);
		}
	}

	// Helper for debugging component dependencies : only if the CVar requires it
	TOptional<FComponentDependenciesVisLogHelper> VisualLogDependencyHelper;
	UE_IFVLOG(
			if ((MergeRenderContext.IsVisualLogEnabled()) && ((VisualLogShowComponentDependencies > 0) || (VisualLogShowComponentDependenciesIndex != INDEX_NONE)))
			{
				// Force the display of all info when we show the dependencies of one component in particular : 
				const FComponentDependenciesVisLogHelper::EShowNodeInfo ShowNodeInfo = (VisualLogShowComponentDependenciesIndex != INDEX_NONE)
					? FComponentDependenciesVisLogHelper::EShowNodeInfo::Detailed
					: static_cast<FComponentDependenciesVisLogHelper::EShowNodeInfo>(VisualLogShowComponentDependencies);
				VisualLogDependencyHelper.Emplace(this, MergeRenderContext.IsHeightmapMerge(), ShowNodeInfo, MergeRenderContext);
			}
		);
	auto VisLogDependency = [VisualLogShowComponentDependenciesIndex, &VisualLogDependencyHelper, &AllComponentsToRenderInfos, &OrderedEditLayerRendererRenderInfos]
		(int32 InSourceComponentIndex, int32 InSourceRendererIndex, int32 InDestinationComponentIndex, int32 InDestinationRendererIndex)
		{
			if (VisualLogDependencyHelper.IsSet() 
				&& (InSourceRendererIndex >= 0) 
				&& ((VisualLogShowComponentDependenciesIndex == INDEX_NONE) || (InSourceComponentIndex == VisualLogShowComponentDependenciesIndex)))
			{
				VisualLogDependencyHelper->AddDependency(AllComponentsToRenderInfos[InSourceComponentIndex], OrderedEditLayerRendererRenderInfos[InSourceRendererIndex],
					AllComponentsToRenderInfos[InDestinationComponentIndex], OrderedEditLayerRendererRenderInfos[InDestinationRendererIndex]);
			}
		};
#endif // ENABLE_VISUAL_LOG

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PrepareRenderersAnalysis);

		AllComponentsToRenderInfos.AddDefaulted(NumAllComponents);
		for (TConstSetBitIterator It(AllValidComponentBitIndices); It; ++It)
		{
			const int32 ComponentIndex = It.GetIndex();
			AllComponentsToRenderInfos[ComponentIndex] = FComponentToRenderInfo(AllComponents[ComponentIndex], ComponentIndex, NumAllComponents, NumAllTargetLayerNames);
			MergeRenderContext.MaxLocalHeight = FMath::Max(MergeRenderContext.MaxLocalHeight, AllComponentsToRenderInfos[ComponentIndex].LocalBounds.Max.Z);
		}

		// Initiate the process by flipping a bit for each component to merge, for the last renderer in the stack, then we'll register the dependencies 
		//  between components on this renderer and those on the previous renderer by iterating on renderers in reverse stack order : 
		for (ULandscapeComponent* Component : InMergeRenderParams.ComponentsToMerge)
		{
			const int32 ComponentIndex = Component2DIndexer.GetValueIndexChecked(Component);
			FinalComponentsToRenderInfoBitIndices[ComponentIndex] = true;
		}

		// Prepare the render infos of all these renderers : 
		OrderedEditLayerRendererRenderInfos.Reserve(FinalNumRenderers);
		for (int32 RendererIndex = 0; RendererIndex < FinalNumRenderers; ++RendererIndex)
		{
			OrderedEditLayerRendererRenderInfos.Emplace(FinalEditLayerRendererStates[RendererIndex], RendererIndex, LandscapeTransform, MergeRenderContext.MaxLocalHeight, NumAllComponents);
		}
	}

	// Iterate over all renderers in inverse order to compute which landscape component needs to be included in the render. This way, the renderers on top are able to 
	//  request potentially more components from renderers underneath (e.g. if the renderer performs a blur, it will require an area around the component's area, thus additional components, potentially, 
	//  which will then request potentially more components on the renderer underneath, etc.)
	// For each renderer we'll only iterate on nodes that correspond to it. Since they're added in reverse order, this is just a matter of starting the iteration from the first of the renderer's nodes :
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AnalyzeRenderers);
		for (int32 CurrentRendererIndex = FinalNumRenderers - 1; CurrentRendererIndex >= 0; --CurrentRendererIndex)
		{
			FEditLayerRendererRenderInfo& EditLayerRendererRenderInfo = OrderedEditLayerRendererRenderInfos[CurrentRendererIndex];

			const FString EditLayerRendererDebugName = EditLayerRendererRenderInfo.RendererState.GetRenderer()->GetEditLayerRendererDebugName();
			const bool bPassesVisualLogRenderItemFilter = VisualLogShowRenderItemsEditLayerRendererFilter.IsEmpty() || EditLayerRendererDebugName.Contains(VisualLogShowRenderItemsEditLayerRendererFilter);

			// This renderer will eventually append new components to render along the way so we add them to a new list and swap at the end : 
			TBitArray<> ComponentsToRenderInfoBitIndicesAfterRenderer = FinalComponentsToRenderInfoBitIndices;

			// Intersect this renderer's render items' outputs with the components to render to find out which ones should participate to the merge :
			for (const FEditLayerRenderItem& RenderItem : EditLayerRendererRenderInfo.RendererState.GetRenderer()->GetRenderItems(&MergeRenderContext))
			{
				checkf(RenderItem.GetTargetTypeState().Intersect(EditLayerRendererRenderInfo.RendererState.GetSupportedTargetTypeState()) == RenderItem.GetTargetTypeState(),
					TEXT("One of edit layer renderer %s's render items target type state is not supported by the renderer's supported target type state. "
					"This is invalid : the renderer's supported target type state should be a superset of its render item's target type state. \n"
					"- Render item state: %s \n"
					"- Renderer supported state: %s \n"), *EditLayerRendererDebugName, *RenderItem.GetTargetTypeState().ToString(), *EditLayerRendererRenderInfo.RendererState.GetSupportedTargetTypeState().ToString());

				// Only consider render items which intersect with the renderer's active state (e.g. one a given renderer, a render item (A) might affect heightmaps only and another one (B) weightmaps only, 
				//  then if performing a heightmap merge, the RendererState's mask here will be ELandscapeToolTargetTypeFlags::Heightmap, so this allows to filter out item B: 
				const FEditLayerTargetTypeState FilteredRenderItemTargetTypeState = RenderItem.GetTargetTypeState().Intersect(EditLayerRendererRenderInfo.RendererState.GetActiveTargetTypeState());

				bool bOutputRenderItem = false;
				const FOutputWorldArea& OutputWorldArea = RenderItem.GetOutputWorldArea();
				
				const bool bRenderItemAffectsMergeType = EnumHasAnyFlags(FilteredRenderItemTargetTypeState.GetTargetTypeMask(), MergeTypeMask);
				if (bRenderItemAffectsMergeType)
				{
					switch (OutputWorldArea.GetType())
					{
					case FOutputWorldArea::EType::LocalComponent:
					{
						// This render item writes to every component : 
						bOutputRenderItem = true;
						break;
					}
					case FOutputWorldArea::EType::SpecificComponent:
					{
						// This render item writes to only 1 component :
						if (int32 ComponentIndex = Component2DIndexer.GetValueIndexForKeySafe(OutputWorldArea.GetSpecificComponentKey()); ComponentIndex != INDEX_NONE)
						{
							// Retain the render item if the component it renders to overlaps with one the components to render :
							bOutputRenderItem = FinalComponentsToRenderInfoBitIndices[ComponentIndex];
						}
						break;
					}
					case FOutputWorldArea::EType::OOBox:
					{
						FIntRect ComponentIndicesBoundingRect;
						TMap<FIntPoint, ULandscapeComponent*> DependentComponents;
						// TODO [jonathan.bard] : Change this to OOBB to OOBB test to cull more components : look at FOrientedBoxHelpers
						const FOOBox2D& OOBox = OutputWorldArea.GetOOBox();
						Info->GetOverlappedComponents(OOBox.Transform, FBox2D(-OOBox.Extents / 2.0, OOBox.Extents / 2.0), DependentComponents, ComponentIndicesBoundingRect);
						for (auto It = DependentComponents.CreateConstIterator(); It && !bOutputRenderItem; ++It)
						{
							int32 ComponentIndex = Component2DIndexer.GetValueIndexForKeyChecked(It.Key());
							// Retain the render item if one of the components it renders to overlaps with one the components to render :
							bOutputRenderItem = FinalComponentsToRenderInfoBitIndices[ComponentIndex];
						}
						break;
					}
					default:
						check(false);
					}
				}

				// Mark which output layers of the component this render item will only need to affect :
				// For a weightmap merge, it's possible the render item will only end up modifying the existing ones (as opposed to "generating" new ones).
				//  In that case, restrain the render item from modifying the target layer mask for this component :
				const bool bAffectsOutputLayerBitIndices = MergeRenderContext.IsHeightmapMerge() || !RenderItem.GetModifyExistingWeightmapsOnly();

				UE_IFVLOG(
					{
						if (MergeRenderContext.IsVisualLogEnabled() && bRenderItemAffectsMergeType && bPassesVisualLogRenderItemFilter)
						{
							if (bVisualLogShowRenderItemsInput && (bOutputRenderItem || bVisualLogShowAllRenderItems))
							{
								TArray<ULandscapeComponent*> AllComponentsToVisLog = bVisualLogShowAllRenderItems ? Component2DIndexer.GetValidValues() : Component2DIndexer.GetValidValuesForBitIndices(FinalComponentsToRenderInfoBitIndices);
								VisLogRenderItemInput(this, RenderItem.GetInputWorldArea(), EditLayerRendererRenderInfo, LandscapeTransform, LandscapeLoadedBounds, MakeArrayView(AllComponentsToVisLog));
							}

							if (bVisualLogShowRenderItemsOutput && (bOutputRenderItem || bVisualLogShowAllRenderItems))
							{
								TArray<ULandscapeComponent*> AllComponentsToVisLog = bVisualLogShowAllRenderItems ? Component2DIndexer.GetValidValues() : Component2DIndexer.GetValidValuesForBitIndices(FinalComponentsToRenderInfoBitIndices);
								VisLogRenderItemOutput(this, MergeRenderContext.IsHeightmapMerge(), bAffectsOutputLayerBitIndices, MergeRenderContext.ConvertTargetLayerBitIndicesToNames(FilteredRenderItemTargetTypeState.GetActiveWeightmapBitIndices()),
									OutputWorldArea, EditLayerRendererRenderInfo, LandscapeTransform, LandscapeLoadedBounds, MakeArrayView(AllComponentsToVisLog));
							}
						}
					});

				if (bOutputRenderItem)
				{
					TBitArray<> InputComponentBitIndices(false, NumAllComponents);
					const TBitArray<>* ComponentsToIterateBitIndices = nullptr;

					const FInputWorldArea& InputWorldArea = RenderItem.GetInputWorldArea();
					switch (InputWorldArea.GetType())
					{
					case FInputWorldArea::EType::Infinite:
					{
						InputComponentBitIndices = Component2DIndexer.GetValidValueBitIndices();
						ComponentsToIterateBitIndices = &InputComponentBitIndices;
						break;
					}
					case FInputWorldArea::EType::LocalComponent:
					{
						// This render item requires the component itself and potentially its neighbors, so we need to iterate on all the components currently being processed : 
						ComponentsToIterateBitIndices = &FinalComponentsToRenderInfoBitIndices;
						break;
					}
					case FInputWorldArea::EType::SpecificComponent:
					{
						InputComponentBitIndices = Component2DIndexer.GetValidValueBitIndicesInBounds(InputWorldArea.GetSpecificComponentKeys(), /* bInInclusiveBounds = */true);
						ComponentsToIterateBitIndices = &InputComponentBitIndices;
						break;
					}
					case FInputWorldArea::EType::OOBox:
					{
						FIntRect ComponentIndicesBoundingRect;
						TMap<FIntPoint, ULandscapeComponent*> DependentComponents;
						// TODO [jonathan.bard] : Change this to OOBB to OOBB test to cull more components
						const FOOBox2D& OOBox = InputWorldArea.GetOOBox();
						Info->GetOverlappedComponents(OOBox.Transform, FBox2D(-OOBox.Extents / 2.0, OOBox.Extents / 2.0), DependentComponents, ComponentIndicesBoundingRect);
						for (auto It : DependentComponents)
						{
							int32 ComponentIndex = Component2DIndexer.GetValueIndexForKeyChecked(It.Key);
							InputComponentBitIndices[ComponentIndex] = true;
						}
						ComponentsToIterateBitIndices = &InputComponentBitIndices;
						break;
					}
					default:
						check(false);
					}
					check(ComponentsToIterateBitIndices != nullptr);

					// List all target layers written by this render item :
					TBitArray<> OutputLayerBitIndices;
					if (bAffectsOutputLayerBitIndices)
					{
						if (MergeRenderContext.IsHeightmapMerge())
						{
							// Only one target layer in the case of a heightmap merge
							check(MergeRenderContext.AllTargetLayerNames.Num() == 1);
							OutputLayerBitIndices.SetNum(1, true);
						}
						else
						{
							OutputLayerBitIndices = FilteredRenderItemTargetTypeState.GetActiveWeightmapBitIndices();
						}
					}

					// Inform the renderer about this how this render item affects it :
					FEditLayerRendererRenderItemRenderInfo& RenderItemRenderInfo = EditLayerRendererRenderInfo.RenderItemRenderInfos.Emplace_GetRef(RenderItem, OutputLayerBitIndices, NumAllComponents);

					// Iterate on all the required components :
					for (TConstSetBitIterator It(*ComponentsToIterateBitIndices); It; ++It)
					{
						const int32 ComponentToRenderIndex = It.GetIndex();
						FComponentToRenderInfo& ComponentToRenderInfo = AllComponentsToRenderInfos[ComponentToRenderIndex];
						check(ComponentToRenderInfo.Component != nullptr);

						// Add the render item's target layers to the component's own : 
						if (bAffectsOutputLayerBitIndices)
						{
							ComponentToRenderInfo.TargetLayerBitIndices.CombineWithBitwiseOR(OutputLayerBitIndices, EBitwiseOperatorFlags::MinSize);
						}

						// Special case for FInputWorldArea::EType::LocalComponent, where the input components are specific to the component being iterated : 
						if (InputWorldArea.GetType() == FInputWorldArea::EType::LocalComponent)
						{
							InputComponentBitIndices = Component2DIndexer.GetValidValueBitIndicesInBounds(InputWorldArea.GetLocalComponentKeys(ComponentToRenderInfo.Component->GetComponentKey()), /* bInInclusiveBounds = */true);
						}

						// There should always be a dependency between this renderer and the previous in the stack for the component itself : 
						InputComponentBitIndices[ComponentToRenderIndex] = true;

						// Tell the render item which component it needs : 
						RenderItemRenderInfo.RenderedComponentBitIndices.CombineWithBitwiseOR(InputComponentBitIndices, EBitwiseOperatorFlags::MinSize);

						// If these components are not yet in the final list of components to render, add them: 
						ComponentsToRenderInfoBitIndicesAfterRenderer.CombineWithBitwiseOR(InputComponentBitIndices, EBitwiseOperatorFlags::MinSize);

						// Add these components to the list that this renderer needs to render: 
						EditLayerRendererRenderInfo.ComponentToRenderInfoBitIndices.CombineWithBitwiseOR(InputComponentBitIndices, EBitwiseOperatorFlags::MinSize);

						// Finally add these components as dependencies to the component we're trying to render : 
						ComponentToRenderInfo.DependentComponentBitIndices.CombineWithBitwiseOR(InputComponentBitIndices, EBitwiseOperatorFlags::MinSize);

#if ENABLE_VISUAL_LOG
						if (VisualLogDependencyHelper.IsSet())
						{
							for (TConstSetBitIterator ItInputComponent(InputComponentBitIndices); ItInputComponent; ++ItInputComponent)
							{
								// Register a dependency from the component we want to render towards all of the components its input area overlaps with on the previous renderer : 
								const int32 DependentComponentIndex = ItInputComponent.GetIndex();
								VisLogDependency(DependentComponentIndex, CurrentRendererIndex - 1, ComponentToRenderIndex, CurrentRendererIndex);
							}
						}
#endif // ENABLE_VISUAL_LOG
					}
				}
				else // !bOutputRenderItem
				{
#if ENABLE_VISUAL_LOG
					// Declare a passthrough dependency between each component on this renderer to the next, to display the full chain of dependencies : 
					if (VisualLogDependencyHelper.IsSet())
					{
						for (TConstSetBitIterator It(ComponentsToRenderInfoBitIndicesAfterRenderer); It; ++It)
						{
							const int32 ComponentToRenderIndex = It.GetIndex();
							VisLogDependency(ComponentToRenderIndex, CurrentRendererIndex - 1, ComponentToRenderIndex, CurrentRendererIndex);
						}
					}
#endif // ENABLE_VISUAL_LOG
				}
			}

			// The renderer has been fully processed, now we can update the list of components to render for the next renderer in line : 
			Swap(ComponentsToRenderInfoBitIndicesAfterRenderer, FinalComponentsToRenderInfoBitIndices);
		}
	}

	for (TConstSetBitIterator It(FinalComponentsToRenderInfoBitIndices); It; ++It)
	{
		const int32 ComponentIndex = It.GetIndex();
		FComponentToRenderInfo& ComponentToRenderInfo = AllComponentsToRenderInfos[ComponentIndex];
		// Now compute the bounds to finalize this component render info (it's faster to do via Component2DIndexer than iterating through components) : 
		FIntRect DependentComponentsInclusiveBounds = Component2DIndexer.GetValidValuesBoundsForBitIndices(ComponentToRenderInfo.DependentComponentBitIndices, /*bInInclusiveBounds = */true);
		ComponentToRenderInfo.Finalize(DependentComponentsInclusiveBounds, ComponentSizeQuads);
	}

#if ENABLE_VISUAL_LOG
	if (VisualLogDependencyHelper.IsSet() && (VisualLogShowComponentDependencies > 0))
	{
		// Display a node for every component that will be rendered in the end: 
		for (TConstSetBitIterator It(FinalComponentsToRenderInfoBitIndices); It; ++It)
		{
			VisualLogDependencyHelper->AddNode(AllComponentsToRenderInfos[It.GetIndex()], OrderedEditLayerRendererRenderInfos[0]);
		}
	}
#endif // ENABLE_VISUAL_LOG

	FIntRect LandscapeExtent;
	verify(Info->GetLandscapeExtent(LandscapeExtent));

	// Now divide the work into batches as large as possible (but fitting in the desired max batch resolution, if possible): 
	TArray<FRenderBatchInfo> AllBatchInfos = DivideIntoBatches(FinalComponentsToRenderInfoBitIndices, AllComponentsToRenderInfos, bWarnedLayerMergeResolution, bWarnedGlobalMergeDimensionsExceeded);
	
	// Early out if we will fail to render : 
	if (AllBatchInfos.IsEmpty())
	{
		return DummyMergeRenderContext;
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PrepareBatches);
		MergeRenderContext.RenderBatches.Reserve(AllBatchInfos.Num());
		MergeRenderContext.TargetLayersToComponents.AddDefaulted(MergeRenderContext.AllTargetLayerNames.Num());
		for (const FRenderBatchInfo& BatchInfo : AllBatchInfos)
		{
			int32 MergeBatchIndex = MergeRenderContext.RenderBatches.Num();
			FMergeRenderBatch& MergeRenderBatch = MergeRenderContext.RenderBatches.Emplace_GetRef();
			MergeRenderBatch.Landscape = this;
			MergeRenderBatch.LandscapeExtent = LandscapeExtent;
			MergeRenderBatch.SectionRect = BatchInfo.CombinedSectionRect;
			MergeRenderBatch.MinComponentKey = MergeRenderBatch.MinComponentKey.ComponentMin(BatchInfo.MinComponentKey);
			MergeRenderBatch.MaxComponentKey = MergeRenderBatch.MaxComponentKey.ComponentMax(BatchInfo.MaxComponentKey);
			// We'll reuse the same merge render targets in order to generate the mips, which include the duplicate borders, so we need to expand the render target's size to accommodate for this : 
			MergeRenderBatch.Resolution = (MergeRenderBatch.MaxComponentKey - MergeRenderBatch.MinComponentKey + 1) * NumSubsections * (SubsectionSizeQuads + 1);
			MergeRenderBatch.TargetLayersToComponents.AddDefaulted(MergeRenderContext.AllTargetLayerNames.Num());
			MergeRenderBatch.TargetLayerBitIndices.SetNum(MergeRenderContext.AllTargetLayerNames.Num(), false);

			MergeRenderContext.MaxNeededResolution = MergeRenderContext.MaxNeededResolution.ComponentMax(MergeRenderBatch.Resolution);
			MergeRenderContext.MaxNumComponents = MergeRenderContext.MaxNumComponents.ComponentMax(MergeRenderBatch.MaxComponentKey - MergeRenderBatch.MinComponentKey + 1);

			// For each renderer, find the list of components actually needed and store that in a separate array, to avoid recomputing it for each target group : 
			int32 LocalNumRenderers = OrderedEditLayerRendererRenderInfos.Num();
			TArray<FPerBatchEditLayerRendererRenderInfo> BatchEditLayerRendererRenderInfos;
			BatchEditLayerRendererRenderInfos.SetNum(LocalNumRenderers);
			for (int32 EditLayerRendererIndex = 0; EditLayerRendererIndex < LocalNumRenderers; ++EditLayerRendererIndex)
			{
				const FEditLayerRendererRenderInfo& EditLayerRendererRenderInfo = OrderedEditLayerRendererRenderInfos[EditLayerRendererIndex];
				// Find out all components that are in common between the renderer's list and the batch's list : only these will need to be rendered in that render step : 
				FPerBatchEditLayerRendererRenderInfo& PerBatchEditLayerRendererRenderInfo = BatchEditLayerRendererRenderInfos[EditLayerRendererIndex];
				PerBatchEditLayerRendererRenderInfo.ComponentsToRenderBitIndices = TBitArray<>::BitwiseAND(EditLayerRendererRenderInfo.ComponentToRenderInfoBitIndices, BatchInfo.ComponentToRenderInfoBitIndices, EBitwiseOperatorFlags::MinSize);
				PerBatchEditLayerRendererRenderInfo.ComponentsToRender.Reserve(PerBatchEditLayerRendererRenderInfo.ComponentsToRenderBitIndices.CountSetBits());
				// Transform the bit indices back into a proper component list : 
				for (TConstSetBitIterator ItComponent(PerBatchEditLayerRendererRenderInfo.ComponentsToRenderBitIndices); ItComponent; ++ItComponent)
				{
					FComponentToRenderInfo& ComponentToRenderInfo = AllComponentsToRenderInfos[ItComponent.GetIndex()];
					PerBatchEditLayerRendererRenderInfo.ComponentsToRender.Add(ComponentToRenderInfo.Component);

					// Inform the render batch and context of the target layer names associated with each component :

					// Declare these target layers as being in use for the batch :
					MergeRenderBatch.TargetLayerBitIndices.CombineWithBitwiseOR(ComponentToRenderInfo.TargetLayerBitIndices, EBitwiseOperatorFlags::MinSize);
					MergeRenderBatch.ComponentToTargetLayerBitIndices.FindOrAdd(ComponentToRenderInfo.Component).CombineWithBitwiseOR(ComponentToRenderInfo.TargetLayerBitIndices, EBitwiseOperatorFlags::MaxSize); // Use EBitwiseOperatorFlags::MaxSize here in order to allocate NumTargetLayerNames entries to the resulting bit array in case FindOrAdd is an add
					MergeRenderContext.ComponentToTargetLayerBitIndices.FindOrAdd(ComponentToRenderInfo.Component).CombineWithBitwiseOR(ComponentToRenderInfo.TargetLayerBitIndices, EBitwiseOperatorFlags::MaxSize); // Use EBitwiseOperatorFlags::MaxSize here in order to allocate NumTargetLayerNames entries to the resulting bit array in case FindOrAdd is an add
					for (TConstSetBitIterator ItTargetLayer(ComponentToRenderInfo.TargetLayerBitIndices); ItTargetLayer; ++ItTargetLayer)
					{
						const int32 TargetLayerIndex = ItTargetLayer.GetIndex();
						MergeRenderBatch.TargetLayersToComponents[TargetLayerIndex].Add(ComponentToRenderInfo.Component);
						MergeRenderContext.TargetLayersToComponents[TargetLayerIndex].Add(ComponentToRenderInfo.Component);
					}
				}

				// Declare these components as being in use for the batch : 
				MergeRenderBatch.ComponentsToRender.Append(PerBatchEditLayerRendererRenderInfo.ComponentsToRender);
			}

			// Now, we have all the info to build our list of successive render steps : process group by group :
			for (const TBitArray<>& TargetLayerGroup : FinalTargetLayerGroups)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(BuildRenderSteps);
				bool bInRecordedSequence = false;
				int32 RenderLayerGroupStartIndex = INDEX_NONE;
				int32 RenderLayerGroupEndIndex = INDEX_NONE;

				// Execute the required operations for the whole stack of renderers for this group :
				for (int32 RendererIndex = 0; RendererIndex < LocalNumRenderers; ++RendererIndex)
				{
					const FEditLayerRendererRenderInfo& EditLayerRendererRenderInfo = OrderedEditLayerRendererRenderInfos[RendererIndex];
					TArrayView<ULandscapeComponent*> ComponentsToRender = MakeArrayView(BatchEditLayerRendererRenderInfos[RendererIndex].ComponentsToRender);
					const ERenderFlags CurrentRenderFlags = EditLayerRendererRenderInfo.RendererState.GetRenderer()->GetRenderFlags(&MergeRenderContext);
					const ERenderFlags CurrentRenderMode = CurrentRenderFlags & ERenderFlags::RenderMode_Mask;
					checkf(FMath::CountBits(static_cast<uint8>(CurrentRenderMode)) == 1, TEXT("Render mode should be either Immediate or Recorded"));
					const bool bIsLastRenderer = (RendererIndex == (LocalNumRenderers - 1));
					const bool bHasSeparateBlend = EnumHasAnyFlags(CurrentRenderFlags, ERenderFlags::BlendMode_SeparateBlend);
					const bool bSupportsGrouping = EnumHasAnyFlags(CurrentRenderFlags, ERenderFlags::RenderLayerGroup_SupportsGrouping);
					checkf(!bSupportsGrouping || bHasSeparateBlend, TEXT("RenderLayerGroup_SupportsGrouping expects BlendMode_SeparateBlend"));

					// TODO [jonathan.bard] : take into account target layer bit indices : only take into account render items that output the same layers as this target layer group
					auto IsCompatibleWithRenderLayerGroup = [&]() -> bool
						{
							// Global switch to disable grouping altogether for debug purposes : 
							// Note : in order not to force the renderers to implement the grouping AND non-grouping behavior, we simply force render groups to contain a single element :
							if (!bEnableRenderLayerGrouping)
							{
								return false;
							}

							check((RenderLayerGroupStartIndex != INDEX_NONE) && (RendererIndex > RenderLayerGroupStartIndex));

							const FPerBatchEditLayerRendererRenderInfo& PerBatchEditLayerRendererRenderInfo = BatchEditLayerRendererRenderInfos[RendererIndex];
							for (int32 OtherRendererIndex = RenderLayerGroupStartIndex; OtherRendererIndex < RendererIndex; ++OtherRendererIndex)
							{
								const FEditLayerRendererRenderInfo& OtherEditLayerRendererRenderInfo = OrderedEditLayerRendererRenderInfos[OtherRendererIndex];
								const FPerBatchEditLayerRendererRenderInfo& OtherPerBatchEditLayerRendererRenderInfo = BatchEditLayerRendererRenderInfos[OtherRendererIndex];

								// The renderer itself can tell whether it is compatible with others in that group : 
								if (!EditLayerRendererRenderInfo.RendererState.GetRenderer()->CanGroupRenderLayerWith(OtherEditLayerRendererRenderInfo.RendererState.GetRenderer()))
								{
									return false;
								}
								// The renderer is compatible with others in the layer group if none of its render items overlaps with any of the their respective render items (a blend is necessary in that case) : 
								// First, perform a first cheap test with the components overlapped by each renderer in this batch. If none match, no need to look further, the renderers are compatible: 
								else if (TBitArray<>::BitwiseAND(PerBatchEditLayerRendererRenderInfo.ComponentsToRenderBitIndices, OtherPerBatchEditLayerRendererRenderInfo.ComponentsToRenderBitIndices, EBitwiseOperatorFlags::MinSize).Contains(true))
								{
									// We have a potential overlap, let's test render item per render item now : 
									for (const FEditLayerRendererRenderItemRenderInfo& RenderItemRenderInfo : EditLayerRendererRenderInfo.RenderItemRenderInfos)
									{
										const TBitArray<> RenderItemRenderedComponentsThisBatch = TBitArray<>::BitwiseAND(RenderItemRenderInfo.RenderedComponentBitIndices, PerBatchEditLayerRendererRenderInfo.ComponentsToRenderBitIndices, EBitwiseOperatorFlags::MinSize);
										for (const FEditLayerRendererRenderItemRenderInfo& OtherRenderItemRenderInfo : OtherEditLayerRendererRenderInfo.RenderItemRenderInfos)
										{
											const TBitArray<> OtherRenderItemRenderedComponentsThisBatch = TBitArray<>::BitwiseAND(OtherRenderItemRenderInfo.RenderedComponentBitIndices, PerBatchEditLayerRendererRenderInfo.ComponentsToRenderBitIndices, EBitwiseOperatorFlags::MinSize);

											// Another cheap test, this time with the render item's rendered components : 
											if (TBitArray<>::BitwiseAND(RenderItemRenderedComponentsThisBatch, OtherRenderItemRenderedComponentsThisBatch, EBitwiseOperatorFlags::MinSize).Contains(true))
											{
												// If both render items are OOBBoxes, we can even have a finer-grained overlap test : 
												// TODO [jonathan.bard] : perform a OOBB to OOBB test here to reduce the potential overlaps :
												const FOOBox2D* RenderItemInputOOBox = RenderItemRenderInfo.RenderItem.GetInputWorldArea().TryGetOOBox();
												const FOOBox2D* RenderItemOutputOOBox = RenderItemRenderInfo.RenderItem.GetOutputWorldArea().TryGetOOBox();
												const FOOBox2D* OtherRenderItemInputOOBox = OtherRenderItemRenderInfo.RenderItem.GetInputWorldArea().TryGetOOBox();
												const FOOBox2D* OtherRenderItemOutputOOBox = OtherRenderItemRenderInfo.RenderItem.GetOutputWorldArea().TryGetOOBox();
												if ((RenderItemInputOOBox != nullptr) && (RenderItemOutputOOBox != nullptr)
													&& (OtherRenderItemInputOOBox != nullptr) && (OtherRenderItemOutputOOBox != nullptr))
												{
													FBox RenderItemAABB = RenderItemInputOOBox->BuildAABB() + RenderItemOutputOOBox->BuildAABB();
													FBox OtherRenderItemAABB = OtherRenderItemInputOOBox->BuildAABB() + OtherRenderItemOutputOOBox->BuildAABB();
													// If the 2 boxes overlap, the render item cannot be part of the group :
													if (RenderItemAABB.IntersectXY(OtherRenderItemAABB))
													{
														return false;
													}
												}
												else
												{
													// We cannot perform a more precise test, so consider it's an overlap, since we know that at least, the components rendered by these 2 render items overlap : 
													return false;
												}
											}
										}
									}
								}
							}
							return true;
						};

					auto AppendBeginRenderLayerGroupStep = [&]()
						{
							check((RenderLayerGroupStartIndex == INDEX_NONE) && (RenderLayerGroupEndIndex == INDEX_NONE));
							MergeRenderBatch.RenderSteps.Emplace(FMergeRenderStep::EType::BeginRenderLayerGroup, CurrentRenderFlags, EditLayerRendererRenderInfo.RendererState, TargetLayerGroup, ComponentsToRender);
							RenderLayerGroupStartIndex = RendererIndex;
						};

					auto AppendEndRenderLayerGroupAndBlendLayerSteps = [&]()
						{
							check((RenderLayerGroupStartIndex != INDEX_NONE) && (RenderLayerGroupEndIndex != INDEX_NONE));
							const FEditLayerRendererRenderInfo& LastEditLayerRendererRenderInfo = OrderedEditLayerRendererRenderInfos[RenderLayerGroupEndIndex];
							const ERenderFlags LastRenderFlags = LastEditLayerRendererRenderInfo.RendererState.GetRenderer()->GetRenderFlags(&MergeRenderContext);
							const ERenderFlags LastRenderMode = LastRenderFlags & ERenderFlags::RenderMode_Mask;
							TArrayView<ULandscapeComponent*> LastComponentsToRender = MakeArrayView(BatchEditLayerRendererRenderInfos[RenderLayerGroupEndIndex].ComponentsToRender);
							check((LastRenderMode == ERenderFlags::RenderMode_Recorded) == bInRecordedSequence); // when closing a group, the last blend step should be in the same mode as the last render step
							MergeRenderBatch.RenderSteps.Emplace(FMergeRenderStep::EType::EndRenderLayerGroup, LastRenderFlags, LastEditLayerRendererRenderInfo.RendererState, TargetLayerGroup, LastComponentsToRender);
							MergeRenderBatch.RenderSteps.Emplace(FMergeRenderStep::EType::BlendLayer, LastRenderFlags, LastEditLayerRendererRenderInfo.RendererState, TargetLayerGroup, LastComponentsToRender);
							RenderLayerGroupStartIndex = INDEX_NONE;
							RenderLayerGroupEndIndex = INDEX_NONE;
						};

					if (!ComponentsToRender.IsEmpty())
					{
						if (bSupportsGrouping)
						{
							// Start a new render layer group if this renderer supports it and none is currently being built
							if (RenderLayerGroupStartIndex == INDEX_NONE)
							{
								AppendBeginRenderLayerGroupStep();
							}
							// If a render layer group is currently being built but the renderer cannot be added to it, we need to stop the group, perform the blend and start a new group :
							else if (!IsCompatibleWithRenderLayerGroup())
							{
								AppendEndRenderLayerGroupAndBlendLayerSteps();
								AppendBeginRenderLayerGroupStep();
							}
						}
						// Stop the current render layer group if this renderer doesn't support it and one is currently being built
						else if (RenderLayerGroupStartIndex != INDEX_NONE)
						{
							AppendEndRenderLayerGroupAndBlendLayerSteps();
						}

						// Initiate the "render command recorder" sequence if necessary :
						if (CurrentRenderMode == ERenderFlags::RenderMode_Recorded)
						{
							if (!bInRecordedSequence)
							{
								MergeRenderBatch.RenderSteps.Emplace(FMergeRenderStep::EType::BeginRenderCommandRecorder);
								bInRecordedSequence = true;
							}
						}
						// Or terminate the "render command recorder" sequence if necessary :
						else if (bInRecordedSequence)
						{
							MergeRenderBatch.RenderSteps.Emplace(FMergeRenderStep::EType::EndRenderCommandRecorder);
							bInRecordedSequence = false;
						}

						// Render the content of this layer : 
						MergeRenderBatch.RenderSteps.Emplace(FMergeRenderStep::EType::RenderLayer, CurrentRenderFlags, EditLayerRendererRenderInfo.RendererState, TargetLayerGroup, ComponentsToRender);
						if (bSupportsGrouping)
						{
							RenderLayerGroupEndIndex = RendererIndex;
						}
						else
						{
							check((RenderLayerGroupStartIndex == INDEX_NONE) && (RenderLayerGroupEndIndex == INDEX_NONE));
						}

						// Add the blend step of this layer if it's separate and not part of an on-going group :
						if (bHasSeparateBlend && (RenderLayerGroupStartIndex == INDEX_NONE))
						{
							MergeRenderBatch.RenderSteps.Emplace(FMergeRenderStep::EType::BlendLayer, CurrentRenderFlags, EditLayerRendererRenderInfo.RendererState, TargetLayerGroup, ComponentsToRender);
						}
					}

					if (bIsLastRenderer)
					{
						// Terminate the current render layer group if necessary :
						if (RenderLayerGroupStartIndex != INDEX_NONE)
						{
							AppendEndRenderLayerGroupAndBlendLayerSteps();
						}

						// Terminate the current "render command recorder" sequence if necessary :
						if (bInRecordedSequence)
						{
							MergeRenderBatch.RenderSteps.Emplace(FMergeRenderStep::EType::EndRenderCommandRecorder);
							bInRecordedSequence = false;
						}
					}
				}
				check(!bInRecordedSequence);
				check((RenderLayerGroupStartIndex == INDEX_NONE) && (RenderLayerGroupEndIndex == INDEX_NONE));

				// Finally, signal the group as done for this batch
				MergeRenderBatch.RenderSteps.Emplace(FMergeRenderStep::EType::SignalBatchMergeGroupDone, TargetLayerGroup, MergeRenderBatch.ComponentsToRender.Array());

				MergeRenderContext.MaxNeededNumSlices = FMath::Max(MergeRenderContext.MaxNeededNumSlices, TargetLayerGroup.CountSetBits());
			}
		}

		// Last chance to bail out from the merge if we are beyond the capabilities for rendering batches
		if ((static_cast<int64>(MergeRenderContext.MaxNumComponents.X) * static_cast<int64>(MergeRenderContext.MaxNumComponents.Y)) >= static_cast<int64>(FMergeRenderContext::MaxComponentsPerBatch))
		{
			if (!bWarnedNumComponentsPerBatchExceeded)
			{
				UE_LOGF(LogLandscape, Error, "Cannot render landscape edit layers because one of the batches requires more than %u components, (component count : %ix%i). "
					"Please reduce the loaded landscape size, or make sure that all edit layer renderers participating to this landscape don't require an input area with "
					"this many components.",
					FMergeRenderContext::MaxComponentsPerBatch, MergeRenderContext.MaxNumComponents.X, MergeRenderContext.MaxNumComponents.Y);
				bWarnedNumComponentsPerBatchExceeded = true;
			}
			return DummyMergeRenderContext;
		}

		// Sort the batches in a deterministic order :
		MergeRenderContext.RenderBatches.Sort();
	}

	return MergeRenderContext;
}

int32 ALandscape::PerformLayersHeightmapsBatchedMerge(const FUpdateLayersContentContext& InUpdateLayersContentContext, const FEditLayersHeightmapMergeParams& InMergeParams)
{
	using namespace UE::Landscape::EditLayers;
	using namespace UE::Landscape::EditLayers::Private;

	TRACE_CPUPROFILER_EVENT_SCOPE(ALandscape::PerformLayersHeightmapsBatchedMerge);
	RHI_BREADCRUMB_EVENT_GAMETHREAD("PerformLayersHeightmapsBatchedMerge");

	FMergeContext MergeContext(/*InLandscape = */this, /*bInIsHeightmapMerge = */true, InMergeParams.bSkipBrush);

	// Prepare the heightmap merge operations for all components that need to be updated :
	TArray<FEditLayerRendererState> RendererStates = GetEditLayerRendererStates(&MergeContext);
	// Add an edit layer renderer at the top of the stack in order to add dependencies between each landscape component and its immediate neighbors in order to ensure they end up
	//  in the same render batch. The renderer is responsible for computing the normals at the end of the batch : 
	ULandscapeHeightmapNormalsEditLayerRenderer* HeightmapNormalsRenderer = ULandscapeHeightmapNormalsEditLayerRenderer::StaticClass()->GetDefaultObject<ULandscapeHeightmapNormalsEditLayerRenderer>();
	RendererStates.Emplace(&MergeContext, HeightmapNormalsRenderer);

	FMergeRenderParams MergeRenderParams(InUpdateLayersContentContext.LandscapeComponentsHeightmapsToRender, RendererStates);
	FMergeRenderContext MergeRenderContext = PrepareEditLayersMergeRenderContext(MergeContext, MergeRenderParams);
	if (!MergeRenderContext.IsValid())
	{
		// Nothing to do : 
		return InMergeParams.HeightmapUpdateModes;
	}

	// For each batch, render and resolve the raw heightmaps into the individual textures : 
	TSet<ULandscapeComponent*> ResolvedComponents;
	ResolvedComponents.Reserve(InUpdateLayersContentContext.LandscapeComponentsHeightmapsToRender.Num());

	// TODO [jonathan.bard] : this could also be recorded into RDGBuilderRecorder to avoid using additional FRDGBuilders at this step...
	// Callback executed each time a render batch is done computing the requested into, just before releasing the render resources : 
	auto OnRenderBatchGroupDone = [&ResolvedComponents, &InUpdateLayersContentContext, OnEditLayersMergedDelegate = &OnEditLayersMergedDelegate]
		(const FMergeRenderContext::FOnRenderBatchTargetGroupDoneParams& InParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder)
	{
		ALandscape* Landscape = InParams.MergeRenderContext->GetLandscape();
		const FMergeRenderBatch* RenderBatch = InParams.MergeRenderContext->GetCurrentRenderBatch();

		//  Note: thanks to HeightmapNormalsRenderer, we have the guarantee that the (up to) 8 neighbors of each of the components requested 
		//  for are present in the batch, which means we have all the data to generate the normals already 
		TSet<ULandscapeComponent*> ComponentsToResolveThisBatch;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(PrepareResolve);
			ComponentsToResolveThisBatch.Reserve(RenderBatch->ComponentsToRender.Num());
			for (ULandscapeComponent* RenderedComponent : RenderBatch->ComponentsToRender)
			{
				checkf(InParams.SortedComponentMergeRenderInfos.FindByPredicate([RenderedComponent](const FComponentMergeRenderInfo& InComponentMergeInfo) { return InComponentMergeInfo.Component == RenderedComponent; }) != nullptr,
					TEXT("All components in the batch must be present in SortedComponentMergeRenderInfos"));

				ULandscapeComponent** ResolvedComponent = ResolvedComponents.Find(RenderedComponent);
				if (ResolvedComponent == nullptr)
				{
					TStaticArray<ULandscapeComponent*, 9> NeighborComponents;
					RenderedComponent->GetLandscapeComponentNeighbors3x3(NeighborComponents);
					TSet<ULandscapeComponent*> ValidNeighborComponents;
					for (ULandscapeComponent* NeighborComponent : NeighborComponents)
					{
						if ((NeighborComponent != nullptr) && (NeighborComponent != RenderedComponent))
						{
							ValidNeighborComponents.Add(NeighborComponent);
						}
					}

					// We need all neighbors to be present in this batch in order to be able to finalize that component :
					if ((ValidNeighborComponents.Intersect(RenderBatch->ComponentsToRender).Num() == ValidNeighborComponents.Num())
						&& InUpdateLayersContentContext.LandscapeComponentsHeightmapsToResolve.Contains(RenderedComponent))
					{
						ResolvedComponents.Add(RenderedComponent);
						ComponentsToResolveThisBatch.Add(RenderedComponent);
					}
				}
			}
		}

		// Copy to mip0 of the final textures and expand the vertices on borders so that we can generate the mips from it:
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(CopyMip0AndExpand);
			RHI_BREADCRUMB_EVENT_GAMETHREAD("CopyMip0AndExpand");

			// Recompose mip0 of the final heightmaps, subsection by subsection, to duplicate borders : 
			InParams.MergeRenderContext->CycleBlendRenderTargets(RDGBuilderRecorder); 
			ULandscapeScratchRenderTarget* ReadRT = InParams.MergeRenderContext->GetBlendRenderTargetRead();

			ReadRT->TransitionTo(ERHIAccess::SRVMask, RDGBuilderRecorder); // TODO [jonathan.bard] : This should be CopyDst but ExecuteCopyLayersTexture doesn't allow for it ATM

			OnEditLayersMergedDelegate->Broadcast(FOnLandscapeEditLayersMergedParams(ReadRT->GetRenderTarget(),
				/*InRenderAreaResolution = */RenderBatch->GetRenderTargetResolution(/*bInWithDuplicateBorders = */false), /*bInIsHeightmapMerge = */true));

			ULandscapeScratchRenderTarget* WriteRT = InParams.MergeRenderContext->GetBlendRenderTargetWrite();

			struct FComponentCopyInfo
			{
				UTexture2D* Texture = nullptr;
				ULandscapeComponent* Component = nullptr;
				FIntPoint TextureOffset = FIntPoint::ZeroValue;
				TArray<FIntRect, TInlineAllocator<4>> SourceSubsectionRects;
				TArray<FIntRect, TInlineAllocator<4>> DestinationSubsectionRects;
			};
			TArray<FComponentCopyInfo> ComponentCopyInfos;
			const int32 TotalNumSubsections = Landscape->NumSubsections * Landscape->NumSubsections;
			const int32 ComponentSubsectionVerts = Landscape->SubsectionSizeQuads + 1;

			for (ULandscapeComponent* Component : RenderBatch->ComponentsToRender)
			{
				FComponentCopyInfo& ComponentCopyInfo = ComponentCopyInfos.Emplace_GetRef();
				ComponentCopyInfo.Component = Component;
				ComponentCopyInfo.Texture = Component->GetHeightmap(false);
				// At this point, the heightmaps we're writing to should have been fully compiled and streamed in by PrepareLayersResources, assert that this is the case : 
				check(!ComponentCopyInfo.Texture->IsDefaultTexture() && !ComponentCopyInfo.Texture->HasPendingInitOrStreaming() && ComponentCopyInfo.Texture->IsFullyStreamedIn());

				// Effective area of the texture affecting this component (because of texture sharing) :
				ComponentCopyInfo.TextureOffset = FIntPoint(static_cast<int32>(Component->HeightmapScaleBias.Z * ComponentCopyInfo.Texture->Source.GetSizeX()), static_cast<int32>(Component->HeightmapScaleBias.W * ComponentCopyInfo.Texture->Source.GetSizeY()));
		
				RenderBatch->ComputeSubsectionRects(Component, /*OutSubsectionRects = */ComponentCopyInfo.SourceSubsectionRects, /*OutSubsectionRectsWithDuplicateBorders = */ComponentCopyInfo.DestinationSubsectionRects);
				check(ComponentCopyInfo.SourceSubsectionRects.Num() == TotalNumSubsections);
				check(ComponentCopyInfo.DestinationSubsectionRects.Num() == TotalNumSubsections);
			}

			// TODO [jonathan.bard] : move this after expand (and rename "Expand" to "Generate mip 0")
			{
				RHI_BREADCRUMB_EVENT_GAMETHREAD("CopyToMip0");
				// Copy sub-section by sub-section in order to duplicate borders :
				TArray<FLandscapeLayersCopyTextureParams> DeferredCopyTextures;
				for (const FComponentCopyInfo& ComponentCopyInfo : ComponentCopyInfos)
				{
					for (int32 SubSectionIndex = 0; SubSectionIndex < TotalNumSubsections; ++SubSectionIndex)
					{
						FIntPoint SubSection(SubSectionIndex % Landscape->NumSubsections, SubSectionIndex / Landscape->NumSubsections);
						const FIntRect& SourceSubSectionRect = ComponentCopyInfo.SourceSubsectionRects[SubSectionIndex];
		
						// Copy to mip0 of the final texture if requested : 
						if (ComponentsToResolveThisBatch.Contains(ComponentCopyInfo.Component))
						{
							FLandscapeLayersCopyTextureParams& CopyTextureParams = DeferredCopyTextures.Add_GetRef(FLandscapeLayersCopyTextureParams(ReadRT->GetRenderTarget(), ComponentCopyInfo.Texture));
							CopyTextureParams.SourcePosition = SourceSubSectionRect.Min;
							CopyTextureParams.CopySize = SourceSubSectionRect.Size();
							CopyTextureParams.DestPosition = ComponentCopyInfo.TextureOffset + FIntPoint(SubSection.X * ComponentSubsectionVerts, SubSection.Y * ComponentSubsectionVerts);
						}
					}
				}
				ExecuteCopyLayersTexture(MoveTemp(DeferredCopyTextures));
			}

			// "Expand" the scratch render target : 
			// Copy sub-section by sub-section in order to duplicate borders :
			InParams.MergeRenderContext->RenderExpandedRenderTarget(RDGBuilderRecorder);
		}

		// Generate the mips from the expanded RT and copy to the final texture mips
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(GenerateMips);
			const int32 NumMips = (int32)FMath::CeilLogTwo(Landscape->SubsectionSizeQuads) + 1;
			RHI_BREADCRUMB_EVENT_GAMETHREAD_F("Generate remaining mips", "Generate %i remaining mips", NumMips - 1);

			FIntPoint CurrentMipResolution = RenderBatch->GetRenderTargetResolution(/*bInWithDuplicateBorders = */true); // Mips are generated after the borders have been duplicated
			FIntPoint CurrentMipSubsectionSize = Landscape->SubsectionSizeQuads + 1;
			for (int32 MipIndex = 1; MipIndex < NumMips; ++MipIndex)
			{
				InParams.MergeRenderContext->CycleBlendRenderTargets(RDGBuilderRecorder);
				ULandscapeScratchRenderTarget* WriteRT = InParams.MergeRenderContext->GetBlendRenderTargetWrite();
				ULandscapeScratchRenderTarget* ReadRT = InParams.MergeRenderContext->GetBlendRenderTargetRead();

				WriteRT->TransitionTo(ERHIAccess::RTV, RDGBuilderRecorder);
				ReadRT->TransitionTo(ERHIAccess::SRVMask, RDGBuilderRecorder);

				CurrentMipResolution /= 2;
				check(CurrentMipResolution.X > 0 && CurrentMipResolution.Y > 0);
				CurrentMipSubsectionSize /= 2;
				check(CurrentMipSubsectionSize.X > 0 && CurrentMipSubsectionSize.Y > 0);

				{
					RHI_BREADCRUMB_EVENT_GAMETHREAD_F("Generate mip", "Generate mip %i", MipIndex);

					ENQUEUE_RENDER_COMMAND(LandscapeLayers_Cmd_HeightmapsGenerateMips)(
						[ OutputResource = WriteRT->GetRenderTarget2D()->GetResource()
						, SourceResource = ReadRT->GetRenderTarget2D()->GetResource()
						, CurrentMipResolution
						, CurrentMipSubsectionSize
						, MipIndex ]
						(FRHICommandListImmediate& InRHICmdList)
					{
						FRDGBuilder GraphBuilder(InRHICmdList, RDG_EVENT_NAME("HeightmapsGenerateMips"));

						FRDGTextureRef OutputTextureRef = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(OutputResource->TextureRHI, TEXT("OutputTexture")));
						FRDGTextureRef SourceTextureRef = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(SourceResource->TextureRHI, TEXT("SourceTexture")));

						FLandscapeEditLayersHeightmapsGenerateMipsPS::FParameters* PSParams = GraphBuilder.AllocParameters<FLandscapeEditLayersHeightmapsGenerateMipsPS::FParameters>();
						PSParams->RenderTargets[0] = FRenderTargetBinding(OutputTextureRef, ERenderTargetLoadAction::ENoAction);
						PSParams->InCurrentMipSubsectionSize = FUintVector2(CurrentMipSubsectionSize.X, CurrentMipSubsectionSize.Y);
						PSParams->InSourceHeightmap = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(SourceTextureRef));
						FLandscapeEditLayersHeightmapsGenerateMipsPS::GenerateMipsPS(GraphBuilder, PSParams, CurrentMipResolution);

						// We need to specify the final state of the external texture to prevent the graph builder from transitioning it to SRVMask :
						GraphBuilder.SetTextureAccessFinal(OutputTextureRef, ERHIAccess::RTV);

						GraphBuilder.Execute();
					});
				}

				// Then copy the appropriate regions to the destination texture mips : 
				// TODO [jonathan.bard] : add this when we don't auto-transition to SRV in the copy texture thing : WriteRT->TransitionTo(ERHIAccess::CopySrc);
				{
					RHI_BREADCRUMB_EVENT_GAMETHREAD_F("Copy mip", "Copy mip %i", MipIndex);

					WriteRT->TransitionTo(ERHIAccess::SRVMask, RDGBuilderRecorder);
					TArray<FLandscapeLayersCopyTextureParams> DeferredCopyTextures;
					for (ULandscapeComponent* Component : ComponentsToResolveThisBatch)
					{
						UTexture2D* ComponentHeightmap = Component->GetHeightmap(false);

						FIntRect SourceSectionRect = RenderBatch->ComputeSectionRect(Component, /*bInWithDuplicateBorders = */true);
						SourceSectionRect.Min.X >>= MipIndex;
						SourceSectionRect.Min.Y >>= MipIndex;
						SourceSectionRect.Max.X >>= MipIndex;
						SourceSectionRect.Max.Y >>= MipIndex;

						// Effective area of the texture affecting this component (because of texture sharing) :
						FIntPoint TextureOffset = FIntPoint(static_cast<int32>(Component->HeightmapScaleBias.Z * ComponentHeightmap->Source.GetSizeX()), static_cast<int32>(Component->HeightmapScaleBias.W * ComponentHeightmap->Source.GetSizeY()));
						TextureOffset.X >>= MipIndex;
						TextureOffset.Y >>= MipIndex;

						FLandscapeLayersCopyTextureParams& CopyTextureParams = DeferredCopyTextures.Add_GetRef(FLandscapeLayersCopyTextureParams(WriteRT->GetRenderTarget(), ComponentHeightmap));
						CopyTextureParams.SourcePosition = SourceSectionRect.Min;
						CopyTextureParams.CopySize = SourceSectionRect.Size();
						CopyTextureParams.DestPosition = TextureOffset;
						CopyTextureParams.DestMip = MipIndex;
					}
					ExecuteCopyLayersTexture(MoveTemp(DeferredCopyTextures));
				}
			}
		}
	};

	// Render everything now. Every time a group from a batch is done (there's only one group per batch for heightmaps), the OnRenderBatchGroupDone callback is called : 
	MergeRenderContext.Render(OnRenderBatchGroupDone);

	// All requested components must have been resolved by now :
	check(ResolvedComponents.Num() == InUpdateLayersContentContext.LandscapeComponentsHeightmapsToResolve.Num());

	// Prepare the UTexture2D readbacks we'll need to perform :
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CopyToReadback);
		RHI_BREADCRUMB_EVENT_GAMETHREAD("CopyToReadback");

		TArray<FLandscapeLayersCopyReadbackTextureParams> DeferredCopyReadbackTextures = PrepareLandscapeLayersCopyReadbackTextureParams(InUpdateLayersContentContext.MapHelper, InUpdateLayersContentContext.HeightmapsToResolve.Array(), /*bWeightmaps = */false);
		ExecuteCopyToReadbackTexture(DeferredCopyReadbackTextures);
	}

	return InMergeParams.HeightmapUpdateModes;
}

bool ALandscape::PerformSelectiveRenderEditLayersHeightmapsCPU(const FUpdateLayersContentContext& InUpdateLayersContentContext, const FLandscapeEditLayerRenderHeightParams& InSelectiveRenderParams)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PerformSelectiveRenderEditLayersHeightmapsCPU);
	RHI_BREADCRUMB_EVENT_GAMETHREAD("PerformSelectiveRenderEditLayersHeightmapsCPU");

	using namespace UE::Landscape;
	using namespace UE::Landscape::Private;
	using namespace UE::Landscape::EditLayers;
	using namespace UE::Landscape::EditLayers::Private;

	check(InSelectiveRenderParams.CpuResult.Num() >= InSelectiveRenderParams.Bounds.Area());

	// Set up the merge batches similar to PerformLayersHeightmapsBatchedMerge.  Difference is that the edit layer visibility is overridden
	// and ULandscapeHeightmapNormalsEditLayerRenderer is omitted.  The completion callback is very different.
	// Note: not using SelectiveBounds in the actual Merge process, just to retrieve the data.  The render region is based only on the
	// components from InUpdateLayersContentContext.

	FMergeContext MergeContext(/*InLandscape = */this, /*bInIsHeightmapMerge = */true, !InSelectiveRenderParams.bRenderBrushes);
	TArray<FEditLayerRendererState> RendererStates = GetEditLayerRendererStatesEnableOverride(&MergeContext, InSelectiveRenderParams.ActiveEditLayers);

	FMergeRenderParams MergeRenderParams(InUpdateLayersContentContext.LandscapeComponentsHeightmapsToRender, RendererStates);
	FMergeRenderContext MergeRenderContext = PrepareEditLayersMergeRenderContext(MergeContext, MergeRenderParams);
	if (!MergeRenderContext.IsValid())
	{
		// Nothing to do 
		return false;
	}

	// Set up a FRHIGPUTextureReadback for each merge batch
	struct FSyncRenderBatchResult
	{
		FRHIGPUTextureReadback Readback = { "SyncRenderResult" };
		FIntRect SectionRect;
		FIntPoint Size;  // RT size
		TArray<FIntRect> ComponentRects;
	};
	TArray<FSyncRenderBatchResult> SyncRenderBatchResults;
	SyncRenderBatchResults.AddDefaulted(MergeRenderContext.GetRenderBatches().Num());  // allocate in advance

	// Batch completion callback
	auto EnqueueBatchReadback = [&SyncRenderBatchResults]
	(const FMergeRenderContext::FOnRenderBatchTargetGroupDoneParams& InParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ExtractSelectiveMergeData);
			RHI_BREADCRUMB_EVENT_GAMETHREAD("ExtractSelectiveMergeData");

			ALandscape* Landscape = InParams.MergeRenderContext->GetLandscape();
			const FMergeRenderBatch* RenderBatch = InParams.MergeRenderContext->GetCurrentRenderBatch();
			int32 CurrentBatch = InParams.MergeRenderContext->GetCurrentRenderBatchIdx();
			FSyncRenderBatchResult& BatchResult = SyncRenderBatchResults[CurrentBatch];

			BatchResult.SectionRect = RenderBatch->SectionRect;
			BatchResult.Size = RenderBatch->Resolution;  //Only used for sanity check on bounds
			BatchResult.ComponentRects = RenderBatch->ComputeAllComponentSectionRects(/*bInWithDuplicateBorders = */false);

			InParams.MergeRenderContext->CycleBlendRenderTargets(RDGBuilderRecorder);
			ULandscapeScratchRenderTarget* ReadRT = InParams.MergeRenderContext->GetBlendRenderTargetRead();

			// Enqueue async readback direct from the current scratch read target.
			ENQUEUE_RENDER_COMMAND(LandscapeLayers_Cmd_SyncExtractResults)(
				[&BatchResult,
				SourceResource = ReadRT->GetRenderTarget2D()->GetResource()]
				(FRHICommandListImmediate& InRHICmdList)
				{
					FRDGBuilder GraphBuilder(InRHICmdList, RDG_EVENT_NAME("HeightmapsExtractResult"));

					FRDGTextureRef SourceTextureRef = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(SourceResource->TextureRHI, TEXT("SourceTexture")));
					AddEnqueueCopyPass(GraphBuilder, &BatchResult.Readback, SourceTextureRef);

					// We need to specify the final state of the external texture to prevent the graph builder from transitioning it to SRVMask :
					GraphBuilder.SetTextureAccessFinal(SourceTextureRef, ERHIAccess::RTV);
					GraphBuilder.Execute();
				});
		};

	// Render everything now. Every time a group from a batch is done (there's only one group per batch for heightmaps), SyncExtractResults is called : 
	MergeRenderContext.Render(EnqueueBatchReadback);

	// Pick up readback results from SyncRenderBatchResults.  Enqueue this on the render thread because FRHIGPUTextureReadback can only be locked there.
	ENQUEUE_RENDER_COMMAND(LandscapeLayers_Cmd_SyncFetchResults)(
		[&SyncRenderBatchResults,
		DestRect = InSelectiveRenderParams.Bounds,
		Dest = InSelectiveRenderParams.CpuResult]
		(FRHICommandListImmediate& InRHICmdList)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(LockAndFetchReadbackData);
			const FBlitBuffer2DDesc DestDesc{ Dest.Num(), DestRect.Width(), DestRect };

			for (FSyncRenderBatchResult& BatchResult : SyncRenderBatchResults)
			{
				int32 BatchStride = 0;
				int32 BatchHeight = 0;
				uint32* SrcData = static_cast<uint32*>(BatchResult.Readback.Lock(BatchStride, &BatchHeight));
				int32 SrcBufferSize = BatchStride * BatchHeight; //the implied buffer size of the locked readback buffer, not necessarily of the native render target
				check(BatchResult.Size.X <= BatchStride && BatchResult.Size.Y <= BatchHeight);

				// The full batch rects are not all valid.  Only rects of each component in the batch are valid.  Blit one component at a time.
				for (const FIntRect& ComponentBatchRect : BatchResult.ComponentRects)
				{
					// ComponentBatchRect is relative to the batch RT.  Convert back into overall landscape coords.
					FIntRect ComponentRect = ComponentBatchRect + BatchResult.SectionRect.Min;
					ComponentRect.Max += FIntPoint(1, 1);  //quads to verts
					check(BatchResult.SectionRect.Contains(ComponentRect));
					
					const FBlitBuffer2DDesc SrcDesc{ SrcBufferSize, BatchStride, BatchResult.SectionRect };
					BlitHeightChannelsToUint16(Dest.GetData(), DestDesc, SrcData, SrcDesc, ComponentRect);
				}

				BatchResult.Readback.Unlock();
			}
		});

	// Sync the game thread with all of the above.  After this, the results are reliably accessible from the game thread.
	FlushRenderingCommands();

	return true;
}

bool ALandscape::PerformSelectiveRenderEditLayersHeightmapsRT(const FUpdateLayersContentContext& InUpdateLayersContentContext, const FLandscapeEditLayerRenderHeightParams& InSelectiveRenderParams)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PerformSelectiveRenderEditLayersHeightmapsRT);
	RHI_BREADCRUMB_EVENT_GAMETHREAD("PerformSelectiveRenderEditLayersHeightmapsRT");

	using namespace UE::Landscape;
	using namespace UE::Landscape::Private;
	using namespace UE::Landscape::EditLayers;
	using namespace UE::Landscape::EditLayers::Private;

	check(InSelectiveRenderParams.RTResult);

	FMergeContext MergeContext(/*InLandscape = */this, /*bInIsHeightmapMerge = */true, !InSelectiveRenderParams.bRenderBrushes);
	TArray<FEditLayerRendererState> RendererStates = GetEditLayerRendererStatesEnableOverride(&MergeContext, InSelectiveRenderParams.ActiveEditLayers);

	FMergeRenderParams MergeRenderParams(InUpdateLayersContentContext.LandscapeComponentsHeightmapsToRender, RendererStates);
	FMergeRenderContext MergeRenderContext = PrepareEditLayersMergeRenderContext(MergeContext, MergeRenderParams);
	if (!MergeRenderContext.IsValid())
	{
		// Nothing to do 
		return false;
	}

	auto EnqueueCopyToResultRT = [&InSelectiveRenderParams, DestRT = InSelectiveRenderParams.RTResult]
	(const FMergeRenderContext::FOnRenderBatchTargetGroupDoneParams& InParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ExtractSelectiveMergeBatchData);
			RHI_BREADCRUMB_EVENT_GAMETHREAD("ExtractSelectiveMergeBatchData");

			ALandscape* Landscape = InParams.MergeRenderContext->GetLandscape();
			const FMergeRenderBatch* RenderBatch = InParams.MergeRenderContext->GetCurrentRenderBatch();

			InParams.MergeRenderContext->CycleBlendRenderTargets(RDGBuilderRecorder);
			ULandscapeScratchRenderTarget* SrcScratchRT = InParams.MergeRenderContext->GetBlendRenderTargetRead();
			UTextureRenderTarget2D* ReadRT = SrcScratchRT->GetRenderTarget2D();
			FIntRect DestRect = InSelectiveRenderParams.Bounds;

			TArray<FIntRect> ComponentRects = RenderBatch->ComputeAllComponentSectionRects(/*bInWithDuplicateBorders = */false);

			TArray<FRHICopyTextureInfo> ComponentCopyInfos;
			ComponentCopyInfos.Reserve(ComponentRects.Num());

			for (FIntRect ComponentRect : ComponentRects)
			{
				ComponentRect.Max += FIntPoint(1, 1);  //quads to verts

				FIntRect CopyRect = ComponentRect + RenderBatch->SectionRect.Min;  //coords in the landscape
				CopyRect.Clip(RenderBatch->SectionRect);    //  limit to the intersection of the component, the batch, and the destination buffer
				CopyRect.Clip(DestRect);
				FIntRect SrcRect = CopyRect - RenderBatch->SectionRect.Min;  // relative to the batch

				if (SrcRect.Area() <= 0)
				{
					continue;
				}

				check(SrcRect.Min.X >= 0 && SrcRect.Min.Y >= 0);
				check(SrcRect.Size().X <= RenderBatch->SectionRect.Size().X && SrcRect.Size().Y <= RenderBatch->SectionRect.Size().Y);
				FIntPoint DestPosition = CopyRect.Min - DestRect.Min;

				FRHICopyTextureInfo Info;
				Info.Size = { SrcRect.Size().X, SrcRect.Size().Y, 1 };
				Info.SourcePosition = { SrcRect.Min.X, SrcRect.Min.Y, 0 };
				Info.DestPosition = { DestPosition.X, DestPosition.Y, 0 };

				ComponentCopyInfos.Add(Info);
			}

			// Enqueue async texture copy into the destination render target
			ENQUEUE_RENDER_COMMAND(LandscapeLayers_Cmd_ExtractSelectiveMergeBatchData)(
				[ReadRT, DestRT, CopyInfos = MoveTemp(ComponentCopyInfos)]
				(FRHICommandListImmediate& InRHICmdList)
				{
					FRDGBuilder GraphBuilder(InRHICmdList, RDG_EVENT_NAME("HeightmapsCopyComponentsToRT"));

					FTextureRenderTargetResource* SrcResource = ReadRT->GetRenderTargetResource();
					FTextureRenderTargetResource* DestResource = DestRT->GetRenderTargetResource();
					FRDGTextureRef SourceTextureRef = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(SrcResource->GetTextureRHI(), TEXT("CopySourceTexture")));
					FRDGTextureRef DestTextureRef = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(DestResource->GetTextureRHI(), TEXT("CopyDestTexture")));

					for (const FRHICopyTextureInfo& CopyInfo : CopyInfos)
					{
						AddCopyTexturePass(GraphBuilder, SourceTextureRef, DestTextureRef, CopyInfo);
					}

					// Leave the scratch target in RTV, as it requires.
					GraphBuilder.SetTextureAccessFinal(SourceTextureRef, ERHIAccess::RTV);
					GraphBuilder.Execute();
				});
		};

	// Render everything now. Every time a group from a batch is done (there's only one group per batch for heightmaps), the OnRenderBatchGroupDone callback is called : 
	MergeRenderContext.Render(EnqueueCopyToResultRT);

	// No flush.  Return without blocking.  The caller may need to flush if it wants to access the render target from the game thread.

	return true;
}

int32 ALandscape::RegenerateLayersHeightmaps(const FUpdateLayersContentContext& InUpdateLayersContentContext)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeLayers_RegenerateLayersHeightmaps);
	ULandscapeInfo* Info = GetLandscapeInfo();

	const int32 HeightmapUpdateModes = LayerContentUpdateModes & ELandscapeLayerUpdateMode::Update_Heightmap_Types;
	const bool bSkipBrush = CVarLandscapeLayerBrushOptim.GetValueOnAnyThread() == 1 && (HeightmapUpdateModes == ELandscapeLayerUpdateMode::Update_Heightmap_Editing);

	if ((HeightmapUpdateModes == 0) || Info == nullptr)
	{
		return 0;
	}

	// Nothing to do (return that we did the processing)
	if (InUpdateLayersContentContext.LandscapeComponentsHeightmapsToRender.Num() == 0)
	{
		return HeightmapUpdateModes;
	}

	// Lazily create CPU read back objects as required
	if (HeightmapUpdateModes)
	{
		for (ULandscapeComponent* Component : InUpdateLayersContentContext.LandscapeComponentsHeightmapsToRender)
		{
			UTexture2D* ComponentHeightmap = Component->GetHeightmap(false);
			ALandscapeProxy* Proxy = Component->GetLandscapeProxy();
			FLandscapeEditLayerReadback** CPUReadback = Proxy->HeightmapsCPUReadback.Find(ComponentHeightmap);
			if (CPUReadback == nullptr)
			{
				FLandscapeEditLayerReadback* NewCPUReadback = new FLandscapeEditLayerReadback();
				// gather the existing hash, pre-readback
				bool bHashIsValid = true;
				const uint64 Hash = ULandscapeTextureHash::CalculateTextureHash64(ComponentHeightmap, ELandscapeTextureType::Heightmap, bHashIsValid);
				NewCPUReadback->SetHash(Hash);
				Proxy->HeightmapsCPUReadback.Add(ComponentHeightmap, NewCPUReadback);
			}
		}
	}

	if (HeightmapUpdateModes)
	{
		RenderCaptureInterface::FScopedCapture RenderCapture((RenderCaptureLayersNextHeightmapDraws != 0), TEXT("LandscapeLayersHeightmapCapture"));
		RenderCaptureLayersNextHeightmapDraws = FMath::Max(0, RenderCaptureLayersNextHeightmapDraws - 1);

		FEditLayersHeightmapMergeParams MergeParams;
		MergeParams.HeightmapUpdateModes = HeightmapUpdateModes;
		MergeParams.bSkipBrush = bSkipBrush;

		return PerformLayersHeightmapsBatchedMerge(InUpdateLayersContentContext, MergeParams);
	}

	return 0;
}

void ALandscape::UpdateForChangedHeightmaps(const TArrayView<FLandscapeEditLayerComponentReadbackResult>& InComponentReadbackResults)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeLayers_UpdateForChangedHeightmaps);

	for (const FLandscapeEditLayerComponentReadbackResult& ComponentReadbackResult : InComponentReadbackResults)
	{
		// If the source data has changed, mark the component as needing a collision data update:
		//  - If ELandscapeComponentUpdateFlag::Component_Update_Heightmap_Collision is passed, it will be done immediately
		//  - If not, at least the component's collision data will still get updated eventually, when the flag is finally passed :
		if (ComponentReadbackResult.bModified)
		{
			ComponentReadbackResult.LandscapeComponent->SetPendingCollisionDataUpdate(true);
		}

		const uint32 HeightUpdateMode = ComponentReadbackResult.UpdateModes & (ELandscapeLayerUpdateMode::Update_Heightmap_All_Modes | ELandscapeLayerUpdateMode::Update_Heightmap_Editing | ELandscapeLayerUpdateMode::Update_Heightmap_Editing_NoCollision);

		// Only update collision if there was an actual change performed on the source data : 
		if (ComponentReadbackResult.LandscapeComponent->GetPendingCollisionDataUpdate())
		{
			if (IsUpdateFlagEnabledForModes(ELandscapeComponentUpdateFlag::Component_Update_Heightmap_Collision, HeightUpdateMode))
			{
				ComponentReadbackResult.LandscapeComponent->UpdateCachedBounds();
				ComponentReadbackResult.LandscapeComponent->UpdateComponentToWorld();

				// Avoid updating height field if we are going to recreate collision in this update
				bool bUpdateHeightfieldRegion = !IsUpdateFlagEnabledForModes(ELandscapeComponentUpdateFlag::Component_Update_Recreate_Collision, HeightUpdateMode);
				ComponentReadbackResult.LandscapeComponent->UpdateCollisionData(bUpdateHeightfieldRegion);
				ComponentReadbackResult.LandscapeComponent->SetPendingCollisionDataUpdate(false);
			}
			else if (IsUpdateFlagEnabledForModes(ELandscapeComponentUpdateFlag::Component_Update_Approximated_Bounds, HeightUpdateMode))
			{
				// Update bounds with an approximated value (real computation will be done anyways when computing collision)
				const bool bInApproximateBounds = true;
				ComponentReadbackResult.LandscapeComponent->UpdateCachedBounds(bInApproximateBounds);
				ComponentReadbackResult.LandscapeComponent->UpdateComponentToWorld();
			}
		}
	}
}

void ALandscape::ResolveLayersHeightmapTexture(
	FTextureToComponentHelper const& MapHelper,
	TSet<UTexture2D*> const& HeightmapsToResolve,
	TArray<FLandscapeEditLayerComponentReadbackResult>& InOutComponentReadbackResults)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeLayers_ResolveLayersHeightmapTexture);

	ULandscapeInfo* Info = GetLandscapeInfo();
	if (Info == nullptr || HeightmapsToResolve.Num() == 0)
	{
		return;
	}

	TArray<ULandscapeComponent*> ChangedComponents;
	for (UTexture2D* Heightmap : HeightmapsToResolve)
	{
		ALandscapeProxy* LandscapeProxy = Heightmap->GetTypedOuter<ALandscapeProxy>();
		check(LandscapeProxy);
		if (FLandscapeEditLayerReadback** CPUReadback = LandscapeProxy->HeightmapsCPUReadback.Find(Heightmap))
		{
			// Only process the readback object if it has active tasks.
			if (!(*CPUReadback)->HasActiveTasks())
			{
				continue;
			}

			const bool bChanged = ResolveLayersTexture(MapHelper, *CPUReadback, Heightmap, InOutComponentReadbackResults, /*bIsWeightmap = */false);
			if (bChanged)
			{
				ChangedComponents.Append(MapHelper.HeightmapToComponents[Heightmap]);
			}

			// issue the edge update request whether it was changed or not, as we need to update GPU edge hashes anyways
			if (TObjectPtr<ULandscapeComponent>* ComponentPtr = FLandscapeGroup::HeightmapTextureToActiveComponent.Find(Heightmap))
			{
				if (ULandscapeHeightmapTextureEdgeFixup* Fixup = (*ComponentPtr)->RegisteredEdgeFixup)
				{
					// since the texture source was just updated via GPU-readback, also update the GPU edge hashes when updating edge data
					const bool bUpdateGPUEdgeHashes = true;
					Fixup->RequestEdgeSnapshotUpdateFromHeightmapSource(bUpdateGPUEdgeHashes);
				}
			}
		}
	}

	const bool bInvalidateLightingCache = true;
	InvalidateGeneratedComponentData(ChangedComponents, bInvalidateLightingCache);
}

void ALandscape::ClearDirtyData(ULandscapeComponent* InLandscapeComponent)
{
	if (InLandscapeComponent->EditToolRenderData.DirtyTexture == nullptr)
	{
		return;
	}

	if (!CVarLandscapeTrackDirty.GetValueOnAnyThread())
	{
		return;
	}

	FLandscapeEditDataInterface LandscapeEdit(GetLandscapeInfo());
	const int32 X1 = InLandscapeComponent->GetSectionBase().X;
	const int32 X2 = X1 + ComponentSizeQuads;
	const int32 Y1 = InLandscapeComponent->GetSectionBase().Y;
	const int32 Y2 = Y1 + ComponentSizeQuads;
	const int32 ComponentWidth = ComponentSizeQuads + 1;
	const int32 DirtyDataSize = ComponentWidth * ComponentWidth;
	TUniquePtr<uint8[]> DirtyData = MakeUnique<uint8[]>(DirtyDataSize);
	FMemory::Memzero(DirtyData.Get(), DirtyDataSize);
	LandscapeEdit.SetDirtyData(X1, Y1, X2, Y2, DirtyData.Get(), 0);
}

void ALandscape::UpdateWeightDirtyData(ULandscapeComponent* InLandscapeComponent, UTexture2D const* InWeightmap, FColor const* InOldData, FColor const* InNewData, uint8 InChannel)
{
	check(InOldData && InNewData);

	FLandscapeEditDataInterface LandscapeEdit(GetLandscapeInfo());
	const int32 X1 = InLandscapeComponent->GetSectionBase().X;
	const int32 X2 = X1 + ComponentSizeQuads;
	const int32 Y1 = InLandscapeComponent->GetSectionBase().Y;
	const int32 Y2 = Y1 + ComponentSizeQuads;
	const int32 ComponentWidth = ComponentSizeQuads + 1;
	const int32 DirtyDataSize = ComponentWidth * ComponentWidth;
	const int32 SizeU = InWeightmap->Source.GetSizeX();
	const int32 SizeV = InWeightmap->Source.GetSizeY();
	const uint8 DirtyWeight = 1 << 1;

	TUniquePtr<uint8[]> DirtyData = MakeUnique<uint8[]>(DirtyDataSize);
	LandscapeEdit.GetDirtyData(X1, Y1, X2, Y2, DirtyData.Get(), 0);

	// COMMENT [jonathan.bard] : this isn't quite working, because of weightmap re-assignment during painting, which can lead to InOldData to be totally different than the previous frame, which 
	//  will mark pretty much everything as dirty. This will be this way until we stop using weightmap sharing in the tool
	FLandscapeComponentDataInterface CDI(InLandscapeComponent);
	for (int32 X = 0; X < ComponentWidth; ++X)
	{
		for (int32 Y = 0; Y < ComponentWidth; ++Y)
		{
			int32 TexX, TexY;
			CDI.VertexXYToTexelXY(X, Y, TexX, TexY);
			int32 TexIndex = TexX + TexY * SizeU;
			check(TexIndex < SizeU * SizeV);
			if (InOldData[TexIndex] != InNewData[TexIndex])
			{
				DirtyData[X + Y * ComponentWidth] |= DirtyWeight;
			}
		}
	}

	LandscapeEdit.SetDirtyData(X1, Y1, X2, Y2, DirtyData.Get(), 0);
}

void ALandscape::OnDirtyWeightmap(FTextureToComponentHelper const& MapHelper, UTexture2D const* InWeightmap, FColor const* InOldData, FColor const* InNewData, int32 InMipLevel, uint8 ChangedChannelsMask)
{
	using namespace UE::Landscape::Private;

	int32 DumpWeightmapDiff = CVarLandscapeDumpWeightmapDiff.GetValueOnGameThread();
	const bool bDumpDiff = (DumpWeightmapDiff > 0);
	const bool bDumpDiffAllMips = (DumpWeightmapDiff > 1);
	const bool bDumpDiffDetails = CVarLandscapeDumpDiffDetails.GetValueOnGameThread();
	const bool bTrackDirty = CVarLandscapeTrackDirty.GetValueOnGameThread() != 0;
	ULandscapeSubsystem* LandscapeSubsystem = GetWorld()->GetSubsystem<ULandscapeSubsystem>();
	check(LandscapeSubsystem != nullptr);
	const FDateTime CurrentTime = LandscapeSubsystem->GetAppCurrentDateTime();

	if ((!bDumpDiff && !bTrackDirty)
		|| (bDumpDiff && !bDumpDiffAllMips && (InMipLevel > 0))
		|| (bTrackDirty && (InMipLevel > 0)))
	{
		return;
	}

	check(ChangedChannelsMask != 0);

	TArray<ULandscapeComponent*> const* Components = MapHelper.WeightmapToComponents.Find(InWeightmap);
	if (Components != nullptr)
	{
		for (ULandscapeComponent* Component : *Components)
		{
			const TArray<UTexture2D*>& WeightmapTextures = Component->GetWeightmapTextures();
			const TArray<FWeightmapLayerAllocationInfo>& AllocInfos = Component->GetWeightmapLayerAllocations();

			for (FWeightmapLayerAllocationInfo const& AllocInfo : AllocInfos)
			{
				check(AllocInfo.IsAllocated() && AllocInfo.WeightmapTextureIndex < WeightmapTextures.Num());
				if (InWeightmap == WeightmapTextures[AllocInfo.WeightmapTextureIndex] 
					// Only dump if that particular weightmap channel has changed
					&& ((1 << AllocInfo.WeightmapTextureChannel) & ChangedChannelsMask) != 0)
				{
					if (bTrackDirty)
					{
						UpdateWeightDirtyData(Component, InWeightmap, InOldData, InNewData, AllocInfo.WeightmapTextureChannel);
					}

					if (bDumpDiff)
					{
						const int32 SizeU = InWeightmap->Source.GetSizeX() >> InMipLevel;
						const int32 SizeV = InWeightmap->Source.GetSizeY() >> InMipLevel;

						FString WorldName = GetWorld()->GetName();
						FString ParentLandscapeActorName = GetActorLabel();
						ALandscapeProxy* Proxy = Cast<ALandscapeProxy>(Component->GetOwner());
						check(Proxy);
						FString ActorName = Proxy->GetActorLabel();
						FString FilePattern = FString::Format(TEXT("{0}/LandscapeLayers/{1}/{2}/{3}/Weightmaps/{4}/{5}-{6}-{7}[mip{8}]"), 
							{ FPaths::ProjectSavedDir(), CurrentTime.ToString(), WorldName, ParentLandscapeActorName, AllocInfo.GetLayerName().ToString(), ActorName, Component->GetDebugName(), InWeightmap->GetName(), InMipLevel});

						FFileHelper::EColorChannel ColorChannel = GetWeightmapColorChannel(AllocInfo);
						FFileHelper::CreateBitmap(*(FilePattern + "_a(pre).bmp"), SizeU, SizeV, InOldData, /*SubRectangle = */nullptr, &IFileManager::Get(), /*OutFilename = */nullptr, /*bool bInWriteAlpha = */true, ColorChannel);
						FFileHelper::CreateBitmap(*(FilePattern + "_b(post).bmp"), SizeU, SizeV, InNewData, /*SubRectangle = */nullptr, &IFileManager::Get(), /*OutFilename = */nullptr, /*bool bInWriteAlpha = */true, ColorChannel);
			
						if (bDumpDiffDetails)
						{
							static const TCHAR* Channels = TEXT("RGBA");
							int32 NumDifferentPixels = 0;
							uint8 MaxDiff = 0;
							TStringBuilder<0> StrBuilder;
							FIntPoint MaxDiffUV(ForceInit);
							for (int32 V = 0; V < SizeV; ++V)
							{
								for (int32 U = 0; U < SizeU; ++U)
								{
									const FColor* OldDataPtr = InOldData + (V * SizeU + U);
									const FColor* NewDataPtr = InNewData + (V * SizeU + U);
									if (*OldDataPtr != *NewDataPtr)
									{
										uint32 OldValueAsUInt32 = OldDataPtr->ToPackedRGBA();
										uint8 OldValue = (OldValueAsUInt32 >> ((3 - AllocInfo.WeightmapTextureChannel) * 8)) & 0xff;
										uint32 NewValueAsUInt32 = NewDataPtr->ToPackedRGBA();
										uint8 NewValue = (NewValueAsUInt32 >> ((3 - AllocInfo.WeightmapTextureChannel) * 8)) & 0xff;
										uint8 Diff = (NewValue > OldValue) ? NewValue - OldValue : OldValue - NewValue;
										if (Diff > 0)
										{
											if (Diff > MaxDiff)
											{
												MaxDiffUV = FIntPoint(U, V);
												MaxDiff = Diff;
											}

											TCHAR Channel = (AllocInfo.WeightmapTextureChannel == 0) ? TEXT('R') 
												: (AllocInfo.WeightmapTextureChannel == 1) ? TEXT('G') 
												: (AllocInfo.WeightmapTextureChannel == 2) ? TEXT('B') 
												: TEXT('A');

											StrBuilder.Appendf(TEXT("Pixel (%4u,%4u) : RGBA ((%3u,%3u,%3u,%3u) -> (%3u,%3u,%3u,%3u)) : channel %c (%3u -> %3u, absdiff %3u)\n"), 
												U, V, OldDataPtr->R, OldDataPtr->G, OldDataPtr->B, OldDataPtr->A, NewDataPtr->R, NewDataPtr->G, NewDataPtr->B, NewDataPtr->A, Channels[AllocInfo.WeightmapTextureChannel], OldValue, NewValue, Diff);

											++NumDifferentPixels;
										}
									}
								}
							}
							StrBuilder.InsertAt(0, FString::Printf(TEXT("----------------------------------------\n")));
							StrBuilder.InsertAt(0, FString::Printf(TEXT("Max diff (at %s) = %u (%1.3f%%)\n"), *MaxDiffUV.ToString(), MaxDiff, 100.0 * static_cast<float>(MaxDiff) / MAX_uint8));
							StrBuilder.InsertAt(0, FString::Printf(TEXT("Num diffs = %u\n"), NumDifferentPixels));
							StrBuilder.InsertAt(0, FString::Printf(TEXT("Layer %s is packed in channel %c\n"), *AllocInfo.GetLayerName().ToString(), Channels[AllocInfo.WeightmapTextureChannel]));
							FFileHelper::SaveStringToFile(StrBuilder.ToView(), *(FilePattern + "_diff.txt"));
						}
					}
				}
			}
		}
	}
}

void ALandscape::UpdateHeightDirtyData(ULandscapeComponent* InLandscapeComponent, UTexture2D const* InHeightmap, FColor const* InOldData, FColor const* InNewData)
{
	check(InOldData && InNewData);

	FLandscapeEditDataInterface LandscapeEdit(GetLandscapeInfo());
	const int32 X1 = InLandscapeComponent->GetSectionBase().X;
	const int32 X2 = X1 + ComponentSizeQuads;
	const int32 Y1 = InLandscapeComponent->GetSectionBase().Y;
	const int32 Y2 = Y1 + ComponentSizeQuads;
	const int32 ComponentWidth = ComponentSizeQuads + 1;
	const int32 DirtyDataSize = ComponentWidth * ComponentWidth;
	TUniquePtr<uint8[]> DirtyData = MakeUnique<uint8[]>(DirtyDataSize);
	const int32 SizeU = InHeightmap->Source.GetSizeX();
	const int32 SizeV = InHeightmap->Source.GetSizeY();
	const int32 HeightmapOffsetX = static_cast<int32>(InLandscapeComponent->HeightmapScaleBias.Z * SizeU);
	const int32 HeightmapOffsetY = static_cast<int32>(InLandscapeComponent->HeightmapScaleBias.W * SizeV);
	const uint8 DirtyHeight = 1 << 0;
	LandscapeEdit.GetDirtyData(X1, Y1, X2, Y2, DirtyData.Get(), 0);

	FLandscapeComponentDataInterface CDI(InLandscapeComponent);
	for (int32 X = 0; X < ComponentWidth; ++X)
	{
		for (int32 Y = 0; Y < ComponentWidth; ++Y)
		{
			int32 TexX, TexY;
			CDI.VertexXYToTexelXY(X, Y, TexX, TexY);
			TexX += HeightmapOffsetX;
			TexY += HeightmapOffsetY;
			int32 TexIndex = TexX + TexY * SizeU;
			check(TexIndex < SizeU * SizeV);
			if (InOldData[TexIndex] != InNewData[TexIndex])
			{
				DirtyData[X + Y * ComponentWidth] |= DirtyHeight;
			}
		}
	}

	LandscapeEdit.SetDirtyData(X1, Y1, X2, Y2, DirtyData.Get(), 0);
}

void ALandscape::OnDirtyHeightmap(FTextureToComponentHelper const& MapHelper, UTexture2D const* InHeightmap, FColor const* InOldData, FColor const* InNewData, int32 InMipLevel)
{
	int32 DumpHeightmapDiff = CVarLandscapeDumpHeightmapDiff.GetValueOnGameThread();
	const bool bDumpDiff = (DumpHeightmapDiff > 0);
	const bool bDumpDiffAllMips = (DumpHeightmapDiff > 1);
	const bool bDumpDiffDetails = CVarLandscapeDumpDiffDetails.GetValueOnGameThread();
	const bool bTrackDirty = CVarLandscapeTrackDirty.GetValueOnGameThread() != 0;
	ULandscapeSubsystem* LandscapeSubsystem = GetWorld()->GetSubsystem<ULandscapeSubsystem>();
	check(LandscapeSubsystem != nullptr);
	const FDateTime CurrentTime = LandscapeSubsystem->GetAppCurrentDateTime();

	if ((!bDumpDiff && !bTrackDirty)
		|| (bDumpDiff && !bDumpDiffAllMips && (InMipLevel > 0))
		|| (bTrackDirty && (InMipLevel > 0)))
	{
		return;
	}

	TArray<ULandscapeComponent*> const* Components = MapHelper.HeightmapToComponents.Find(InHeightmap);
	if (Components != nullptr)
	{
		for (ULandscapeComponent* Component : *Components)
		{
			if (bTrackDirty)
			{
				UpdateHeightDirtyData(Component, InHeightmap, InOldData, InNewData);
			}

			if (bDumpDiff)
			{
				FString WorldName = GetWorld()->GetName();
				FString ParentLandscapeActorName = GetActorLabel();
				ALandscapeProxy* Proxy = Cast<ALandscapeProxy>(Component->GetOwner());
				check(Proxy);
				FString ActorName = Proxy->GetActorLabel();
				FString FilePattern = FString::Format(TEXT("{0}/LandscapeLayers/{1}/{2}/{3}/Heightmaps/{4}-{5}-{6}[mip{7}]"), 
					{ FPaths::ProjectSavedDir(), CurrentTime.ToString(), WorldName, ParentLandscapeActorName, ActorName, Component->GetDebugName(), InHeightmap->GetName(), InMipLevel });

				const int32 SizeU = InHeightmap->Source.GetSizeX() >> InMipLevel;
				const int32 SizeV = InHeightmap->Source.GetSizeY() >> InMipLevel;
				const int32 HeightmapOffsetX = static_cast<int32>(Component->HeightmapScaleBias.Z * SizeU);
				const int32 HeightmapOffsetY = static_cast<int32>(Component->HeightmapScaleBias.W * SizeV);
				const int32 ComponentWidth = ((SubsectionSizeQuads + 1) * NumSubsections) >> InMipLevel;
				FIntRect SubRegion(HeightmapOffsetX, HeightmapOffsetY, HeightmapOffsetX + ComponentWidth, HeightmapOffsetY + ComponentWidth);

				int32 NumDifferentPixels = 0;
				uint16 MaxHeightDiff = 0;
				FIntPoint MaxHeightDiffUV(ForceInit);
				uint8 MaxNormalDiff = 0;
				FIntPoint MaxNormalDiffUV(ForceInit);
				TStringBuilder<0> StrBuilder;
				const FColor* OldDataStartPtr = InOldData + (HeightmapOffsetY * SizeU + HeightmapOffsetX);
				const FColor* NewDataStartPtr = InNewData + (HeightmapOffsetY * SizeU + HeightmapOffsetX);
				for (int32 V = 0; V < ComponentWidth; ++V)
				{
					for (int32 U = 0; U < ComponentWidth; ++U)
					{
						const FColor* OldDataPtr = OldDataStartPtr + (V * SizeU + U);
						const FColor* NewDataPtr = NewDataStartPtr + (V * SizeU + U);
						if (*OldDataPtr != *NewDataPtr)
						{
							uint16 OldHeight = ((static_cast<uint16>(OldDataPtr->R) << 8) | static_cast<uint16>(OldDataPtr->G));
							uint16 NewHeight = ((static_cast<uint16>(NewDataPtr->R) << 8) | static_cast<uint16>(NewDataPtr->G));
							uint16 HeightDiff = (NewHeight > OldHeight) ? NewHeight - OldHeight : OldHeight - NewHeight;
							if (HeightDiff > MaxHeightDiff)
							{
								MaxHeightDiffUV = FIntPoint(U, V);
								MaxHeightDiff = HeightDiff;
							}

							uint8 OldNormalX = (OldDataPtr->B);
							uint8 NewNormalX = (NewDataPtr->B);
							uint16 NormalXDiff = (NewNormalX > OldNormalX) ? NewNormalX - OldNormalX : OldNormalX - NewNormalX;
							if (NormalXDiff > MaxNormalDiff)
							{
								MaxNormalDiffUV = FIntPoint(U, V);
								MaxNormalDiff = NormalXDiff;
							}

							uint8 OldNormalY = (OldDataPtr->A);
							uint8 NewNormalY = (NewDataPtr->A);
							uint16 NormalYDiff = (NewNormalY > OldNormalY) ? NewNormalY - OldNormalY : OldNormalY - NewNormalY;
							if (NormalYDiff > MaxNormalDiff)
							{
								MaxNormalDiffUV = FIntPoint(U, V);
								MaxNormalDiff = NormalYDiff;
							}

							StrBuilder.Appendf(TEXT("Pixel (%4u,%4u) : Height (%5u -> %5u, absdiff %5u), Normal ((%3u,%3u) -> (%3u,%3u), absdiff %3u)\n"),
								U, V, OldHeight, NewHeight, HeightDiff, OldNormalX, OldNormalY, NewNormalX, NewNormalY, FMath::Max(NormalXDiff, NormalYDiff));

							++NumDifferentPixels;
						}
					}
				}

				if (NumDifferentPixels > 0)
				{
					FFileHelper::CreateBitmap(*(FilePattern + "_a(pre).bmp"), SizeU, SizeV, InOldData, &SubRegion, &IFileManager::Get(), /*OutFilename = */nullptr, /*bInWriteAlpha = */true);
					FFileHelper::CreateBitmap(*(FilePattern + "_b(post).bmp"), SizeU, SizeV, InNewData, &SubRegion, &IFileManager::Get(), /*OutFilename = */nullptr, /*bInWriteAlpha = */true);

					if (bDumpDiffDetails)
					{
						StrBuilder.InsertAt(0, FString::Printf(TEXT("----------------------------------------\n")));
						StrBuilder.InsertAt(0, FString::Printf(TEXT("Max normal diff (at %s) = %u (%1.3f%%)\n"), *MaxNormalDiffUV.ToString(), MaxNormalDiff, 100.0f * static_cast<float>(MaxNormalDiff) / MAX_uint8));
						StrBuilder.InsertAt(0, FString::Printf(TEXT("Max height diff (at %s) = %u (%1.3f%%)\n"), *MaxHeightDiffUV.ToString(), MaxHeightDiff, 100.0f * static_cast<float>(MaxHeightDiff) / MAX_uint16));
						StrBuilder.InsertAt(0, FString::Printf(TEXT("Num diffs = %u\n"), NumDifferentPixels));
						FFileHelper::SaveStringToFile(StrBuilder.ToView(), *(FilePattern + "_diff.txt"));
					}
				}
			}
		}
	}
}

bool ALandscape::ResolveLayersTexture(
	FTextureToComponentHelper const& MapHelper,
	FLandscapeEditLayerReadback* InCPUReadback,
	UTexture2D* InOutputTexture,
	TArray<FLandscapeEditLayerComponentReadbackResult>& InOutComponentReadbackResults,
	bool bIsWeightmap)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeLayers_ResolveLayersTexture);

	InCPUReadback->Tick();

	const int32 CompletedReadbackNum = InCPUReadback->GetCompletedResultNum();

	bool bUserTriggered = false;

	if (TArray<ULandscapeComponent*> const * Components = bIsWeightmap ? MapHelper.WeightmapToComponents.Find(InOutputTexture) : MapHelper.HeightmapToComponents.Find(InOutputTexture))
	{
		for (ULandscapeComponent* Component : *Components)
		{
			if (Component->GetUserTriggeredChangeRequested())
			{
				bUserTriggered = true;
				break;
			}
		}
	}

	bool bChanged = false;
	TOptional<uint8> ChangedChannelsMask;
	if (bIsWeightmap)
	{
		// Request a precise report of which channel have changed if we need to dump the weightmap diffs :
		if ((CVarLandscapeDumpWeightmapDiff.GetValueOnGameThread() != 0)
			|| CVarLandscapeTrackDirty.GetValueOnGameThread() != 0)
		{
			ChangedChannelsMask.Emplace(0);
		}
	}

	if (CompletedReadbackNum > 0)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeLayers_PerformReadbacks);

		// Copy final result to texture source.
		TArray<TArray<FColor>> const& NewMipsData = InCPUReadback->GetResult(CompletedReadbackNum - 1);

		ELandscapeTextureType TextureType = bIsWeightmap ? ELandscapeTextureType::Weightmap : ELandscapeTextureType::Heightmap;

		// Keep track if we locked the texture for write or not.  bChanged implies bLockedForWrite, but not the other way around.  We might have
		// bLockedForWrite because of a data hash change, but bChanged=false due to the threshold check.
		bool bLockedForWrite = false;
		uint64 NewHash = 0;
		uint64 OldHash = InCPUReadback->GetHash();

		for (int8 MipIndex = 0; MipIndex < NewMipsData.Num(); ++MipIndex)
		{
			int32 MipTexels = NewMipsData[MipIndex].Num();
			if (MipTexels > 0)
			{
				FColor* MipDataWriteable = nullptr;
				const FColor* NewMipData = NewMipsData[MipIndex].GetData();

				// Do dirty detection on first mip.
				if (MipIndex == 0)
				{
					NewHash = ULandscapeTextureHash::CalculateTextureHash64(NewMipData, MipTexels, TextureType);
					if (NewHash != OldHash)
					{
						// Defer locking the texture for ReadWrite until after we know that the data is changing at all.  Unlocking after a ReadWrite causes an 
						// expensive rehashing using the slower hash function used in IoHash.  
						const FColor* OldMipDataReadOnly = (const FColor*) InOutputTexture->Source.LockMipReadOnly(MipIndex);
						check(OldMipDataReadOnly);

						if (ULandscapeTextureHash::DoesTextureDataChangeExceedThreshold(OldMipDataReadOnly, NewMipData, MipTexels, bIsWeightmap ? ELandscapeTextureType::Weightmap : ELandscapeTextureType::Heightmap, OldHash, NewHash, ChangedChannelsMask))
						{
							// convert the lock to a read/write lock (first have to release the existing read-only lock, then acquire a new read/write lock)
							InOutputTexture->Source.UnlockMip(MipIndex);
							OldMipDataReadOnly = nullptr;
							MipDataWriteable = (FColor*) InOutputTexture->Source.LockMip(MipIndex);
							check(!bLockedForWrite);
							bLockedForWrite = true;

							// We skip the dirty notice if it hasn't changed.
							bChanged |= InCPUReadback->SetHash(NewHash);
							if (bChanged)
							{
								// We're about to modify the texture's source data, the texture needs to know so that it can handle properly update cached platform data (additionally, the package needs to be dirtied) :
								ULandscapeInfo* LandscapeInfo = GetLandscapeInfo();
								if (GetDefault<ULandscapeSettings>()->LandscapeDirtyingMode == ELandscapeDirtyingMode::InLandscapeModeAndUserTriggeredChanges)
								{
									FLandscapeDirtyOnlyInModeScope Scope(LandscapeInfo, !bUserTriggered);
									LandscapeInfo->ModifyObject(InOutputTexture);
									if (ULandscapeTextureHash* TextureHash = InOutputTexture->GetAssetUserData<ULandscapeTextureHash>())
									{
										LandscapeInfo->ModifyObject(TextureHash);
									}
								}
								else
								{
									LandscapeInfo->ModifyObject(InOutputTexture);
									if (ULandscapeTextureHash* TextureHash = InOutputTexture->GetAssetUserData<ULandscapeTextureHash>())
									{
										LandscapeInfo->ModifyObject(TextureHash);
									}
								}
							}
						}
						else
						{
							// release the read-only lock
							InOutputTexture->Source.UnlockMip(MipIndex);
						}
					}
				}
				else if (bLockedForWrite)
				{
					// If we locked mip 0, lock the rest too, even though FTextureSource isn't tracking mip-level locking.
					MipDataWriteable = (FColor*) InOutputTexture->Source.LockMip(MipIndex);
					bLockedForWrite = true;
				}

				if (bChanged)
				{
					// issue callbacks before overwriting the MipData (so the callback can compare old vs new)
					check(bLockedForWrite && MipDataWriteable);
					if (bIsWeightmap)
					{
						OnDirtyWeightmap(MapHelper, InOutputTexture, (FColor*)MipDataWriteable, NewMipData, MipIndex, ChangedChannelsMask.Get(0));
					}
					else
					{
						OnDirtyHeightmap(MapHelper, InOutputTexture, (FColor*)MipDataWriteable, NewMipData, MipIndex);
					}
				}

				if (bLockedForWrite)
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(ReadbackToCPU);
					FMemory::Memcpy(MipDataWriteable, NewMipData, MipTexels * sizeof(FColor));
				}
			}
		}

		if (bLockedForWrite)
		{
			// Unlock all mips at once because there's a lock counter in FTextureSource that recomputes the content hash when reaching 0 (which means we'd recompute the hash several times over if we Lock/Unlock/Lock/Unloc/... for each mip ):
			for (int8 MipIndex = 0; MipIndex < NewMipsData.Num(); ++MipIndex)
			{
				if (NewMipsData[MipIndex].Num() > 0)
				{
					InOutputTexture->Source.UnlockMip(MipIndex);
				}
			}

			// update the hash (New hash if it's considered changed, otherwise continue to use the old hash)
			// this must happen after UnlockMip, so the SourceID is up to date.
			ULandscapeTextureHash::SetHash64(InOutputTexture, bChanged ? NewHash : OldHash, ELandscapeTextureUsage::FinalData, bIsWeightmap ? ELandscapeTextureType::Weightmap : ELandscapeTextureType::Heightmap);
		}

		// change lighting guid to be the hash of the source data (so we can use lighting guid to detect when it actually changes)
		InOutputTexture->SetLightingGuid(ULandscapeTextureHash::GetHash(InOutputTexture));

		// Find out whether some channels from this weightmap are now all zeros : 
		static constexpr uint32 AllChannelsAllZerosMask = 15;
		uint32 AllZerosTextureChannelMask = AllChannelsAllZerosMask;
		const bool bCheckForEmptyChannels = CVarLandscapeRemoveEmptyPaintLayersOnEdit.GetValueOnGameThread() != 0;
		if (bIsWeightmap && bCheckForEmptyChannels)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeLayers_AnalyzeWeightmap);
			const FColor* TextureData = reinterpret_cast<const FColor*>(InOutputTexture->Source.LockMipReadOnly(0));
			const int32 TexSize = NewMipsData[0].Num();
			// We can stop iterating as soon as all of the channels are non-zero :
			for (int32 Index = 0; (Index < TexSize) && (AllZerosTextureChannelMask != 0); ++Index)
			{
				AllZerosTextureChannelMask &= (((TextureData[Index].R == 0) ? 1 : 0) << 0)
					| (((TextureData[Index].G == 0) ? 1 : 0) << 1)
					| (((TextureData[Index].B == 0) ? 1 : 0) << 2)
					| (((TextureData[Index].A == 0) ? 1 : 0) << 3);
			}
			InOutputTexture->Source.UnlockMip(0);
		}

		// Process component flags from all result contexts.
		for (int32 ResultIndex = 0; ResultIndex < CompletedReadbackNum; ++ResultIndex)
		{
			const FLandscapeEditLayerReadback::FReadbackContext& ResultContext = InCPUReadback->GetResultContext(ResultIndex);
			for (const FLandscapeEditLayerReadback::FComponentReadbackContext& ComponentContext : ResultContext)
			{
				ULandscapeComponent** Component = GetLandscapeInfo()->XYtoComponentMap.Find(ComponentContext.ComponentKey);
				if (Component != nullptr && *Component != nullptr)
				{
					FLandscapeEditLayerComponentReadbackResult* ComponentReadbackResult = UE::Landscape::Private::FindOrAddByComponent(InOutComponentReadbackResults, *Component, ELandscapeLayerUpdateMode::Update_None);
					ComponentReadbackResult->UpdateModes |= ComponentContext.UpdateModes;
					ComponentReadbackResult->bModified |= bChanged ? 1 : 0;
				}
			}
		}

		// We need to find the weightmap layers that are effectively empty in order to let the component clean them up eventually :
		if (bIsWeightmap && bCheckForEmptyChannels && (AllZerosTextureChannelMask != 0))
		{
			// Only use the latest readback context, since it's the only one we've actually read back : 
			const FLandscapeEditLayerReadback::FReadbackContext& EffectiveResultContext = InCPUReadback->GetResultContext(CompletedReadbackNum - 1);
			while (AllZerosTextureChannelMask != 0)
			{
				int32 AllZerosTextureChannelIndex = NumBitsPerDWORD - 1 - FMath::CountLeadingZeros(AllZerosTextureChannelMask);
				for (const FLandscapeEditLayerReadback::FComponentReadbackContext& ComponentContext : EffectiveResultContext)
				{
					FName AllZerosLayerInfoName = ComponentContext.PerChannelLayerNames[AllZerosTextureChannelIndex];
					ULandscapeComponent** Component = GetLandscapeInfo()->XYtoComponentMap.Find(ComponentContext.ComponentKey);
					if (Component != nullptr && *Component != nullptr)
					{
						const TArray<FWeightmapLayerAllocationInfo>& WeightmapLayerAllocations = (*Component)->GetWeightmapLayerAllocations();
						const TArray<UTexture2D*>& WeightmapTextures = (*Component)->GetWeightmapTextures();
						for (const FWeightmapLayerAllocationInfo& WeightmapLayerAllocation : WeightmapLayerAllocations)
						{
							if (WeightmapLayerAllocation.IsAllocated())
							{
								UTexture2D* Texture = WeightmapTextures[WeightmapLayerAllocation.WeightmapTextureIndex];
								if ((Texture == InOutputTexture) && (AllZerosLayerInfoName == WeightmapLayerAllocation.LayerInfo->GetLayerName()))
								{
									FLandscapeEditLayerComponentReadbackResult* ComponentReadbackResult = InOutComponentReadbackResults.FindByPredicate([LandscapeComponent = *Component](const FLandscapeEditLayerComponentReadbackResult& Element) { return Element.LandscapeComponent == LandscapeComponent; });
									check(ComponentReadbackResult != nullptr);

									// Mark this layer info within this component as being all-zero :
									ComponentReadbackResult->AllZeroLayers.AddUnique(WeightmapLayerAllocation.LayerInfo);
								}
							}
						}
					}
				}
				AllZerosTextureChannelMask &= ~((uint32)1 << AllZerosTextureChannelIndex);
			}
		}

		// Release the processed read backs
		InCPUReadback->ReleaseCompletedResults(CompletedReadbackNum);
	}

	return bChanged;
}

void ALandscape::ReallocateLayersWeightmaps(FUpdateLayersContentContext& InUpdateLayersContentContext, 
	const TMap<ULandscapeComponent*, TArray<ULandscapeLayerInfoObject*>>& InPerComponentAllocations, 
	const TSet<ULandscapeComponent*>& InRestrictTextureSharingToComponents)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeLayers_ReallocateLayersWeightmaps);

	ULandscapeInfo* Info = GetLandscapeInfo();

	if (Info == nullptr)
	{
		return;
	}

	// Clear allocation data
	for (ULandscapeComponent* Component : InUpdateLayersContentContext.LandscapeComponentsWeightmapsToResolve)
	{
		TArray<FWeightmapLayerAllocationInfo>& BaseLayerAllocations = Component->GetWeightmapLayerAllocations();
		for (FWeightmapLayerAllocationInfo& BaseWeightmapAllocation : BaseLayerAllocations)
		{
			BaseWeightmapAllocation.Free();
		}

		TArray<TObjectPtr<ULandscapeWeightmapUsage>>& WeightmapTexturesUsage = Component->GetWeightmapTexturesUsage();
		for (int32 i = 0; i < WeightmapTexturesUsage.Num(); ++i)
		{
			ULandscapeWeightmapUsage* Usage = WeightmapTexturesUsage[i];
			check(Usage != nullptr);

			Usage->ClearUsage(Component);
		}
	}

	int32 NumToResolve = InUpdateLayersContentContext.LandscapeComponentsWeightmapsToResolve.Num();
	int32 NumNoAllocs = 0;
	int32 NumUndoFlagged = 0;
	int32 NumWithAdds = 0;
	int32 NumWithRemoves = 0;

	// Trim the components that don't need weightmaps anymore (e.g. all edit layers are made invisible : there were some
	// components in LandscapeComponentsWeightmapsToResolve but there aren't anymore now).
	// Record which components have no allocations or had previously had their allocations changed by undo/redo.  These might need InvalidateGeneratedComponentData to fix up dependencies.
	TSet<ULandscapeComponent*> ComponentsWithChangedAllocs;
	ComponentsWithChangedAllocs.Reserve(InUpdateLayersContentContext.LandscapeComponentsWeightmapsToResolve.Num());
	InUpdateLayersContentContext.LandscapeComponentsWeightmapsToResolve.RemoveAllSwap([&InPerComponentAllocations, &ComponentsWithChangedAllocs, &NumNoAllocs, &NumUndoFlagged](ULandscapeComponent* Component) -> bool
		{
			const TArray<ULandscapeLayerInfoObject*>* ComponentLayerAlloc = InPerComponentAllocations.Find(Component);
			check(ComponentLayerAlloc != nullptr);
			bool bNoAllocs = ComponentLayerAlloc->IsEmpty();
			NumNoAllocs += bNoAllocs ? 1 : 0;
			bool bUndoChangedAllocs = Component->GetUndoChangedWeightmapAllocs();

			if (bUndoChangedAllocs)
			{
				NumUndoFlagged++;
				ComponentsWithChangedAllocs.Add(Component);
				Component->SetUndoChangedWeightmapAllocs(false);  // Reset the flag since it's being handled.
			}
			return bNoAllocs;
		},
		EAllowShrinking::No);

	// Determine if the Final layer need to add/remove some alloc
	for (auto& ItPair : InPerComponentAllocations)
	{
		ULandscapeComponent* Component = ItPair.Key;
		const TArray<ULandscapeLayerInfoObject*>& ComponentLayerAlloc = ItPair.Value;
		TArray<FWeightmapLayerAllocationInfo>& ComponentBaseLayerAlloc = Component->GetWeightmapLayerAllocations();

		bool bRemoved = false;
		bool bAdded = false;

		// Deal with the one that need removal
		for (int32 i = ComponentBaseLayerAlloc.Num() - 1; i >= 0; --i)
		{
			const FWeightmapLayerAllocationInfo& Alloc = ComponentBaseLayerAlloc[i];

			if (!ComponentLayerAlloc.Contains(Alloc.LayerInfo))
			{
				bRemoved = true;
				ComponentBaseLayerAlloc.RemoveAt(i);
			}
		}

		// Then add the new one
		for (ULandscapeLayerInfoObject* LayerAlloc : ComponentLayerAlloc)
		{
			const bool AllocExist = ComponentBaseLayerAlloc.ContainsByPredicate([&LayerAlloc](const FWeightmapLayerAllocationInfo& BaseLayerAlloc) { return (LayerAlloc == BaseLayerAlloc.LayerInfo); });

			if (!AllocExist)
			{
				bAdded = true;
				ComponentBaseLayerAlloc.Add(FWeightmapLayerAllocationInfo(LayerAlloc));
			}
		}

		if (bRemoved || bAdded)
		{
			ComponentsWithChangedAllocs.Add(Component);

			NumWithRemoves += bRemoved ? 1 : 0;
			NumWithAdds += bAdded ? 1 : 0;
		}
	}

	// Realloc the weightmap so it will create proper texture (if needed) and will set the allocations information
	TSet<UTexture*> NewCreatedTextures;
	for (ULandscapeComponent* Component : InUpdateLayersContentContext.LandscapeComponentsWeightmapsToResolve)
	{
		// If requested, don't allow the component to use textures that have a weightmap in another component than those provided when reallocating weightmaps :
		NewCreatedTextures.Append(Component->ReallocateWeightmaps(/*DataInterface = */nullptr, /*InEditLayerGuid = */FGuid(), /*bInSaveToTransactionBuffer = */false, /*bool bInForceReallocate = */false, 
			/*InTargetProxy = */nullptr, &InRestrictTextureSharingToComponents));
	}

	// TODO: correctly only recreate what is required instead of everything..
	//GDisableAutomaticTextureMaterialUpdateDependencies = true;

	FTextureCompilingManager::Get().FinishCompilation(NewCreatedTextures.Array());
	FLandscapeTextureStreamingManager* TextureStreamingManager = GetWorld()->GetSubsystem<ULandscapeSubsystem>()->GetTextureStreamingManager();
	for (UTexture* Texture : NewCreatedTextures)
	{
		check(Texture);
		TextureStreamingManager->RequestTextureFullyStreamedInForever(Texture, /* bWaitForStreaming= */ true);
	}

	//GDisableAutomaticTextureMaterialUpdateDependencies = false;

	// Clean-up unused weightmap CPUReadback resources
	Info->ForEachLandscapeProxy([](ALandscapeProxy* Proxy)
	{
		TArray<UTexture2D*, TInlineAllocator<64>> EntriesToRemoveFromMap;
		for (auto& Pair : Proxy->WeightmapsCPUReadback)
		{
			UTexture2D* WeightmapTextureKey = Pair.Key;
			bool IsTextureReferenced = false;
			for (ULandscapeComponent* Component : Proxy->LandscapeComponents)
			{
				for (UTexture2D* WeightmapTexture : Component->GetWeightmapTextures(false))
				{
					if (WeightmapTexture == WeightmapTextureKey)
					{
						IsTextureReferenced = true;
						break;
					}
				}
			}
			if (!IsTextureReferenced)
			{
				EntriesToRemoveFromMap.Add(WeightmapTextureKey);
			}
		}

		if (EntriesToRemoveFromMap.Num())
		{
			for (UTexture2D* OldWeightmapTexture : EntriesToRemoveFromMap)
			{
				if (FLandscapeEditLayerReadback** CPUReadbackToDelete = Proxy->WeightmapsCPUReadback.Find(OldWeightmapTexture))
				{
					check(*CPUReadbackToDelete);
					delete* CPUReadbackToDelete;
					Proxy->WeightmapsCPUReadback.Remove(OldWeightmapTexture);
				}
			}
		}

		return true;
	});

	int32 NumInvalidated = ComponentsWithChangedAllocs.Num();

	// Very spammy logging for interactive edits.
	UE_LOGF(LogLandscape, VeryVerbose, "ReallocateLayersWeightmaps - Components ToResolve: %d, Invalidated: %d - Added: %d, Removed: %d, NoAlloc: %d, UndoFlagged: %d",
		NumToResolve, NumInvalidated, NumWithAdds, NumWithRemoves, NumNoAllocs, NumUndoFlagged);

	// When the last pixels are removed from a component it loses its allocations, which means we stop tracking it through the update pipeline.  Add a tracking
	// object if needed and mark it with bCleared so that the code later on will know to update the collision object.

	for (ULandscapeComponent* Component : ComponentsWithChangedAllocs)
	{
		FLandscapeEditLayerComponentReadbackResult* ComponentReadbackResult =  UE::Landscape::Private::FindOrAddByComponent(InUpdateLayersContentContext.AllLandscapeComponentReadbackResults, Component, ELandscapeLayerUpdateMode::Update_None);
		ComponentReadbackResult->bCleared = true;
		ComponentReadbackResult->bModified = true;
	}

	InvalidateGeneratedComponentData(ComponentsWithChangedAllocs, false);
	ValidateProxyLayersWeightmapUsage();

	InUpdateLayersContentContext.Refresh(FUpdateLayersContentContext::ERefreshFlags::RefreshWeightmapInfos | FUpdateLayersContentContext::ERefreshFlags::RefreshMapHelper);
}

struct FEditLayersWeightmapMergeParams
{
	int32 WeightmapUpdateModes;
	bool bSkipBrush;
};

int32 ALandscape::PerformLayersWeightmapsBatchedMerge(FUpdateLayersContentContext& InUpdateLayersContentContext, const FEditLayersWeightmapMergeParams& InMergeParams)
{
	using namespace UE::Landscape;
	using namespace UE::Landscape::EditLayers;
	using namespace UE::Landscape::EditLayers::Private;

	TRACE_CPUPROFILER_EVENT_SCOPE(ALandscape::PerformLayersWeightmapsBatchedMerge);
	RHI_BREADCRUMB_EVENT_GAMETHREAD("PerformLayersWeightmapsBatchedMerge");

	FMergeContext MergeContext(/*InLandscape = */this, /*bInIsHeightmapMerge = */false, InMergeParams.bSkipBrush);
	TArray<FEditLayerRendererState> RendererStates = GetEditLayerRendererStates(&MergeContext);

	const TSet<FName> RequestedWeightmapLayerNames(GetTargetLayerNames(/*bInIncludeVisibilityLayer = */ true));

	// Prepare the merge : 
	TSet<ULandscapeComponent*> LandscapeComponentsWeightmapsToRenderBefore(InUpdateLayersContentContext.LandscapeComponentsWeightmapsToRender);
	// Not yet ready to selectively render layers, so request all layers
	const bool bRequestAllLayers = true;
	FMergeRenderParams MergeRenderParams(InUpdateLayersContentContext.LandscapeComponentsWeightmapsToRender, RendererStates, RequestedWeightmapLayerNames, bRequestAllLayers);
	FMergeRenderContext MergeRenderContext = PrepareEditLayersMergeRenderContext(MergeContext, MergeRenderParams);
	if (!MergeRenderContext.IsValid())
	{
		return InMergeParams.WeightmapUpdateModes;
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ReallocateWeightmaps);
		TMap<ULandscapeComponent*, TArray<ULandscapeLayerInfoObject*>> PerComponentAllocations;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(PrepareComponentAllocations);
			PerComponentAllocations.Reserve(InUpdateLayersContentContext.LandscapeComponentsWeightmapsToResolve.Num());
			for (ULandscapeComponent* ComponentToResolve : InUpdateLayersContentContext.LandscapeComponentsWeightmapsToResolve)
			{
				const TBitArray<>* ComponentTargetLayerBitIndices = MergeRenderContext.ComponentToTargetLayerBitIndices.Find(ComponentToResolve);
				check(ComponentTargetLayerBitIndices != nullptr);
				PerComponentAllocations.Add(ComponentToResolve, MergeRenderContext.ConvertTargetLayerBitIndicesToLayerInfos(*ComponentTargetLayerBitIndices));
			}
		}

		// We don't want new components to be required for rendering because of ReallocateLayersWeightmaps, as that would so would require re-running the merge preparation step. 
		//  So we prevent new texture allocations from using textures from components that are not already in the list of components to render : 
		TSet<ULandscapeComponent*> RestrictTextureSharingToComponents(InUpdateLayersContentContext.LandscapeComponentsWeightmapsToRender);
		ReallocateLayersWeightmaps(InUpdateLayersContentContext, PerComponentAllocations, RestrictTextureSharingToComponents);

		checkf(Algo::AllOf(InUpdateLayersContentContext.LandscapeComponentsWeightmapsToRender, [&LandscapeComponentsWeightmapsToRenderBefore](ULandscapeComponent* InComponent) { return LandscapeComponentsWeightmapsToRenderBefore.Contains(InComponent); }), 
			TEXT("If this asserts, the list of components involved in the weightmaps being merged has changed, which would require re-running the merge preparation step. bInRestrictTextureSharingToTheseComponents should prevent this from happening"));
	}

	TSet<ULandscapeComponent*> ResolvedLandscapeComponents;
	// Key = texture, Value = channels mask resolved so far. If all channels have been resolved, we remove the entry as the texture is already fully resolved :
	TMap<UTexture2D*, uint8> TexturesNeedingResolve; 
	TexturesNeedingResolve.Reserve(InUpdateLayersContentContext.WeightmapsToResolve.Num());
	for (UTexture2D* Weightmap : InUpdateLayersContentContext.WeightmapsToResolve)
	{ 
		// At this point, the weightmaps we're writing to should have been fully compiled and streamed in by PrepareLayersResources, assert that this is the case : 
		check(!Weightmap->IsDefaultTexture() && !Weightmap->HasPendingInitOrStreaming() && Weightmap->IsFullyStreamedIn());
		TexturesNeedingResolve.Add(Weightmap, 0);
	}

	// Callback executed each time a render batch is done computing the requested into, just before releasing the render resources : 
	auto OnRenderBatchGroupDone = [&ResolvedLandscapeComponents, &TexturesNeedingResolve, &InUpdateLayersContentContext, OnEditLayersMergedDelegate = &OnEditLayersMergedDelegate]
		(const FMergeRenderContext::FOnRenderBatchTargetGroupDoneParams& InParams, FRDGBuilderRecorder& RDGBuilderRecorder)
	{
		// We can now finalize the weightmaps : since we don't use SUPPORTS_LANDSCAPE_EDITORONLY_UBER_MATERIAL yet, each component that has been rendered should have a complete set of weightmaps that has been regenerated
		//  already so we need to repack those into the appropriate weightmap channels

		ALandscape* Landscape = InParams.MergeRenderContext->GetLandscape();
		const FMergeRenderBatch* RenderBatch = InParams.MergeRenderContext->GetCurrentRenderBatch();

		// Cycle render targets one last time so we can read from the last RT we've written to :
		InParams.MergeRenderContext->CycleBlendRenderTargets(RDGBuilderRecorder);
		ULandscapeScratchRenderTarget* ReadRT = InParams.MergeRenderContext->GetBlendRenderTargetRead();

		ReadRT->TransitionTo(ERHIAccess::SRVMask, RDGBuilderRecorder);

		OnEditLayersMergedDelegate->Broadcast(FOnLandscapeEditLayersMergedParams(ReadRT->GetRenderTarget(),
			/*InRenderAreaResolution = */RenderBatch->GetRenderTargetResolution(/*bInWithDuplicateBorders = */false), /*bInIsHeightmapMerge = */false));

		const int32 TotalNumSubsections = Landscape->NumSubsections * Landscape->NumSubsections;
		FIntPoint MinWeightmapResolution(MAX_int32, MAX_int32);
		FIntPoint MaxWeightmapResolution(MIN_int32, MIN_int32);

		struct FWeightmapResolveInfo
		{
			// Weightmap to resolve : 
			FTextureResource* TextureResource = nullptr;
			FString TextureResourceDebugName;
			// Slice index (in the batch's source texture array) that needs to be copied onto each individual channel of this weightmap:
			FIntVector4 SourceSliceIndexPerChannel = FIntVector4(INDEX_NONE);
			// The rects that correspond to the component to read (without border expansion) in the batch (source) texture (one per channel)
			TStaticArray<TArray<FIntRect, TInlineAllocator<4>>, 4> SourceSubsectionRectsPerChannel;
			// Indicates which channel(s) to resolve : 
			uint8 ChannelMask = 0;
			// Indicates whether this resolve operation should be done additively (e.g. there could be a first resolve on texture T for channels .rgb and another later on for channel .a : the former 
			//  would *not* be done additively but the latter, yes... you can thank weightmap channel sharing for all that complexity)
			bool bIsAdditiveResolve = false;
			// Indicates this resolve operation is the final one, i.e. all channels from this weightmap are now resolved so we can finalize the texture entirely (generate mips, final copy and readback) :
			bool bIsFinalResolve = false;
		};
		// List of weightmaps to resolve this batch and how to resolve them: 
		TMap<UTexture2D*, FWeightmapResolveInfo> WeightmapResolveInfosForBatch;
		TSet<ULandscapeComponent*> ComponentsResolvedInBatch;

		// Process the list of textures that are not yet resolved and find if there are components in this batch that we participate to it so we can resolve it either partially or fully : 
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(PrepareResolve);
			for (auto ItWeightmapNeedingResolve = TexturesNeedingResolve.CreateIterator(); ItWeightmapNeedingResolve; ++ItWeightmapNeedingResolve)
			{
				UTexture2D* Weightmap = ItWeightmapNeedingResolve.Key();
				uint8& AlreadyResolvedChannelMask = ItWeightmapNeedingResolve.Value();

				uint8 WeightmapChannelMask = InUpdateLayersContentContext.MapHelper.WeightmapToChannelMask.FindChecked(Weightmap);
				check((WeightmapChannelMask > 0) && (WeightmapChannelMask <= 15)); // a mask of 0 means nothing needs to be resolved, so it shouldn't ever be in the TexturesNeedingResolve list
			
				const TArray<ULandscapeComponent*>& ComponentsForWeightmap = InUpdateLayersContentContext.MapHelper.WeightmapToComponents.FindChecked(Weightmap);
				for (ULandscapeComponent* RenderedComponent : ComponentsForWeightmap)
				{
					// If the component is present in this batch, we can resolve its channels :
					if (RenderBatch->ComponentsToRender.Contains(RenderedComponent) 
						// The component was possibly rendered in a previous batch, in which case we don't have to resolve it again : 
						&& !ResolvedLandscapeComponents.Contains(RenderedComponent))
					{
						TArray<FIntRect, TInlineAllocator<4>> SourceSubsectionRects, DummySubsectionRects;
						RenderBatch->ComputeSubsectionRects(RenderedComponent, /*OutSubsectionRects = */SourceSubsectionRects, /*OutSubsectionRectsWithDuplicateBorders = */DummySubsectionRects);
						check(SourceSubsectionRects.Num() == TotalNumSubsections);

						FWeightmapResolveInfo* WeightmapResolveInfo = WeightmapResolveInfosForBatch.Find(Weightmap);
						if (WeightmapResolveInfo == nullptr)
						{
							// This is a new weightmap to resolve this batch, let's create the info : 
							WeightmapResolveInfo = &WeightmapResolveInfosForBatch.Add(Weightmap, FWeightmapResolveInfo());
							WeightmapResolveInfo->TextureResource = Weightmap->GetResource();
							WeightmapResolveInfo->TextureResourceDebugName = Weightmap->GetName();
							// We need the resolve operation to be additive if some channels have already been resolved in a previous batch: 
							WeightmapResolveInfo->bIsAdditiveResolve = (AlreadyResolvedChannelMask != 0);

							ALandscapeProxy* Proxy = RenderedComponent->GetLandscapeProxy();
							check(Proxy != nullptr);
							// Setup the CPU readback if it does not already exist:
							FLandscapeEditLayerReadback** CPUReadback = Proxy->WeightmapsCPUReadback.Find(Weightmap);
							if (CPUReadback == nullptr)
							{
								// Lazily create the readback objects as required (ReallocateLayersWeightmaps might have created new weightmaps)
								FLandscapeEditLayerReadback* NewCPUReadback = new FLandscapeEditLayerReadback();
								bool bHashIsValid = true;
								const uint64 Hash = ULandscapeTextureHash::CalculateTextureHash64(Weightmap, ELandscapeTextureType::Weightmap, bHashIsValid);
								NewCPUReadback->SetHash(Hash);
								CPUReadback = &Proxy->WeightmapsCPUReadback.Add(Weightmap, NewCPUReadback);
							}
							check(*CPUReadback != nullptr);

							FIntPoint WeightmapResolution(Weightmap->Source.GetSizeX(), Weightmap->Source.GetSizeY());
							MinWeightmapResolution = MinWeightmapResolution.ComponentMin(WeightmapResolution);
							MaxWeightmapResolution = MaxWeightmapResolution.ComponentMax(WeightmapResolution);
						}


						// Select only the allocations of this component that involve this texture : 
						const TArray<UTexture2D*>& ComponentTextures = RenderedComponent->GetWeightmapTextures();
						const int32 WeightmapIndex = ComponentTextures.IndexOfByKey(Weightmap);
						check(WeightmapIndex != INDEX_NONE);
						const TArray<FWeightmapLayerAllocationInfo>& ComponentAllocationInfos = RenderedComponent->GetWeightmapLayerAllocations();
						TArray<FWeightmapLayerAllocationInfo, TInlineAllocator<4>> AllocationInfosForTexture;
						Algo::TransformIf(ComponentAllocationInfos, AllocationInfosForTexture,
							[WeightmapIndex](const FWeightmapLayerAllocationInfo& InAllocationInfo) { return (InAllocationInfo.LayerInfo != nullptr) && InAllocationInfo.IsAllocated() && (InAllocationInfo.WeightmapTextureIndex == WeightmapIndex); },
							[](const FWeightmapLayerAllocationInfo& InAllocationInfo) { return InAllocationInfo; });
						check(!AllocationInfosForTexture.IsEmpty() && (AllocationInfosForTexture.Num() <= 4));

						for (const FWeightmapLayerAllocationInfo& AllocationInfo : AllocationInfosForTexture)
						{
							check((AllocationInfo.WeightmapTextureChannel >= 0) && (AllocationInfo.WeightmapTextureChannel < 4));
							checkf(((WeightmapResolveInfo->ChannelMask & (1 << AllocationInfo.WeightmapTextureChannel)) == 0), TEXT("This channel has already been resolved, it shouldn't happen, it would mean that 2 allocations are using the same channel"));
							int32 SliceIndex = InParams.TargetLayerGroupLayerNames.IndexOfByKey(AllocationInfo.GetLayerName());
							checkf(SliceIndex != INDEX_NONE, TEXT("Couldn't find %s in the list of weightmaps that have been produced"), *AllocationInfo.GetLayerName().ToString());
							WeightmapResolveInfo->SourceSliceIndexPerChannel[AllocationInfo.WeightmapTextureChannel] = SliceIndex;
							WeightmapResolveInfo->SourceSubsectionRectsPerChannel[AllocationInfo.WeightmapTextureChannel] = SourceSubsectionRects;
							WeightmapResolveInfo->ChannelMask |= (1 << AllocationInfo.WeightmapTextureChannel);
						}
						check(WeightmapResolveInfo->ChannelMask != 0);
						// We can now consider these channels resolved for this texture : 
						AlreadyResolvedChannelMask |= WeightmapResolveInfo->ChannelMask;

						// If all of this weightmap's channels have been resolved, we can finalize it and remove it from our list of weightmaps to resolve :
						if (AlreadyResolvedChannelMask == WeightmapChannelMask)
						{
							WeightmapResolveInfo->bIsFinalResolve = true;
							ItWeightmapNeedingResolve.RemoveCurrent();
						}

						ComponentsResolvedInBatch.Add(RenderedComponent);
					}
				}
			}
		}

		if (!WeightmapResolveInfosForBatch.IsEmpty())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FinalizeWeightmaps);
			RHI_BREADCRUMB_EVENT_GAMETHREAD("FinalizeWeightmaps");

			check(MinWeightmapResolution == MaxWeightmapResolution);

			ENQUEUE_RENDER_COMMAND(LandscapeLayers_Cmd_WeightmapsPackWeightmap)(
				[ SourceResource = ReadRT->GetRenderTarget2DArray()->GetResource()
				, WeightmapResolveInfosForBatch
				, WeightmapResolution = MinWeightmapResolution
				, NumMips = (int32)FMath::CeilLogTwo(Landscape->SubsectionSizeQuads) + 1
				, TotalNumSubsections = Landscape->NumSubsections * Landscape->NumSubsections
				, NumSubsections = Landscape->NumSubsections
				, ComponentSubsectionVerts = Landscape->SubsectionSizeQuads + 1 ]
				(FRHICommandListImmediate& InRHICmdList)
			{
				FRDGBuilder GraphBuilder(InRHICmdList, RDG_EVENT_NAME("WeightmapsFinalizeWeightmaps"));

				FRDGTextureSRVRef BlackDummySRVRef = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(GSystemTextures.GetBlackDummy(GraphBuilder)));
				FRDGTextureRef SourceTextureRef = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(SourceResource->TextureRHI, TEXT("SourceTexture")));
				FRDGTextureSRVRef SourceTextureSRVRef = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(SourceTextureRef));
				FRDGTextureDesc Desc = FRDGTextureDesc::Create2D(WeightmapResolution, PF_B8G8R8A8, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource, NumMips);
				FRDGTextureRef PackedTextureRef = GraphBuilder.CreateTexture(Desc, TEXT("PackedWeightmap"));

				for (auto It : WeightmapResolveInfosForBatch)
				{
					const FWeightmapResolveInfo& WeightmapResolveInfo = It.Value;
					FRDGTextureRef DestinationTextureRef = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(WeightmapResolveInfo.TextureResource->TextureRHI, TEXT("DestinationTexture")));

					RDG_EVENT_SCOPE(GraphBuilder, "%sFinalize %s", WeightmapResolveInfo.bIsFinalResolve ? TEXT("") : TEXT("(partially)"), *WeightmapResolveInfo.TextureResourceDebugName);
					{
						RDG_EVENT_SCOPE(GraphBuilder, "Pack %d channels %s", FMath::CountBits(WeightmapResolveInfo.ChannelMask), WeightmapResolveInfo.bIsAdditiveResolve ? TEXT("(additive)") : TEXT(""));

						// If the resolve is additive, it means we need to use the destination texture as an input to the pack operation, because we've stored the previous (partial) resolve in it :
						FRDGTextureSRVRef WeightmapBeingPackedSRVRef = WeightmapResolveInfo.bIsAdditiveResolve ? GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(DestinationTextureRef)) : BlackDummySRVRef;

						// Operate sub-section by sub-section in order to both pack the 4 channels and duplicate borders :
						for (int32 SubSectionIndex = 0; SubSectionIndex < TotalNumSubsections; ++SubSectionIndex)
						{
							FIntPoint SubSection(SubSectionIndex % NumSubsections, SubSectionIndex / NumSubsections);
							const FIntRect OutputRect = FIntRect(SubSection * ComponentSubsectionVerts, SubSection * ComponentSubsectionVerts + FIntPoint(ComponentSubsectionVerts, ComponentSubsectionVerts));

							FLandscapeEditLayersWeightmapsPackWeightmapPS::FParameters* PSParams = GraphBuilder.AllocParameters<FLandscapeEditLayersWeightmapsPackWeightmapPS::FParameters>();
							PSParams->RenderTargets[0] = FRenderTargetBinding(PackedTextureRef, ERenderTargetLoadAction::ELoad);
							PSParams->InSourceSliceIndices = WeightmapResolveInfo.SourceSliceIndexPerChannel;
							for (int32 ChannelIndex = 0; ChannelIndex < 4; ++ChannelIndex)
							{
								if ((WeightmapResolveInfo.SourceSliceIndexPerChannel[ChannelIndex] != INDEX_NONE))
								{
									const FIntRect& SourceSubSectionRect = WeightmapResolveInfo.SourceSubsectionRectsPerChannel[ChannelIndex][SubSectionIndex];
									PSParams->InSourcePixelOffsets[ChannelIndex] = FUintVector4(SourceSubSectionRect.Min.X, SourceSubSectionRect.Min.Y, 0, 0);
								}
								else
								{
									PSParams->InSourcePixelOffsets[ChannelIndex] = FUintVector4(0, 0, 0, 0);
								}
							}
							PSParams->InSubsectionPixelOffset = FUintVector2(OutputRect.Min.X, OutputRect.Min.Y);
							PSParams->InIsAdditive = WeightmapResolveInfo.bIsAdditiveResolve ? 1 : 0;
							PSParams->InSourceWeightmaps = SourceTextureSRVRef;
							PSParams->InWeightmapBeingPacked = WeightmapBeingPackedSRVRef;
							FLandscapeEditLayersWeightmapsPackWeightmapPS::PackWeightmapPS(GraphBuilder, PSParams, OutputRect);
						}
					}

					if (WeightmapResolveInfo.bIsFinalResolve)
					{
						// Only generate the mips if it's the final resolve : 
						if (NumMips > 1)
						{
							RDG_EVENT_SCOPE(GraphBuilder, "Generate %d remaining mips", NumMips - 1);

							FIntPoint CurrentMipSize = WeightmapResolution;
							for (int32 MipLevel = 1; MipLevel < NumMips; ++MipLevel)
							{
								CurrentMipSize.X >>= 1;
								CurrentMipSize.Y >>= 1;

								// Read from scratch weightmap texture (mip N - 1) -> write to scratch weightmap texture (mip N) :
								FLandscapeEditLayersWeightmapsGenerateMipsPS::FParameters* PSParams = GraphBuilder.AllocParameters<FLandscapeEditLayersWeightmapsGenerateMipsPS::FParameters>();
								PSParams->RenderTargets[0] = FRenderTargetBinding(PackedTextureRef, ERenderTargetLoadAction::ENoAction, static_cast<uint8>(MipLevel));
								PSParams->InCurrentMipSubsectionSize = FUintVector2(CurrentMipSize.X / NumSubsections, CurrentMipSize.Y / NumSubsections);
								PSParams->InSourceWeightmap = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(PackedTextureRef, MipLevel - 1));

								FLandscapeEditLayersWeightmapsGenerateMipsPS::GenerateMipsPS(GraphBuilder, PSParams, CurrentMipSize);
							}
						}
					}

					{
						// We use the final texture as a temporary buffer when it's a partial resolve, so we only need to copy mip 0 then. All mips will be generated/copy when the resolve step is final :
						const int32 NumMipsToCopy = WeightmapResolveInfo.bIsFinalResolve ? NumMips : 1;
						RDG_EVENT_SCOPE(GraphBuilder, "Copy %d mips", NumMips);
						for (int32 MipLevel = 0; MipLevel < NumMipsToCopy; ++MipLevel)
						{
							FRHICopyTextureInfo CopyInfo;
							CopyInfo.SourceMipIndex = MipLevel;
							CopyInfo.DestMipIndex = MipLevel;
							AddCopyTexturePass(GraphBuilder, PackedTextureRef, DestinationTextureRef, CopyInfo);
						}
					}
				}

				GraphBuilder.Execute();
			});
		}

		// Remember all components resolved this batch so that we don't have to resolve them ever again : 
		ResolvedLandscapeComponents.Append(ComponentsResolvedInBatch);
	};

	// Render everything now. Every time a group from a batch is done, the OnRenderBatchGroupDone callback is called : 
	MergeRenderContext.Render(OnRenderBatchGroupDone);

	check(TexturesNeedingResolve.IsEmpty());

	// Prepare the UTexture2D readbacks we'll need to perform :
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CopyToReadback);
		RHI_BREADCRUMB_EVENT_GAMETHREAD("CopyToReadback");

		TArray<FLandscapeLayersCopyReadbackTextureParams> DeferredCopyReadbackTextures = PrepareLandscapeLayersCopyReadbackTextureParams(InUpdateLayersContentContext.MapHelper, InUpdateLayersContentContext.WeightmapsToResolve.Array(), /*bWeightmaps = */true);
		ExecuteCopyToReadbackTexture(DeferredCopyReadbackTextures);
	}

	// Finally, update the material instances to take into account potentially new material combinations : 
	UpdateLayersMaterialInstances(InUpdateLayersContentContext.LandscapeComponentsWeightmapsToResolve);

	return InMergeParams.WeightmapUpdateModes;
}

bool ALandscape::PerformSelectiveRenderEditLayersWeightmapsCPU(FUpdateLayersContentContext& InUpdateLayersContentContext, const FLandscapeEditLayerRenderWeightParams& InSelectiveRenderParams)
{
	using namespace UE::Landscape;
	using namespace UE::Landscape::Private;
	using namespace UE::Landscape::EditLayers;
	using namespace UE::Landscape::EditLayers::Private;

	TRACE_CPUPROFILER_EVENT_SCOPE(ALandscape::PerformSelectiveRenderEditLayersWeightmapsCPU);
	RHI_BREADCRUMB_EVENT_GAMETHREAD("PerformSelectiveRenderEditLayersWeightmapsCPU");

	FMergeContext MergeContext(/*InLandscape = */this, /*bInIsHeightmapMerge = */false, !InSelectiveRenderParams.bRenderBrushes);
	TArray<FEditLayerRendererState> RendererStates = GetEditLayerRendererStatesEnableOverride(&MergeContext, InSelectiveRenderParams.ActiveEditLayers);

	TMap<FName, int32> RequestedNameToIndex;
	for (int Idx = 0; Idx < InSelectiveRenderParams.WeightmapNames.Num(); ++Idx)
	{
		RequestedNameToIndex.Emplace(InSelectiveRenderParams.WeightmapNames[Idx], Idx);
	}

	// Prepare the merge : 
	// Not yet ready to selectively render layers, so request all layers.  Asserts deep in in FMergeRenderContext::Render if this is false.
	const bool bRequestAllLayers = true;
	const TSet<FName> RequestedWeightmapLayerNames(InSelectiveRenderParams.WeightmapNames);
	FMergeRenderParams MergeRenderParams(InUpdateLayersContentContext.LandscapeComponentsWeightmapsToRender, RendererStates, RequestedWeightmapLayerNames, bRequestAllLayers);
	FMergeRenderContext MergeRenderContext = PrepareEditLayersMergeRenderContext(MergeContext, MergeRenderParams);
	if (!MergeRenderContext.IsValid())
	{
		return false;
	}

	// Set up a FRHIGPUTextureReadback for each merge batch
	struct FSyncReadback
	{
		// FRHIGPUTextureReadback only supports one slice each
		FRHIGPUTextureReadback Readback = { "SyncRenderResult" };
		FName TargetLayerName;
		int32 SliceIdx = 0;
	};
	struct FSyncRenderBatchResult
	{
		FIntRect SectionRect;
		FIntPoint Size;  // RT size
		TArray<FIntRect> ComponentRects;
		TArray<FSyncReadback> SliceData;  // one slice per target layer
	};
	TArray<FSyncRenderBatchResult> SyncRenderBatchResults;
	SyncRenderBatchResults.AddDefaulted(MergeRenderContext.GetRenderBatches().Num());  // allocate in advance.  accessed from the render thread, so can't re-size.

	// Batch completion callback
	// New version, based on selective render heightmaps.  no resolve stuff.
	auto EnqueueBatchReadback = [&RequestedWeightmapLayerNames, &SyncRenderBatchResults]
	(const FMergeRenderContext::FOnRenderBatchTargetGroupDoneParams& InParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ExtractSelectiveMergeData);
			RHI_BREADCRUMB_EVENT_GAMETHREAD("ExtractSelectiveMergeData");

			InParams.MergeRenderContext->CycleBlendRenderTargets(RDGBuilderRecorder);
			ULandscapeScratchRenderTarget* ReadRT = InParams.MergeRenderContext->GetBlendRenderTargetRead();
			UTextureRenderTarget2DArray* SourceTexArray = ReadRT->GetRenderTarget2DArray();
			check(SourceTexArray->GetSurfaceArraySize() == InParams.TargetLayerGroupLayerNames.Num());

			ALandscape* Landscape = InParams.MergeRenderContext->GetLandscape();
			const FMergeRenderBatch* RenderBatch = InParams.MergeRenderContext->GetCurrentRenderBatch();
			int32 CurrentBatch = InParams.MergeRenderContext->GetCurrentRenderBatchIdx();
			FSyncRenderBatchResult& BatchResult = SyncRenderBatchResults[CurrentBatch];

			BatchResult.SectionRect = RenderBatch->SectionRect;
			BatchResult.Size = RenderBatch->Resolution;  //Only used for sanity check on bounds
			BatchResult.ComponentRects = RenderBatch->ComputeAllComponentSectionRects(/*bInWithDuplicateBorders = */false);

			BatchResult.SliceData.Reserve(InParams.TargetLayerGroupLayerNames.Num());
			for (int32 Idx = 0; Idx < InParams.TargetLayerGroupLayerNames.Num(); ++Idx)
			{
				// Only need the layers we asked for.  The merge code could include other layers based on internal requirements.
				if (RequestedWeightmapLayerNames.Contains(InParams.TargetLayerGroupLayerNames[Idx]))
				{
					FSyncReadback& ReadbackData = BatchResult.SliceData.AddDefaulted_GetRef();
					ReadbackData.TargetLayerName = InParams.TargetLayerGroupLayerNames[Idx];
					ReadbackData.SliceIdx = Idx;
				}
			}

			// Enqueue async readback direct from the current scratch read target.  One readback per texture slice, but only on the slices that we requested.
			// Extra layers are present because bRequestAllLayers=true is required.
			ENQUEUE_RENDER_COMMAND(LandscapeLayers_Cmd_SyncExtractResults)(
				[&BatchResult,
				SourceResource = SourceTexArray->GetResource()]
				(FRHICommandListImmediate& InRHICmdList)
				{
					FRDGBuilder GraphBuilder(InRHICmdList, RDG_EVENT_NAME("WeightmapsExtractResult"));

					FRDGTextureRef SourceTextureRef = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(SourceResource->TextureRHI, TEXT("SourceTexture")));
					for (FSyncReadback& SliceData : BatchResult.SliceData)
					{
						AddEnqueueCopyPass(GraphBuilder, &SliceData.Readback, SourceTextureRef, FResolveRect(), SliceData.SliceIdx);
					}

					// We need to specify the final state of the external texture to prevent the graph builder from transitioning it to SRVMask :
					GraphBuilder.SetTextureAccessFinal(SourceTextureRef, ERHIAccess::RTV);
					GraphBuilder.Execute();
				});
		};
	
	// Render everything now. Every time a group from a batch is done, the EnqueueBatchReadback callback is called : 
	MergeRenderContext.Render(EnqueueBatchReadback);

	// Pick up readback results from SyncRenderBatchResults.  Enqueue this on the render thread because FRHIGPUTextureReadback can only be locked there.
	ENQUEUE_RENDER_COMMAND(LandscapeLayers_Cmd_SyncFetchResults)(
		[&SyncRenderBatchResults,
		&RequestedNameToIndex,
		DestRect = InSelectiveRenderParams.Bounds,
		Dest = InSelectiveRenderParams.CpuResult]
		(FRHICommandListImmediate& InRHICmdList)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(LockAndFetchReadbackData);
			const FBlitBuffer2DDesc DestDesc{ Dest.Num(), DestRect.Width(), DestRect };

			int32 DestSliceStride = DestRect.Area();

			for (FSyncRenderBatchResult& BatchResult : SyncRenderBatchResults)
			{
				int32 NumSlices = BatchResult.SliceData.Num();
				for (FSyncReadback& SliceData : BatchResult.SliceData)
				{
					FName LayerName = SliceData.TargetLayerName;
					int32* DestLayerIndexPtr = RequestedNameToIndex.Find(LayerName);
					if (DestLayerIndexPtr == nullptr)
					{
						continue;  // didn't ask for this layer
					}
					int32 DestLayerIndex = *DestLayerIndexPtr;

					int32 BatchStride = 0;
					int32 BatchHeight = 0;
					uint32* SrcData = static_cast<uint32*>(SliceData.Readback.Lock(BatchStride, &BatchHeight));
					int32 SrcBufferSize = BatchStride * BatchHeight; //the implied buffer size of the locked readback buffer, not necessarily of the native render target
					check(BatchResult.Size.X <= BatchStride && BatchResult.Size.Y <= BatchHeight);
					uint8* DestPtr = Dest.GetData() + (DestSliceStride * DestLayerIndex);
					check(DestSliceStride * (DestLayerIndex + 1) <= Dest.Num());

					// The full batch rects are not all valid.  Only rects of each component in the batch are valid.  Blit one component at a time.
					for (const FIntRect& ComponentBatchRect : BatchResult.ComponentRects)
					{
						// ComponentBatchRect is relative to the batch RT.  Convert back into overall landscape coords.
						FIntRect ComponentRect = ComponentBatchRect + BatchResult.SectionRect.Min;
						ComponentRect.Max += FIntPoint(1, 1);  //quads to verts
						check(BatchResult.SectionRect.Contains(ComponentRect));

						const FBlitBuffer2DDesc SrcDesc{ SrcBufferSize, BatchStride, BatchResult.SectionRect };
						BlitWeightChannelsToUint8(DestPtr, DestDesc, SrcData, SrcDesc, ComponentRect);
					}

					SliceData.Readback.Unlock();
				}
			}
		});

	// Sync the game thread with all of the above.  After this, the results are reliably accessible from the game thread.
	FlushRenderingCommands();

	return true;
}

int32 ALandscape::RegenerateLayersWeightmaps(FUpdateLayersContentContext& InUpdateLayersContentContext)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeLayers_RegenerateLayersWeightmaps);
	const int32 WeightmapUpdateModes = LayerContentUpdateModes & ELandscapeLayerUpdateMode::Update_Weightmap_Types;
	const bool bSkipBrush = CVarLandscapeLayerBrushOptim.GetValueOnAnyThread() == 1 && (WeightmapUpdateModes == ELandscapeLayerUpdateMode::Update_Weightmap_Editing);

	ULandscapeInfo* Info = GetLandscapeInfo();

	if (WeightmapUpdateModes == 0)
	{
		return 0;
	}

	if (InUpdateLayersContentContext.LandscapeComponentsWeightmapsToResolve.Num() == 0 || Info == nullptr)
	{
		return WeightmapUpdateModes;
	}

	if (WeightmapUpdateModes)
	{
		RenderCaptureInterface::FScopedCapture RenderCapture((RenderCaptureLayersNextWeightmapDraws != 0), TEXT("LandscapeLayersWeightmapCapture"));
		RenderCaptureLayersNextWeightmapDraws = FMath::Max(0, RenderCaptureLayersNextWeightmapDraws - 1);

		FEditLayersWeightmapMergeParams MergeParams;
		MergeParams.WeightmapUpdateModes = WeightmapUpdateModes;
		MergeParams.bSkipBrush = bSkipBrush;

		return PerformLayersWeightmapsBatchedMerge(InUpdateLayersContentContext, MergeParams);
	}

	return 0;
}

void ALandscape::UpdateForChangedWeightmaps(const TArrayView<FLandscapeEditLayerComponentReadbackResult>& InComponentReadbackResults)
{
	TArray<ULandscapeComponent*> ComponentsNeedingMaterialInstanceUpdates;

	for (const FLandscapeEditLayerComponentReadbackResult& ComponentReadbackResult : InComponentReadbackResults)
	{
		// If the source data has changed, mark the component as needing a collision layer data update:
		//  - If ELandscapeComponentUpdateFlag::Component_Update_Weightmap_Collision is passed, it will be done immediately
		//  - If not, at least the component's collision layer data will still get updated eventually, when the flag is finally passed :
		if (ComponentReadbackResult.bModified)
		{
			ComponentReadbackResult.LandscapeComponent->SetPendingLayerCollisionDataUpdate(true);
		}

		// If this component has a layer with only zeros, remove it so that we don't end up with weightmaps we don't end up using :
		if (!ComponentReadbackResult.AllZeroLayers.IsEmpty())
		{
			const TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = ComponentReadbackResult.LandscapeComponent->GetWeightmapLayerAllocations(FGuid());
			for (ULandscapeLayerInfoObject* AllZeroLayerInfo : ComponentReadbackResult.AllZeroLayers)
			{
				check(AllZeroLayerInfo != nullptr);
				// Find the index for this layer in this component.
				int32 AllZeroLayerIndex = ComponentWeightmapLayerAllocations.IndexOfByPredicate(
					[AllZeroLayerInfo](const FWeightmapLayerAllocationInfo& Allocation) { return Allocation.LayerInfo == AllZeroLayerInfo; });
				check(AllZeroLayerIndex != INDEX_NONE);

				ComponentReadbackResult.LandscapeComponent->DeleteLayerAllocation(FGuid(), AllZeroLayerIndex, /*bShouldDirtyPackage = */true);

				// We removed a weightmap allocation so the material instance for this landscape component needs updating :
				ComponentsNeedingMaterialInstanceUpdates.Add(ComponentReadbackResult.LandscapeComponent);
			}
		}

		const int32 WeightUpdateMode = ComponentReadbackResult.UpdateModes & (ELandscapeLayerUpdateMode::Update_Weightmap_All_Modes | ELandscapeLayerUpdateMode::Update_Weightmap_Editing | ELandscapeLayerUpdateMode::Update_Weightmap_Editing_NoCollision);
		if (IsUpdateFlagEnabledForModes(ELandscapeComponentUpdateFlag::Component_Update_Weightmap_Collision, WeightUpdateMode) 
			|| ComponentReadbackResult.bCleared)
		{
			// Only update collision data if there was an actual change performed on the source data : 
			if (ComponentReadbackResult.LandscapeComponent->GetPendingLayerCollisionDataUpdate())
			{
				ComponentReadbackResult.LandscapeComponent->UpdateCollisionLayerData();
				ComponentReadbackResult.LandscapeComponent->SetPendingLayerCollisionDataUpdate(false);
			}
		}
	}

	UpdateLayersMaterialInstances(ComponentsNeedingMaterialInstanceUpdates);
}

void ULandscapeComponent::GetUsedPaintLayers(const FGuid& InLayerGuid, TArray<ULandscapeLayerInfoObject*>& OutUsedLayerInfos) const
{
	const TArray<FWeightmapLayerAllocationInfo>& AllocInfos = GetWeightmapLayerAllocations(InLayerGuid);
	for (const FWeightmapLayerAllocationInfo& AllocInfo : AllocInfos)
	{
		if (AllocInfo.LayerInfo != nullptr)
		{
			OutUsedLayerInfos.AddUnique(AllocInfo.LayerInfo);
		}
	}
}

uint32 ULandscapeComponent::ComputeWeightmapsHash()
{
	uint32 Hash = 0;
	const TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapAllocations = GetWeightmapLayerAllocations();
	for (const FWeightmapLayerAllocationInfo& AllocationInfo : ComponentWeightmapAllocations)
	{
		Hash = HashCombine(AllocationInfo.GetHash(), Hash);
	}

	const TArray<UTexture2D*>& ComponentWeightmapTextures = GetWeightmapTextures();
	const TArray<ULandscapeWeightmapUsage*>& ComponentWeightmapTextureUsage = GetWeightmapTexturesUsage();
	for (int32 i = 0; i < ComponentWeightmapTextures.Num(); ++i)
	{
		Hash = PointerHash(ComponentWeightmapTextures[i], Hash);
		Hash = PointerHash(ComponentWeightmapTextureUsage[i], Hash);
		for (int32 j = 0; j < ULandscapeWeightmapUsage::NumChannels; ++j)
		{
			Hash = PointerHash(ComponentWeightmapTextureUsage[i]->ChannelUsage[j], Hash);
		}
	}
	return Hash;
}

void ALandscape::UpdateLayersMaterialInstances(const TArray<ULandscapeComponent*>& InLandscapeComponents)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeLayers_UpdateLayersMaterialInstances);
	TArray<ULandscapeComponent*> ComponentsToUpdate;

	// Compute Weightmap usage changes
	if (ULandscapeInfo* Info = GetLandscapeInfo())
	{
		for (ULandscapeComponent* LandscapeComponent : InLandscapeComponents)
		{
			uint32 NewHash = LandscapeComponent->ComputeWeightmapsHash();
			if (LandscapeComponent->WeightmapsHash != NewHash)
			{
				ComponentsToUpdate.Add(LandscapeComponent);
				LandscapeComponent->WeightmapsHash = NewHash;
			}
		}
	}

	if (ComponentsToUpdate.Num() == 0)
	{
		return;
	}

	// we're not having the material update context recreate render states because we will manually do it for only our components
	TArray<FComponentRecreateRenderStateContext> RecreateRenderStateContexts;
	RecreateRenderStateContexts.Reserve(ComponentsToUpdate.Num());

	for (ULandscapeComponent* Component : ComponentsToUpdate)
	{
		RecreateRenderStateContexts.Emplace(Component);
	}
	TOptional<FMaterialUpdateContext> MaterialUpdateContext;
	MaterialUpdateContext.Emplace(FMaterialUpdateContext::EOptions::Default & ~FMaterialUpdateContext::EOptions::RecreateRenderStates);

	bool bHasUniformExpressionUpdatePending = false;

	for (ULandscapeComponent* Component : ComponentsToUpdate)
	{
		int32 MaxLOD = FMath::CeilLogTwo(SubsectionSizeQuads + 1) - 1;
		decltype(Component->MaterialPerLOD) NewMaterialPerLOD;
		Component->LODIndexToMaterialIndex.SetNumUninitialized(MaxLOD + 1);
		int8 LastLODIndex = INDEX_NONE;

		UMaterialInterface* BaseMaterial = Component->GetLandscapeMaterial();
		UMaterialInterface* LOD0Material = Component->GetLandscapeMaterial(0);

		for (int32 LODIndex = 0; LODIndex <= MaxLOD; ++LODIndex)
		{
			UMaterialInterface* CurrentMaterial = Component->GetLandscapeMaterial(static_cast<int8>(LODIndex));

			// if we have a LOD0 override, do not let the base material override it, it should override everything!
			if (CurrentMaterial == BaseMaterial && BaseMaterial != LOD0Material)
			{
				CurrentMaterial = LOD0Material;
			}

			const int8* MaterialLOD = NewMaterialPerLOD.Find(CurrentMaterial);

			if (MaterialLOD != nullptr)
			{
				Component->LODIndexToMaterialIndex[LODIndex] = *MaterialLOD > LastLODIndex ? *MaterialLOD : LastLODIndex;
			}
			else
			{
				int32 AddedIndex = NewMaterialPerLOD.Num();
				NewMaterialPerLOD.Add(CurrentMaterial, static_cast<int8>(LODIndex));
				Component->LODIndexToMaterialIndex[LODIndex] = static_cast<int8>(AddedIndex);
				LastLODIndex = static_cast<int8>(AddedIndex);
			}
		}

		Component->MaterialPerLOD = NewMaterialPerLOD;

		Component->MaterialInstances.SetNumZeroed(Component->MaterialPerLOD.Num()); 
		int8 MaterialIndex = 0;

		const TArray<FWeightmapLayerAllocationInfo>& WeightmapBaseLayerAllocation = Component->GetWeightmapLayerAllocations();

		const TArray<UTexture2D*>& ComponentWeightmapTextures = Component->GetWeightmapTextures();

		for (auto& ItPair : Component->MaterialPerLOD)
		{
			const int8 MaterialLOD = ItPair.Value;

			// Find or set a matching MIC in the Landscape's map.
			UMaterialInstanceConstant* CombinationMaterialInstance = Component->GetCombinationMaterial(&MaterialUpdateContext.GetValue(), WeightmapBaseLayerAllocation, MaterialLOD, false);

			if (CombinationMaterialInstance != nullptr)
			{
				UMaterialInstanceConstant* MaterialInstance = Component->MaterialInstances[MaterialIndex];
				bool NeedToCreateMIC = MaterialInstance == nullptr;

				if (NeedToCreateMIC)
				{
					// Create the instance for this component, that will use the layer combination instance.
					MaterialInstance = NewObject<ULandscapeMaterialInstanceConstant>(this);
					Component->MaterialInstances[MaterialIndex] = MaterialInstance;
				}

				MaterialInstance->SetParentEditorOnly(CombinationMaterialInstance);

				MaterialUpdateContext.GetValue().AddMaterialInstance(MaterialInstance); // must be done after SetParent				

				FLinearColor Masks[4] = { FLinearColor(1.0f, 0.0f, 0.0f, 0.0f), FLinearColor(0.0f, 1.0f, 0.0f, 0.0f), FLinearColor(0.0f, 0.0f, 1.0f, 0.0f), FLinearColor(0.0f, 0.0f, 0.0f, 1.0f) };

				// Set the layer mask
				for (int32 AllocIdx = 0; AllocIdx < WeightmapBaseLayerAllocation.Num(); AllocIdx++)
				{
					const FWeightmapLayerAllocationInfo& Allocation = WeightmapBaseLayerAllocation[AllocIdx];
					MaterialInstance->SetVectorParameterValueEditorOnly(FName(*FString::Printf(TEXT("LayerMask_%s"), *Allocation.GetLayerName().ToString())), Masks[Allocation.WeightmapTextureChannel]);
				}

				// Set the weightmaps
				for (int32 i = 0; i < ComponentWeightmapTextures.Num(); i++)
				{
					MaterialInstance->SetTextureParameterValueEditorOnly(FName(*FString::Printf(TEXT("Weightmap%d"), i)), ComponentWeightmapTextures[i]);
				}

				if (NeedToCreateMIC)
				{
					MaterialInstance->PostEditChange();
				}
				else
				{
					bHasUniformExpressionUpdatePending = true;
					MaterialInstance->RecacheUniformExpressions(true);
				}
			}

			++MaterialIndex;
		}

		if (Component->MaterialPerLOD.Num() == 0)
		{
			Component->MaterialInstances.Empty(1);
			Component->MaterialInstances.Add(nullptr);
			Component->LODIndexToMaterialIndex.Empty(1);
			Component->LODIndexToMaterialIndex.Add(0);
		}

		Component->EditToolRenderData.UpdateDebugColorMaterial(Component);
	}

	// End material update
	MaterialUpdateContext.Reset();

	// Recreate the render state for our components, needed to update the static drawlist which has cached the MaterialRenderProxies
	// Must be after the FMaterialUpdateContext is destroyed
	RecreateRenderStateContexts.Empty();

	if (bHasUniformExpressionUpdatePending)
	{
		ENQUEUE_RENDER_COMMAND(LandscapeLayers_Cmd_UpdateMaterial)(
			[](FRHICommandList& RHICmdList)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeLayers_RT_UpdateMaterial);
			FMaterialRenderProxy::UpdateDeferredCachedUniformExpressions();
		});
	}
}

void ALandscape::ResolveLayersWeightmapTexture(
	FTextureToComponentHelper const& MapHelper,
	TSet<UTexture2D*> const& WeightmapsToResolve,
	TArray<FLandscapeEditLayerComponentReadbackResult>& InOutComponentReadbackResults)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeLayers_ResolveLayersWeightmapTexture);

	ULandscapeInfo* Info = GetLandscapeInfo();
	if (Info == nullptr)
	{
		return;
	}

	TArray<ULandscapeComponent*> ChangedComponents;
	for (UTexture2D* Weightmap : WeightmapsToResolve)
	{
		ALandscapeProxy* LandscapeProxy = Weightmap->GetTypedOuter<ALandscapeProxy>();
		check(LandscapeProxy);
		if (FLandscapeEditLayerReadback** CPUReadback = LandscapeProxy->WeightmapsCPUReadback.Find(Weightmap))
		{
			// Only process the readback object if it has active tasks.
			if (!(*CPUReadback)->HasActiveTasks())
			{
				continue;
			}

			const bool bChanged = ResolveLayersTexture(MapHelper, *CPUReadback, Weightmap, InOutComponentReadbackResults, /*bIsWeightmap = */true);
			if (bChanged)
			{
				ChangedComponents.Append(MapHelper.WeightmapToComponents[Weightmap]);
			}
		}
	}

	// Weightmaps shouldn't invalidate lighting
	const bool bInvalidateLightingCache = false;
	InvalidateGeneratedComponentData(ChangedComponents, bInvalidateLightingCache);
}

// Deprecated
bool ALandscape::HasLayersContent() const
{
	return true;
}

// Deprecated
void ALandscape::UpdateCachedHasLayersContent(bool bInCheckComponentDataIntegrity)
{
}

void ALandscape::RequestLayersInitialization(bool bInRequestContentUpdate, bool bInForceLayerResourceReset)
{
	if (IsTemplate())
	{
		return;
	}

	bLandscapeLayersAreInitialized = false;
	LandscapeSplinesAffectedComponents.Empty();

	if (bInRequestContentUpdate)
	{
		RequestLayersContentUpdateForceAll();
	}
}

void ALandscape::RequestSplineLayerUpdate()
{
	if (!IsTemplate() && FindLayerOfTypeConst(ULandscapeEditLayerSplines::StaticClass()) != nullptr)
	{
		bSplineLayerUpdateRequested = true;
	}
}

void ALandscape::RequestLayersContentUpdate(ELandscapeLayerUpdateMode InUpdateMode)
{
	LayerContentUpdateModes |= InUpdateMode;
}

void ALandscape::RequestLayersContentUpdateForceAll(ELandscapeLayerUpdateMode InModeMask, bool bInUserTriggered)
{
	// Ignore Update requests while in PostLoad (to avoid dirtying package on load)
	if (FUObjectThreadContext::Get().IsRoutingPostLoad)
	{
		return;
	}

	if (IsTemplate())
	{
		return;
	}

	const bool bUpdateWeightmap = (InModeMask & (ELandscapeLayerUpdateMode::Update_Weightmap_All_Modes | ELandscapeLayerUpdateMode::Update_Weightmap_Editing | ELandscapeLayerUpdateMode::Update_Weightmap_Editing_NoCollision)) != 0;
	const bool bUpdateHeightmap = (InModeMask & (ELandscapeLayerUpdateMode::Update_Heightmap_All_Modes | ELandscapeLayerUpdateMode::Update_Heightmap_Editing | ELandscapeLayerUpdateMode::Update_Heightmap_Editing_NoCollision)) != 0;
	const bool bUpdateWeightCollision = (InModeMask & (ELandscapeLayerUpdateMode::Update_Weightmap_All_Modes | ELandscapeLayerUpdateMode::Update_Weightmap_Editing)) != 0;
	const bool bUpdateHeightCollision = (InModeMask & (ELandscapeLayerUpdateMode::Update_Heightmap_All_Modes | ELandscapeLayerUpdateMode::Update_Heightmap_Editing)) != 0;
	const bool bUpdateAllHeightmapModes = (InModeMask & ELandscapeLayerUpdateMode::Update_Heightmap_All_Modes) != 0;
	const bool bUpdateAllWeightmapModes = (InModeMask & ELandscapeLayerUpdateMode::Update_Weightmap_All_Modes) != 0;
	const bool bUpdateClientUpdateEditing = (InModeMask & ELandscapeLayerUpdateMode::Update_Client_Editing) != 0;
	if (ULandscapeInfo* LandscapeInfo = GetLandscapeInfo())
	{
		LandscapeInfo->ForEachLandscapeProxy([bUpdateHeightmap, bUpdateWeightmap, bUpdateAllHeightmapModes, bUpdateAllWeightmapModes, bUpdateHeightCollision, bUpdateWeightCollision, bUpdateClientUpdateEditing, bInUserTriggered](ALandscapeProxy* Proxy)
		{
			if (Proxy)
			{
				for (ULandscapeComponent* Component : Proxy->LandscapeComponents)
				{
					if (bUpdateHeightmap)
					{
						Component->RequestHeightmapUpdate(bUpdateAllHeightmapModes, bUpdateHeightCollision, bInUserTriggered);
					}

					if (bUpdateWeightmap)
					{
						Component->RequestWeightmapUpdate(bUpdateAllWeightmapModes, bUpdateWeightCollision, bInUserTriggered);
					}

					if (bUpdateClientUpdateEditing)
					{
						Component->RequestEditingClientUpdate(bInUserTriggered);
					}
				}
			}
			return true;
		});
	}

	RequestLayersContentUpdate(InModeMask);
}

bool ALandscape::IsUpdateFlagEnabledForModes(ELandscapeComponentUpdateFlag InFlag, uint32 InUpdateModes)
{
	if (InUpdateModes & ELandscapeLayerUpdateMode::Update_Heightmap_All_Modes)
	{
		const uint32 HeightmapAllFlags = ELandscapeComponentUpdateFlag::Component_Update_Heightmap_Collision | ELandscapeComponentUpdateFlag::Component_Update_Recreate_Collision | ELandscapeComponentUpdateFlag::Component_Update_Client;
		if (HeightmapAllFlags & InFlag)
		{
			return true;
		}
	}

	if (InUpdateModes & ELandscapeLayerUpdateMode::Update_Heightmap_Editing)
	{
		const uint32 HeightmapEditingFlags = ELandscapeComponentUpdateFlag::Component_Update_Heightmap_Collision | ELandscapeComponentUpdateFlag::Component_Update_Client_Editing;
		if (HeightmapEditingFlags & InFlag)
		{
			return true;
		}
	}

	if (InUpdateModes & ELandscapeLayerUpdateMode::Update_Weightmap_All_Modes)
	{
		const uint32 WeightmapAllFlags = ELandscapeComponentUpdateFlag::Component_Update_Weightmap_Collision | ELandscapeComponentUpdateFlag::Component_Update_Recreate_Collision | ELandscapeComponentUpdateFlag::Component_Update_Client;
		if (WeightmapAllFlags & InFlag)
		{
			return true;
		}
	}

	if (InUpdateModes & ELandscapeLayerUpdateMode::Update_Weightmap_Editing)
	{
		const uint32 WeightmapEditingFlags = ELandscapeComponentUpdateFlag::Component_Update_Weightmap_Collision | ELandscapeComponentUpdateFlag::Component_Update_Client_Editing;
		if (WeightmapEditingFlags & InFlag)
		{
			return true;
		}
	}

	if (InUpdateModes & ELandscapeLayerUpdateMode::Update_Client_Editing)
	{
		const uint32 WeightmapEditingFlags = ELandscapeComponentUpdateFlag::Component_Update_Client_Editing;
		if (WeightmapEditingFlags & InFlag)
		{
			return true;
		}
	}

	if (InUpdateModes & ELandscapeLayerUpdateMode::Update_Client_Deferred)
	{
		const uint32 DeferredClientUpdateFlags = ELandscapeComponentUpdateFlag::Component_Update_Client;
		if (DeferredClientUpdateFlags & InFlag)
		{
			return true;
		}
	}

	if (InUpdateModes & (ELandscapeLayerUpdateMode::Update_Heightmap_Editing_NoCollision | ELandscapeLayerUpdateMode::Update_Weightmap_Editing_NoCollision))
	{
		const uint32 EditingNoCollisionFlags = ELandscapeComponentUpdateFlag::Component_Update_Approximated_Bounds;
		if (EditingNoCollisionFlags & InFlag)
		{
			return true;
		}
	}

	return false;
}

void ULandscapeComponent::ClearUpdateFlagsForModes(uint32 InModeMask)
{
	LayerUpdateFlagPerMode &= ~InModeMask;
}

void ULandscapeComponent::RequestDeferredClientUpdate()
{
	LayerUpdateFlagPerMode |= ELandscapeLayerUpdateMode::Update_Client_Deferred;
}

void ULandscapeComponent::RequestEditingClientUpdate(bool bInUserTriggered)
{
	bUserTriggeredChangeRequested = bInUserTriggered; 
	
	LayerUpdateFlagPerMode |= ELandscapeLayerUpdateMode::Update_Client_Editing;
	if (ALandscape* LandscapeActor = GetLandscapeActor())
	{
		LandscapeActor->RequestLayersContentUpdate(ELandscapeLayerUpdateMode::Update_Client_Editing);
	}
}

void ULandscapeComponent::RequestHeightmapUpdate(bool bUpdateAll, bool bUpdateCollision, bool bInUserTriggered)
{
	bUserTriggeredChangeRequested = bInUserTriggered;
	uint32 NewLayerUpdateFlags = 0;
	if (bUpdateAll || bUpdateCollision)
	{
		NewLayerUpdateFlags |= ELandscapeLayerUpdateMode::Update_Heightmap_Editing;
		NewLayerUpdateFlags |= ELandscapeLayerUpdateMode::Update_Heightmap_All_Modes;
	}
	else
	{
		NewLayerUpdateFlags |= ELandscapeLayerUpdateMode::Update_Heightmap_Editing_NoCollision;
	}

	LayerUpdateFlagPerMode |= NewLayerUpdateFlags;

	// The typical codepath for sculpt or paint will have bUpdateCollision=true and bUpdateAll=false.  Only pass through
	// the _All_Modes flag in the case of bUpdateAll, not just from bUpdateCollision.
	uint32 ActorFlags = NewLayerUpdateFlags;
	if (!bUpdateAll)
	{
		ActorFlags &= ~ELandscapeLayerUpdateMode::Update_Heightmap_All_Modes;  // Remove _All_Modes flag
	}

	if (ALandscape* LandscapeActor = GetLandscapeActor())
	{
		LandscapeActor->RequestLayersContentUpdate((ELandscapeLayerUpdateMode)ActorFlags);
	}
	else
	{
		// This condition is only valid with streaming proxies, since the landscape actor will always be reachable in the non-WP Case
		ALandscapeStreamingProxy* LandscapeStreamingProxy = CastChecked<ALandscapeStreamingProxy>(GetLandscapeProxy());
		LandscapeStreamingProxy->RequestDeferredLayerContentUpdateModes((ELandscapeLayerUpdateMode)ActorFlags);
	}
}

void ULandscapeComponent::RequestWeightmapUpdate(bool bUpdateAll, bool bUpdateCollision, bool bInUserTriggered)
{
	bUserTriggeredChangeRequested = bInUserTriggered;
	uint32 NewLayerUpdateFlags = 0;
	if (bUpdateAll || bUpdateCollision)
	{
		NewLayerUpdateFlags |= ELandscapeLayerUpdateMode::Update_Weightmap_Editing;
		NewLayerUpdateFlags |= ELandscapeLayerUpdateMode::Update_Weightmap_All_Modes;
	}
	else
	{
		NewLayerUpdateFlags |= ELandscapeLayerUpdateMode::Update_Weightmap_Editing_NoCollision;
	}

	LayerUpdateFlagPerMode |= NewLayerUpdateFlags;

	// The typical codepath for sculpt or paint will have bUpdateCollision=true and bUpdateAll=false.  Only pass through
	// the _All_Modes flag in the case of bUpdateAll, not just from bUpdateCollision.
	uint32 ActorFlags = NewLayerUpdateFlags;
	if (!bUpdateAll)
	{
		ActorFlags &= ~ELandscapeLayerUpdateMode::Update_Weightmap_All_Modes;  // Remove _All_Modes flag
	}

	if (ALandscape* LandscapeActor = GetLandscapeActor())
	{
		LandscapeActor->RequestLayersContentUpdate((ELandscapeLayerUpdateMode)ActorFlags);
	}
	else
	{
		// This condition is only valid with streaming proxies, since the landscape actor will always be reachable in the non-WP Case
		ALandscapeStreamingProxy* LandscapeStreamingProxy = CastChecked<ALandscapeStreamingProxy>(GetLandscapeProxy());
		LandscapeStreamingProxy->RequestDeferredLayerContentUpdateModes((ELandscapeLayerUpdateMode)ActorFlags);
	}
}

void ALandscape::MonitorLandscapeEdModeChanges()
{
	bool bRequiredEditingClientFullUpdate = false;
	if (LandscapeEdModeInfo.ViewMode != GLandscapeViewMode)
	{
		LandscapeEdModeInfo.ViewMode = GLandscapeViewMode;
		bRequiredEditingClientFullUpdate = true;
	}

	ELandscapeToolTargetType NewValue = LandscapeEdMode ? LandscapeEdMode->GetLandscapeToolTargetType() : ELandscapeToolTargetType::Invalid;
	if (LandscapeEdModeInfo.ToolTarget != NewValue)
	{
		LandscapeEdModeInfo.ToolTarget = NewValue;
		bRequiredEditingClientFullUpdate = true;
	}

	const ULandscapeEditLayerBase* SelectedEditLayer = LandscapeEdMode ? LandscapeEdMode->GetLandscapeSelectedLayer() : nullptr;
	FGuid NewSelectedLayer = SelectedEditLayer && SelectedEditLayer->IsVisible() ? SelectedEditLayer->GetGuid() : FGuid();
	if (LandscapeEdModeInfo.SelectedLayer != NewSelectedLayer)
	{
		LandscapeEdModeInfo.SelectedLayer = NewSelectedLayer;
		bRequiredEditingClientFullUpdate = true;
	}

	TWeakObjectPtr<ULandscapeLayerInfoObject> NewLayerInfoObject;
	if (LandscapeEdMode)
	{
		NewLayerInfoObject = LandscapeEdMode->GetSelectedLandscapeLayerInfo();
	}
	if (LandscapeEdModeInfo.SelectedLayerInfoObject != NewLayerInfoObject)
	{
		LandscapeEdModeInfo.SelectedLayerInfoObject = NewLayerInfoObject;
		bRequiredEditingClientFullUpdate = true;
	}

	if (bRequiredEditingClientFullUpdate && (LandscapeEdModeInfo.ViewMode == ELandscapeViewMode::LayerContribution))
	{
		// Update_Client_Editing does not trigger readback changes. As a result, UpdateCollisionAndClients will not be run
		// Use Update_All_Editing to force an update, making sure the contribution mode visualization state is updated with the read back results
		RequestLayersContentUpdateForceAll(ELandscapeLayerUpdateMode::Update_All_Editing);
	}
}

void ULandscapeComponent::GetLandscapeComponentNeighborsToRender(TSet<ULandscapeComponent*>& OutNeighborComponents) const
{
	ULandscapeInfo* LandscapeInfo = GetLandscapeInfo();
	FIntPoint ComponentKey = GetComponentKey();

	for (int32 IndexX = ComponentKey.X - 1; IndexX <= ComponentKey.X + 1; ++IndexX)
	{
		for (int32 IndexY = ComponentKey.Y - 1; IndexY <= ComponentKey.Y + 1; ++IndexY)
		{
			ULandscapeComponent* Result = LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(IndexX, IndexY));
			if (Result && Result != this)
			{
				OutNeighborComponents.Add(Result);
			}
		}
	}
}

void ULandscapeComponent::GetLandscapeComponentNeighbors3x3(TStaticArray<ULandscapeComponent*, 9>& OutNeighborComponents) const
{
	ULandscapeInfo* LandscapeInfo = GetLandscapeInfo();
	FIntPoint ComponentKey = GetComponentKey();

	int32 LinearIndex = 0;
	for (int32 IndexY = ComponentKey.Y - 1; IndexY <= ComponentKey.Y + 1; ++IndexY)
	{
		for (int32 IndexX = ComponentKey.X - 1; IndexX <= ComponentKey.X + 1; ++IndexX)
		{
			OutNeighborComponents[LinearIndex] = LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(IndexX, IndexY));
			++LinearIndex;
		}
	}
}

void ULandscapeComponent::GetLandscapeComponentWeightmapsToRender(TSet<ULandscapeComponent*>& OutWeightmapComponents) const
{
	// Fill with Components that share the same weightmaps so that the Resolve of Weightmap Texture doesn't resolve null data.
	for (ULandscapeWeightmapUsage* Usage : GetWeightmapTexturesUsage(/*InReturnEditingWeightmap = */false))
	{
		for (int32 Channel = 0; Channel < ULandscapeWeightmapUsage::NumChannels; ++Channel)
		{
			if (Usage != nullptr && Usage->ChannelUsage[Channel] != nullptr)
			{
				ULandscapeComponent* Component = Usage->ChannelUsage[Channel];
				OutWeightmapComponents.Add(Component);
			}
		}
	}
}

void ALandscape::FWaitingForResourcesNotificationHelper::Notify(ALandscape* InLandscape, FLandscapeNotificationManager* InNotificationManager, ELandscapeNotificationType InNotificationType, const FText& InNotificationText)
{
	// We need to wait until layers texture resources are ready to initialize the landscape to avoid taking the sizes and format of the default texture:
	static constexpr double TimeBeforeDisplayingWaitingForResourcesNotification = 3.0;

	WaitingForResourcesStartTime = FSlateApplicationBase::IsInitialized() ? FSlateApplicationBase::Get().GetCurrentTime() : 0.0f;
	if (!Notification.IsValid())
	{
		Notification = MakeShared<FLandscapeNotification>(InLandscape, InNotificationType);
		Notification->NotificationText = InNotificationText;
		Notification->NotificationStartTime = WaitingForResourcesStartTime + TimeBeforeDisplayingWaitingForResourcesNotification;
	}
	InNotificationManager->RegisterNotification(Notification);
}

void ALandscape::FWaitingForResourcesNotificationHelper::Reset()
{
	Notification.Reset();
	WaitingForResourcesStartTime = -1.0;
}

bool ALandscape::CanUpdateLayersContent() const
{
	ULandscapeInfo* LandscapeInfo = GetLandscapeInfo();
	UWorld* World = GetWorld();
	ULandscapeSubsystem* Subsystem = World ? GetWorld()->GetSubsystem<ULandscapeSubsystem>() : nullptr;

	return FApp::CanEverRender() && (LandscapeInfo != nullptr) && !IsTemplate() &&
		LandscapeInfo->AreAllComponentsRegistered() && LandscapeInfo->SupportsLandscapeEditing() &&
		Subsystem && Subsystem->GetTextureStreamingManager();
}

// Render the edit layers for an explicit set of components, for a subset of layers.  Do not modify the regular persistent state of edit layer data.
// Empty InLayerVisibility implies using each layer's default visibility status.
bool ALandscape::SelectiveRenderEditLayersHeightmaps(const FLandscapeEditLayerRenderHeightParams& InSelectiveRenderParams)
{
	check((InSelectiveRenderParams.CpuResult.Num() > 0) ^ (InSelectiveRenderParams.RTResult != nullptr));  //Expecting either a cpu result buffer or a render target.  Not both.

	if (InSelectiveRenderParams.Bounds.Area() <= 0)
	{
		return true;  //nothing to render
	}
	
	// Expecting to be already initialized for BatchMerge.
	check(bLandscapeLayersAreInitialized);

	if (!CanUpdateLayersContent())
	{
		return false;
	}

	UWorld* World = GetWorld();
	check(World != nullptr);
	ULandscapeSubsystem* LandscapeSubsystem = World->GetSubsystem<ULandscapeSubsystem>();
	check(LandscapeSubsystem != nullptr);
	ULandscapeInfo* LandscapeInfo = GetLandscapeInfo();
	check(LandscapeInfo);

	const bool bWaitForStreaming = true;
	if (!PrepareTextureResources(bWaitForStreaming) ||
		!PrepareLayersTextureResources(bWaitForStreaming) ||
		!PrepareLayersResources(GetFeatureLevelShaderPlatform_Checked(World->GetFeatureLevel()), bWaitForStreaming))
	{
		return false;
	}

	TSet<ULandscapeComponent*> AffectedComponents;
	LandscapeInfo->GetComponentsInRegion(InSelectiveRenderParams.Bounds.Min.X, InSelectiveRenderParams.Bounds.Min.Y, InSelectiveRenderParams.Bounds.Max.X, InSelectiveRenderParams.Bounds.Max.Y, AffectedComponents);
	if (AffectedComponents.Num() == 0)
	{
		// No loaded components in region
		UE_LOGF(LogLandscape, Display, "SelectiveRenderLayers with no affected components");
		return true;
	}

	FTextureToComponentHelper Helper(*LandscapeInfo, AffectedComponents.Array(), FTextureToComponentHelper::ERefreshFlags::RefreshHeightmaps);
	FUpdateLayersContentContext LayerContext(Helper, FUpdateLayersContentContext::FSelectiveRenderTag(), FUpdateLayersContentContext::ERefreshFlags::RefreshHeightmapInfos);

	{
		RenderCaptureInterface::FScopedCapture RenderCapture((RenderCaptureLayersNextHeightmapDraws != 0), TEXT("SelectiveRenderEditLayersHeightmaps"));
		RenderCaptureLayersNextHeightmapDraws = FMath::Max(0, RenderCaptureLayersNextHeightmapDraws - 1);

		if (!InSelectiveRenderParams.CpuResult.IsEmpty())
		{
			return PerformSelectiveRenderEditLayersHeightmapsCPU(LayerContext, InSelectiveRenderParams);
		}
		else
		{
			return PerformSelectiveRenderEditLayersHeightmapsRT(LayerContext, InSelectiveRenderParams);
		}
	}
}

bool ALandscape::SelectiveRenderEditLayersWeightmaps(const FLandscapeEditLayerRenderWeightParams& InSelectiveRenderParams)
{
	check((InSelectiveRenderParams.CpuResult.Num() > 0) ^ (InSelectiveRenderParams.RTResult != nullptr));  //Expecting either a cpu result buffer or a render target.  Not both.

	if (InSelectiveRenderParams.Bounds.Area() <= 0)
	{
		return true;  //nothing to render
	}

	// Expecting to be already initialized for BatchMerge.
	check(bLandscapeLayersAreInitialized);

	if (!CanUpdateLayersContent())
	{
		return false;
	}

	UWorld* World = GetWorld();
	check(World != nullptr);
	ULandscapeSubsystem* LandscapeSubsystem = World->GetSubsystem<ULandscapeSubsystem>();
	check(LandscapeSubsystem != nullptr);
	ULandscapeInfo* LandscapeInfo = GetLandscapeInfo();
	check(LandscapeInfo);

	const bool bWaitForStreaming = true;
	if (!PrepareTextureResources(bWaitForStreaming) ||
		!PrepareLayersTextureResources(bWaitForStreaming) ||
		!PrepareLayersResources(GetFeatureLevelShaderPlatform_Checked(World->GetFeatureLevel()), bWaitForStreaming))
	{
		return false;
	}

	TSet<ULandscapeComponent*> AffectedComponents;
	LandscapeInfo->GetComponentsInRegion(InSelectiveRenderParams.Bounds.Min.X, InSelectiveRenderParams.Bounds.Min.Y, InSelectiveRenderParams.Bounds.Max.X, InSelectiveRenderParams.Bounds.Max.Y, AffectedComponents);
	if (AffectedComponents.Num() == 0)
	{
		// No loaded components in region
		UE_LOGF(LogLandscape, Display, "SelectiveRenderLayers with no affected components");
		return true;
	}

	FTextureToComponentHelper Helper(*LandscapeInfo, AffectedComponents.Array(), FTextureToComponentHelper::ERefreshFlags::RefreshWeightmaps);
	FUpdateLayersContentContext LayerContext(Helper, FUpdateLayersContentContext::FSelectiveRenderTag(), FUpdateLayersContentContext::ERefreshFlags::RefreshWeightmapInfos);

	{
		RenderCaptureInterface::FScopedCapture RenderCapture((RenderCaptureLayersNextHeightmapDraws != 0), TEXT("SelectiveRenderEditLayersWeightmaps"));
		RenderCaptureLayersNextHeightmapDraws = FMath::Max(0, RenderCaptureLayersNextHeightmapDraws - 1);

		if (!InSelectiveRenderParams.CpuResult.IsEmpty())
		{
			return PerformSelectiveRenderEditLayersWeightmapsCPU(LayerContext, InSelectiveRenderParams);
		}
		else
		{
			unimplemented();
			return false;
		}
	}
}

void ALandscape::UpdateLayersContent(bool bInWaitForStreaming, bool bInSkipMonitorLandscapeEdModeChanges, bool bFlushRender)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeLayers_UpdateLayersContent);

	// Detect any attempt to re-enter.  If called from blueprint, log an error and return early instead of asserting.
	if (LayerUpdateCount > 0 && UE::Landscape::Private::InBPCallstack())
	{
		UE_LOGF(LogLandscapeBP, Error, "Attempting to make illegal re-entrant call to UpdateLayersContent.");
		return;
	}

	check(LayerUpdateCount == 0);
	LayerUpdateCount++;

	bool bHideNotifications = true;
	ON_SCOPE_EXIT
	{
		// Make sure that we don't leave any notification behind when we leave this function without explicitly displaying one :
		if (bHideNotifications)
		{
			WaitingForTexturesNotificationHelper.Reset();
			WaitingForEditLayerResourcesNotificationHelper.Reset();
		}

		// If nothing to do, let's do some garbage collecting on async readback tasks so that we slowly get rid of staging textures 
		//  (don't do it while waiting for read backs because something might prevent us from updating the readbacks (e.g. waiting for resources to compiling...), which would 
		//  lead to FLandscapeEditReadbackTaskPool's frame count increasing while readback tasks don't have the chance to complete, leading to the "readback leak" warning to incorrectly be triggered) :
		if(IsUpToDate())
		{
			FLandscapeEditLayerReadback::GarbageCollectTasks();
		}

		LayerUpdateCount--;
	};

	// Note : no early-out allowed before this : even if not actually updating edit layers, we need to poll our resources in order to make sure we register to streaming events when needed:
	// Always bypass the rate limit on PrepareTextureResources from here.  The rate limit is to prevent hammering many times per frame from physical material updates.
	EPrepareTextureResourcesLimitedMode PrepareFlags = EPrepareTextureResourcesLimitedMode::BypassRateLimit;
	if (bInWaitForStreaming)
	{
		PrepareFlags |= EPrepareTextureResourcesLimitedMode::WaitForStreaming;
	}
	bool bResourcesReady = PrepareTextureResourcesLimited(PrepareFlags);

	ULandscapeInfo* LandscapeInfo = GetLandscapeInfo();
	if (!CanUpdateLayersContent())
	{
		return;
	}

	UWorld* World = GetWorld();
	check(World != nullptr);
	ULandscapeSubsystem* LandscapeSubsystem = World->GetSubsystem<ULandscapeSubsystem>();
	check(LandscapeSubsystem != nullptr);
	FLandscapeNotificationManager* LandscapeNotificationManager = LandscapeSubsystem->GetNotificationManager();

	// Make sure Update doesn't dirty Landscape packages when not in Landscape Ed Mode
	FLandscapeDirtyOnlyInModeScope DirtyOnlyInMode(LandscapeInfo);

	if (!bLandscapeLayersAreInitialized)
	{
		InitializeLayers();
	}

	if (!bLandscapeLayersAreInitialized)
	{
		// we failed to initialize layers, cannot continue
		return;
	}

	if (!bInSkipMonitorLandscapeEdModeChanges)
	{
		MonitorLandscapeEdModeChanges();
	}

	// Make sure Brush get a chance to request an update of the landscape
	for (FLandscapeLayer& Layer : LandscapeEditLayers)
	{
		for (FLandscapeLayerBrush& Brush : Layer.Brushes)
		{
			if (ALandscapeBlueprintBrushBase* LandscapeBrush = Brush.GetBrush())
			{
				LandscapeBrush->PushDeferredLayersContentUpdate();
			}
		}
	}

	// Make sure weightmap usages that need updating are processed before doing any update on the landscape : 
	UpdateProxyLayersWeightmapUsage();

	if (bSplineLayerUpdateRequested)
	{
		if (const FLandscapeLayer* SplinesLayer = FindLayerOfTypeConst(ULandscapeEditLayerSplines::StaticClass()))
		{
			// We need the spline layer resources to all be ready before updating it:
			if (!PrepareLayersTextureResources({ *SplinesLayer }, bInWaitForStreaming))
			{
				return;
			}

			UpdateAllLandscapeSplines();
			bSplineLayerUpdateRequested = false;
		}
	}

	const bool bProcessReadbacks = FLandscapeEditLayerReadback::HasAnyGlobalWork();

	// User triggered change has been completely processed, resetting user triggered flag on all components.
	if(IsUpToDate())
	{
		GetLandscapeInfo()->ForAllLandscapeComponents(
			[this](ULandscapeComponent* Component) -> void
			{
				if (Component->GetUserTriggeredChangeRequested())
				{
					check(Component->GetLayerUpdateFlagPerMode() == 0);
					Component->SetUserTriggeredChangeRequested(/* bInUserTriggered = */false);	
				}
			}
		);

		return;
	}

	bResourcesReady &= PrepareLayersTextureResources(bInWaitForStreaming);
	if (!bResourcesReady && LandscapeNotificationManager)
	{
		WaitingForTexturesNotificationHelper.Notify(this, LandscapeNotificationManager, ELandscapeNotificationType::LandscapeTextureResourcesNotReady, LOCTEXT("WaitForLandscapeTextureResources", "Waiting for texture resources to be ready"));
		bHideNotifications = false;
	}
	else
	{
		WaitingForTexturesNotificationHelper.Reset();
	}

	bResourcesReady &= PrepareLayersResources(GetFeatureLevelShaderPlatform_Checked(World->GetFeatureLevel()), bInWaitForStreaming);
	if (!bResourcesReady && LandscapeNotificationManager)
	{
		WaitingForEditLayerResourcesNotificationHelper.Notify(this, LandscapeNotificationManager, ELandscapeNotificationType::LandscapeEditLayerResourcesNotReady, LOCTEXT("WaitForLandscapeEditLayerResources", "Waiting for edit layer resources to be ready"));
		bHideNotifications = false;
	}
	else
	{
		WaitingForEditLayerResourcesNotificationHelper.Reset();
	}

	if (!bResourcesReady)
	{
		return;
	}

	// Gather mappings between heightmaps/weightmaps and components
	FTextureToComponentHelper MapHelper(*LandscapeInfo);

	// Poll and complete any outstanding resolve work
	if (bProcessReadbacks)
	{
		// Flushing once all readback tasks is much faster than asking each to do it so start by doing just this :
		if (bFlushRender)
		{
			FLandscapeEditLayerReadback::FlushAllReadbackTasks();
		}

		TArray<FLandscapeEditLayerComponentReadbackResult> ComponentReadbackResults;
		ResolveLayersHeightmapTexture(MapHelper, MapHelper.Heightmaps, ComponentReadbackResults);
		ResolveLayersWeightmapTexture(MapHelper, MapHelper.Weightmaps, ComponentReadbackResults);
		LayerContentUpdateModes |= UpdateAfterReadbackResolves(ComponentReadbackResults);
	}

	// The Edit layers shaders only work on SM5+ : log warning that any landscape updates will not happen when SM5+ shading model is not active :
	if (World->GetFeatureLevel() >= ERHIFeatureLevel::SM5)
	{
		bHasWarnedAboutInvalidShadingModel = false;
	}
	else
	{
		// Log once when an update is requested, then early out so no updates are processed until a valid feature level is re-enabled
		if (LayerContentUpdateModes != ELandscapeLayerUpdateMode::Update_None)
		{
			if (!bHasWarnedAboutInvalidShadingModel)
			{
				UE_LOGF(LogLandscape, Warning, "Cannot update landscape with a feature level less than SM5");
				bHasWarnedAboutInvalidShadingModel = true;
			}
		}
		return;
	}

	if (LayerContentUpdateModes == 0)
	{
		return;
	}

	bool bUpdateAllComponents = LayerContentUpdateModes & Update_All_Components;
	bool bPartialUpdate = !bUpdateAllComponents && CVarLandscapeLayerOptim.GetValueOnAnyThread() == 1;

	FUpdateLayersContentContext UpdateLayersContentContext(MapHelper, bPartialUpdate);

	// Regenerate any heightmaps and weightmaps
	int32 ProcessedModes = 0;
	ProcessedModes |= RegenerateLayersHeightmaps(UpdateLayersContentContext);
	ProcessedModes |= RegenerateLayersWeightmaps(UpdateLayersContentContext);
	ProcessedModes |= (LayerContentUpdateModes & ELandscapeLayerUpdateMode::Update_Client_Deferred);
	ProcessedModes |= (LayerContentUpdateModes & ELandscapeLayerUpdateMode::Update_Client_Editing);
	ProcessedModes |= (LayerContentUpdateModes & ELandscapeLayerUpdateMode::Update_All_Components);

	// If we are flushing then read back resolved textures immediately
	if (bFlushRender || CVarLandscapeForceFlush.GetValueOnGameThread() != 0)
	{
		// Flushing once all readback tasks is much faster than asking each to do it so start by doing just this :
		FLandscapeEditLayerReadback::FlushAllReadbackTasks();
		// When flushing, don't bother resolving textures that weren't requested to be updated in the first place
		//  We cannot do this in the non-flush case above, because LayerContentUpdateModes might have changed since the readbacks have been requested so we still need to perform the readbacks on all textures
		if ((LayerContentUpdateModes & ELandscapeLayerUpdateMode::Update_Heightmap_Types) != 0)
		{
			ResolveLayersHeightmapTexture(UpdateLayersContentContext.MapHelper, UpdateLayersContentContext.HeightmapsToResolve, UpdateLayersContentContext.AllLandscapeComponentReadbackResults);
		}
		if ((LayerContentUpdateModes & ELandscapeLayerUpdateMode::Update_Weightmap_Types) != 0)
		{
			ResolveLayersWeightmapTexture(UpdateLayersContentContext.MapHelper, UpdateLayersContentContext.WeightmapsToResolve, UpdateLayersContentContext.AllLandscapeComponentReadbackResults);
		}
	}

	// Clear processed mode flags
	LayerContentUpdateModes &= ~ProcessedModes;
	for (ULandscapeComponent* Component : UpdateLayersContentContext.AllLandscapeComponentsToResolve)
	{
		Component->ClearUpdateFlagsForModes(ProcessedModes);
	}

	// Apply post resolve updates
	const uint32 ToProcessModes = UpdateAfterReadbackResolves(UpdateLayersContentContext.AllLandscapeComponentReadbackResults);
	LayerContentUpdateModes |= ToProcessModes;
	if (LandscapeEdMode)
	{
		LandscapeEdMode->PostUpdateLayerContent();
	}

	// Additional validation that at the end of an update, we haven't screwed up anything in the weightmap allocations/usages : 
	ValidateProxyLayersWeightmapUsage();
}

// not thread safe
struct FEnableCollisionHashOptimScope
{
	FEnableCollisionHashOptimScope(ULandscapeHeightfieldCollisionComponent* InCollisionComponent)
	{
		CollisionComponent = InCollisionComponent;
		if (CollisionComponent)
		{
			// not reentrant
			check(!CollisionComponent->bEnableCollisionHashOptim);
			CollisionComponent->bEnableCollisionHashOptim = true;
		}
	}

	~FEnableCollisionHashOptimScope()
	{
		if (CollisionComponent)
		{
			CollisionComponent->bEnableCollisionHashOptim = false;
		}
	}

private:
	ULandscapeHeightfieldCollisionComponent* CollisionComponent;
};

uint32 ALandscape::UpdateCollisionAndClients(const TArrayView<FLandscapeEditLayerComponentReadbackResult>& InComponentReadbackResults)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeLayers_PostResolve_CollisionAndClients);

	bool bAllClientsUpdated = true;

	const uint16 DefaultHeightValue = LandscapeDataAccess::GetTexHeight(0.f);
	const uint8 MaxLayerContributingValue = UINT8_MAX;
	const float HeightValueNormalizationFactor = 1.f / (0.5f * UINT16_MAX);
	TArray<uint16> HeightData;
	TArray<uint8> LayerContributionMaskData;

	for (const FLandscapeEditLayerComponentReadbackResult& ComponentReadbackResult : InComponentReadbackResults)
	{
		ULandscapeComponent* LandscapeComponent = ComponentReadbackResult.LandscapeComponent;
		
		bool bDeferClientUpdateForComponent = false;
		bool bDoUpdateClient = true;
		if (IsUpdateFlagEnabledForModes(ELandscapeComponentUpdateFlag::Component_Update_Recreate_Collision, ComponentReadbackResult.UpdateModes)
			|| ComponentReadbackResult.bCleared)
		{
			if (ULandscapeHeightfieldCollisionComponent* CollisionComp = LandscapeComponent->GetCollisionComponent())
			{
				FEnableCollisionHashOptimScope Scope(CollisionComp);
				bDoUpdateClient = CollisionComp->RecreateCollision();
			}
		}

		if (bDoUpdateClient && IsUpdateFlagEnabledForModes(ELandscapeComponentUpdateFlag::Component_Update_Client, ComponentReadbackResult.UpdateModes))
		{
			if (!GUndo)
			{
				if (ULandscapeHeightfieldCollisionComponent* CollisionComp = LandscapeComponent->GetCollisionComponent())
				{
					FNavigationSystem::UpdateComponentData(*CollisionComp);

					const EFoliageLandscapeSnappingMode SnappingMode = GetDefault<ULandscapeSettings>()->GetFoliageLandscapeSnappingMode();
					if (SnappingMode != EFoliageLandscapeSnappingMode::DontSnap)
					{
						const ULandscapeHeightfieldCollisionComponent::ESnapFoliageInstancesFlags Flags = (SnappingMode == EFoliageLandscapeSnappingMode::SnapUnderBrushOnly)
							? ULandscapeHeightfieldCollisionComponent::ESnapFoliageInstancesFlags::MarkedAreasOnly
							: ULandscapeHeightfieldCollisionComponent::ESnapFoliageInstancesFlags::ForceAll;
						CollisionComp->SnapFoliageInstances(Flags);
					}
				}
			}
			else
			{
				bDeferClientUpdateForComponent = true;
				bAllClientsUpdated = false;
			}
		}

		if (IsUpdateFlagEnabledForModes(ELandscapeComponentUpdateFlag::Component_Update_Client_Editing, ComponentReadbackResult.UpdateModes))
		{
			if (LandscapeEdModeInfo.ViewMode == ELandscapeViewMode::LayerContribution)
			{
				check(ComponentSizeQuads == LandscapeComponent->ComponentSizeQuads);
				const int32 Stride = (1 + ComponentSizeQuads);
				const int32 ArraySize = Stride * Stride;
				if (LayerContributionMaskData.Num() != ArraySize)
				{
					LayerContributionMaskData.AddZeroed(ArraySize);
				}
				uint8* LayerContributionMaskDataPtr = LayerContributionMaskData.GetData();
				const int32 X1 = LandscapeComponent->GetSectionBase().X;
				const int32 X2 = X1 + ComponentSizeQuads;
				const int32 Y1 = LandscapeComponent->GetSectionBase().Y;
				const int32 Y2 = Y1 + ComponentSizeQuads;
				bool bLayerContributionWrittenData = false;

				ULandscapeInfo* Info = LandscapeComponent->GetLandscapeInfo();
				check(Info);
				FLandscapeEditDataInterface LandscapeEdit(Info);

				if (LandscapeEdModeInfo.SelectedLayer.IsValid())
				{
					FScopedSetLandscapeEditingLayer Scope(this, LandscapeEdModeInfo.SelectedLayer);
					if (LandscapeEdModeInfo.ToolTarget == ELandscapeToolTargetType::Heightmap)
					{
						if (HeightData.Num() != ArraySize)
						{
							HeightData.AddZeroed(ArraySize);
						}
						LandscapeEdit.GetHeightDataFast(X1, Y1, X2, Y2, HeightData.GetData(), Stride);
						for (int i = 0; i < ArraySize; ++i)
						{
							LayerContributionMaskData[i] = HeightData[i] != DefaultHeightValue ? (uint8)(FMath::Pow(FMath::Clamp((HeightValueNormalizationFactor * FMath::Abs(HeightData[i] - DefaultHeightValue)), 0.f, (float)1.f), 0.25f) * MaxLayerContributingValue) : 0;
						}
						bLayerContributionWrittenData = true;
					}
					else if (LandscapeEdModeInfo.ToolTarget == ELandscapeToolTargetType::Weightmap || LandscapeEdModeInfo.ToolTarget == ELandscapeToolTargetType::Visibility)
					{
						ULandscapeLayerInfoObject* LayerObject = (LandscapeEdModeInfo.ToolTarget == ELandscapeToolTargetType::Visibility) ? ALandscapeProxy::VisibilityLayer : LandscapeEdModeInfo.SelectedLayerInfoObject.Get();
						if (LayerObject)
						{
							LandscapeEdit.GetWeightDataFast(LayerObject, X1, Y1, X2, Y2, LayerContributionMaskData.GetData(), Stride);
							bLayerContributionWrittenData = true;
						}
					}
				}
				if (!bLayerContributionWrittenData)
				{
					FMemory::Memzero(LayerContributionMaskDataPtr, ArraySize);
				}
				LandscapeEdit.SetLayerContributionData(X1, Y1, X2, Y2, LayerContributionMaskDataPtr, 0);
			}
		}

		if (bDeferClientUpdateForComponent)
		{
			LandscapeComponent->RequestDeferredClientUpdate();
		}
	}

	// Some clients not updated so return the Deferred flag to trigger processing next update.
	return bAllClientsUpdated ? 0 : ELandscapeLayerUpdateMode::Update_Client_Deferred;
}

uint32 ALandscape::UpdateAfterReadbackResolves(const TArrayView<FLandscapeEditLayerComponentReadbackResult>& InComponentReadbackResults)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeLayers_PostResolve_Updates);

	uint32 NewUpdateFlags = 0;

	if (InComponentReadbackResults.Num())
	{
		UpdateForChangedHeightmaps(InComponentReadbackResults);
		UpdateForChangedWeightmaps(InComponentReadbackResults);

		GetLandscapeInfo()->UpdateAllAddCollisions();

		NewUpdateFlags |= UpdateCollisionAndClients(InComponentReadbackResults);
	}

	return NewUpdateFlags;
}

void ALandscape::InitializeLayers()
{
	InitializeLandscapeLayersWeightmapUsage();
	bLandscapeLayersAreInitialized = true;
}

void ALandscape::OnPreSave()
{
	// Note:: This is only called if the outer level is saved.
	FlushLayerContentThisFrame();
}

// If any work is pending in the layer update system, wait for it to complete.  Won't flush more than once per frame.
void ALandscape::FlushLayerContentThisFrame()
{
	// Only call ForceUpdateLayersContent once per frame. Each proxy might trigger this, so only do it for the first one.
	// Don't attempt this if the ULandscapeSubsystem or TextureStreamingManager don't exist.
	uint32 CurrentFrame = GFrameNumber;
	if (LastFlushedLayerUpdateFrame != CurrentFrame && CanUpdateLayersContent())
	{
		LastFlushedLayerUpdateFrame = CurrentFrame;
		ForceUpdateLayersContent();
	}
}

void ALandscape::ForceUpdateLayersContent()
{
	UpdateLayersContent(/*bWaitForStreaming = */true, /*bInSkipMonitorLandscapeEdModeChanges = */true, /*bFlushRender = */true);
}

// Deprecated
void ALandscape::ForceUpdateLayersContent(bool bIntermediateRender)
{
	ForceUpdateLayersContent();
}

void ALandscape::ForceLayersFullUpdate()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ALandscape::ForceLayersFullUpdate);

	FAssetCompilingManager::Get().FinishAllCompilation();

	FStreamingManagerCollection& StreamingManagers = IStreamingManager::Get();
	StreamingManagers.UpdateResourceStreaming(GetWorld()->GetDeltaSeconds(), /* bProcessEverything */ true);
	StreamingManagers.BlockTillAllRequestsFinished();

	RequestSplineLayerUpdate();
	RequestLayersContentUpdateForceAll();
	ForceUpdateLayersContent();
}

void ALandscape::TickLayers(float DeltaTime)
{
	check(GIsEditor);

	if (!bEnableEditorLayersTick)
	{
		return;
	}

	UWorld* World = GetWorld();
	if (World && !World->IsPlayInEditor() && GetLandscapeInfo() && GEditor->PlayWorld == nullptr)
	{
		if (CVarLandscapeSimulatePhysics.GetValueOnAnyThread() == 1)
		{
			World->bShouldSimulatePhysics = true;
		}

		UpdateLayersContent(/*bInWaitForStreaming = */false, /*bInSkipMonitorLandscapeEdModeChanges = */false, /*bFlushRender = */false);
	}
}

#endif

void ALandscapeProxy::FinishDestroy()
{
#if WITH_EDITORONLY_DATA
	for (auto& ItPair : HeightmapsCPUReadback)
	{
		FLandscapeEditLayerReadback* HeightmapCPUReadback = ItPair.Value;
		delete HeightmapCPUReadback;
	}
	HeightmapsCPUReadback.Empty();

	for (auto& ItPair : WeightmapsCPUReadback)
	{
		FLandscapeEditLayerReadback* WeightmapCPUReadback = ItPair.Value;
		delete WeightmapCPUReadback;
	}
	WeightmapsCPUReadback.Empty();
#endif

	Super::FinishDestroy();
}

#if WITH_EDITOR
// Deprecated
bool ALandscapeProxy::CanHaveLayersContent() const
{
	return true;
}

// Deprecated
bool ALandscapeProxy::HasLayersContent() const
{
	return true;
}

// Deprecated
void ALandscapeProxy::UpdateCachedHasLayersContent(bool InCheckComponentDataIntegrity)
{
}

namespace
{
	bool DeleteUnusedLayersImpl(ULandscapeComponent* InComponent, const FGuid& InLayerGuid)
	{
		TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = InComponent->GetWeightmapLayerAllocations(InLayerGuid);
		bool bWasModified = false;

		for (int32 LayerIdx = 0; LayerIdx < ComponentWeightmapLayerAllocations.Num();)
		{
			const FWeightmapLayerAllocationInfo& Allocation = ComponentWeightmapLayerAllocations[LayerIdx];
			const TArray<TObjectPtr<UTexture2D>>& WeightmapTextures = InComponent->GetWeightmapTextures(InLayerGuid);
			UTexture2D* Texture = WeightmapTextures[Allocation.WeightmapTextureIndex];

			if (Texture == nullptr)
			{
				++LayerIdx;
				continue;
			}

			const uint8* MipDataPtr = Texture->Source.LockMipReadOnly(0);

			if (MipDataPtr == nullptr)
			{
				++LayerIdx;
				continue;
			}
				
			const uint8* const TextDataPtr = MipDataPtr + ChannelOffsets[Allocation.WeightmapTextureChannel];

			constexpr bool bShouldDirtyPackage = true;

			// If DeleteLayerIfAllZero returns true, We just removed the current layer allocation, so we need to iterate on the new current index.
			if (InComponent->DeleteLayerIfAllZero(InLayerGuid, TextDataPtr, Texture->GetSizeX(), LayerIdx, bShouldDirtyPackage))
			{
				bWasModified = true;
			}
			else
			{
				++LayerIdx;
			}

			Texture->Source.UnlockMip(0);
		}

		if (bWasModified)
		{
			InComponent->UpdateMaterialInstances();
			InComponent->MarkRenderStateDirty();
		}

		return bWasModified;
	}
}

void ALandscapeProxy::DeleteUnusedLayers()
{
	bool bWasModified = false;
	
	for (ULandscapeComponent* Component : LandscapeComponents)
	{
		if (Component == nullptr)
		{
			continue;
		}

		Component->ForEachLayer([Component, &bWasModified](const FGuid& LayerGuid, FLandscapeLayerComponentData& LayerData)
		{
			bWasModified = DeleteUnusedLayersImpl(Component, LayerGuid);
		});

		// Execute ClearUnusedLayersImpl on the final Layer.
		bWasModified = DeleteUnusedLayersImpl(Component, FGuid());

		if (bWasModified)
		{
			InvalidateNaniteRepresentation(false);
		}
	}
}

// Deprecated
bool ALandscapeProxy::RemoveObsoleteLayers(const TSet<FGuid>& InExistingLayers)
{
	return RemoveObsoleteEditLayers(/*bShowMapCheckWarning =*/true);
}

bool ALandscapeProxy::RemoveObsoleteEditLayers(bool bShowMapCheckWarning)
{
	bool bModified = false;
	ULandscapeSubsystem* LandscapeSubsystem = GetWorld()->GetSubsystem<ULandscapeSubsystem>();
	if (!LandscapeSubsystem || !GetLandscapeActor())
	{
		UE_LOGF(LogLandscape, Error, "RemoveObsoleteEditLayers: Cannot remove obsolete edit layer data until proxy is registered with parent ALandscape.");
		return bModified;
	}

	for (const TPair<FGuid, FName>& LayerGuidToDebugName: GetObsoleteEditLayersInComponents())
	{
		if (bShowMapCheckWarning)
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("LayerName"), FText::FromString(LayerGuidToDebugName.Value.ToString()));
			Arguments.Add(TEXT("LayerGuid"), FText::FromString(LayerGuidToDebugName.Key.ToString(EGuidFormats::HexValuesInBraces)));

			FMessageLog("MapCheck").Info()
				->AddToken(FUObjectToken::Create(this, FText::FromString(GetActorNameOrLabel())))
				->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_LandscapeProxyObsoleteLayer","Layer '{LayerName}' ('{LayerGuid}') was removed from LandscapeProxy because it doesn't match any Edit Layers in the parent Landscape. Saving will remove this data for good."), Arguments)))
				->AddToken(FActionToken::Create(LOCTEXT("MapCheck_RemoveObsoleteLayers", "Save Modified Landscapes"), LOCTEXT("MapCheck_RemoveObsoleteLayers_Desc", "Saves the modified landscape proxy actors"),
					FOnActionTokenExecuted::CreateUObject(LandscapeSubsystem, &ULandscapeSubsystem::SaveModifiedLandscapes, UE::Landscape::EBuildFlags::WriteFinalLog),
					FCanExecuteActionToken::CreateUObject(LandscapeSubsystem, &ULandscapeSubsystem::HasModifiedLandscapes),
					/*bInSingleUse = */false))
				->AddToken(FMapErrorToken::Create(FMapErrors::LandscapeComponentPostLoad_Warning));
		}

		for (ULandscapeComponent* Component : LandscapeComponents)
		{
			Component->RemoveObsoleteEditLayerData(LayerGuidToDebugName.Key);
		}
		bModified = true;
	}

	if (bModified)
	{
		if (ALandscape* LandscapeActor = GetLandscapeActor())
		{
			LandscapeActor->RequestLayersContentUpdateForceAll();
		}
	}

	return bModified;
}

TMap<FGuid, FName> ALandscapeProxy::GetObsoleteEditLayersInComponents()
{
	TMap<FGuid, FName> ObsoleteEditLayers;

	if (!GetLandscapeActor())
	{
		UE_LOGF(LogLandscape, Error, "GetObsoleteEditLayersInComponents: Cannot get obsolete edit layer data until proxy is registered with parent ALandscape.");
		return ObsoleteEditLayers;
	}

	for (ULandscapeComponent* Component : LandscapeComponents)
	{
		if (Component != nullptr)
		{
			for (TPair<FGuid, FLandscapeLayerComponentData> ObsoleteLayer : Component->GetObsoleteEditLayersData())
			{
				if (ObsoleteLayer.Key.IsValid())
				{
					ObsoleteEditLayers.Add(TPair<FGuid, FName>(ObsoleteLayer.Key, ObsoleteLayer.Value.DebugName));
				}
			}
		}
	}

	return ObsoleteEditLayers;
}

bool ALandscapeProxy::AddLayer(const FGuid& InLayerGuid)
{
	bool bModified = false;
	for (ULandscapeComponent* Component : LandscapeComponents)
	{
		if ((Component != nullptr) && !Component->GetLayerData(InLayerGuid))
		{
			const ULandscapeEditLayerBase* EditLayer = GetLandscapeActor() ? GetLandscapeActor()->GetEditLayerConst(InLayerGuid) : nullptr;
			Component->AddLayerData(InLayerGuid, FLandscapeLayerComponentData(EditLayer ? EditLayer->GetName() : FName()));
			bModified = true;
		}
	}

	if (bModified)
	{
		InitializeLayerWithEmptyContent(InLayerGuid);
	}

	return bModified;
}

void ALandscapeProxy::DeleteLayer(const FGuid& InLayerGuid)
{
	for (ULandscapeComponent* Component : LandscapeComponents)
	{
		if (Component != nullptr)
		{
			const FLandscapeLayerComponentData* LayerComponentData = Component->GetLayerData(InLayerGuid);

			if (LayerComponentData != nullptr)
			{
				for (const FWeightmapLayerAllocationInfo& Allocation : LayerComponentData->WeightmapData.LayerAllocations)
				{
					UTexture2D* WeightmapTexture = LayerComponentData->WeightmapData.Textures[Allocation.WeightmapTextureIndex];
					TObjectPtr<ULandscapeWeightmapUsage>* Usage = WeightmapUsageMap.Find(WeightmapTexture);

					if (Usage != nullptr && (*Usage) != nullptr)
					{
						(*Usage)->ChannelUsage[Allocation.WeightmapTextureChannel] = nullptr;

						if ((*Usage)->IsEmpty())
						{
							WeightmapUsageMap.Remove(WeightmapTexture);
						}
					}
				}
				Component->RemoveLayerData(InLayerGuid);
			}
		}
	}
}

void ALandscapeProxy::InitializeLayerWithEmptyContent(const FGuid& InLayerGuid)
{
	if (IsPendingKillPending() || !GetLandscapeActor() || !LandscapeGuid.IsValid())
	{
		return;
	}

	// Build a mapping between each Heightmaps and Component in them
	TMap<UTexture2D*, TArray<ULandscapeComponent*>> ComponentsPerHeightmaps;

	for (ULandscapeComponent* Component : LandscapeComponents)
	{
		if (Component != nullptr)
		{
			UTexture2D* ComponentHeightmapTexture = Component->GetHeightmap();
			TArray<ULandscapeComponent*>& ComponentList = ComponentsPerHeightmaps.FindOrAdd(ComponentHeightmapTexture);
			ComponentList.Add(Component);
		}
	}

	// Init layers with valid "empty" data
	TMap<UTexture2D*, UTexture2D*> CreatedHeightmapTextures; // < Final layer texture, New created texture for layer

	for (ULandscapeComponent* Component : LandscapeComponents)
	{
		if (Component != nullptr)
		{
			UTexture2D* ComponentHeightmap = Component->GetHeightmap();
			const TArray<ULandscapeComponent*>* ComponentsUsingHeightmap = ComponentsPerHeightmaps.Find(ComponentHeightmap);
			check(ComponentsUsingHeightmap != nullptr);

			Component->AddDefaultLayerData(InLayerGuid, *ComponentsUsingHeightmap, CreatedHeightmapTextures);
		}
	}
}

TArray<FName> ALandscapeProxy::SynchronizeUnmarkedSharedProperties(ALandscapeProxy* InLandscape)
{
	check(InLandscape != nullptr);
	TArray<FName> SynchronizedProperties;
	USceneComponent* OwnRootComponent = GetRootComponent();
	USceneComponent* ProxyRootComponent = InLandscape->GetRootComponent();

	if ((OwnRootComponent != nullptr) && (ProxyRootComponent != nullptr) && ProxyRootComponent->HasBeenInitialized())
	{
		FVector ProxyScale3D = ProxyRootComponent->GetComponentToWorld().GetScale3D();

		if (!OwnRootComponent->GetRelativeScale3D().Equals(ProxyScale3D))
		{
			OwnRootComponent->SetRelativeScale3D(ProxyScale3D);
			SynchronizedProperties.Emplace(TEXT("RelativeScale3D"));
		}
	}

	return SynchronizedProperties;
}

#endif

bool ALandscape::IsUpToDate() const
{
	if (!FApp::CanEverRender())
	{
		return true;
	}

#if WITH_EDITORONLY_DATA
	if (!IsTemplate() && GetWorld() != nullptr && !GetWorld()->IsGameWorld())
	{
		return LayerContentUpdateModes == 0 && !FLandscapeEditLayerReadback::HasAnyGlobalWork();
	}
#endif

	return true;
}

#if WITH_EDITOR
bool ALandscape::IsLayerNameUnique(const FName& InName) const
{
	return Algo::CountIf(LandscapeEditLayers, [InName](const FLandscapeLayer& Layer) { return (Layer.EditLayer != nullptr) && (Layer.EditLayer->GetName() == InName); }) == 0;
}

void ALandscape::OnEditLayerDataChanged(const FOnLandscapeEditLayerDataChangedParams& InParams)
{
	const ULandscapeSettings* LandscapeSettings = GetDefault<ULandscapeSettings>();
	check(LandscapeSettings != nullptr);

	const bool bAllowLandscapeUpdate = (InParams.PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive) || LandscapeSettings->GetShouldUpdateEditLayersDuringInteractiveChanges();
	if (InParams.bRequiresLandscapeUpdate && bAllowLandscapeUpdate)
	{
		RequestLayersContentUpdateForceAll(ELandscapeLayerUpdateMode::Update_All, InParams.bUserTriggered);
	}
}

FLandscapeLayer* ALandscape::GetLayerInternal(int32 InLayerIndex)
{
	if (LandscapeEditLayers.IsValidIndex(InLayerIndex))
	{
		return &LandscapeEditLayers[InLayerIndex];
	}
	return nullptr;
}

ULandscapeEditLayerBase* ALandscape::GetEditLayerInternal(int32 InLayerIndex)
{
	if (LandscapeEditLayers.IsValidIndex(InLayerIndex))
	{
			return LandscapeEditLayers[InLayerIndex].EditLayer;
	}
	return nullptr;
}

TArrayView<const FLandscapeLayer> ALandscape::GetLayersConst() const
{
	return MakeArrayView(LandscapeEditLayers); 
}

const FLandscapeLayer* ALandscape::GetLayerConst(int32 InLayerIndex) const
{
	if (LandscapeEditLayers.IsValidIndex(InLayerIndex))
	{
		return &LandscapeEditLayers[InLayerIndex];
	}
	return nullptr;
}

int32 ALandscape::GetLayerIndex(const FGuid& InLayerGuid) const
{
	return LandscapeEditLayers.IndexOfByPredicate([&InLayerGuid](const FLandscapeLayer& Other) { return Other.EditLayer->GetGuid() == InLayerGuid; });
}

const FLandscapeLayer* ALandscape::GetLayerConst(const FGuid& InLayerGuid) const
{
	return LandscapeEditLayers.FindByPredicate([&InLayerGuid](const FLandscapeLayer& Other) { return Other.EditLayer->GetGuid() == InLayerGuid; });
}

const FLandscapeLayer* ALandscape::GetLayerConst(const FName& InLayerName) const
{
	return LandscapeEditLayers.FindByPredicate([InLayerName](const FLandscapeLayer& Layer) { return Layer.EditLayer->GetName() == InLayerName; });
}

const TArray<const ULandscapeEditLayerBase*> ALandscape::GetEditLayersConst() const
{
	TArray<const ULandscapeEditLayerBase*> EditLayers;

	for (const FLandscapeLayer& Layer : LandscapeEditLayers)
	{
		const ULandscapeEditLayerBase* EditLayer = Layer.EditLayer;
		check(EditLayer != nullptr);
		EditLayers.Add(EditLayer);
	}

	return EditLayers;
}

const TArray<ULandscapeEditLayerBase*> ALandscape::GetEditLayers() const
{
	TArray<ULandscapeEditLayerBase*> EditLayers;

	for (const FLandscapeLayer& Layer : GetLayersConst())
	{
		ULandscapeEditLayerBase* EditLayer = Layer.EditLayer;
		check(EditLayer != nullptr);
		EditLayers.Add(EditLayer);
	}

	return EditLayers;
}

const ULandscapeEditLayerBase* ALandscape::GetEditLayerConst(int32 InLayerIndex) const
{
	if (const FLandscapeLayer* Layer = GetLayerConst(InLayerIndex))
	{
		check(Layer->EditLayer != nullptr);
		return Layer->EditLayer;
	}
	return nullptr;
}

const ULandscapeEditLayerBase* ALandscape::GetEditLayerConst(const FGuid& InLayerGuid) const
{
	if (const FLandscapeLayer* Layer = GetLayerConst(InLayerGuid))
	{
		check(Layer->EditLayer != nullptr);
		return Layer->EditLayer;
	}
	return nullptr;
}

const ULandscapeEditLayerBase* ALandscape::GetEditLayerConst(const FName& InLayerName) const
{
	if (const FLandscapeLayer* Layer = GetLayerConst(InLayerName))
	{
		check(Layer->EditLayer != nullptr);
		return Layer->EditLayer;
	}
	return nullptr;
}

ULandscapeEditLayerBase* ALandscape::GetEditLayer(int32 InLayerIndex) const
{
	return const_cast<ULandscapeEditLayerBase*>(GetEditLayerConst(InLayerIndex));
}

ULandscapeEditLayerBase* ALandscape::GetEditLayer(const FGuid& InLayerGuid) const
{
	return const_cast<ULandscapeEditLayerBase*>(GetEditLayerConst(InLayerGuid));
}

ULandscapeEditLayerBase* ALandscape::GetEditLayer(const FName& InLayerName) const
{
	return const_cast<ULandscapeEditLayerBase*>(GetEditLayerConst(InLayerName));
}

const ULandscapeEditLayerBase* ALandscape::FindEditLayerOfTypeConst(const TSubclassOf<ULandscapeEditLayerBase>& InLayerClass) const
{
	if (const FLandscapeLayer* Layer = FindLayerOfTypeConst(InLayerClass))
	{
		check(Layer->EditLayer != nullptr);
		return Layer->EditLayer;
	}
	return nullptr;
}

ULandscapeEditLayerBase* ALandscape::FindEditLayerOfType(const TSubclassOf<ULandscapeEditLayerBase>& InLayerClass) const
{
	if (const FLandscapeLayer* Layer = FindLayerOfTypeConst(InLayerClass))
	{
		check(Layer->EditLayer != nullptr);
		return const_cast<FLandscapeLayer*>(Layer)->EditLayer;
	}
	return nullptr;
}

TArray<const ULandscapeEditLayerBase*> ALandscape::GetEditLayersOfTypeConst(const TSubclassOf<ULandscapeEditLayerBase>& InLayerClass) const
{
	TArray<const ULandscapeEditLayerBase*> EditLayers;
	EditLayers.Reserve(LandscapeEditLayers.Num());
	for (const FLandscapeLayer& InLayer : LandscapeEditLayers)
	{
		check(InLayer.EditLayer != nullptr);
		EditLayers.Add(InLayer.EditLayer);
	}
	return EditLayers;
}

TArray<ULandscapeEditLayerBase*> ALandscape::GetEditLayersOfType(const TSubclassOf<ULandscapeEditLayerBase>& InLayerClass) const
{
	TArray<ULandscapeEditLayerBase*> EditLayers;
	EditLayers.Reserve(LandscapeEditLayers.Num());
	for (const FLandscapeLayer& InLayer : LandscapeEditLayers)
	{
		check(InLayer.EditLayer != nullptr);
		EditLayers.Add(const_cast<ULandscapeEditLayerBase*>(InLayer.EditLayer.Get()));
	}
	return EditLayers;
}

int32 ALandscape::GetLayerIndex(FName InLayerName) const
{
	return LandscapeEditLayers.IndexOfByPredicate([InLayerName](const FLandscapeLayer& Layer) { return Layer.EditLayer->GetName() == InLayerName; });
}

void ALandscape::ForEachLayerConst(TFunctionRef<bool(const FLandscapeLayer&)> Fn)
{
	for (FLandscapeLayer& Layer : LandscapeEditLayers)
	{
		if (!Fn(Layer))
		{
			return;
		}
	}
}

void ALandscape::ForEachEditLayerConst(TFunctionRef<bool(const ULandscapeEditLayerBase*)> Fn)
{
	for (const ULandscapeEditLayerBase* EditLayer : GetEditLayersConst())
	{
		if (!Fn(EditLayer))
		{
			return;
		}
	}
}

const FLandscapeLayer* ALandscape::FindLayerOfTypeConst(const TSubclassOf<ULandscapeEditLayerBase>& InLayerClass) const
{
	return LandscapeEditLayers.FindByPredicate([&InLayerClass](const FLandscapeLayer& InLayer) { check(InLayer.EditLayer != nullptr); return InLayer.EditLayer->GetClass()->IsChildOf(InLayerClass); });
}

TArray<const FLandscapeLayer*> ALandscape::GetLayersOfTypeConst(const TSubclassOf<ULandscapeEditLayerBase>& InLayerClass) const
{
	TArray<const FLandscapeLayer*> Result;
	Result.Reserve(LandscapeEditLayers.Num());
	Algo::TransformIf(LandscapeEditLayers, Result,
		[&InLayerClass](const FLandscapeLayer& InLayer) { check(InLayer.EditLayer != nullptr); return InLayer.EditLayer->GetClass()->IsChildOf(InLayerClass); },
		[](const FLandscapeLayer& InLayer) { return &InLayer; });
	return Result;
}

// Deprecated
void ALandscape::DeleteLayers()
{
}

void ALandscape::CollapseAllEditLayers()
{
	if (IsTemplate())
	{
		return;
	}

	CopyOldDataToDefaultLayer();
	SetSelectedEditLayerIndex(0);

	// Delete all brushes and keep only the base edit layer
	for (int32 EditLayerIndex = LandscapeEditLayers.Num() - 1; EditLayerIndex >= 0; --EditLayerIndex)
	{
		for (int32 BrushIndex = LandscapeEditLayers[EditLayerIndex].Brushes.Num() - 1; BrushIndex >= 0; --BrushIndex)
		{
			ALandscapeBlueprintBrushBase* Brush = LandscapeEditLayers[EditLayerIndex].Brushes[BrushIndex].GetBrush();
			RemoveBrushFromLayer(EditLayerIndex, Brush);
		}

		if (EditLayerIndex > 0)
		{
			DeleteLayer(EditLayerIndex);
		}
	}
	 
	// Force an update before DeleteUnusedLayers runs to ensure newly copied data is not accidentally deleted
	ForceUpdateLayersContent();

	// After removing individual edit layer data, clear any empty weightmap allocations from the components' final data
	DeleteUnusedLayers();
}

bool ALandscape::DeleteLayer(int32 InLayerIndex)
{
	if (IsTemplate())
	{
		return false;
	}

	// Detect any attempt to call this in the middle of UpdateLayersContent.  If called from blueprint, log an error and return early instead of asserting.
	if (LayerUpdateCount > 0 && UE::Landscape::Private::InBPCallstack())
	{
		UE_LOGF(LogLandscapeBP, Error, "Attempting to make illegal call to DeleteLayer during UpdateLayersContent.");
		return false;
	}
	check(LayerUpdateCount == 0);

	const FLandscapeLayer* LayerStruct = GetLayerConst(InLayerIndex);
	if (!LayerStruct)
	{
		return false;
	}

	Modify();

	// If the layer to delete is below the current selected layer index, shift the selected index down
	if (SelectedEditLayerIndex >= InLayerIndex)
	{
		if (LandscapeEditLayers.IsValidIndex(InLayerIndex - 1))
		{
			SetSelectedEditLayerIndex(InLayerIndex - 1);
		}
		else
		{
			SetSelectedEditLayerIndex(0);
		}
	}

	// We're about to remove the layer from our list, which will invalidate our LayerStruct pointer.
	// We'll need to call OnLayerRemoved afterward, though, so keep pointer to the UObject.
	ULandscapeEditLayerBase* EditLayer = LayerStruct->EditLayer;
	// It's possible the edit layer UObject is missing (e.g. when we're trying to load an invalid UObject layer class), 
	//  so we do our best to cleanup the associated data if we have access to it, but otherwise, it should be deleted on load : 
	if (EditLayer != nullptr)
	{
		FGuid LayerGuid = LayerStruct->EditLayer->GetGuid();

		// Clean up Weightmap usage in LandscapeProxies
		if (ULandscapeInfo* LandscapeInfo = GetLandscapeInfo())
		{
			LandscapeInfo->ForEachLandscapeProxy([&LayerGuid](ALandscapeProxy* Proxy)
			{
				Proxy->DeleteLayer(LayerGuid);
				return true;
			});
		}
	}

	// Remove layer from list
	LandscapeEditLayers.RemoveAt(InLayerIndex);
	LayerStruct = nullptr;

	if (EditLayer != nullptr)
	{
		EditLayer->OnLayerRemoved();

		// Unregister from data change events on the edit layer so that we can update the landscape accordingly : 
		EditLayer->OnLayerDataChanged().RemoveAll(this);
	}

	// Request Update
	RequestLayersContentUpdateForceAll();

	return true;
}

void ALandscape::CollapseLayer(int32 InLayerIndex)
{
	check(InLayerIndex >= 1);

	ULandscapeInfo* LandscapeInfo = GetLandscapeInfo();
	check(LandscapeInfo);

	FScopedSlowTask SlowTask(static_cast<float>(LandscapeInfo->XYtoComponentMap.Num()), LOCTEXT("Landscape_CollapseLayer_SlowWork", "Collapsing Layer..."));
	SlowTask.MakeDialog();

	// Only render the two layers being combined
	TBitArray<> EditLayerVis;
	for (int32 i = 0; i < LandscapeEditLayers.Num(); ++i)
	{
		bool bRenderLayer = i == InLayerIndex || i == InLayerIndex - 1;
		EditLayerVis.Add(bRenderLayer);
	}

	TArray<FName> AllTargetNames = GetTargetLayerNames(/*bInIncludeVisibilityLayer = */true);

	FIntRect ExtentInc;
	verify(LandscapeInfo->GetLandscapeExtent(ExtentInc));
	FIntRect ExtentExcl = ExtentInc;
	ExtentExcl.Max += FIntPoint(1, 1);  //convert to exclusive range
	int32 LayerSamples = ExtentExcl.Area();
	check(LayerSamples > 0);

	// ForceUpdateLayersContent without requesting any updates to flush any pending evaluation.
	ForceUpdateLayersContent();

	TArray<uint16> HeightData;
	HeightData.AddZeroed(LayerSamples);

	TArray<uint8> WeightData;
	bool bHasTargetLayers = !AllTargetNames.IsEmpty();
	if (bHasTargetLayers)
	{
		WeightData.AddZeroed(LayerSamples * AllTargetNames.Num());
	}

	// Render heightmap
	FLandscapeEditLayerRenderHeightParams RenderHeightParams;
	RenderHeightParams.Bounds = ExtentExcl;
	RenderHeightParams.ActiveEditLayers = EditLayerVis;
	RenderHeightParams.bRenderBrushes = false;
	RenderHeightParams.CpuResult = HeightData;
	bool bRenderRet = SelectiveRenderEditLayersHeightmaps(RenderHeightParams);
	check(bRenderRet);

	if (bHasTargetLayers)
	{
		// Render weightmaps
		FLandscapeEditLayerRenderWeightParams RenderWeightParams;
		RenderWeightParams.Bounds = ExtentExcl;
		RenderWeightParams.ActiveEditLayers = EditLayerVis;
		RenderWeightParams.bRenderBrushes = false;
		RenderWeightParams.WeightmapNames = AllTargetNames;
		RenderWeightParams.CpuResult = WeightData;
		bRenderRet = SelectiveRenderEditLayersWeightmaps(RenderWeightParams);
		check(bRenderRet);
	}

	// Assign combined data to destination layer
	{
		FLandscapeEditDataInterface DataInterface(LandscapeInfo);
		DataInterface.SetShouldDirtyPackage(true);
		const ULandscapeEditLayerBase* DestLayer = GetEditLayerConst(InLayerIndex - 1);
		FGuid DestLayerGuid = DestLayer->GetGuid();
		DataInterface.SetEditLayer(DestLayerGuid);
		int32 DataStride = 0;  //automatic

		DataInterface.SetHeightData(ExtentInc.Min.X, ExtentInc.Min.Y, ExtentInc.Max.X, ExtentInc.Max.Y,
			HeightData.GetData(), DataStride, /*InCalcNormals=*/ true);

		for (int32 TargetLayerIdx = 0; TargetLayerIdx < AllTargetNames.Num(); ++TargetLayerIdx)
		{
			// Set weightmap data one target layer at a time.  There is a multi-layer version, but it expects interleaved channel data.
			ULandscapeLayerInfoObject* LayerInfo = LandscapeInfo->GetLayerInfoByName(AllTargetNames[TargetLayerIdx]);
			check(LayerInfo);
			int32 DataOffset = LayerSamples * TargetLayerIdx;
			check(WeightData.IsValidIndex(DataOffset + LayerSamples - 1));
			DataInterface.SetAlphaData(LayerInfo, ExtentInc.Min.X, ExtentInc.Min.Y, ExtentInc.Max.X, ExtentInc.Max.Y,
				WeightData.GetData() + DataOffset, DataStride);
		}
	}

	// Note that this code theoretically supports brushes, to handle the non-brush persistent layer data that brush-containing layers
	// are allowed to hold. But the UI doesn't allow CollapseLayer to be used if either layer contains brushes.
	for (int32 BrushIndex = LandscapeEditLayers[InLayerIndex].Brushes.Num() - 1; BrushIndex >= 0; --BrushIndex)
	{
		ALandscapeBlueprintBrushBase* Brush = LandscapeEditLayers[InLayerIndex].Brushes[BrushIndex].GetBrush();
		RemoveBrushFromLayer(InLayerIndex, Brush);
		AddBrushToLayer(InLayerIndex - 1, Brush);
	}

	DeleteLayer(InLayerIndex);

	RequestLayersContentUpdateForceAll();
}

void ALandscape::GetUsedPaintLayers(int32 InLayerIndex, TArray<ULandscapeLayerInfoObject*>& OutUsedLayerInfos) const
{
	if (const ULandscapeEditLayerBase* EditLayer = GetEditLayer(InLayerIndex))
	{
		GetUsedPaintLayers(EditLayer->GetGuid(), OutUsedLayerInfos);
	}
}

void ALandscape::GetUsedPaintLayers(const FGuid& InLayerGuid, TArray<ULandscapeLayerInfoObject*>& OutUsedLayerInfos) const
{
	ULandscapeInfo* LandscapeInfo = GetLandscapeInfo();
	if (!LandscapeInfo)
	{
		return;
	}

	LandscapeInfo->GetUsedPaintLayers(InLayerGuid, OutUsedLayerInfos);
}

void ALandscape::ClearPaintLayer(int32 InLayerIndex, ULandscapeLayerInfoObject* InLayerInfo)
{
	if (const ULandscapeEditLayerBase* EditLayer = GetEditLayer(InLayerIndex))
	{
		ClearPaintLayer(EditLayer->GetGuid(), InLayerInfo);
	}
}

void ALandscape::ClearPaintLayer(const FGuid& InLayerGuid, ULandscapeLayerInfoObject* InLayerInfo)
{
	ULandscapeInfo* LandscapeInfo = GetLandscapeInfo();
	if (!LandscapeInfo)
	{
		return;
	}

	Modify();
	FScopedSetLandscapeEditingLayer Scope(this, InLayerGuid, [this] { RequestLayersContentUpdate(ELandscapeLayerUpdateMode::Update_Weightmap_All); });

	FLandscapeEditDataInterface LandscapeEdit(LandscapeInfo);
	LandscapeInfo->ForEachLandscapeProxy([&](ALandscapeProxy* Proxy)
	{
		Proxy->Modify();
		for (ULandscapeComponent* Component : Proxy->LandscapeComponents)
		{
			Component->DeleteLayer(InLayerInfo, LandscapeEdit);
		}
		return true;
	});
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
// Deprecated
void ALandscape::ClearLayer(int32 InLayerIndex, TSet<TObjectPtr<ULandscapeComponent>>* InComponents, ELandscapeClearMode InClearMode)
{
	ClearEditLayer(InLayerIndex, InComponents, InClearMode == ELandscapeClearMode::Clear_Weightmap ? ELandscapeToolTargetTypeFlags::Weightmap : ELandscapeToolTargetTypeFlags::Heightmap);
}

// Deprecated
void ALandscape::ClearLayer(const FGuid& InLayerGuid, TSet<TObjectPtr<ULandscapeComponent>>* InComponents, ELandscapeClearMode InClearMode, bool bMarkPackageDirty)
	{
	ClearEditLayer(InLayerGuid, InComponents, InClearMode == ELandscapeClearMode::Clear_Weightmap ? ELandscapeToolTargetTypeFlags::Weightmap : ELandscapeToolTargetTypeFlags::Heightmap, bMarkPackageDirty);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void ALandscape::ClearEditLayer(int32 InLayerIndex, TSet<TObjectPtr<ULandscapeComponent>>* InComponents, ELandscapeToolTargetTypeFlags InClearMode)
{
	if (const ULandscapeEditLayerBase* EditLayer = GetEditLayer(InLayerIndex))
	{
		ClearEditLayer(EditLayer->GetGuid(), InComponents, InClearMode);
	}
}

void ALandscape::ClearEditLayer(const FGuid& InLayerGuid, TSet<TObjectPtr<ULandscapeComponent>>* InComponents, ELandscapeToolTargetTypeFlags InClearMode, bool bMarkPackageDirty)
{
	const ULandscapeEditLayerBase* EditLayer = GetEditLayerConst(InLayerGuid);
	ULandscapeInfo* LandscapeInfo = GetLandscapeInfo();
	if (!LandscapeInfo || !EditLayer)
	{
		return;
	}

	Modify(bMarkPackageDirty);
	FScopedSetLandscapeEditingLayer Scope(this, EditLayer->GetGuid(), [this] { RequestLayersContentUpdate(ELandscapeLayerUpdateMode::Update_All); });

	TArray<uint16> NewHeightData;
	NewHeightData.AddZeroed(FMath::Square(ComponentSizeQuads + 1));
	uint16 ZeroValue = LandscapeDataAccess::GetTexHeight(0.f);
	for (uint16& NewHeightDataValue : NewHeightData)
	{
		NewHeightDataValue = ZeroValue;
	}

	TArray<uint16> NewHeightAlphaBlendData;
	TArray<uint8> NewHeightFlagsData;
	if (EnumHasAllFlags(InClearMode, ELandscapeToolTargetTypeFlags::Heightmap))
	{
		if (EditLayer->GetBlendMode() == LSBM_AlphaBlend)
		{
			NewHeightAlphaBlendData.Init(MAX_uint16, FMath::Square(ComponentSizeQuads + 1));
			NewHeightFlagsData.AddZeroed(FMath::Square(ComponentSizeQuads + 1));
		}
	}

	TArray<ULandscapeComponent*> Components;
	if (InComponents)
	{
		TSet<ALandscapeProxy*> Proxies;
		Components.Reserve(InComponents->Num());
		for (ULandscapeComponent* Component : *InComponents)
		{
			if (Component)
			{
				Components.Add(Component);
				ALandscapeProxy* Proxy = Component->GetLandscapeProxy();
				if (!Proxies.Find(Proxy))
				{
					Proxies.Add(Proxy);
					Proxy->Modify(bMarkPackageDirty);
				}
			}
		}
	}
	else
	{
		LandscapeInfo->ForEachLandscapeProxy([&](ALandscapeProxy* Proxy)
		{
			Proxy->Modify(bMarkPackageDirty);
			Components.Append(Proxy->LandscapeComponents);
			return true;
		});
	}

	FLandscapeEditDataInterface LandscapeEdit(LandscapeInfo);
	FLandscapeDoNotDirtyScope DoNotDirtyScope(LandscapeEdit, !bMarkPackageDirty);
	for (ULandscapeComponent* Component : Components)
	{
		if (EnumHasAllFlags(InClearMode, ELandscapeToolTargetTypeFlags::Heightmap))
		{
			int32 MinX = MAX_int32;
			int32 MinY = MAX_int32;
			int32 MaxX = MIN_int32;
			int32 MaxY = MIN_int32;
			Component->GetComponentExtent(MinX, MinY, MaxX, MaxY);
			check(ComponentSizeQuads == (MaxX - MinX));
			check(ComponentSizeQuads == (MaxY - MinY));
			LandscapeEdit.SetHeightData(MinX, MinY, MaxX, MaxY, NewHeightData.GetData(), 0, false, nullptr, NewHeightAlphaBlendData.GetData(), NewHeightFlagsData.GetData());
		}

		if (EnumHasAllFlags(InClearMode, ELandscapeToolTargetTypeFlags::Weightmap))
		{
			// Clear weight maps
			for (FLandscapeInfoLayerSettings& LayerSettings : LandscapeInfo->Layers)
			{
				if (LayerSettings.LayerInfoObj != ALandscapeProxy::VisibilityLayer)
				{
				Component->DeleteLayer(LayerSettings.LayerInfoObj, LandscapeEdit);
			}
		}
	}

		if (EnumHasAllFlags(InClearMode, ELandscapeToolTargetTypeFlags::Visibility))
		{
			// Clear visibility layer only
			for (FLandscapeInfoLayerSettings& LayerSettings : LandscapeInfo->Layers)
			{
				if (LayerSettings.LayerInfoObj == ALandscapeProxy::VisibilityLayer)
				{
					Component->DeleteLayer(LayerSettings.LayerInfoObj, LandscapeEdit);
					break;
				}
			}
		}
	}
}

void ALandscape::ShowOnlySelectedLayer(int32 InLayerIndex)
{
	const ULandscapeEditLayerBase* VisibleEditLayer = GetEditLayerConst(InLayerIndex);
	if (VisibleEditLayer)
	{
		for (ULandscapeEditLayerBase* EditLayer : GetEditLayers())
			{
			bool bDesiredVisible = (EditLayer == VisibleEditLayer);
			if (EditLayer->IsVisible() != bDesiredVisible)
				{
				EditLayer->SetVisible(bDesiredVisible, /* bInModify = */ true);
		}
	}
}
}

void ALandscape::ShowAllLayers()
{
	if (LandscapeEditLayers.Num() > 0)
	{
		for (ULandscapeEditLayerBase* EditLayer : GetEditLayers())
			{
			if (EditLayer->IsVisible() != true)
				{
				EditLayer->SetVisible(/*bInVisible = */ true, /*bInModify = */ true);
	}
}
}
}

LANDSCAPE_API extern bool GDisableUpdateLandscapeMaterialInstances;

uint32 ULandscapeComponent::ComputeLayerHash(bool InReturnEditingHash) const
{
	UTexture2D* Heightmap = GetHeightmap(InReturnEditingHash);
	const uint8* MipData = Heightmap->Source.LockMipReadOnly(0);
	uint32 Hash = FCrc::MemCrc32(MipData, Heightmap->Source.GetSizeX() * Heightmap->Source.GetSizeY() * sizeof(FColor));
	Heightmap->Source.UnlockMip(0);

	// Copy to sort
	const TArray<UTexture2D*>& Weightmaps = GetWeightmapTextures(InReturnEditingHash);
	TArray<FWeightmapLayerAllocationInfo> AllocationInfos = GetWeightmapLayerAllocations(InReturnEditingHash);

	// Sort allocations infos by LayerInfo Path so the Weightmaps hashes get ordered properly
	AllocationInfos.Sort([](const FWeightmapLayerAllocationInfo& A, const FWeightmapLayerAllocationInfo& B)
	{
		FString PathA(A.LayerInfo ? A.LayerInfo->GetPathName() : FString());
		FString PathB(B.LayerInfo ? B.LayerInfo->GetPathName() : FString());

		return PathA < PathB;
	});

	for (const FWeightmapLayerAllocationInfo& AllocationInfo : AllocationInfos)
	{
		if (AllocationInfo.IsAllocated())
		{
			// Compute hash of actual data of the texture that is owned by the component (per Texture Channel)
			UTexture2D* Weightmap = Weightmaps[AllocationInfo.WeightmapTextureIndex];
			MipData = Weightmap->Source.LockMipReadOnly(0) + ChannelOffsets[AllocationInfo.WeightmapTextureChannel];
			TArray<uint8> ChannelData;
			ChannelData.AddDefaulted(Weightmap->Source.GetSizeX() * Weightmap->Source.GetSizeY());
			int32 TexSize = (SubsectionSizeQuads + 1) * NumSubsections;
			for (int32 TexY = 0; TexY < TexSize; TexY++)
			{
				for (int32 TexX = 0; TexX < TexSize; TexX++)
				{
					const int32 Index = (TexX + TexY * TexSize);

					ChannelData.GetData()[Index] = MipData[4 * Index];
				}
			}

			Hash = FCrc::MemCrc32(ChannelData.GetData(), Weightmap->GetSizeX() * Weightmap->GetSizeY(), Hash);
			Weightmap->Source.UnlockMip(0);
		}
	}

	return Hash;
}

// Deprecated
void ALandscape::UpdateLandscapeSplines(const FGuid& InTargetLayer, bool bInUpdateOnlySelected, bool bInForceUpdateAllCompoments)
{
	UpdateAllLandscapeSplines(InTargetLayer, bInForceUpdateAllCompoments);
}

void ALandscape::UpdateAllLandscapeSplines(const FGuid& InTargetLayer, bool bInForceUpdateAllCompoments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeLayers_UpdateAllLandscapeSplines);
	check(!IsTemplate());
	ULandscapeInfo* LandscapeInfo = GetLandscapeInfo();
	const ULandscapeEditLayerBase* SplinesEditLayer = FindEditLayerOfTypeConst(ULandscapeEditLayerSplines::StaticClass());
	FGuid TargetLayerGuid = (SplinesEditLayer != nullptr) ? SplinesEditLayer->GetGuid() : InTargetLayer;
	const ULandscapeEditLayerBase* TargetLayer = GetEditLayerConst(TargetLayerGuid);
	if (LandscapeInfo && TargetLayer)
	{
		FScopedSetLandscapeEditingLayer Scope(this, TargetLayerGuid, [this] { this->RequestLayersContentUpdate(ELandscapeLayerUpdateMode::Update_All); });
		// Temporarily disable material instance updates since it will be done once at the end (requested by RequestLayersContentUpdateForceAll)
		GDisableUpdateLandscapeMaterialInstances = true;
		TSet<TObjectPtr<ULandscapeComponent>>* ModifiedComponents = nullptr;
		if (SplinesEditLayer != nullptr)
		{
			// Check that we can modify data
			if (!LandscapeInfo->AreAllComponentsRegistered())
			{
				return;
			}

			TMap<ULandscapeComponent*, uint32> PreviousHashes;
			{
				FLandscapeEditDataInterface LandscapeEdit(LandscapeInfo);

				LandscapeInfo->ForAllLandscapeComponents([&](ULandscapeComponent* Component)
				{
					// Was never computed
					if (Component->SplineHash == 0)
					{
						LandscapeInfo->ModifyObject(Component);
						Component->SplineHash = DefaultSplineHash;
					}

					PreviousHashes.Add(Component, Component->SplineHash);
					LandscapeInfo->ModifyObject(Component, false);
					Component->SplineHash = DefaultSplineHash;
				});
			}

			// Clear layers without affecting weightmap allocations
			const bool bMarkPackageDirty = false;
			ClearEditLayer(SplinesEditLayer->GetGuid(), (!bInForceUpdateAllCompoments && LandscapeSplinesAffectedComponents.Num()) ? &LandscapeSplinesAffectedComponents : nullptr, ELandscapeToolTargetTypeFlags::All, bMarkPackageDirty);
			LandscapeSplinesAffectedComponents.Empty();
			ModifiedComponents = &LandscapeSplinesAffectedComponents;

			// Apply splines without clearing up weightmap allocations
			LandscapeInfo->ApplySplines(ModifiedComponents, bMarkPackageDirty);

			for (const TPair<ULandscapeComponent*, uint32>& Pair : PreviousHashes)
			{
				if (LandscapeSplinesAffectedComponents.Contains(Pair.Key))
				{
					uint32 NewHash = Pair.Key->ComputeLayerHash();
					if (NewHash != Pair.Value)
					{
						LandscapeInfo->MarkObjectDirty(Pair.Key);
					}
					Pair.Key->SplineHash = NewHash;
				}
				else if (Pair.Key->SplineHash == DefaultSplineHash && Pair.Value != DefaultSplineHash)
				{
					LandscapeInfo->MarkObjectDirty(Pair.Key);
				}
			}
		}
		else
		{
			LandscapeInfo->ApplySplines(ModifiedComponents);
		}
		GDisableUpdateLandscapeMaterialInstances = false;
	}
}

FScopedSetLandscapeEditingLayer::FScopedSetLandscapeEditingLayer(ALandscape* InLandscape, const FGuid& InLayerGUID, TFunction<void()> InCompletionCallback)
	: Landscape(InLandscape)
	, CompletionCallback(MoveTemp(InCompletionCallback))
{
	if (Landscape.IsValid() && !Landscape.Get()->IsTemplate())
	{
		PreviousLayerGUID = Landscape->GetEditingLayer();
		Landscape->SetEditingLayer(InLayerGUID);
	}
}

FScopedSetLandscapeEditingLayer::~FScopedSetLandscapeEditingLayer()
{
	if (Landscape.IsValid() && !Landscape.Get()->IsTemplate())
	{
		Landscape->SetEditingLayer(PreviousLayerGUID);
		if (CompletionCallback)
		{
			CompletionCallback();
		}
	}
}

void ALandscape::SetEditingLayer(const FGuid& InLayerGuid)
{
	ensure(!IsTemplate());

	ULandscapeInfo* LandscapeInfo = GetLandscapeInfo();
	if (!LandscapeInfo)
	{
		EditingLayer.Invalidate();
		return;
	}

	EditingLayer = InLayerGuid;
}

void ALandscape::SetGrassUpdateEnabled(bool bInGrassUpdateEnabled)
{
#if WITH_EDITORONLY_DATA
	bGrassUpdateEnabled = bInGrassUpdateEnabled;
#endif
}

const FGuid& ALandscape::GetEditingLayer() const
{
	return EditingLayer;
}

void ALandscape::SetSelectedEditLayerIndex(const int32 InEditLayerIndex)
{
	check(LandscapeEditLayers.IsValidIndex(InEditLayerIndex) && !IsTemplate());
	SelectedEditLayerIndex = InEditLayerIndex;
}

const int32 ALandscape::GetSelectedEditLayerIndex() const
{
	// When edit layers are not supported, index should always be NONE
	check(LandscapeEditLayers.IsValidIndex(SelectedEditLayerIndex) && !IsTemplate());
	return SelectedEditLayerIndex;
}

bool ALandscape::IsMaxLayersReached() const
{
	return LandscapeEditLayers.Num() >= GetDefault<ULandscapeSettings>()->MaxNumberOfLayers;
}

void ALandscape::CreateDefaultLayer()
{
	if (IsTemplate())
	{
		return;
	}

	check(LandscapeEditLayers.Num() == 0); // We can only call this function if we have no layers

	CreateLayer(FName("Layer"));
	SetSelectedEditLayerIndex(0);
}

FLandscapeLayer* ALandscape::DuplicateLayerAndMoveBrushes(const FLandscapeLayer& InOtherLayer)
{
	ULandscapeInfo* LandscapeInfo = GetLandscapeInfo();
	if (!LandscapeInfo || IsTemplate())
	{
		return nullptr;
	}

	if (IsMaxLayersReached())
	{
		UE_LOGF(LogLandscape, Warning, "Cannot duplicate layer : %ls as the max number of layers (%i) has been reached", *InOtherLayer.EditLayer->GetName().ToString(), GetDefault<ULandscapeSettings>()->MaxNumberOfLayers);
		return nullptr;
	}

	Modify();

	FLandscapeLayer NewLayer(InOtherLayer);

	// Duplicate the internal edit layer object by hand : 
	check(InOtherLayer.EditLayer != nullptr);
	NewLayer.EditLayer = DuplicateObject(InOtherLayer.EditLayer, this, MakeUniqueObjectName(this, InOtherLayer.EditLayer->GetClass(), InOtherLayer.EditLayer->GetFName()));
	NewLayer.EditLayer->SetGuid(FGuid::NewGuid(), /*bInModify =*/true);

	// Update owning landscape and reparent to landscape's level if necessary
	for (FLandscapeLayerBrush& Brush : NewLayer.Brushes)
	{
		Brush.SetOwner(this);
	}

	int32 AddedIndex = LandscapeEditLayers.Add(NewLayer);
	check(LandscapeEditLayers.IsValidIndex(SelectedEditLayerIndex));

	OnLayerCreatedInternal(NewLayer.EditLayer);

	return &LandscapeEditLayers[AddedIndex];
}

int32 ALandscape::CreateLayer(FName InName, const TSubclassOf<ULandscapeEditLayerBase>& InEditLayerClass, bool bInIgnoreLayerCountLimit)
{
	// Detect any attempt to call this in the middle of UpdateLayersContent.  If called from blueprint, log an error and return early instead of asserting.
	if (LayerUpdateCount > 0 && UE::Landscape::Private::InBPCallstack())
	{
		UE_LOGF(LogLandscapeBP, Error, "Attempting to make illegal call to CreateLayer during UpdateLayersContent.");
		return INDEX_NONE;
	}
	check(LayerUpdateCount == 0);

	if ((!bInIgnoreLayerCountLimit && IsMaxLayersReached()) || IsTemplate())
	{
		return INDEX_NONE;
	}

	Modify();

	const UClass* EditLayerClass = (InEditLayerClass.Get() != nullptr) ? InEditLayerClass.Get() : ULandscapeEditLayer::StaticClass();
	int32 LayerIndex = LandscapeEditLayers.Emplace();
	FLandscapeLayer& NewLayer = LandscapeEditLayers[LayerIndex];
	NewLayer.EditLayer = NewObject<ULandscapeEditLayerBase>(this, EditLayerClass, MakeUniqueObjectName(this, EditLayerClass), RF_Transactional);
	NewLayer.EditLayer->SetBackPointer(this);
	NewLayer.EditLayer->SetName(GenerateUniqueLayerName(InName), /*bInModify = */true);

	OnLayerCreatedInternal(NewLayer.EditLayer);
	return LayerIndex;
}

void ALandscape::OnLayerCreatedInternal(ULandscapeEditLayerBase* EditLayer)
{
	// TODO: Might not be necessary eventually, if EditLayer has ability to trigger landscape updates
	// and has access to its guid.
	EditLayer->SetBackPointer(this);
	EditLayer->SetFlags(RF_Transactional);

	// Register to data change events on the edit layer so that we can update the landscape accordingly : 
	check(!EditLayer->OnLayerDataChanged().IsBoundToObject(this));
	EditLayer->OnLayerDataChanged().AddUObject(this, &ALandscape::OnEditLayerDataChanged);

	// Add to self first instead of from the loop.  Avoids potential load-order problems if this is called at load-time,
	// before the LandscapeInfo is created.  The other proxies can fix themselves if they load later.
	AddLayer(EditLayer->GetGuid());

	// Create associated layer data in each landscape proxy
	if (ULandscapeInfo* LandscapeInfo = GetLandscapeInfo())
	{
		LandscapeInfo->ForEachLandscapeProxy([&EditLayer, this](ALandscapeProxy* Proxy)
		{
			if (Proxy != this)
			{
			Proxy->AddLayer(EditLayer->GetGuid());
			}
			return true;
		});
	}

	// Request Update
	// Force update rendering resources
	RequestLayersInitialization(/*bInRequestContentUpdate = */true);
	RequestSplineLayerUpdate(); // Request a spline update as well, in case we already had spline actors and a spline layers was just created
}

void ALandscape::AddLayersToProxy(ALandscapeProxy* InProxy)
{
	ULandscapeInfo* LandscapeInfo = GetLandscapeInfo();
	if (!LandscapeInfo || IsTemplate())
	{
		return;
	}

	check(InProxy != this);
	check(InProxy != nullptr);

	ForEachEditLayerConst([&](const ULandscapeEditLayerBase* EditLayer)
	{
		InProxy->AddLayer(EditLayer->GetGuid());
		return true;
	});

	// Force update rendering resources
	RequestLayersInitialization();
}

bool ALandscape::ReorderLayer(int32 InStartingLayerIndex, int32 InDestinationLayerIndex)
{
	if (InStartingLayerIndex != InDestinationLayerIndex &&
		LandscapeEditLayers.IsValidIndex(InStartingLayerIndex) &&
		LandscapeEditLayers.IsValidIndex(InDestinationLayerIndex))
	{
		Modify();
		FLandscapeLayer Layer = LandscapeEditLayers[InStartingLayerIndex];
		LandscapeEditLayers.RemoveAt(InStartingLayerIndex);
		LandscapeEditLayers.Insert(Layer, InDestinationLayerIndex);
		RequestLayersContentUpdateForceAll();
		return true;
	}
	return false;
}

FName ALandscape::GenerateUniqueLayerName(FName InName) const
{
	// If we are receiving a unique name, use it.
	if (InName != NAME_None && !LandscapeEditLayers.ContainsByPredicate([InName](const FLandscapeLayer& Layer) { return Layer.EditLayer->GetName() == InName; }))
	{
		return InName;
	}

	FString BaseName = (InName == NAME_None) ? "Layer" : InName.ToString();
	FName NewName;
	int32 LayerIndex = 0;
	do
	{
		++LayerIndex;
		NewName = FName(*FString::Printf(TEXT("%s%d"), *BaseName, LayerIndex));
	} while (LandscapeEditLayers.ContainsByPredicate([NewName](const FLandscapeLayer& Layer) { return Layer.EditLayer->GetName() == NewName; }));

	return NewName;
}

void ALandscape::ReplaceLayerSubstractiveBlendStatus(ULandscapeLayerInfoObject* InFromLayerInfo, ULandscapeLayerInfoObject* InToLayerInfo, bool bInShouldDirtyPackage)
{
	for (ULandscapeEditLayerBase* EditLayer : GetEditLayers())
	{
		bool OutValue;
		if (EditLayer->RemoveAndCopyWeightmapAllocationLayerBlend(InFromLayerInfo, OutValue, /*bInModify = */ bInShouldDirtyPackage))
		{
			EditLayer->AddOrUpdateWeightmapAllocationLayerBlend(InToLayerInfo, OutValue, /*bInModify = */ bInShouldDirtyPackage);
		}
	}
}

bool ALandscape::ReorderLayerBrush(int32 InLayerIndex, int32 InStartingLayerBrushIndex, int32 InDestinationLayerBrushIndex)
{
	if (FLandscapeLayer* Layer = GetLayerInternal(InLayerIndex))
	{
		if (InStartingLayerBrushIndex != InDestinationLayerBrushIndex &&
			Layer->Brushes.IsValidIndex(InStartingLayerBrushIndex) &&
			Layer->Brushes.IsValidIndex(InDestinationLayerBrushIndex))
		{
			Modify();
			FLandscapeLayerBrush MovingBrush = Layer->Brushes[InStartingLayerBrushIndex];
			Layer->Brushes.RemoveAt(InStartingLayerBrushIndex);
			Layer->Brushes.Insert(MovingBrush, InDestinationLayerBrushIndex);
			RequestLayersContentUpdateForceAll();
			return true;
		}
	}
	return false;
}

int32 ALandscape::GetBrushLayer(const ALandscapeBlueprintBrushBase* InBrush) const
{
	for (int32 LayerIndex = 0; LayerIndex < LandscapeEditLayers.Num(); ++LayerIndex)
	{
		for (const FLandscapeLayerBrush& Brush : LandscapeEditLayers[LayerIndex].Brushes)
		{
			if (Brush.GetBrush() == InBrush)
			{
				return LayerIndex;
			}
		}
	}

	return INDEX_NONE;
}

void ALandscape::AddBrushToLayer(int32 InLayerIndex, ALandscapeBlueprintBrushBase* InBrush)
{
	check(GetBrushLayer(InBrush) == INDEX_NONE);
	if (FLandscapeLayer* Layer = GetLayerInternal(InLayerIndex))
	{
		Modify();
		// ensure the brush has the correct parent on creation
		InBrush->SetOwner(this);
		Layer->Brushes.Add(FLandscapeLayerBrush(InBrush));
		InBrush->SetOwningLandscape(this);
		RequestLayersContentUpdateForceAll();
	}
}

void ALandscape::RemoveBrush(ALandscapeBlueprintBrushBase* InBrush)
{
	int32 LayerIndex = GetBrushLayer(InBrush);
	if (LayerIndex != INDEX_NONE)
	{
		RemoveBrushFromLayer(LayerIndex, InBrush);
	}
}

void ALandscape::RemoveBrushFromLayer(int32 InLayerIndex, ALandscapeBlueprintBrushBase* InBrush)
{
	int32 BrushIndex = GetBrushIndexForLayer(InLayerIndex, InBrush);
	if (BrushIndex != INDEX_NONE)
	{
		RemoveBrushFromLayer(InLayerIndex, BrushIndex);
	}
}

void ALandscape::RemoveBrushFromLayer(int32 InLayerIndex, int32 InBrushIndex)
{
	if (FLandscapeLayer* Layer = GetLayerInternal(InLayerIndex))
	{
		if (Layer->Brushes.IsValidIndex(InBrushIndex))
		{
			Modify();
			ALandscapeBlueprintBrushBase* Brush = Layer->Brushes[InBrushIndex].GetBrush();
			Layer->Brushes.RemoveAt(InBrushIndex);
			if (Brush != nullptr)
			{
				Brush->SetOwningLandscape(nullptr);
			}
			RequestLayersContentUpdateForceAll();
		}
	}
}

int32 ALandscape::GetBrushIndexForLayer(int32 InLayerIndex, ALandscapeBlueprintBrushBase* InBrush)
{
	if (const FLandscapeLayer* Layer = GetLayerConst(InLayerIndex))
	{
		for (int32 i = 0; i < Layer->Brushes.Num(); ++i)
		{
			if (Layer->Brushes[i].GetBrush() == InBrush)
			{
				return i;
			}
		}
	}

	return INDEX_NONE;
}


void ALandscape::OnBlueprintBrushChanged()
{
#if WITH_EDITORONLY_DATA
	LandscapeBlueprintBrushChangedDelegate.Broadcast();
	RequestLayersContentUpdateForceAll();
#endif
}

void ALandscape::OnLayerInfoSplineFalloffModulationChanged(const ULandscapeLayerInfoObject* InLayerInfo)
{
	ULandscapeInfo* LandscapeInfo = GetLandscapeInfo();

	if (!LandscapeInfo)
	{
		return;
	}

	ALandscape* Landscape = LandscapeInfo->LandscapeActor.Get();
	if (!Landscape || IsTemplate())
	{
		return;
	}

	bool bUsedForSplines = false;
	LandscapeInfo->ForAllSplineActors([&](TScriptInterface<ILandscapeSplineInterface> SplineOwner)
	{
		bUsedForSplines |= (SplineOwner->GetSplinesComponent() && SplineOwner->GetSplinesComponent()->IsUsingLayerInfo(InLayerInfo));
	});

	if (bUsedForSplines)
	{
		Landscape->RequestSplineLayerUpdate();
	}
}

ALandscapeBlueprintBrushBase* ALandscape::GetBrushForLayer(int32 InLayerIndex, int32 InBrushIndex) const
{
	if (const FLandscapeLayer* Layer = GetLayerConst(InLayerIndex))
	{
		if (Layer->Brushes.IsValidIndex(InBrushIndex))
		{
			return Layer->Brushes[InBrushIndex].GetBrush();
		}
	}
	return nullptr;
}

TArray<ALandscapeBlueprintBrushBase*> ALandscape::GetBrushesForLayer(int32 InLayerIndex) const
{
	TArray<ALandscapeBlueprintBrushBase*> Brushes;
	if (const FLandscapeLayer* Layer = GetLayerConst(InLayerIndex))
	{
		Brushes.Reserve(Layer->Brushes.Num());
		for (const FLandscapeLayerBrush& Brush : Layer->Brushes)
		{
			Brushes.Add(Brush.GetBrush());
		}
	}
	return Brushes;
}

ALandscapeBlueprintBrushBase* FLandscapeLayerBrush::GetBrush() const
{
#if WITH_EDITORONLY_DATA
	return BlueprintBrush;
#else
	return nullptr;
#endif
}

void FLandscapeLayerBrush::SetOwner(ALandscape* InOwner)
{
#if WITH_EDITORONLY_DATA
	if (BlueprintBrush && InOwner)
	{
		if (BlueprintBrush->GetTypedOuter<ULevel>() != InOwner->GetTypedOuter<ULevel>())
		{
			BlueprintBrush->Rename(nullptr, InOwner->GetTypedOuter<ULevel>());
		}
		BlueprintBrush->SetOwningLandscape(InOwner);
	}
#endif
}

bool FLandscapeLayerBrush::AffectsHeightmap() const
{
#if WITH_EDITORONLY_DATA
	return BlueprintBrush && BlueprintBrush->AffectsHeightmap();
#else
	return false;
#endif
}

bool FLandscapeLayerBrush::AffectsWeightmapLayer(const FName& InWeightmapLayerName) const
{
#if WITH_EDITORONLY_DATA
	return BlueprintBrush && BlueprintBrush->AffectsWeightmapLayer(InWeightmapLayerName);
#else
	return false;
#endif
}

bool FLandscapeLayerBrush::AffectsVisibilityLayer() const
{
#if WITH_EDITORONLY_DATA
	return BlueprintBrush && BlueprintBrush->AffectsVisibilityLayer();
#else
	return false;
#endif
}

void LandscapeDumpSelectiveLayerRender(const TArray<FString>& Args)
{
	// Example usages:
	//		landscape.DumpSelectiveLayerRender -landscape Landscape -h -layermask 255 -region 0 0 1000 500
	//		landscape.DumpSelectiveLayerRender -landscape Landscape -w -wname Grass -layermask 255 -region 0 0 1000 500
	// override default layer visibility with a bitmask:  -layermask 255
	// render a region of the landscape based on landscape coords (integers):  -region minX minY maxX maxY
	// All params optional except -w or -h.
	// -w with no -wname means all weightmaps.

	const TCHAR* LandscapeNameArg = TEXT("-landscape");
	const TCHAR* LayerMaskArg = TEXT("-layermask");
	const TCHAR* RegionArg = TEXT("-region");
	const TCHAR* HeightmapArg = TEXT("-h");
	const TCHAR* WeightmapArg = TEXT("-w");
	const TCHAR* WeightmapNameArg = TEXT("-wname");
	const TCHAR* RTArg = TEXT("-rt");

	ALandscape* Landscape = nullptr;
	FString LandscapeName;
	bool bHasLayerMask = false;
	uint32 LayerMask = ~0;
	bool bHasRegion = false;
	FIntRect Region;
	bool bUseRenderTarget = false;

	bool bRenderHeight = false;
	bool bRenderWeight = false;
	TArray<FName> WeightmapNames;

	UWorld* World = GEditor->GetEditorWorldContext().World();
	if (!World)
	{
		return;
	}

	// Parse args

	for (int32 i = 0; i < Args.Num(); ++i)
	{
		const FString& Arg = Args[i];

		if (Arg == LandscapeNameArg)
		{
			++i;
			if (i < Args.Num())
			{
				LandscapeName = Args[i];
			}
		}
		else if (Arg == LayerMaskArg)
		{
			++i;
			if (i < Args.Num())
			{
				LexFromString(LayerMask, Args[i]);
				bHasLayerMask = true;
			}
		}
		else if (Arg == RegionArg)
		{
			if (i + 4 < Args.Num())
			{
				LexFromString(Region.Min.X, Args[i + 1]);
				LexFromString(Region.Min.Y, Args[i + 2]);
				LexFromString(Region.Max.X, Args[i + 3]);
				LexFromString(Region.Max.Y, Args[i + 4]);
				bHasRegion = true;
			}
			i += 4;
		}
		else if (Arg == HeightmapArg)
		{
			bRenderHeight = true;
		}
		else if (Arg == WeightmapArg)
		{
			bRenderWeight = true;
		}
		else if (Arg == WeightmapNameArg)
		{
			++i;
			if (i < Args.Num())
			{
				WeightmapNames.Add(FName(Args[i]));
			}
		}
		else if (Arg == RTArg)
		{
			bUseRenderTarget = true;
		}
		else
		{
			UE_LOGF(LogLandscape, Error, "LandscapeDumpSelectiveLayerRender: Unknown parameter %ls", *Arg);
		}
	}

	// Validate args and fill in implied params

	// If landscape name not specified, just take the first one
	for (ALandscape* Actor : TActorRange<ALandscape>(World))
	{
		if (LandscapeName.IsEmpty() || LandscapeName == Actor->GetActorNameOrLabel())
		{
			Landscape = Actor;
			break;
		}
	}

	if (!Landscape)
	{
		UE_LOGF(LogLandscape, Error, "LandscapeDumpSelectiveLayerRender: landscape not found");
		return;
	}

	if (!bRenderHeight && !bRenderWeight)
	{
		UE_LOGF(LogLandscape, Error, "LandscapeDumpSelectiveLayerRender: must specify either -h or -w");
		return;
	}

	if (!bHasRegion)
	{
		if (ULandscapeInfo* LandscapeInfo = ULandscapeInfo::Find(World, Landscape->GetLandscapeGuid()))
		{
			bHasRegion = LandscapeInfo->GetLandscapeExtent(Region);
			Region.Max += FIntPoint(1, 1);  //convert to exclusive range
		}
	}

	if (!bHasRegion)
	{
		UE_LOGF(LogLandscape, Warning, "LandscapeDumpSelectiveLayerRender: couldn't determine landscape extent.  Nothing loaded?");
		return;
	}

	TBitArray<> LayerVis;  // leave empty for default layer visibility
	if (bHasLayerMask)
	{
		int32 NumEditLayers = Landscape->GetEditLayersConst().Num();
		for (int32 i = 0; i < NumEditLayers; ++i)
		{
			LayerVis.Add((LayerMask & (1 << i)) != 0);
		}
	}

	if (bRenderWeight)
	{
		TArray<FName> AllNames = Landscape->GetTargetLayerNames(/*bInIncludeVisibilityLayer = */true);
		if (WeightmapNames.IsEmpty())
		{
			WeightmapNames = AllNames;
		}
		else
		{
			// Filter arg-provided names by available names.
			WeightmapNames.SetNum(Algo::RemoveIf(WeightmapNames, [&AllNames](FName TestName)
				{
					if (!AllNames.Contains(TestName))
					{
						UE_LOGF(LogLandscape, Warning, "LandscapeDumpSelectiveLayerRender: weightmap named %ls not found.", *TestName.ToString());
						return true;
					}
					return false;
				}));
		}

		if (WeightmapNames.IsEmpty())
		{
			UE_LOGF(LogLandscape, Warning, "LandscapeDumpSelectiveLayerRender: no weightmaps to render.");
			bRenderWeight = false;
		}
	}

	ULandscapeSubsystem* Subsystem = World->GetSubsystem<ULandscapeSubsystem>();
	const FDateTime CurrentTime = Subsystem->GetAppCurrentDateTime();
	FString FilePattern = FString::Format(TEXT("{0}/Landscape/LandscapeRender{1}"), { FPaths::ProjectSavedDir(), CurrentTime.ToString() });

	// Render the heightmaps

	if (bRenderHeight)
	{
		UE_LOGF(LogLandscape, Display, "LandscapeDumpSelectiveLayerRender: SelectiveRenderLayers heightmap, landscape=%ls region=(%d,%d)-(%d,%d) layers=%x",
			*Landscape->GetActorNameOrLabel(), Region.Min.X, Region.Min.Y, Region.Max.X, Region.Max.Y, LayerMask);

		TArray<uint16> HeightData;
		HeightData.AddZeroed(Region.Area());
		TObjectPtr<UTextureRenderTarget2D> HeightRT;

		FLandscapeEditLayerRenderHeightParams RenderParams;
		RenderParams.Bounds = Region;
		RenderParams.ActiveEditLayers = LayerVis;

		if (bUseRenderTarget)
		{
			// Prepare the render target
			FName RTName = MakeUniqueObjectName(GetTransientPackage(), UTextureRenderTarget2D::StaticClass(), TEXT("LandscapeDumpHeightRT"));
			HeightRT = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), RTName, RF_Transient);
			HeightRT->bCanCreateUAV = false;
			HeightRT->RenderTargetFormat = RTF_RGBA8;
			HeightRT->ClearColor = FColor(0);
			FIntPoint Size = Region.Size();
			HeightRT->InitAutoFormat(Size.X, Size.Y);
			HeightRT->UpdateResourceImmediate(/*bClearRenderTarget = */true);

			RenderParams.RTResult = HeightRT;
		}
		else
		{
			RenderParams.CpuResult = HeightData;
		}

		bool bRenderRet = Landscape->SelectiveRenderEditLayersHeightmaps(RenderParams);
		if (!bRenderRet)
		{
			UE_LOGF(LogLandscape, Error, "LandscapeDumpSelectiveLayerRender: SelectiveRenderEditLayersHeightmaps failed");
			return;
		}

		if (bUseRenderTarget)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(BatchedMergeHeightmapsSyncResults_ReadPixels);
			FlushRenderingCommands();

			// Note:  GameThread_GetRenderTargetResource is documented as returning a pointer which cannot be used from the game thread,
			// but other callers say this can be used if you don't care about performance and flush the render thread first.
			FTextureRenderTargetResource* RenderTargetResource = HeightRT->GameThread_GetRenderTargetResource();

			// Make a blocking read from the render target.
			TArray<FColor> RGBABuffer;
			FIntRect ReadRect = Region;  // destination rect in landscape coords
			RGBABuffer.Reserve(Region.Area());
			FIntPoint Offset = ReadRect.Min;
			ReadRect -= Offset;  //offset so that the first pixel of the buffer is (0,0).
			FReadSurfaceDataFlags ReadFlags(RCM_MinMax);
			bool bRet = RenderTargetResource->ReadPixels(RGBABuffer, ReadFlags, ReadRect);

			// Convert FColor (R+G) to uint16
			TArrayView<uint16> DestBuffer = HeightData;
			ensure(bRet);
			ensure(RGBABuffer.Num() == DestBuffer.Num());
			if (bRet)
			{
				for (int32 i = 0; i < RGBABuffer.Num(); ++i)
				{
					FColor HeightAndNormal = RGBABuffer[i];
					DestBuffer[i] = (static_cast<uint16>(HeightAndNormal.R) << 8) | static_cast<uint16>(HeightAndNormal.G);
				}
			}
			else
			{
				UE_LOGF(LogLandscape, Error, "LandscapeDumpSelectiveLayerRender: ReadPixels failed");
				return;
			}

			HeightRT->ReleaseResource();
		}

		// Write the data to an image file
		FImageInfo ImageInfo(Region.Size().X, Region.Size().Y, 1, ERawImageFormat::G16, EGammaSpace::Linear);
		FImageView ImageView(ImageInfo, HeightData.GetData());
		FString Filename = FilePattern + "Heightmap.png";
		if (FImageUtils::SaveImageByExtension(*Filename, ImageView))
		{
			UE_LOGF(LogLandscape, Display, "LandscapeDumpSelectiveLayerRender: wrote heightmap data to %ls", *Filename);
		}
		else
		{
			UE_LOGF(LogLandscape, Error, "LandscapeDumpSelectiveLayerRender: failed to write heightmap data to %ls", *Filename);
		}
	}

	// Render the weightmaps

	if (bRenderWeight)
	{
		UE_LOGF(LogLandscape, Display, "LandscapeDumpSelectiveLayerRender: SelectiveRenderLayers weightmaps, landscape=%ls region=(%d,%d)-(%d,%d) layers=%x",
			*Landscape->GetActorNameOrLabel(), Region.Min.X, Region.Min.Y, Region.Max.X, Region.Max.Y, LayerMask);

		const int32 SingleWeightmapSizePixels = Region.Area();
		TArray<uint8> WeightData;

		WeightData.AddZeroed(SingleWeightmapSizePixels * WeightmapNames.Num());

		FLandscapeEditLayerRenderWeightParams RenderParams;
		RenderParams.Bounds = Region;
		RenderParams.ActiveEditLayers = LayerVis;
		RenderParams.WeightmapNames = WeightmapNames;

		RenderParams.CpuResult = WeightData;

		bool bRenderRet = Landscape->SelectiveRenderEditLayersWeightmaps(RenderParams);
		if (!bRenderRet)
		{
			UE_LOGF(LogLandscape, Error, "LandscapeDumpSelectiveLayerRender: SelectiveRenderEditLayersHeightmaps failed");
			return;
		}

		FImageInfo ImageInfo(Region.Size().X, Region.Size().Y, 1, ERawImageFormat::G8, EGammaSpace::Linear);

		for (int32 NameIdx = 0; NameIdx < WeightmapNames.Num(); NameIdx++)
		{
			// Write each weightmap to an image file.
			FName WeightmapName = WeightmapNames[NameIdx];
			FImageView ImageView(ImageInfo, WeightData.GetData() + (SingleWeightmapSizePixels * NameIdx));  // Offset into image buffer
			FString Filename = FilePattern + "Weightmap_" + WeightmapName.ToString() + ".png";
			if (FImageUtils::SaveImageByExtension(*Filename, ImageView))
			{
				UE_LOGF(LogLandscape, Display, "LandscapeDumpSelectiveLayerRender: wrote weightmap data to %ls", *Filename);
			}
			else
			{
				UE_LOGF(LogLandscape, Error, "LandscapeDumpSelectiveLayerRender: failed to write weightmap data to %ls", *Filename);
			}
		}

	}
}

void LandscapeForceLayersFullUpdate(const TArray<FString>& Args)
{
	TArray<FString> StringFilters(Args);

	UWorld* World = GEditor->GetEditorWorldContext().World();
	if (!World)
	{
		return;
	}

	TSet<ALandscape*> UpdatedLandscapes;
	auto UpdateLandscapeLayers = [&UpdatedLandscapes](ALandscape* InLandscape, const FString& InSuffix)
		{
			if (!UpdatedLandscapes.Contains(InLandscape))
			{
				UE_LOGF(LogLandscape, Display, "LandscapeForceLayersFullUpdate : %ls updated%ls", *InLandscape->GetActorNameOrLabel(), *InSuffix);
				InLandscape->ForceLayersFullUpdate();
				UpdatedLandscapes.Add(InLandscape);
			}
		};

	TArray<FString> RemovedFilters;
	if (StringFilters.IsEmpty())
	{
		for (ALandscape* Landscape : TActorRange<ALandscape>(World))
		{
			UpdateLandscapeLayers(Landscape, TEXT(""));
		}
	}
	else
	{
		// Look for perfect matches first :
	for (ALandscape* Landscape : TActorRange<ALandscape>(World))
	{
		const FString ActorName = Landscape->GetActorNameOrLabel();
			// If we've found a landscape whose name perfectly matches, update layers and remove the name from the filters so that we don't run the command
			//  on other landscapes with a partial name match
			if (int32 PerfectMatchFilterIndex = StringFilters.IndexOfByKey(ActorName); PerfectMatchFilterIndex != INDEX_NONE)
		{
				UpdateLandscapeLayers(Landscape, FString::Format(TEXT(" (perfect name match with: \"{0}\")"), { StringFilters[PerfectMatchFilterIndex] }));
				// Note : don't use RemoveAt here in order to remove all potential duplicate filters
				StringFilters.Remove(ActorName);
				RemovedFilters.Add(ActorName);
		}
	}

		// Run again for the landscapes whose name may only partially match one of the filters
		for (ALandscape* Landscape : TActorRange<ALandscape>(World))
		{
			if (!UpdatedLandscapes.Contains(Landscape))
			{
				const FString ActorName = Landscape->GetActorNameOrLabel();
				if (int32 PartialMatchFilterIndex = StringFilters.IndexOfByPredicate([ActorName](const FString& InStringFilter) { return ActorName.Find(InStringFilter) != INDEX_NONE; }); 
					PartialMatchFilterIndex != INDEX_NONE)
				{
					UpdateLandscapeLayers(Landscape, FString::Format(TEXT(" (partial name match with: \"{0}\")"), { StringFilters[PartialMatchFilterIndex] }));
				}
				else if (int32 PartialMatchWithRemovedFilterIndex = RemovedFilters.IndexOfByPredicate([ActorName](const FString& InStringFilter) { return ActorName.Find(InStringFilter) != INDEX_NONE; });
					PartialMatchWithRemovedFilterIndex != INDEX_NONE)
				{
					UE_LOGF(LogLandscape, Display, "LandscapeForceLayersFullUpdate : %ls skipped  (partial name match with: \"%ls\" but another landscape with that exact name was found)", 
						*ActorName, *RemovedFilters[PartialMatchWithRemovedFilterIndex]);
				}
			}
		}
	}

	if (UpdatedLandscapes.IsEmpty())
	{
		UE_LOGF(LogLandscape, Display, "LandscapeForceLayersFullUpdate : no landscape updated");
	}
}

#endif // WITH_EDITOR


#undef LOCTEXT_NAMESPACE
