// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeEditReadback.h"

#include "Engine/Texture2D.h"
#include "LandscapePrivate.h"
#include "RenderingThread.h"
#include "RenderUtils.h"
#include "Hash/CityHashHelpers.h"

namespace UE::Landscape::Private
{
static TAutoConsoleVariable<int32> CVarReadbackPoolSizeMB(
	TEXT("landscape.ReadbackPoolSizeMB"),
	256,
	TEXT("Minimum pool size (in MB) for the editor readbacks. This ensures a minimum amount of readback textures are left in the pool when reclaiming memory, which avoids severe hiccups when reallocating a lot of resources."));

static int32 TotalStagingTexturesAllocatedSize = 0;
}

/** Data for a read back task. */
struct FLandscapeEditReadbackTaskImpl
{
	/** Completion state for a read back task. */
	enum class ECompletionState : uint64
	{
		None = 0,		// Copy not submitted
		Pending = 1,	// Copy submitted, waiting for GPU
		Complete = 2	// Result copied back from GPU
	};

	// Create on game thread
	FTextureResource const* TextureResource = nullptr;
	FLandscapeEditLayerReadback::FReadbackContext ReadbackContext;
	uint32 InitFrameId = 0;
	FIntPoint Size = FIntPoint(ForceInitToZero);
	uint32 NumMips = 0;
	EPixelFormat Format = PF_Unknown;
	int32 StagingTexturesAllocatedSize = 0;

	// Create on render thread
	TArray<FTextureRHIRef> StagingTextures;
	FGPUFenceRHIRef ReadbackFence;

	// Result written on render thread and read on game thread
	ECompletionState CompletionState = ECompletionState::None;
	TArray<TArray<FColor>> Result;

};

/** Initialize the read back task data that is written by game thread. */
void InitTask_GameThread(FLandscapeEditReadbackTaskImpl& Task, UTexture2D const* InTexture, FLandscapeEditLayerReadback::FReadbackContext&& InReadbackContext, uint32 InFrameId)
{
	Task.TextureResource = InTexture->GetResource();
	Task.ReadbackContext = MoveTemp(InReadbackContext);
	Task.InitFrameId = InFrameId;
	Task.Size = FIntPoint(InTexture->GetSizeX(), InTexture->GetSizeY());
	Task.NumMips = InTexture->GetNumMips();
	Task.Format = InTexture->GetPixelFormat();
	Task.CompletionState = FLandscapeEditReadbackTaskImpl::ECompletionState::None;
}

/** Initialize the read back task resources. */
bool InitTask_RenderThread(FRHICommandListImmediate& RHICmdList, FLandscapeEditReadbackTaskImpl& Task)
{
	using namespace UE::Landscape::Private;

	if (Task.StagingTextures.Num() == 0 || !Task.StagingTextures[0].IsValid() || Task.StagingTextures[0]->GetSizeXYZ() != FIntVector(Task.Size.X, Task.Size.Y, 1) || (Task.StagingTextures[0]->GetFormat() != Task.Format))
	{
		Task.StagingTextures.SetNum(Task.NumMips);

		for (uint32 MipIndex = 0; MipIndex < Task.NumMips; ++MipIndex)
		{
			const int32 MipWidth = FMath::Max(Task.Size.X >> MipIndex, 1);
			const int32 MipHeight = FMath::Max(Task.Size.Y >> MipIndex, 1);

			const FRHITextureCreateDesc Desc =
				FRHITextureCreateDesc::Create2D(TEXT("LandscapeEditReadbackTask"), MipWidth, MipHeight, Task.Format)
				.SetFlags(ETextureCreateFlags::CPUReadback);

			Task.StagingTextures[MipIndex] = RHICmdList.CreateTexture(Desc);
		}

		Task.StagingTexturesAllocatedSize += CalcTextureSize(Task.Size.X, Task.Size.Y, Task.Format, Task.NumMips);
		TotalStagingTexturesAllocatedSize += Task.StagingTexturesAllocatedSize;
	}

	if (!Task.ReadbackFence.IsValid())
	{
		Task.ReadbackFence = RHICreateGPUFence(TEXT("LandscapeEditReadbackTask"));
	}
	Task.ReadbackFence->Clear();

	return true;
}

/** Kick the GPU work for the read back task. */
void KickTask_RenderThread(FRHICommandListImmediate& RHICmdList, FLandscapeEditReadbackTaskImpl& Task)
{
	// Transition staging textures for write.
	TArray <FRHITransitionInfo> Transitions;
	Transitions.Add(FRHITransitionInfo(Task.TextureResource->GetTexture2DRHI(), ERHIAccess::SRVMask, ERHIAccess::CopySrc));
	for (uint32 MipIndex = 0; MipIndex < Task.NumMips; ++MipIndex)
	{
		Transitions.Add(FRHITransitionInfo(Task.StagingTextures[MipIndex], ERHIAccess::Unknown, ERHIAccess::CopyDest));
	}
	RHICmdList.Transition(Transitions);

	// Copy to staging textures.
	for (uint32 MipIndex = 0; MipIndex < Task.NumMips; ++MipIndex)
	{
		const int32 MipWidth = FMath::Max(Task.Size.X >> MipIndex, 1);
		const int32 MipHeight = FMath::Max(Task.Size.Y >> MipIndex, 1);

		FRHICopyTextureInfo Info;
		Info.Size = FIntVector(MipWidth, MipHeight, 1);
		Info.SourceMipIndex = MipIndex;

		RHICmdList.CopyTexture(Task.TextureResource->GetTexture2DRHI(), Task.StagingTextures[MipIndex], Info);
	}

	// Transition staging textures for read.
	Transitions.Reset();
	Transitions.Add(FRHITransitionInfo(Task.TextureResource->GetTexture2DRHI(), ERHIAccess::CopySrc, ERHIAccess::SRVMask));
	for (uint32 MipIndex = 0; MipIndex < Task.NumMips; ++MipIndex)
	{
		Transitions.Add(FRHITransitionInfo(Task.StagingTextures[MipIndex], ERHIAccess::Unknown, ERHIAccess::CPURead));
	}
	RHICmdList.Transition(Transitions);

	// Write fence used to read back without stalling.
	RHICmdList.WriteGPUFence(Task.ReadbackFence);

	Task.CompletionState = FLandscapeEditReadbackTaskImpl::ECompletionState::Pending;
}

/**
 * Update the read back task on the render thread. Check if the GPU work is complete and if it is copy the data.
 * @return true if the task's state is Complete, false if it is still Pending : 
 */
bool UpdateTask_RenderThread(FRHICommandListImmediate& RHICmdList, FLandscapeEditReadbackTaskImpl& Task, bool bFlush)
{
	if (Task.CompletionState == FLandscapeEditReadbackTaskImpl::ECompletionState::Pending && (bFlush || Task.ReadbackFence->Poll()))
	{
		// Read back to Task.Result
		Task.Result.SetNum(Task.NumMips);

		for (uint32 MipIndex = 0; MipIndex < Task.NumMips; ++MipIndex)
		{
			const int32 MipWidth = FMath::Max(Task.Size.X >> MipIndex, 1);
			const int32 MipHeight = FMath::Max(Task.Size.Y >> MipIndex, 1);

			// Editor always runs on GPU zero
			const uint32 GPUIndex = 0;

			Task.Result[MipIndex].SetNum(MipWidth * MipHeight);

			void* Data = nullptr;
			int32 TargetWidth, TargetHeight;
			RHICmdList.MapStagingSurface(Task.StagingTextures[MipIndex], Task.ReadbackFence.GetReference(), Data, TargetWidth, TargetHeight, GPUIndex);
			check(Data != nullptr);
			check(MipWidth <= TargetWidth && MipHeight <= TargetHeight);

			FColor const* ReadPtr = (FColor*)Data;
			FColor* WritePtr = Task.Result[MipIndex].GetData();
			for (int32 Y = 0; Y < MipHeight; ++Y)
			{
				FMemory::Memcpy(WritePtr, ReadPtr, MipWidth * sizeof(FColor));
				ReadPtr += TargetWidth;
				WritePtr += MipWidth;
			}

			RHICmdList.UnmapStagingSurface(Task.StagingTextures[MipIndex], GPUIndex);
		}

		// Write completion flag for game thread.
		FPlatformMisc::MemoryBarrier();
		Task.CompletionState = FLandscapeEditReadbackTaskImpl::ECompletionState::Complete;
	}

	return (Task.CompletionState == FLandscapeEditReadbackTaskImpl::ECompletionState::Complete);
}


/** 
 * Pool of read back tasks. 
 * Decouples task ownership so that that tasks can be easily released and recycled.
 */
class FLandscapeEditReadbackTaskPool : public FRenderResource
{
public:
	/** Pool uses chunked array to avoid task data being moved by a realloc. */
	TChunkedArray< FLandscapeEditReadbackTaskImpl > Pool;
	/** Allocation count used to check if there are any tasks to Tick. */
	uint32 AllocCount = 0;
	/** Frame count used to validate and garbage collect. */
	uint32 FrameCount = 0;

	void ReleaseRHI() override
	{
		Pool.Empty();
	}

	/** Allocate task data from the pool. */
	int32 Allocate(UTexture2D const* InTexture, FLandscapeEditLayerReadback::FReadbackContext&& InReadbackContext)
	{
		int32 CurrentIndex = 0;
		int32 BestEntryIndex = INDEX_NONE;
		FIntVector TextureSize(InTexture->GetSizeX(), InTexture->GetSizeY(), 1);
		auto ItEnd = Pool.end();
		for (auto It = Pool.begin(); It != ItEnd; ++It, ++CurrentIndex)
		{
			FLandscapeEditReadbackTaskImpl& Task = *It;
			// If the entry is unused, it's a candidate 
			if (Task.TextureResource == nullptr)
			{
				BestEntryIndex = CurrentIndex;
				// Check the entry's texture size to ensure it's the best possible candidate. If so, no need to look further :
				if (!Task.StagingTextures.IsEmpty() && Task.StagingTextures[0].IsValid() && (Task.StagingTextures[0]->GetSizeXYZ() == TextureSize) && (Task.Format == InTexture->GetPixelFormat()))
				{
					break;
				}
			}
		}

		if (BestEntryIndex == INDEX_NONE)
		{
			Pool.Add();
			BestEntryIndex = Pool.Num() - 1;
		}

		InitTask_GameThread(Pool[BestEntryIndex], InTexture, MoveTemp(InReadbackContext), FrameCount);
		++AllocCount;
		return BestEntryIndex;
	}

	/** Return task data to the pool. */
	void Free(int32 InTaskHandle)
	{
		check(InTaskHandle != -1);
		check(AllocCount > 0);
		AllocCount --;

		// Submit render thread command to mark pooled task as free.
		ENQUEUE_RENDER_COMMAND(FLandscapeEditLayerReadback_Free)([Task = &Pool[InTaskHandle]](FRHICommandListImmediate& RHICmdList)
		{
			Task->TextureResource = nullptr;
		});
	}

	/** Free render resources that have been unused for long enough. */
	void GarbageCollect()
	{
		using namespace UE::Landscape::Private;
		
		const uint32 PoolSize = Pool.Num();
		if (PoolSize > 0)
		{
			// Garbage collect a maximum of one item per call to reduce overhead if pool has grown large.
			FLandscapeEditReadbackTaskImpl* Task = &Pool[FrameCount % PoolSize];
			if (Task->InitFrameId + 100 < FrameCount)
			{
				if (Task->TextureResource != nullptr)
				{
					// Task not completed after 100 updates. We are probably leaking tasks!
					UE_LOGF(LogLandscape, Warning, "Leaking landscape edit layer read back tasks.")
				}
				else
				{
					// Free data allocations
					Task->ReadbackContext.Empty();
					Task->Result.Empty();

					const int32 MinPoolSize = CVarReadbackPoolSizeMB.GetValueOnGameThread();
					const int32 MinPoolSizeBytes = MinPoolSize * 1024 * 1024;
					if ((!Task->StagingTextures.IsEmpty() || Task->ReadbackFence.IsValid()) 
						&& ((TotalStagingTexturesAllocatedSize - Task->StagingTexturesAllocatedSize) > MinPoolSizeBytes)) // Don't deplete the pool under the minimum limit
					{
						TotalStagingTexturesAllocatedSize -= Task->StagingTexturesAllocatedSize;
						check(TotalStagingTexturesAllocatedSize >= 0);

						// Release the render resources (which may already be released)
						ENQUEUE_RENDER_COMMAND(FLandscapeEditLayerReadback_Release)([Task](FRHICommandListImmediate& RHICmdList)
						{
							Task->StagingTextures.Reset();
							Task->ReadbackFence.SafeRelease();
						});
					}
				}
			}
		}

		FrameCount++;
	}

	void FlushAll()
	{
		// Flush all pending tasks in a single command
		ENQUEUE_RENDER_COMMAND(FLandscapeEditLayerReadback_FlushAll)([this](FRHICommandListImmediate& RHICmdList)
		{
			auto ItEnd = Pool.end();
			for (auto It = Pool.begin(); It != ItEnd; ++It)
			{
				FLandscapeEditReadbackTaskImpl& Task = *It;
				if (Task.TextureResource != nullptr)
				{
					bool bTaskComplete = UpdateTask_RenderThread(RHICmdList, Task, /*bFlush = */true);
					check(bTaskComplete); // Flush should never fail to complete
				}
			}
		});

		TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeLayers_ReadbackFlushAll);
		FlushRenderingCommands();
	}
};

/** Static global pool object. */
static TGlobalResource< FLandscapeEditReadbackTaskPool > GReadbackTaskPool;


FLandscapeEditLayerReadback::FLandscapeEditLayerReadback()
{}

FLandscapeEditLayerReadback::~FLandscapeEditLayerReadback()
{
	for (int32 TaskHandle : TaskHandles)
	{
		GReadbackTaskPool.Free(TaskHandle);
	}
}

bool FLandscapeEditLayerReadback::SetHash(uint64 InHash)
{
	const bool bChanged = InHash != Hash;
	Hash = InHash;
	return bChanged;
}

void FLandscapeEditLayerReadback::Enqueue(UTexture2D const* InSourceTexture, FReadbackContext&& InReadbackContext)
{
	const int32 TaskHandle = GReadbackTaskPool.Allocate(InSourceTexture, MoveTemp(InReadbackContext));
	if (ensure(TaskHandle != -1))
	{
		TaskHandles.Add(TaskHandle);

		ENQUEUE_RENDER_COMMAND(FLandscapeEditLayerReadback_Queue)([TaskHandle](FRHICommandListImmediate& RHICmdList)
		{
			InitTask_RenderThread(RHICmdList, GReadbackTaskPool.Pool[TaskHandle]);
			KickTask_RenderThread(RHICmdList, GReadbackTaskPool.Pool[TaskHandle]);
		});
	}
}

void FLandscapeEditLayerReadback::Tick()
{
	TArray<int32> TaskHandlesCopy(TaskHandles);

	ENQUEUE_RENDER_COMMAND(FLandscapeEditLayerReadback_Tick)([TasksToUpdate = MoveTemp(TaskHandlesCopy)](FRHICommandListImmediate& RHICmdList)
	{
		for (int32 TaskHandle : TasksToUpdate)
		{
			// Tick the task : 
			bool bTaskComplete = UpdateTask_RenderThread(RHICmdList, GReadbackTaskPool.Pool[TaskHandle], false);
			// Stop processing at the first incomplete task in order not to get a task's state to Complete before a one of its previous task (in case their GPU fences are written in between the calls to UpdateTask_RenderThread) : 
			if (!bTaskComplete)
			{
				break;
			}
		}
	});
}

void FLandscapeEditLayerReadback::Flush()
{
	TArray<int32> TaskHandlesCopy(TaskHandles);

	ENQUEUE_RENDER_COMMAND(FLandscapeEditLayerReadback_Flush)([TasksToUpdate = MoveTemp(TaskHandlesCopy)](FRHICommandListImmediate& RHICmdList)
	{
		for (int32 TaskHandle : TasksToUpdate)
		{
			bool bTaskComplete = UpdateTask_RenderThread(RHICmdList, GReadbackTaskPool.Pool[TaskHandle], true);
			check(bTaskComplete); // Flush should never fail to complete
		}
	});

	TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeLayers_ReadbackFlush);
	FlushRenderingCommands();
}

int32 FLandscapeEditLayerReadback::GetCompletedResultNum() const
{
	// Find last task marked as complete. We can assume that tasks complete in order.
	for (int32 TaskIndex = TaskHandles.Num() - 1; TaskIndex >= 0; --TaskIndex)
	{
		if (GReadbackTaskPool.Pool[TaskHandles[TaskIndex]].CompletionState == FLandscapeEditReadbackTaskImpl::ECompletionState::Complete)
		{
			return TaskIndex + 1;
		}
	}

	return 0;
}

TArray<TArray<FColor>> const& FLandscapeEditLayerReadback::GetResult(int32 InResultIndex) const
{
	check(InResultIndex >= 0);
	check(InResultIndex < TaskHandles.Num());
	check(GReadbackTaskPool.Pool[TaskHandles[InResultIndex]].CompletionState == FLandscapeEditReadbackTaskImpl::ECompletionState::Complete);

	return GReadbackTaskPool.Pool[TaskHandles[InResultIndex]].Result;
}

FLandscapeEditLayerReadback::FReadbackContext const& FLandscapeEditLayerReadback::GetResultContext(int32 InResultIndex) const
{
	check(InResultIndex >= 0);
	check(InResultIndex < TaskHandles.Num());
	check(GReadbackTaskPool.Pool[TaskHandles[InResultIndex]].CompletionState == FLandscapeEditReadbackTaskImpl::ECompletionState::Complete);

	return GReadbackTaskPool.Pool[TaskHandles[InResultIndex]].ReadbackContext;
}

void FLandscapeEditLayerReadback::ReleaseCompletedResults(int32 InResultNum)
{
	check(InResultNum > 0);
	check(InResultNum <= TaskHandles.Num());
	check(GReadbackTaskPool.Pool[TaskHandles[InResultNum - 1]].CompletionState == FLandscapeEditReadbackTaskImpl::ECompletionState::Complete);

	for (int32 TaskIndex = 0; TaskIndex < InResultNum; ++TaskIndex)
	{
		GReadbackTaskPool.Free(TaskHandles[TaskIndex]);
	}

	TaskHandles.RemoveAt(0, InResultNum, EAllowShrinking::No);
}

bool FLandscapeEditLayerReadback::HasAnyGlobalWork()
{
	return GReadbackTaskPool.AllocCount > 0;
}

void FLandscapeEditLayerReadback::GarbageCollectTasks()
{
	GReadbackTaskPool.GarbageCollect();
}

void FLandscapeEditLayerReadback::FlushAllReadbackTasks()
{
	GReadbackTaskPool.FlushAll();
}