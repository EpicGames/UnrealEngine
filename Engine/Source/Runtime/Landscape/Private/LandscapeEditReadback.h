// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TextureResource.h"

/** 
 * Read back queue for a landscape edit layer texture.
 * This handles reading landscape edit data back from the GPU to the game thread.
 * Multiple read backs can be queued in a single object and any completed results can be retrieved.
 * Tick() must be called regularly to update the internal read back tasks.
 * When results are available GetCompletedResultNum() will be greater than zero.
 * After getting latest results with GetResult()/GetResultContext() you should call ReleaseCompletedResults() to prevent leaks.
 */
class FLandscapeEditLayerReadback
{
public:
	FLandscapeEditLayerReadback();
	~FLandscapeEditLayerReadback();

	/** Update the stored hash value. Return true if this changes the value. */
	bool SetHash(uint64 InHash);
	/** Get the stored hash value. */
	uint64 GetHash() const { return Hash; }

	using FPerChannelLayerNames = TStaticArray<FName, 4>;

	/** Per component context required for processing read back results. */
	struct FComponentReadbackContext
	{
		/** Component identifier key. */
		FIntPoint ComponentKey;
		/** Component ELandscapeLayerUpdateMode flags. */
		int32 UpdateModes = 0;
		/** For weightmaps only : configuration of the channels for this component and texture when the readback was performed (useful in case the channel configuration changed before we could perform the readback) */
		FPerChannelLayerNames PerChannelLayerNames;
	};
	/** Full context for processing read back results. */
	using FReadbackContext = TArray<FComponentReadbackContext>;

	/** Queue a new texture to read back. The GPU copy will be queued to the render thread inside this function. */
	void Enqueue(UTexture2D const* InSourceTexture, FReadbackContext&& InReadbackContext);

	/* Tick the read back tasks. */
	void Tick();
	/* Flush all read back tasks to completion. This may stall waiting for the render thread. */
	void Flush();

	/** Returns a value greater than 0 if there are completed read back tasks for which we can retrieve results. */
	int32 GetCompletedResultNum() const;

	/** Get the result for the completed task index. Returned reference will be valid until we next call ReleaseCompletedResults(). */
	TArray< TArray<FColor> > const& GetResult(int32 InResultIndex) const;
	/** Get the result context for the completed task index. Returned reference will be valid until we next call ReleaseCompletedResults(). */
	FReadbackContext const& GetResultContext(int32 InResultIndex) const;

	/* Release results. Pass in a value which is no more than the value returned by GetCompletedResultNum(). */
	void ReleaseCompletedResults(int32 InResultNum);

	bool HasActiveTasks() const
	{
		return !TaskHandles.IsEmpty();
	}

	/** Returns true if there are any queued tasks that haven't been released for any readback object. */
	static bool HasAnyGlobalWork();

	/** Optional call to allow garbage collection of any task resources and check for leaks. Call once per frame. */
	static void GarbageCollectTasks();

	/** Brute-force flush of all pending readback tasks. Use this when you want to make sure all readbacks are performed. In case of multiple readbacks, this is much more efficient 
	 than individually flushing each task. */
	static void FlushAllReadbackTasks();

private:
	uint64 Hash = 0;
	TArray<int32> TaskHandles;
};
