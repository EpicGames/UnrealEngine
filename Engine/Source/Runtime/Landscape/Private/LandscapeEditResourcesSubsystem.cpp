// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeEditResourcesSubsystem.h"
#include "Engine/Engine.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "RenderingThread.h"
#include "TextureResource.h"
#include "UObject/Package.h"
#include "RenderGraphUtils.h"
#include "LandscapeUtils.h"

#if WITH_EDITOR
#include "LandscapeRender.h"
#include "LandscapeMaterialInstanceConstant.h"
#endif // WITH_EDITOR

// ----------------------------------------------------------------------------------

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapeEditResourcesSubsystem)

namespace UE::Landscape
{

FScratchRenderTargetScope::FScratchRenderTargetScope(const FScratchRenderTargetParams& InParams)
{
	ULandscapeEditResourcesSubsystem* LandscapeEditResourcesSubsystem = GEngine->GetEngineSubsystem<ULandscapeEditResourcesSubsystem>();
	check(LandscapeEditResourcesSubsystem != nullptr);
	RenderTarget = LandscapeEditResourcesSubsystem->RequestScratchRenderTarget(InParams);
}

FScratchRenderTargetScope::~FScratchRenderTargetScope()
{
	ULandscapeEditResourcesSubsystem* LandscapeEditResourcesSubsystem = GEngine->GetEngineSubsystem<ULandscapeEditResourcesSubsystem>();
	check(LandscapeEditResourcesSubsystem != nullptr);
	LandscapeEditResourcesSubsystem->ReleaseScratchRenderTarget(RenderTarget);
}

} // namespace UE::Landscape


// ----------------------------------------------------------------------------------

FRHITransitionInfo ULandscapeScratchRenderTarget::FTransitionInfo::ToRHITransitionInfo() const
{
	return FRHITransitionInfo(Resource->TextureRHI, StateBefore, StateAfter);
}


// ----------------------------------------------------------------------------------

ULandscapeScratchRenderTarget::FTransitionBatcherScope::FTransitionBatcherScope(UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder)
	: RDGBuilderRecorder(RDGBuilderRecorder)
{

}

ULandscapeScratchRenderTarget::FTransitionBatcherScope::~FTransitionBatcherScope()
{
	if (!PendingTransitions.IsEmpty())
	{
		// Don't transition when recording : the graph builder will do it automatically. It is simply required that the user specifies to the command recorder the state of each external texture
		//  ever used in one of the recorded RDG commands, if they want to prevent the auto-transition to SRVMask when the commands are flushed and the FRDGBuilder, executed.
		if (!RDGBuilderRecorder.IsRecording())
		{
			auto PerformRHITransitions = [Transitions = MoveTemp(PendingTransitions)](FRHICommandListImmediate& RHICmdList) mutable
				{
					TArray<FRHITransitionInfo, TInlineAllocator<8>> RHITransitions;
					Algo::Transform(Transitions, RHITransitions, [](const FTransitionInfo& InTransitionInfo) { return InTransitionInfo.ToRHITransitionInfo(); });
					RHICmdList.Transition(RHITransitions);
				};
			RDGBuilderRecorder.EnqueueRenderCommand(PerformRHITransitions);
		}
	}
}

void ULandscapeScratchRenderTarget::FTransitionBatcherScope::TransitionTo(ULandscapeScratchRenderTarget* InScratchRenderTarget, ERHIAccess InStateAfter)
{
	check(InScratchRenderTarget != nullptr);
	if (InScratchRenderTarget->CurrentState != InStateAfter)
	{
		// Append the transition and change the scratch RT's state but only issue the render commands when the object goes out of scope : 
		PendingTransitions.Emplace(InScratchRenderTarget->RenderTarget->GameThread_GetRenderTargetResource(), InScratchRenderTarget->CurrentState, InStateAfter);
		InScratchRenderTarget->CurrentState = InStateAfter;
	}
}


// ----------------------------------------------------------------------------------

ULandscapeScratchRenderTarget::ULandscapeScratchRenderTarget()
{
	
}

UTextureRenderTarget2D* ULandscapeScratchRenderTarget::GetRenderTarget2D() const
{
	UTextureRenderTarget2D* TextureRenderTarget2D = Cast<UTextureRenderTarget2D>(RenderTarget);
	checkf((TextureRenderTarget2D != nullptr) && (CurrentRenderTargetParams.NumSlices <= 0), TEXT("Cannot ask for a render target 2D on a scratch render target that wasn't created as one"));
	return TextureRenderTarget2D;
}

UTextureRenderTarget2D* ULandscapeScratchRenderTarget::TryGetRenderTarget2D() const
{
	return Cast<UTextureRenderTarget2D>(RenderTarget);
}

UTextureRenderTarget2DArray* ULandscapeScratchRenderTarget::GetRenderTarget2DArray() const
{
	UTextureRenderTarget2DArray* RenderTarget2DArray = Cast<UTextureRenderTarget2DArray>(RenderTarget);
	checkf((RenderTarget2DArray != nullptr) && (CurrentRenderTargetParams.NumSlices > 0), TEXT("Cannot ask for a render target 2D array on a scratch render target that wasn't created as one"));
	return RenderTarget2DArray;
}

UTextureRenderTarget2DArray* ULandscapeScratchRenderTarget::TryGetRenderTarget2DArray() const
{
	return Cast<UTextureRenderTarget2DArray>(RenderTarget);
}

const FString& ULandscapeScratchRenderTarget::GetDebugName() const
{
	return CurrentRenderTargetParams.DebugName;
}

FIntPoint ULandscapeScratchRenderTarget::GetResolution() const
{
	if (UTextureRenderTarget2D* RenderTarget2D = TryGetRenderTarget2D())
	{
		return FIntPoint(RenderTarget2D->SizeX, RenderTarget2D->SizeY);
	}
	else if (UTextureRenderTarget2DArray* RenderTarget2DArray = TryGetRenderTarget2DArray())
	{
		return FIntPoint(RenderTarget2DArray->SizeX, RenderTarget2DArray->SizeY);
	}
	return FIntPoint(ForceInitToZero);
}

FIntPoint ULandscapeScratchRenderTarget::GetEffectiveResolution() const
{
	return CurrentRenderTargetParams.Resolution;
}

int32 ULandscapeScratchRenderTarget::GetNumSlices() const
{
	if (UTextureRenderTarget2DArray* RenderTarget2DArray = TryGetRenderTarget2DArray())
	{
		return RenderTarget2DArray->Slices;
	}
	return 0;
}

int32 ULandscapeScratchRenderTarget::GetEffectiveNumSlices() const
{
	return CurrentRenderTargetParams.NumSlices;
}

FLinearColor ULandscapeScratchRenderTarget::GetClearColor() const
{
	if (UTextureRenderTarget2D* RenderTarget2D = TryGetRenderTarget2D())
	{
		return RenderTarget2D->ClearColor;
	}
	else if (UTextureRenderTarget2DArray* RenderTarget2DArray = TryGetRenderTarget2DArray())
	{
		return RenderTarget2DArray->ClearColor;
	}
	return FLinearColor(ForceInitToZero);
}

ETextureRenderTargetFormat ULandscapeScratchRenderTarget::GetFormat() const
{
	return RenderTargetFormat;
}

void ULandscapeScratchRenderTarget::TransitionTo(ERHIAccess InDesiredState, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder)
{
	check(RenderTarget != nullptr);

	FTransitionBatcherScope TransitionScope(RDGBuilderRecorder);
	TransitionScope.TransitionTo(this, InDesiredState);
}

void ULandscapeScratchRenderTarget::Clear(UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder)
{
	using namespace UE::Landscape;

	check(RenderTarget != nullptr);

	TransitionTo(ERHIAccess::RTV, RDGBuilderRecorder);

	auto RDGCommand = 
		[ Resource = RenderTarget->GameThread_GetRenderTargetResource()
		, EffectiveNumSlices = GetEffectiveNumSlices()]
		(FRDGBuilder& GraphBuilder)
		{
			FRDGTextureRef TextureRef = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(Resource->GetTextureRHI(), TEXT("ScratchTexture")));
			FRDGTextureClearInfo ClearInfo;
			if (TextureRef->Desc.IsTextureArray())
			{
				check(EffectiveNumSlices <= TextureRef->Desc.ArraySize);
				ClearInfo.NumSlices = EffectiveNumSlices;
			}
			AddClearRenderTargetPass(GraphBuilder, TextureRef, ClearInfo);
		};

	// We need to specify the final state of the external texture to prevent the graph builder from transitioning it to SRVMask :
	RDGBuilderRecorder.EnqueueRDGCommand(RDGCommand, { { RenderTarget->GetResource(), ERHIAccess::RTV } });
}

namespace UE::Landscape::Private
{

void EnqueueCopyToScratchRTRenderCommand(const ULandscapeScratchRenderTarget::FCopyFromParams& InCopyParams, FTextureResource* InSourceTextureResource, FTextureResource* InDestTextureResource, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder)
{
	FIntPoint SourceSize(InSourceTextureResource->GetSizeX() >> InCopyParams.SourceMip, InSourceTextureResource->GetSizeY() >> InCopyParams.SourceMip);
	FIntPoint DestSize(InDestTextureResource->GetSizeX() >> InCopyParams.DestMip, InDestTextureResource->GetSizeY() >> InCopyParams.DestMip);

	FRHICopyTextureInfo Info;
	// For now this function only supports the copy of a single slice : 
	Info.NumSlices = 1;
	// If CopySize is passed, use that as the size (and don't adjust with the mip level : consider that the user has computed it properly) : 
	Info.Size.X = (InCopyParams.CopySize.X > 0) ? InCopyParams.CopySize.X : SourceSize.X;
	Info.Size.Y = (InCopyParams.CopySize.Y > 0) ? InCopyParams.CopySize.Y : SourceSize.Y;
	Info.Size.Z = 1;
	Info.SourcePosition.X = InCopyParams.SourcePosition.X;
	Info.SourcePosition.Y = InCopyParams.SourcePosition.Y;
	Info.DestPosition.X = InCopyParams.DestPosition.X;
	Info.DestPosition.Y = InCopyParams.DestPosition.Y;
	Info.SourceSliceIndex = InCopyParams.SourceSliceIndex;
	Info.DestSliceIndex = InCopyParams.DestSliceIndex;
	Info.SourceMipIndex = InCopyParams.SourceMip;
	Info.DestMipIndex = InCopyParams.DestMip;

	check((Info.SourcePosition.X >= 0) && (Info.SourcePosition.Y >= 0) && (Info.DestPosition.X >= 0) && (Info.DestPosition.Y >= 0));
	check(Info.SourcePosition.X + Info.Size.X <= SourceSize.X);
	check(Info.SourcePosition.Y + Info.Size.Y <= SourceSize.Y);
	check(Info.DestPosition.X + Info.Size.X <= DestSize.X);
	check(Info.DestPosition.Y + Info.Size.Y <= DestSize.Y);

	auto RDGCommand =
		[ InSourceTextureResource
		, InDestTextureResource
		, Info]
		(FRDGBuilder& GraphBuilder)
		{
			FRDGTextureRef SourceTextureRef = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(InSourceTextureResource->GetTextureRHI(), TEXT("CopySourceTexture")));
			FRDGTextureRef DestTextureRef = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(InDestTextureResource->GetTextureRHI(), TEXT("CopyDestTexture")));
			AddCopyTexturePass(GraphBuilder, SourceTextureRef, DestTextureRef, Info);
		};

	// We need to specify the final state of the external textures to prevent the graph builder from transitioning them to SRVMask :
	TArray<FRDGBuilderRecorder::FRDGExternalTextureAccessFinal> RDGExternalTextureAccessFinalList =
	{
		{ InSourceTextureResource, ERHIAccess::CopySrc },
		{ InDestTextureResource, ERHIAccess::CopyDest }
	};
	RDGBuilderRecorder.EnqueueRDGCommand(RDGCommand, RDGExternalTextureAccessFinalList);
}

} // end namespace UE::Landscape::Private


void ULandscapeScratchRenderTarget::CopyFrom(const FCopyFromTextureParams& InCopyParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder)
{
	using namespace UE::Landscape::Private;

	// The source is expected to be in CopySrc state already. We need to transition the scratch RT to the appropriate state, though : 
	TransitionTo(ERHIAccess::CopyDest, RDGBuilderRecorder);

	EnqueueCopyToScratchRTRenderCommand(InCopyParams, InCopyParams.SourceTexture->GetResource(), RenderTarget->GameThread_GetRenderTargetResource(), RDGBuilderRecorder);
}

void ULandscapeScratchRenderTarget::CopyFrom(const FCopyFromScratchRenderTargetParams& InCopyParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder)
{
	using namespace UE::Landscape::Private;

	// We need to transition both the source and destination scratch RT to the appropriate state:
	InCopyParams.SourceScratchRenderTarget->TransitionTo(ERHIAccess::CopySrc, RDGBuilderRecorder);
	TransitionTo(ERHIAccess::CopyDest, RDGBuilderRecorder);

	EnqueueCopyToScratchRTRenderCommand(InCopyParams, InCopyParams.SourceScratchRenderTarget->GetRenderTarget()->GameThread_GetRenderTargetResource(), RenderTarget->GameThread_GetRenderTargetResource(), RDGBuilderRecorder);
}

bool ULandscapeScratchRenderTarget::IsCompatibleWith(const UE::Landscape::FScratchRenderTargetParams& InParams) const
{
	check(RenderTarget != nullptr);

	// If it's already in use, it cannot be considered compatible (since the purpose is to recycle the scratch RT if possible) : 
	if (IsInUse())
	{
		return false;
	}

	if (RenderTargetFormat != InParams.Format)
	{
		return false;
	}

	FLinearColor RenderTargetClearColor = GetClearColor();
	// Don't use FLinearColor::operator != on purpose : the clear color needs to match exactly to be considered compatible
	if (GetTypeHash(RenderTargetClearColor) != GetTypeHash(InParams.ClearColor))
	{
		return false;
	}

	// If texture flags are different, we cannot be compatible
	if (RenderTarget->bCanCreateUAV != InParams.bUseUAV || RenderTarget->bTargetArraySlicesIndependently != InParams.bTargetArraySlicesIndependently)
	{
		return false;
	}

	int32 RenderTargetNumSlices = GetNumSlices();
	const bool bNeedsTextureArray = (InParams.NumSlices > 0);
	// Only keep RTs that are of the proper type (texture 2D or texture 2D array) :
	if (IsTexture2DArray() != bNeedsTextureArray)
	{
		return false;
	}

	// Only keep RTs that are of the requested format and large enough to fit the requested RT's size : 
	FIntPoint RenderTargetResolution = GetResolution();
	bool bIsCompatibleResolution = (RenderTargetResolution == InParams.Resolution);
	if (!bIsCompatibleResolution && !InParams.bExactDimensions)
	{
		bIsCompatibleResolution = (RenderTargetResolution.X >= InParams.Resolution.X) && (RenderTargetResolution.Y >= InParams.Resolution.Y);
	}

	// For texture arrays, only keep RTs that are of the requested format and large enough to fit the requested RT's size : 
	bool bIsCompatibleNumSlices = true;
	if (bNeedsTextureArray)
	{
		bIsCompatibleNumSlices = (RenderTargetNumSlices == InParams.NumSlices);
		if (!bIsCompatibleNumSlices && !InParams.bExactDimensions)
		{
			bIsCompatibleNumSlices = RenderTargetNumSlices >= InParams.NumSlices;
		}
	}

	return bIsCompatibleResolution && bIsCompatibleNumSlices;
}

void ULandscapeScratchRenderTarget::OnRequested(const UE::Landscape::FScratchRenderTargetParams& InParams)
{
	using namespace UE::Landscape;

	check(!IsInUse());

	// If it's not been initialized yet, create the render target now : 
	if (RenderTarget == nullptr)
	{
		// No existing RT is compatible, create a new one : 
		if (InParams.NumSlices > 0)
		{
			FName RTName = MakeUniqueObjectName(GetTransientPackage(), UTextureRenderTarget2DArray::StaticClass(), TEXT("ScratchLandscapeRT2DArray"));
			UTextureRenderTarget2DArray* RenderTarget2DArray = NewObject<UTextureRenderTarget2DArray>(GetTransientPackage(), RTName, RF_Transient);
			RenderTarget2DArray->bCanCreateUAV = InParams.bUseUAV;
			RenderTarget2DArray->bTargetArraySlicesIndependently = InParams.bTargetArraySlicesIndependently;
			RenderTarget2DArray->OverrideFormat = GetPixelFormatFromRenderTargetFormat(InParams.Format);
			RenderTarget2DArray->ClearColor = InParams.ClearColor;
			RenderTarget2DArray->InitAutoFormat(InParams.Resolution.X, InParams.Resolution.Y, InParams.NumSlices);
			RenderTarget2DArray->UpdateResourceImmediate(/*bClearRenderTarget = */false);
			RenderTarget = RenderTarget2DArray;
		}
		else
		{
			FName RTName = MakeUniqueObjectName(GetTransientPackage(), UTextureRenderTarget2D::StaticClass(), TEXT("ScratchLandscapeRT2D"));
			UTextureRenderTarget2D* RenderTarget2D = NewObject<UTextureRenderTarget2D>(GetTransientPackage(), RTName, RF_Transient);
			RenderTarget2D->bCanCreateUAV = InParams.bUseUAV;
			RenderTarget2D->RenderTargetFormat = InParams.Format;
			RenderTarget2D->ClearColor = InParams.ClearColor;
			RenderTarget2D->InitAutoFormat(InParams.Resolution.X, InParams.Resolution.Y);
			RenderTarget2D->UpdateResourceImmediate(/*bClearRenderTarget = */false);
			RenderTarget = RenderTarget2D;
		}
		check(RenderTarget != nullptr);

		CurrentState = ERHIAccess::SRVMask;
		RenderTargetFormat = InParams.Format;
	}

	bIsInUse = true;
	CurrentRenderTargetParams = InParams;

	if (InParams.InitialState != ERHIAccess::None)
	{
		// Cannot be requested when recording RDGRenderCommandRecorder so we use an immediate recorder : 
		FRDGBuilderRecorder RDGBuilderRecorderImmediate;
		TransitionTo(InParams.InitialState, RDGBuilderRecorderImmediate);
	}
}

void ULandscapeScratchRenderTarget::OnReleased()
{
	check(IsInUse());
	CurrentRenderTargetParams = UE::Landscape::FScratchRenderTargetParams();
	bIsInUse = false;
}


// ----------------------------------------------------------------------------------

ULandscapeEditResourcesSubsystem::ULandscapeEditResourcesSubsystem()
{
}

void ULandscapeEditResourcesSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeEditResourcesSubsystem::Initialize);

	Super::Initialize(Collection);
	
#if WITH_EDITOR  
	LayerDebugColorMaterial = UE::Landscape::CreateToolLandscapeMaterialInstanceConstant (LoadObject<UMaterial>(nullptr, TEXT("/Engine/EditorLandscapeResources/LayerVisMaterial.LayerVisMaterial")));
	SelectionColorMaterial = UE::Landscape::CreateToolLandscapeMaterialInstanceConstant (LoadObject<UMaterialInstanceConstant>(nullptr, TEXT("/Engine/EditorLandscapeResources/SelectBrushMaterial_Selected.SelectBrushMaterial_Selected")));
	SelectionRegionMaterial = UE::Landscape::CreateToolLandscapeMaterialInstanceConstant (LoadObject<UMaterialInstanceConstant>(nullptr, TEXT("/Engine/EditorLandscapeResources/SelectBrushMaterial_SelectedRegion.SelectBrushMaterial_SelectedRegion")));
	MaskRegionMaterial = UE::Landscape::CreateToolLandscapeMaterialInstanceConstant (LoadObject<UMaterialInstanceConstant>(nullptr, TEXT("/Engine/EditorLandscapeResources/MaskBrushMaterial_MaskedRegion.MaskBrushMaterial_MaskedRegion")));
	ColorMaskRegionMaterial = UE::Landscape::CreateToolLandscapeMaterialInstanceConstant (LoadObject<UMaterialInstanceConstant>(nullptr, TEXT("/Engine/EditorLandscapeResources/ColorMaskBrushMaterial_MaskedRegion.ColorMaskBrushMaterial_MaskedRegion")));
	LandscapeDirtyMaterial = UE::Landscape::CreateToolLandscapeMaterialInstanceConstant (LoadObject<UMaterial>(nullptr, TEXT("/Engine/EditorLandscapeResources/LandscapeDirtyMaterial.LandscapeDirtyMaterial")));
	LandscapeLayerUsageMaterial = UE::Landscape::CreateToolLandscapeMaterialInstanceConstant (LoadObject<UMaterial>(nullptr, TEXT("/Engine/EditorLandscapeResources/LandscapeLayerUsageMaterial.LandscapeLayerUsageMaterial")));
	LandscapeBlackTexture = LoadObject<UTexture2D>(nullptr, TEXT("/Engine/EngineResources/Black.Black"));
#endif // WITH_EDITOR
}


void ULandscapeEditResourcesSubsystem::Deinitialize()
{
	Super::Deinitialize();
}

ULandscapeScratchRenderTarget* ULandscapeEditResourcesSubsystem::RequestScratchRenderTarget(const UE::Landscape::FScratchRenderTargetParams& InParams)
{
	TArray<ULandscapeScratchRenderTarget*> CompatibleRTs = ScratchRenderTargets.FilterByPredicate([&InParams](ULandscapeScratchRenderTarget* InScratchRT) { return !InScratchRT->IsInUse() && InScratchRT->IsCompatibleWith(InParams); });

	ULandscapeScratchRenderTarget* ScratchRT = nullptr;
	if (!CompatibleRTs.IsEmpty())
	{
		// Pick the one whose resolution is the closest : 
		int32 MinimumResolutionArea = InParams.Resolution.X * InParams.Resolution.Y;
		CompatibleRTs.Sort([MinimumResolutionArea](const ULandscapeScratchRenderTarget& InLHS, const ULandscapeScratchRenderTarget& InRHS) -> bool
		{
			FIntPoint LHSResolution = InLHS.GetResolution();
			FIntPoint RHSResolution = InRHS.GetResolution();
			return (LHSResolution.X * LHSResolution.Y - MinimumResolutionArea) < (RHSResolution.X * RHSResolution.Y - MinimumResolutionArea);
		});
		ScratchRT = CompatibleRTs[0];
	}
	else
	{
		FName ScratchRTName = MakeUniqueObjectName(GetTransientPackage(), ULandscapeScratchRenderTarget::StaticClass(), TEXT("ScratchLandscapeRT"));
		ScratchRT = NewObject<ULandscapeScratchRenderTarget>(GetTransientPackage(), ScratchRTName, RF_Transient);
		ScratchRenderTargets.Add(ScratchRT);
	}

	ScratchRT->OnRequested(InParams);

	return ScratchRT;
}

void ULandscapeEditResourcesSubsystem::ReleaseScratchRenderTarget(ULandscapeScratchRenderTarget* InScratchRenderTarget)
{
	check(InScratchRenderTarget->IsInUse() && ScratchRenderTargets.Contains(InScratchRenderTarget));
	InScratchRenderTarget->OnReleased();
}
