// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeGizmoActor.h"
#include "Misc/MessageDialog.h"
#include "Misc/FileHelper.h"
#include "Misc/FeedbackContext.h"
#include "SceneView.h"
#include "UObject/ConstructorHelpers.h"
#include "EngineDefines.h"
#include "RHI.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "MaterialShared.h"
#include "LandscapeInfo.h"
#include "Engine/Texture2D.h"
#include "LandscapeLayerInfoObject.h"
#include "LandscapeInfoMap.h"
#include "LandscapeDataAccess.h"
#include "LandscapePrivate.h"
#include "LandscapeRender.h"
#include "LandscapeGizmoActiveActor.h"
#include "LandscapeGizmoRenderComponent.h"
#include "DynamicMeshBuilder.h"
#include "Engine/CollisionProfile.h"
#include "EngineUtils.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Components/BillboardComponent.h"
#include "HAL/PlatformApplicationMisc.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapeGizmoActor)

class FLandscapeGizmoMeshRenderProxy : public FMaterialRenderProxy
{
public:
	const FMaterialRenderProxy* const Parent;
	const float TopHeight;
	const float BottomHeight;
	const UTexture2D* AlphaTexture;
	const FLinearColor ScaleBias;
	const FMatrix WorldToLandscapeMatrix;

	/** Initialization constructor. */
	FLandscapeGizmoMeshRenderProxy(const FMaterialRenderProxy* InParent, const float InTop, const float InBottom, const UTexture2D* InAlphaTexture, const FLinearColor& InScaleBias, const FMatrix& InWorldToLandscapeMatrix)
	:	FMaterialRenderProxy(InParent->GetMaterialName())
	,	Parent(InParent)
	,	TopHeight(InTop)
	,	BottomHeight(InBottom)
	,	AlphaTexture(InAlphaTexture)
	,	ScaleBias(InScaleBias)
	,	WorldToLandscapeMatrix(InWorldToLandscapeMatrix)
	{}

	// FMaterialRenderProxy interface.
	virtual const FMaterial* GetMaterialNoFallback(ERHIFeatureLevel::Type InFeatureLevel) const override
	{
		return Parent->GetMaterialNoFallback(InFeatureLevel);
	}

	virtual const FMaterialRenderProxy* GetFallback(ERHIFeatureLevel::Type InFeatureLevel) const override
	{
		return Parent->GetFallback(InFeatureLevel);
	}

	virtual bool GetParameterValue(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, FMaterialParameterValue& OutValue, const FMaterialRenderContext& Context) const
	{
		auto GetColor = [](const FMatrix::FReal* MatrixColumn)
		{
			return FLinearColor(static_cast<float>(MatrixColumn[0]),
			                    static_cast<float>(MatrixColumn[1]),
			                    static_cast<float>(MatrixColumn[2]),
			                    static_cast<float>(MatrixColumn[3]));
		};
		
		switch (Type)
		{
		case EMaterialParameterType::Vector:
			if (ParameterInfo.Name == FName("AlphaScaleBias"))
			{
				OutValue = ScaleBias;
				return true;
			}
			else if (ParameterInfo.Name == FName("MatrixRow1"))
			{
				OutValue = GetColor(WorldToLandscapeMatrix.M[0]);
				return true;
			}
			else if (ParameterInfo.Name == FName("MatrixRow2"))
			{
				OutValue = GetColor(WorldToLandscapeMatrix.M[1]);
				return true;
			}
			else if (ParameterInfo.Name == FName("MatrixRow3"))
			{
				OutValue = GetColor(WorldToLandscapeMatrix.M[2]);
				return true;
			}
			else if (ParameterInfo.Name == FName("MatrixRow4"))
			{
				OutValue = GetColor(WorldToLandscapeMatrix.M[3]);
				return true;
			}
			break;
		case EMaterialParameterType::Scalar:
			if (ParameterInfo.Name == FName("Top"))
			{
				OutValue = TopHeight;
				return true;
			}
			else if (ParameterInfo.Name == FName("Bottom"))
			{
				OutValue = BottomHeight;
				return true;
			}
			break;
		case EMaterialParameterType::Texture:
			if (ParameterInfo.Name == FName("AlphaTexture"))
			{
				// FIXME: This needs to return a black texture if AlphaTexture is nullptr.
				// Returning nullptr will cause the material to use GWhiteTexture.
				OutValue = AlphaTexture;
				return true;
			}
			break;
		default:
			break;
		}
		return Parent->GetParameterValue(Type, ParameterInfo, OutValue, Context);
	}
};

/** Represents a LandscapeGizmoRenderingComponent to the scene manager. */
class FLandscapeGizmoRenderSceneProxy final : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	FVector XAxis, YAxis, Origin;
	float SampleSizeX = 0.0f;
	float SampleSizeY = 0.0f;
	bool bHeightmapRendering = false;
	bool bIsValid = false;
	HHitProxy* HitProxy = nullptr;
	FMatrix MeshRT;
	FVector FrustumVerts[8];
	TArray<FVector> SampledPositions;
	TArray<FVector> SampledNormals;
	FLandscapeGizmoMeshRenderProxy* HeightmapRenderProxy = nullptr;
	FMaterialRenderProxy* GizmoRenderProxy = nullptr;

	FLandscapeGizmoRenderSceneProxy(const ULandscapeGizmoRenderComponent* InComponent)
		: FPrimitiveSceneProxy(InComponent)
	{
#if WITH_EDITOR	
		ALandscapeGizmoActiveActor* Gizmo = Cast<ALandscapeGizmoActiveActor>(InComponent->GetOwner());
		if (Gizmo && Gizmo->GizmoMeshMaterial && Gizmo->GizmoDataMaterial && Gizmo->GetRootComponent() 
			&& !FMath::IsNearlyZero(Gizmo->CachedWidth) 
			&& !FMath::IsNearlyZero(Gizmo->CachedHeight))
		{
			ULandscapeInfo* LandscapeInfo = Gizmo->TargetLandscapeInfo.Get();
 			if (LandscapeInfo && LandscapeInfo->GetLandscapeProxy() && !FMath::IsNearlyZero(LandscapeInfo->DrawScale.X))
			{
				const float ScaleXY = static_cast<float>(LandscapeInfo->DrawScale.X);
				SampleSizeX = static_cast<float>(Gizmo->SampleSizeX);
				SampleSizeY = static_cast<float>(Gizmo->SampleSizeY);
				bHeightmapRendering = (Gizmo->DataType & LGT_Height);
				FTransform LToW = LandscapeInfo->GetLandscapeProxy()->LandscapeActorToWorld();
				const float W = Gizmo->Width / 2;
				const float H = Gizmo->Height / 2;
				const float L = Gizmo->LengthZ;
				// The Gizmo's coordinate space is weird, it's partially relative to the landscape and partially relative to the world
				const FVector GizmoLocation = Gizmo->GetActorLocation();
				const FQuat   GizmoRotation = FRotator(0, Gizmo->GetActorRotation().Yaw, 0).Quaternion() * LToW.GetRotation();
				const FVector GizmoScale3D  = Gizmo->GetActorScale3D();
				const FTransform GizmoRT = FTransform(GizmoRotation, GizmoLocation, GizmoScale3D);

				FrustumVerts[0] = Gizmo->FrustumVerts[0] = GizmoRT.TransformPosition(FVector( - W, - H, + L ));
				FrustumVerts[1] = Gizmo->FrustumVerts[1] = GizmoRT.TransformPosition(FVector( + W, - H, + L ));
				FrustumVerts[2] = Gizmo->FrustumVerts[2] = GizmoRT.TransformPosition(FVector( + W, + H, + L ));
				FrustumVerts[3] = Gizmo->FrustumVerts[3] = GizmoRT.TransformPosition(FVector( - W, + H, + L ));

				FrustumVerts[4] = Gizmo->FrustumVerts[4] = GizmoRT.TransformPosition(FVector( - W, - H,   0 ));
				FrustumVerts[5] = Gizmo->FrustumVerts[5] = GizmoRT.TransformPosition(FVector( + W, - H,   0 ));
				FrustumVerts[6] = Gizmo->FrustumVerts[6] = GizmoRT.TransformPosition(FVector( + W, + H,   0 ));
				FrustumVerts[7] = Gizmo->FrustumVerts[7] = GizmoRT.TransformPosition(FVector( - W, + H,   0 ));

				XAxis  = GizmoRT.TransformPosition(FVector( + W,   0, + L ));
				YAxis  = GizmoRT.TransformPosition(FVector(   0, + H, + L ));
				Origin = GizmoRT.TransformPosition(FVector(   0,   0, + L ));

				const FMatrix WToL = LToW.ToMatrixWithScale().InverseFast();
				const FVector BaseLocation = WToL.TransformPosition(Gizmo->GetActorLocation());
				
				// Cache sampled height
				float ScaleX = Gizmo->GetWidth() / Gizmo->CachedWidth / ScaleXY * Gizmo->CachedScaleXY;
				float ScaleY = Gizmo->GetHeight() / Gizmo->CachedHeight / ScaleXY * Gizmo->CachedScaleXY;

				if (!FMath::IsNearlyZero(ScaleX) && !FMath::IsNearlyZero(ScaleY))
				{
					MeshRT = FTranslationMatrix(FVector(-W / ScaleXY + 0.5, -H / ScaleXY + 0.5, 0) * GizmoScale3D) * FRotationTranslationMatrix(FRotator(0, Gizmo->GetActorRotation().Yaw, 0), FVector(BaseLocation.X, BaseLocation.Y, 0)) * LToW.ToMatrixWithScale();

					HeightmapRenderProxy = new FLandscapeGizmoMeshRenderProxy(Gizmo->GizmoMeshMaterial->GetRenderProxy(),
					                                                          static_cast<float>(BaseLocation.Z + L),
					                                                          static_cast<float>(BaseLocation.Z),
					                                                          Gizmo->GizmoTexture,
					                                                          FLinearColor(static_cast<float>(Gizmo->TextureScale.X),
					                                                                       static_cast<float>(Gizmo->TextureScale.Y), 0.0f, 0.0f),
					                                                          WToL);
					GizmoRenderProxy = (Gizmo->DataType != LGT_None) ? Gizmo->GizmoDataMaterial->GetRenderProxy() : Gizmo->GizmoMaterial->GetRenderProxy();

					FScaleMatrix Mat(FVector(ScaleX, ScaleY, L));
					FMatrix NormalM = Mat.InverseFast().GetTransposed();

					int32 SamplingSize = Gizmo->SampleSizeX * Gizmo->SampleSizeY;
					SampledPositions.Empty(SamplingSize);
					SampledNormals.Empty(SamplingSize);

					for (int32 Y = 0; Y < Gizmo->SampleSizeY; ++Y)
					{
						for (int32 X = 0; X < Gizmo->SampleSizeX; ++X)
						{
							FVector SampledPos = Gizmo->SampledHeight[X + Y * ALandscapeGizmoActiveActor::DataTexSize];
							SampledPos.X *= ScaleX;
							SampledPos.Y *= ScaleY;
							SampledPos.Z = Gizmo->GetLandscapeHeight(static_cast<float>(SampledPos.Z));

							FVector SampledNormal = NormalM.TransformVector(Gizmo->SampledNormal[X + Y * ALandscapeGizmoActiveActor::DataTexSize]);
							SampledNormal = SampledNormal.GetSafeNormal();

							SampledPositions.Add(SampledPos);
							SampledNormals.Add(SampledNormal);
						}
					}
				}
			}
		}
#endif
	}

	~FLandscapeGizmoRenderSceneProxy()
	{
		delete HeightmapRenderProxy;
		HeightmapRenderProxy = nullptr;
	}

#if WITH_EDITOR
	virtual HHitProxy* CreateHitProxies(UPrimitiveComponent* Component, TArray<TRefCountPtr<HHitProxy> >& OutHitProxies) override
	{
		ALandscapeGizmoActiveActor* Gizmo = CastChecked<ALandscapeGizmoActiveActor>(Component->GetOwner());
		HitProxy = new HTranslucentActor(Gizmo, Component);
		OutHitProxies.Add(HitProxy);

		// by default we're not clickable, to allow the preview heightmap to be non-clickable (only the bounds frame)
		return nullptr;
	}
#endif

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
#if WITH_EDITOR
		if(GizmoRenderProxy && HeightmapRenderProxy)
		{
			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				if (VisibilityMap & (1 << ViewIndex))
				{
					const FSceneView* View = Views[ViewIndex];
					FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);

					// Axis
					PDI->DrawLine( Origin, XAxis, FLinearColor(1, 0, 0), SDPG_World );
					PDI->DrawLine( Origin, YAxis, FLinearColor(0, 1, 0), SDPG_World );

					{
						FDynamicMeshBuilder MeshBuilder(View->GetFeatureLevel());

						const FColor GizmoColor = FColor::White;
						MeshBuilder.AddVertex((FVector3f)FrustumVerts[0], FVector2f(0, 0), FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), GizmoColor);
						MeshBuilder.AddVertex((FVector3f)FrustumVerts[1], FVector2f(1, 0), FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), GizmoColor);
						MeshBuilder.AddVertex((FVector3f)FrustumVerts[2], FVector2f(1, 1), FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), GizmoColor);
						MeshBuilder.AddVertex((FVector3f)FrustumVerts[3], FVector2f(0, 1), FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), GizmoColor);

						MeshBuilder.AddVertex((FVector3f)FrustumVerts[4], FVector2f(0, 0), FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), GizmoColor);
						MeshBuilder.AddVertex((FVector3f)FrustumVerts[5], FVector2f(1, 0), FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), GizmoColor);
						MeshBuilder.AddVertex((FVector3f)FrustumVerts[6], FVector2f(1, 1), FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), GizmoColor);
						MeshBuilder.AddVertex((FVector3f)FrustumVerts[7], FVector2f(0, 1), FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), GizmoColor);

						MeshBuilder.AddVertex((FVector3f)FrustumVerts[1], FVector2f(0, 0), FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), GizmoColor);
						MeshBuilder.AddVertex((FVector3f)FrustumVerts[0], FVector2f(1, 0), FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), GizmoColor);
						MeshBuilder.AddVertex((FVector3f)FrustumVerts[4], FVector2f(1, 1), FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), GizmoColor);
						MeshBuilder.AddVertex((FVector3f)FrustumVerts[5], FVector2f(0, 1), FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), GizmoColor);

						MeshBuilder.AddVertex((FVector3f)FrustumVerts[3], FVector2f(0, 0), FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), GizmoColor);
						MeshBuilder.AddVertex((FVector3f)FrustumVerts[2], FVector2f(1, 0), FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), GizmoColor);
						MeshBuilder.AddVertex((FVector3f)FrustumVerts[6], FVector2f(1, 1), FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), GizmoColor);
						MeshBuilder.AddVertex((FVector3f)FrustumVerts[7], FVector2f(0, 1), FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), GizmoColor);

						MeshBuilder.AddVertex((FVector3f)FrustumVerts[2], FVector2f(0, 0), FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), GizmoColor);
						MeshBuilder.AddVertex((FVector3f)FrustumVerts[1], FVector2f(1, 0), FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), GizmoColor);
						MeshBuilder.AddVertex((FVector3f)FrustumVerts[5], FVector2f(1, 1), FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), GizmoColor);
						MeshBuilder.AddVertex((FVector3f)FrustumVerts[6], FVector2f(0, 1), FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), GizmoColor);

						MeshBuilder.AddVertex((FVector3f)FrustumVerts[0], FVector2f(0, 0), FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), GizmoColor);
						MeshBuilder.AddVertex((FVector3f)FrustumVerts[3], FVector2f(1, 0), FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), GizmoColor);
						MeshBuilder.AddVertex((FVector3f)FrustumVerts[7], FVector2f(1, 1), FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), GizmoColor);
						MeshBuilder.AddVertex((FVector3f)FrustumVerts[4], FVector2f(0, 1), FVector3f(1,0,0), FVector3f(0,1,0), FVector3f(0,0,1), GizmoColor);

						for (int32 i = 0; i < 6; ++i)
						{
							int32 Idx = i*4;
							MeshBuilder.AddTriangle( Idx, Idx+2, Idx+1 );
							MeshBuilder.AddTriangle( Idx, Idx+3, Idx+2 );
						}

						MeshBuilder.GetMesh(FMatrix::Identity, GizmoRenderProxy, SDPG_World, true, false, false, ViewIndex, Collector, HitProxy);
					}

					if (bHeightmapRendering)
					{
						auto GetSampleIndex = [this](int32 X, int32 Y) -> int32
						{
							return static_cast<int32>(X + Y * SampleSizeX);
						};

						FDynamicMeshBuilder MeshBuilder(View->GetFeatureLevel());

						for (int32 Y = 0; Y < SampleSizeY; ++Y)
						{
							for (int32 X = 0; X < SampleSizeX; ++X)
							{
								const int32 SampleIndex = GetSampleIndex(X, Y);
								FVector SampledNormal = SampledNormals[SampleIndex];
								FVector TangentX(SampledNormal.Z, 0, -SampledNormal.X);
								TangentX = TangentX.GetSafeNormal();

								MeshBuilder.AddVertex((FVector3f)SampledPositions[SampleIndex], FVector2f((float)X / (SampleSizeX), (float)Y / (SampleSizeY)), (FVector3f)TangentX, FVector3f(SampledNormal^TangentX), (FVector3f)SampledNormal, FColor::White);
							}
						}

						for (int32 Y = 0; Y < SampleSizeY; ++Y)
						{
							for (int32 X = 0; X < SampleSizeX; ++X)
							{
								if (X < SampleSizeX - 1 && Y < SampleSizeY - 1)
								{
									MeshBuilder.AddTriangle(GetSampleIndex(X + 0, Y + 0), GetSampleIndex(X + 1, Y + 1), GetSampleIndex(X + 1, Y + 0));
									MeshBuilder.AddTriangle(GetSampleIndex(X + 0, Y + 0), GetSampleIndex(X + 0, Y + 1), GetSampleIndex(X + 1, Y + 1));
								}
							}
						}

						MeshBuilder.GetMesh(MeshRT, HeightmapRenderProxy , SDPG_World, false, false, ViewIndex, Collector);
					}
				}
			}
		}
#endif
	};

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;
#if WITH_EDITOR
		const bool bVisible = View->Family->EngineShowFlags.Landscape;
		Result.bDrawRelevance = IsShown(View) && bVisible && !View->bIsGameView && GLandscapeEditRenderMode & ELandscapeEditRenderMode::Gizmo;
		Result.bDynamicRelevance = true;
		// ideally the TranslucencyRelevance should be filled out by the material, here we do it conservative
		Result.bSeparateTranslucency = Result.bNormalTranslucency = true;
#endif
		return Result;
	}

	virtual uint32 GetMemoryFootprint( void ) const override { return( sizeof( *this ) + GetAllocatedSize() ); }
	uint32 GetAllocatedSize( void ) const { return static_cast<uint32>(FPrimitiveSceneProxy::GetAllocatedSize()); }
};

ULandscapeGizmoRenderComponent::ULandscapeGizmoRenderComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bHiddenInGame = true;
	bIsEditorOnly = true;
	SetCollisionProfileName(UCollisionProfile::BlockAll_ProfileName);
}

FPrimitiveSceneProxy* ULandscapeGizmoRenderComponent::CreateSceneProxy()
{
	return new FLandscapeGizmoRenderSceneProxy(this);
}

void ULandscapeGizmoRenderComponent::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const 
{
#if WITH_EDITORONLY_DATA
	ALandscapeGizmoActiveActor* Gizmo = Cast<ALandscapeGizmoActiveActor>(GetOwner());
	if (Gizmo)
	{
		UMaterialInterface* GizmoMat = (Gizmo->DataType != LGT_None) ?
			(UMaterialInterface*)Gizmo->GizmoDataMaterial :
			(UMaterialInterface*)Gizmo->GizmoMaterial;

		if (GizmoMat)
		{
			OutMaterials.Add(GizmoMat);
		}
	}
#endif
}

FBoxSphereBounds ULandscapeGizmoRenderComponent::CalcBounds(const FTransform& LocalToWorld) const
{
#if WITH_EDITOR
	ALandscapeGizmoActiveActor* Gizmo = Cast<ALandscapeGizmoActiveActor>(GetOwner());
	if (Gizmo)
	{
		ULandscapeInfo* LandscapeInfo = Gizmo->TargetLandscapeInfo.Get();
		if (LandscapeInfo && LandscapeInfo->GetLandscapeProxy())
		{
			FTransform LToW = LandscapeInfo->GetLandscapeProxy()->LandscapeActorToWorld();

			// We calculate this ourselves, not from Gizmo->FrustrumVerts, as those haven't been updated yet
			// The Gizmo's coordinate space is weird, it's partially relative to the landscape and partially relative to the world
			const FVector GizmoLocation = Gizmo->GetActorLocation();
			const FQuat   GizmoRotation = FRotator(0, Gizmo->GetActorRotation().Yaw, 0).Quaternion() * LToW.GetRotation();
			const FVector GizmoScale3D = Gizmo->GetActorScale3D();
			const FTransform GizmoRT = FTransform(GizmoRotation, GizmoLocation, GizmoScale3D);
			const float W = Gizmo->Width / 2;
			const float H = Gizmo->Height / 2;
			const float L = Gizmo->LengthZ;
			return FBoxSphereBounds(FBox(FVector(-W, -H, 0), FVector(+W, +H, +L))).TransformBy(GizmoRT);
		}
	}
#endif

	return Super::CalcBounds(LocalToWorld);
}

ALandscapeGizmoActor::ALandscapeGizmoActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	SpriteComponent = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Sprite"));
	if (!IsRunningCommandlet() && (SpriteComponent != nullptr))
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinderOptional<UTexture2D> DecalActorIconTexture;
			FName ID_Misc;
			FText NAME_Misc;
			FConstructorStatics()
				: DecalActorIconTexture(TEXT("Texture2D'/Engine/EditorResources/S_DecalActorIcon.S_DecalActorIcon'"))
				, ID_Misc(TEXT("Misc"))
				, NAME_Misc(NSLOCTEXT("SpriteCategory", "Misc", "Misc"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		SpriteComponent->Sprite = ConstructorStatics.DecalActorIconTexture.Get();
		SpriteComponent->SetRelativeScale3D(FVector(0.5f, 0.5f, 0.5f));
		SpriteComponent->bHiddenInGame = true;
		SpriteComponent->SpriteInfo.Category = ConstructorStatics.ID_Misc;
		SpriteComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Misc;
		SpriteComponent->bIsScreenSizeScaled = true;
	}
#endif

	USceneComponent* SceneComponent = CreateDefaultSubobject<USceneComponent>(TEXT("RootComponent0"));
	RootComponent = SceneComponent;

#if WITH_EDITORONLY_DATA
	bListedInSceneOutliner = false;
	bEditable = false;
	Width = 1280.0f;
	Height = 1280.0f;
	LengthZ = 1280.0f;
	MarginZ = 512.0f;
	MinRelativeZ = 0.0f;
	RelativeScaleZ = 1.0f;
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR

void ALandscapeGizmoActor::Duplicate(ALandscapeGizmoActor* Gizmo)
{
	Gizmo->Width = Width;
	Gizmo->Height = Height;
	Gizmo->LengthZ = LengthZ;
	Gizmo->MarginZ = MarginZ;
	//Gizmo->TargetLandscapeInfo = TargetLandscapeInfo;

	Gizmo->SetActorLocation( GetActorLocation(), false );
	Gizmo->SetActorRotation( GetActorRotation() );

	if (Gizmo->GetRootComponent() != nullptr && GetRootComponent() != nullptr)
	{
		Gizmo->GetRootComponent()->SetRelativeScale3D(GetRootComponent()->GetRelativeScale3D());
	}

	Gizmo->MinRelativeZ = MinRelativeZ;
	Gizmo->RelativeScaleZ = RelativeScaleZ;

	Gizmo->ReregisterAllComponents();
}
#endif	//WITH_EDITOR

ALandscapeGizmoActiveActor::ALandscapeGizmoActiveActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer
		.DoNotCreateDefaultSubobject(TEXT("Sprite"))
	)
{
#if WITH_EDITORONLY_DATA
	if (!IsRunningCommandlet())
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinder<UMaterial> LandscapeGizmo_Mat;
			ConstructorHelpers::FObjectFinder<UMaterialInstanceConstant> LandscapeGizmo_Mat_Copied;
			ConstructorHelpers::FObjectFinder<UMaterial> LandscapeGizmoHeight_Mat;
			FConstructorStatics()
				: LandscapeGizmo_Mat(TEXT("/Engine/EditorLandscapeResources/LandscapeGizmo_Mat"))
				, LandscapeGizmo_Mat_Copied(TEXT("/Engine/EditorLandscapeResources/LandscapeGizmo_Mat_Copied"))
				, LandscapeGizmoHeight_Mat(TEXT("/Engine/EditorLandscapeResources/LandscapeGizmoHeight_Mat"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		GizmoMaterial = ConstructorStatics.LandscapeGizmo_Mat.Object;
		GizmoDataMaterial = ConstructorStatics.LandscapeGizmo_Mat_Copied.Object;
		GizmoMeshMaterial = ConstructorStatics.LandscapeGizmoHeight_Mat.Object;
	}
#endif // WITH_EDITORONLY_DATA

	ULandscapeGizmoRenderComponent* LandscapeGizmoRenderComponent = CreateDefaultSubobject<ULandscapeGizmoRenderComponent>(TEXT("GizmoRendererComponent0"));
	LandscapeGizmoRenderComponent->SetCollisionProfileName(UCollisionProfile::BlockAllDynamic_ProfileName);

	RootComponent = LandscapeGizmoRenderComponent;
#if WITH_EDITORONLY_DATA
	bEditable = true;
	Width = 1280.0f;
	Height = 1280.0f;
	LengthZ = 1280.0f;
	MarginZ = 512.0f;
	DataType = LGT_None;
	SampleSizeX = 0;
	SampleSizeY = 0;
	CachedWidth = 1.0f;
	CachedHeight = 1.0f;
	CachedScaleXY = 1.0f;
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITOR
void ALandscapeGizmoActiveActor::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	if( PropertyName == FName("LengthZ") )
	{
		if (LengthZ < 0)
		{
			LengthZ = MarginZ;
		}
	}
	else if ( PropertyName == FName("TargetLandscapeInfo") )
	{
		SetTargetLandscape(TargetLandscapeInfo.Get());
	}
	// AActor::PostEditChange will ForceUpdateComponents()
	Super::PostEditChangeProperty(PropertyChangedEvent);
}


void ALandscapeGizmoActiveActor::PostEditMove(bool bFinished)
{
	Super::PostEditMove( bFinished );

	if (bFinished)
	{
		UnsnappedRotation = FRotator::ZeroRotator;
	}
}

FVector ALandscapeGizmoActiveActor::SnapToLandscapeGrid(const FVector& GizmoLocation) const
{
	FVector ResultLocation = GizmoLocation;
	if (TargetLandscapeInfo != nullptr)
	{
		const ALandscapeProxy* LandscapeProxy = TargetLandscapeInfo->GetLandscapeProxy();

		const FTransform LToW = LandscapeProxy->LandscapeActorToWorld();
		const FVector LandscapeSpaceLocation = LToW.InverseTransformPosition(GizmoLocation);
		float SnapDimension;
		switch (SnapType)
		{
		case ELandscapeGizmoSnapType::Component:
			SnapDimension = static_cast<float>(LandscapeProxy->ComponentSizeQuads);
			break;
		case ELandscapeGizmoSnapType::Texel:
			SnapDimension = 1.0f;
			break;
		case ELandscapeGizmoSnapType::None:
			SnapDimension = 0.0f;
			break;
		default:
			SnapDimension = 0.0f;
			break;
		}

		if (SnapDimension > 0.0f)
		{
			const FVector SnappedLandscapeSpaceLocation = LandscapeSpaceLocation.GridSnap(SnapDimension);
			ResultLocation = LToW.TransformPosition(SnappedLandscapeSpaceLocation);
		}
		else
		{
			ResultLocation = GizmoLocation;
		}

		if (bFollowTerrainHeight)
		{
			ResultLocation.Z = LandscapeProxy->GetHeightAtLocation(ResultLocation, EHeightfieldSource::Editor).Get(static_cast<float>(GizmoLocation.Z));
		}
	}
	
	return ResultLocation;
}

void ALandscapeGizmoActiveActor::EditorApplyTranslation(const FVector& DeltaTranslation, bool bAltDown, bool bShiftDown, bool bCtrlDown)
{
	const FVector GizmoLocation = GetActorLocation() + DeltaTranslation;
	const FVector ResultLocation = SnapToLandscapeGrid(GizmoLocation);

	SetActorLocation(ResultLocation, false);

	ReregisterAllComponents();
}

FRotator ALandscapeGizmoActiveActor::SnapToLandscapeGrid(const FRotator& GizmoRotation) const
{
	// Snap to multiples of 90 Yaw in landscape coordinate system
	//check(TargetLandscapeInfo && TargetLandscapeInfo->LandscapeProxy);
	//const FTransform LToW = TargetLandscapeInfo->LandscapeProxy->ActorToWorld();
	//const FRotator LandscapeSpaceRotation = (LToW.GetRotation().InverseFast() * GizmoRotation.Quaternion()).Rotator().GetNormalized();
	//const FRotator SnappedLandscapeSpaceRotation = FRotator(0, FMath::GridSnap(LandscapeSpaceRotation.Yaw, 90), 0);
	//const FRotator ResultRotation = (SnappedLandscapeSpaceRotation.Quaternion() * LToW.GetRotation()).Rotator().GetNormalized();

	// Gizmo rotation is used as if it was relative to the landscape even though it isn't, so snap in world space
	const FRotator ResultRotation = FRotator(0.f, FMath::GridSnap(GizmoRotation.Yaw, (FRotator::FReal)90.f), 0.f);
	return ResultRotation;
}

void ALandscapeGizmoActiveActor::EditorApplyRotation(const FRotator& DeltaRotation, bool bAltDown, bool bShiftDown, bool bCtrlDown)
{
	if (SnapType == ELandscapeGizmoSnapType::Texel || SnapType == ELandscapeGizmoSnapType::Component)
	{
		// Based on AActor::EditorApplyRotation
		FRotator GizmoRotation = GetActorRotation() + UnsnappedRotation;
		FRotator Winding, Remainder;
		GizmoRotation.GetWindingAndRemainder(Winding, Remainder);
		const FQuat ActorQ = Remainder.Quaternion();
		const FQuat DeltaQ = DeltaRotation.Quaternion();
		const FQuat ResultQ = DeltaQ * ActorQ;
		const FRotator NewActorRotRem = FRotator( ResultQ );
		FRotator DeltaRot = NewActorRotRem - Remainder;
		DeltaRot.Normalize();

		GizmoRotation += DeltaRot;

		const FRotator ResultRotation = SnapToLandscapeGrid(GizmoRotation);

		UnsnappedRotation = GizmoRotation - ResultRotation;
		UnsnappedRotation.Pitch = 0;
		UnsnappedRotation.Roll = 0;
		UnsnappedRotation.Normalize();

		SetActorRotation(ResultRotation);
	}
	else
	{
		Super::EditorApplyRotation(DeltaRotation, bAltDown, bShiftDown, bCtrlDown);
	}

	ReregisterAllComponents();
}

void ALandscapeGizmoActiveActor::EditorApplyScale(const FVector& DeltaScale, const FVector* PivotLocation, bool bAltDown, bool bShiftDown, bool bCtrlDown)
{
	Super::EditorApplyScale(DeltaScale, PivotLocation, bAltDown, bShiftDown, bCtrlDown);
	ReregisterAllComponents();
} 

ALandscapeGizmoActor* ALandscapeGizmoActiveActor::SpawnGizmoActor()
{
	// ALandscapeGizmoActor is history for ALandscapeGizmoActiveActor
	FActorSpawnParameters SpawnParams;
	SpawnParams.ObjectFlags |= RF_Transient;
	ALandscapeGizmoActor* NewActor = GetWorld()->SpawnActor<ALandscapeGizmoActor>(SpawnParams);
	Duplicate(NewActor);
	return NewActor;
}

void ALandscapeGizmoActiveActor::SetTargetLandscape(ULandscapeInfo* LandscapeInfo)
{
	ULandscapeInfo* PrevInfo = TargetLandscapeInfo.Get();
	if (!LandscapeInfo || LandscapeInfo->HasAnyFlags(RF_BeginDestroyed))
	{
		TargetLandscapeInfo = nullptr;
		if (const UWorld* CurrentWorld = GetWorld())
		{
			for (const TPair<FGuid, TObjectPtr<ULandscapeInfo>>& InfoMapPair : ULandscapeInfoMap::GetLandscapeInfoMap(CurrentWorld).Map)
			{
				ULandscapeInfo* CandidateInfo = InfoMapPair.Value;
				if (CandidateInfo && CandidateInfo->SupportsLandscapeEditing() && !CandidateInfo->HasAnyFlags(RF_BeginDestroyed) && CandidateInfo->GetLandscapeProxy() != nullptr)
				{
					TargetLandscapeInfo = CandidateInfo;
					break;
				}
			}
		}
	}
	else
	{
		TargetLandscapeInfo = LandscapeInfo;
	}

	if (TargetLandscapeInfo.Get() != PrevInfo)
	{
		// if there's no copied data, try to move somewhere useful
		if (TargetLandscapeInfo.IsValid() && DataType == LGT_None)
		{
			MarginZ = static_cast<float>(TargetLandscapeInfo->DrawScale.Z * 3);
			Width = Height = static_cast<float>(TargetLandscapeInfo->DrawScale.X * (TargetLandscapeInfo->ComponentSizeQuads + 1));

			float NewLengthZ;
			FVector NewLocation = TargetLandscapeInfo->GetLandscapeCenterPos(NewLengthZ);
			SetLength(NewLengthZ);
			SetActorLocation(NewLocation, false);
			SetActorRotation(FRotator::ZeroRotator);
		}

		ReregisterAllComponents();
	}
}

void ALandscapeGizmoActiveActor::ClearGizmoData()
{
	DataType = LGT_None;
	SelectedData.Empty();
	LayerInfos.Empty();

	// If the clipboard contains copied gizmo data, clear it also
	FString ClipboardString;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardString);
	const TCHAR* Str = *ClipboardString;
	if (FParse::Command(&Str, TEXT("GizmoData=")))
	{
		FPlatformApplicationMisc::ClipboardCopy(TEXT(""));
	}

	ReregisterAllComponents();
}

void ALandscapeGizmoActiveActor::FitToSelection()
{
	if (TargetLandscapeInfo.IsValid())
	{
		// Find fit size
		int32 MinX = MAX_int32, MinY = MAX_int32;
		int32 MaxX = MIN_int32, MaxY = MIN_int32;
		TargetLandscapeInfo->GetSelectedExtent(MinX, MinY, MaxX, MaxY);
		if (MinX != MAX_int32)
		{
			const FVector LocalScale3D = GetRootComponent()->GetRelativeScale3D();

			const float ScaleXY = static_cast<float>(TargetLandscapeInfo->DrawScale.X);
			Width = static_cast<float>(ScaleXY * (MaxX - MinX + 1) / LocalScale3D.X);
			Height = static_cast<float>(ScaleXY * (MaxY - MinY + 1) / LocalScale3D.Y);
			float NewLengthZ;
			FVector NewLocation = TargetLandscapeInfo->GetLandscapeCenterPos(NewLengthZ, MinX, MinY, MaxX, MaxY);
			SetLength(NewLengthZ);
			SetActorLocation(NewLocation, false);
			SetActorRotation(FRotator::ZeroRotator);
			// Reset Z render scale values...
			MinRelativeZ = 0.f;
			RelativeScaleZ = 1.f;
			ReregisterAllComponents();
		}
	}
}

void ALandscapeGizmoActiveActor::FitMinMaxHeight()
{
	if (TargetLandscapeInfo.IsValid())
	{
		FVector::FReal MinZ = UE_OLD_HALF_WORLD_MAX, MaxZ = -UE_OLD_HALF_WORLD_MAX;
		// Change MinRelativeZ and RelativeZScale to fit Gizmo Box
		for (auto It = SelectedData.CreateConstIterator(); It; ++It )
		{
			const FGizmoSelectData& Data = It.Value();
			MinZ = FMath::Min(MinZ, Data.HeightData);
			MaxZ = FMath::Max(MaxZ, Data.HeightData);
		}

		if (MinZ != UE_OLD_HALF_WORLD_MAX && MaxZ > MinZ + KINDA_SMALL_NUMBER)
		{
			MinRelativeZ = static_cast<float>(MinZ);
			RelativeScaleZ = static_cast<float>(1.f / (MaxZ - MinZ));
			ReregisterAllComponents();
		}
	}
}

float ALandscapeGizmoActiveActor::GetNormalizedHeight(uint16 LandscapeHeight) const
{
	if (TargetLandscapeInfo.IsValid())
	{
		ALandscapeProxy* Proxy = TargetLandscapeInfo->GetLandscapeProxy();
		if (Proxy)
		{
			// Need to make it scale...?
			float ZScale = GetLength();
			if (ZScale > KINDA_SMALL_NUMBER)
			{
				FVector LocalGizmoPos = Proxy->LandscapeActorToWorld().InverseTransformPosition(GetActorLocation());
				return FMath::Clamp<float>(
					static_cast<float>((LandscapeDataAccess::GetLocalHeight(LandscapeHeight) - LocalGizmoPos.Z) * TargetLandscapeInfo->DrawScale.Z / ZScale),
					0.f, 1.f);
			}
		}
	}
	return 0.f;
}

float ALandscapeGizmoActiveActor::GetWorldHeight(float NormalizedHeight) const
{
	if (TargetLandscapeInfo.IsValid())
	{
		ALandscapeProxy* Proxy = TargetLandscapeInfo->GetLandscapeProxy();
		if (Proxy)
		{
			float ZScale = GetLength();
			if (ZScale > KINDA_SMALL_NUMBER)
			{
				FVector LocalGizmoPos = Proxy->LandscapeActorToWorld().InverseTransformPosition(GetActorLocation());
				return static_cast<float>(NormalizedHeight * ZScale + LocalGizmoPos.Z * TargetLandscapeInfo->DrawScale.Z);
			}
		}
	}
	return 0.f;
}

float ALandscapeGizmoActiveActor::GetLandscapeHeight(float NormalizedHeight) const
{
	if (TargetLandscapeInfo.IsValid())
	{
		NormalizedHeight = (NormalizedHeight - MinRelativeZ) * RelativeScaleZ;
		const float ScaleZ = static_cast<float>(TargetLandscapeInfo->DrawScale.Z);
		return (GetWorldHeight(NormalizedHeight) / ScaleZ);
	}
	return 0.f;
}

void ALandscapeGizmoActiveActor::CalcNormal()
{
	int32 SquaredDataTex = DataTexSize * DataTexSize;
	if (SampledHeight.Num() == SquaredDataTex && SampleSizeX > 0 && SampleSizeY > 0 )
	{
		if (SampledNormal.Num() != SquaredDataTex)
		{
			SampledNormal.Empty(SquaredDataTex);
			SampledNormal.AddZeroed(SquaredDataTex);
		}
		for (int32 Y = 0; Y < SampleSizeY-1; ++Y)
		{
			for (int32 X = 0; X < SampleSizeX-1; ++X)
			{
				FVector Vert00 = SampledHeight[X + Y*DataTexSize];
				FVector Vert01 = SampledHeight[X + (Y+1)*DataTexSize];
				FVector Vert10 = SampledHeight[X+1 + Y*DataTexSize];
				FVector Vert11 = SampledHeight[X+1 + (Y+1)*DataTexSize];

				FVector FaceNormal1 = ((Vert00-Vert10) ^ (Vert10-Vert11)).GetSafeNormal();
				FVector FaceNormal2 = ((Vert11-Vert01) ^ (Vert01-Vert00)).GetSafeNormal(); 

				// contribute to the vertex normals.
				SampledNormal[X + Y*DataTexSize] += FaceNormal1;
				SampledNormal[X + (Y+1)*DataTexSize] += FaceNormal2;
				SampledNormal[X+1 + Y*DataTexSize] += FaceNormal1 + FaceNormal2;
				SampledNormal[X+1 + (Y+1)*DataTexSize] += FaceNormal1 + FaceNormal2;
			}
		}
		for (int32 Y = 0; Y < SampleSizeY; ++Y)
		{
			for (int32 X = 0; X < SampleSizeX; ++X)
			{
				SampledNormal[X + Y*DataTexSize] = SampledNormal[X + Y*DataTexSize].GetSafeNormal();
			}
		}
	}
}

void ALandscapeGizmoActiveActor::SampleData(int32 SizeX, int32 SizeY)
{
	if (TargetLandscapeInfo.IsValid() && GizmoTexture)
	{
		// Rasterize rendering Texture...
		int32 TexSizeX = FMath::Min(ALandscapeGizmoActiveActor::DataTexSize, SizeX);
		int32 TexSizeY = FMath::Min(ALandscapeGizmoActiveActor::DataTexSize, SizeY);
		SampleSizeX = TexSizeX;
		SampleSizeY = TexSizeY;

		// Update Data Texture...
		//DataTexture->SetFlags(RF_Transactional);
		//DataTexture->Modify();

		TextureScale = FVector2D( (float)SizeX / FMath::Max(ALandscapeGizmoActiveActor::DataTexSize, SizeX), (float)SizeY / FMath::Max(ALandscapeGizmoActiveActor::DataTexSize, SizeY));
		uint8* TexData = GizmoTexture->Source.LockMip(0);
		int32 GizmoTexSizeX = GizmoTexture->Source.GetSizeX();
		for (int32 Y = 0; Y < TexSizeY; ++Y)
		{
			for (int32 X = 0; X < TexSizeX; ++X)
			{
				float TexX = static_cast<float>(X) * SizeX / TexSizeX;
				float TexY = static_cast<float>(Y) * SizeY / TexSizeY;
				int32 LX = FMath::FloorToInt(TexX);
				int32 LY = FMath::FloorToInt(TexY);

				float FracX = TexX - LX;
				float FracY = TexY - LY;

				FGizmoSelectData* Data00 = SelectedData.Find(FIntPoint(LX, LY));
				FGizmoSelectData* Data10 = SelectedData.Find(FIntPoint(LX+1, LY));
				FGizmoSelectData* Data01 = SelectedData.Find(FIntPoint(LX, LY+1));
				FGizmoSelectData* Data11 = SelectedData.Find(FIntPoint(LX+1, LY+1));

				// Invert Tex Data to show selected region more visible
				TexData[X + Y*GizmoTexSizeX] = static_cast<uint8>(255 - FMath::Lerp(
					FMath::Lerp(Data00 ? Data00->Ratio : 0, Data10 ? Data10->Ratio : 0, FracX),
					FMath::Lerp(Data01 ? Data01->Ratio : 0, Data11 ? Data11->Ratio : 0, FracX),
					FracY
					) * 255);

				if (DataType & LGT_Height)
				{
					float NormalizedHeight = FMath::Lerp(
						FMath::Lerp(Data00 ? Data00->HeightData : 0, Data10 ? Data10->HeightData : 0, FracX),
						FMath::Lerp(Data01 ? Data01->HeightData : 0, Data11 ? Data11->HeightData : 0, FracX),
						FracY
						);

					SampledHeight[X + Y*GizmoTexSizeX] = FVector(LX, LY, NormalizedHeight);
				}
			}
		}

		if (DataType & LGT_Height)
		{
			CalcNormal();
		}

		GizmoTexture->TemporarilyDisableStreaming();
		FUpdateTextureRegion2D Region(0, 0, 0, 0, TexSizeX, TexSizeY);
		GizmoTexture->UpdateTextureRegions(0, 1, &Region, GizmoTexSizeX, sizeof(uint8), TexData);
		FlushRenderingCommands();
		GizmoTexture->Source.UnlockMip(0);

		ReregisterAllComponents();
	}
}

LANDSCAPE_API void ALandscapeGizmoActiveActor::Import( int32 VertsX, int32 VertsY, uint16* HeightData, TArray<ULandscapeLayerInfoObject*> ImportLayerInfos, uint8* LayerDataPointers[] )
{
	if (VertsX <= 0 || VertsY <= 0 || HeightData == nullptr || TargetLandscapeInfo == nullptr || GizmoTexture == nullptr || (ImportLayerInfos.Num() && !LayerDataPointers) )
	{
		return;
	}

	GWarn->BeginSlowTask( NSLOCTEXT("Landscape", "BeginImportingGizmoDataTask", "Importing Gizmo Data"), true);

	ClearGizmoData();

	CachedScaleXY = static_cast<float>(TargetLandscapeInfo->DrawScale.X);
	CachedWidth = CachedScaleXY * VertsX; // (DrawScale * DrawScale3D.X);
	CachedHeight = CachedScaleXY * VertsY; // (DrawScale * DrawScale3D.Y);
	
	float CurrentWidth = GetWidth();
	float CurrentHeight = GetHeight();
	LengthZ = GetLength();

	FVector Scale3D = FVector(CurrentWidth / CachedWidth, CurrentHeight / CachedHeight, 1.f);
	GetRootComponent()->SetRelativeScale3D(Scale3D);

	Width = CachedWidth;
	Height = CachedHeight;

	DataType = ELandscapeGizmoType(DataType | LGT_Height);
	if (ImportLayerInfos.Num())
	{
		DataType = ELandscapeGizmoType(DataType | LGT_Weight);
	}

	for (int32 Y = 0; Y < VertsY; ++Y)
	{
		for (int32 X = 0; X < VertsX; ++X)
		{
			FGizmoSelectData Data;
			Data.Ratio = 1.f;
			Data.HeightData = (float)HeightData[X + Y*VertsX] / LandscapeDataAccess::MaxValue; //GetNormalizedHeight(HeightData[X + Y*VertsX]);
			for (int32 i = 0; i < ImportLayerInfos.Num(); ++i)
			{
				Data.WeightDataMap.Add( ImportLayerInfos[i], LayerDataPointers[i][X + Y*VertsX] );
			}
			SelectedData.Add(FIntPoint(X, Y), Data);
		}
	}

	SampleData(VertsX, VertsY);

	for (auto It = ImportLayerInfos.CreateConstIterator(); It; ++It)
	{
		LayerInfos.Add(*It);
	}

	GWarn->EndSlowTask();

	ReregisterAllComponents();
}

void ALandscapeGizmoActiveActor::Export(int32 Index, TArray<FString>& Filenames)
{
	//guard around case where landscape has no layer structs
	if (Filenames.Num() == 0)
	{
		return;
	}

	bool bExportOneTarget = (Filenames.Num() == 1);

	if (TargetLandscapeInfo.IsValid())
	{
		int32 MinX = MAX_int32, MinY = MAX_int32;
		int32 MaxX = MIN_int32, MaxY = MIN_int32;
		for (const TPair<FIntPoint, FGizmoSelectData>& SelectedDataPair : SelectedData)
		{
			const FIntPoint Key = SelectedDataPair.Key;
			if (MinX > Key.X) MinX = Key.X;
			if (MaxX < Key.X) MaxX = Key.X;
			if (MinY > Key.Y) MinY = Key.Y;
			if (MaxY < Key.Y) MaxY = Key.Y;
		}

		if (MinX != MAX_int32)
		{
			GWarn->BeginSlowTask( NSLOCTEXT("Landscape", "BeginExportingGizmoDataTask", "Exporting Gizmo Data"), true);

			TArray<uint8> HeightData;
			if (!bExportOneTarget || Index == -1)
			{
				HeightData.AddZeroed((1+MaxX-MinX)*(1+MaxY-MinY)*sizeof(uint16));
			}
			uint16* pHeightData = (uint16*)HeightData.GetData();

			TArray<TArray<uint8> > WeightDatas;
			for( int32 i=1;i<Filenames.Num();i++ )
			{
				TArray<uint8> WeightData;
				if (!bExportOneTarget || Index == i-1)
				{
					WeightData.AddZeroed((1+MaxX-MinX)*(1+MaxY-MinY));
				}
				WeightDatas.Add(WeightData);
			}

			for (int32 Y = MinY; Y <= MaxY; ++Y)
			{
				for (int32 X = MinX; X <= MaxX; ++X)
				{
					const FGizmoSelectData* Data = SelectedData.Find(FIntPoint(X, Y));
					if (Data)
					{
						int32 Idx = (X-MinX) + Y *(1+MaxX-MinX);
						if (!bExportOneTarget || Index == -1)
						{
							pHeightData[Idx] = FMath::Clamp<uint16>(static_cast<uint16>(Data->HeightData * LandscapeDataAccess::MaxValue), 0, LandscapeDataAccess::MaxValue);
						}

						for( int32 i=1;i<Filenames.Num();i++ )
						{
							if (!bExportOneTarget || Index == i-1)
							{
								TArray<uint8>& WeightData = WeightDatas[i-1];
								WeightData[Idx] = FMath::Clamp<uint8>(static_cast<uint8>(Data->WeightDataMap.FindRef(LayerInfos[i-1])), 0, 255);
							}
						}
					}
				}
			}

			if (!bExportOneTarget || Index == -1)
			{
				FFileHelper::SaveArrayToFile(HeightData,*Filenames[0]);
			}

			for( int32 i=1;i<Filenames.Num();i++ )
			{
				if (!bExportOneTarget || Index == i-1)
				{
					FFileHelper::SaveArrayToFile(WeightDatas[i-1],*Filenames[bExportOneTarget ? 0 : i]);
				}
			}

			GWarn->EndSlowTask();
		}
		else
		{
			FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "LandscapeGizmoExport_Warning", "Landscape Gizmo has no copyed data. You need to choose proper targets and copy it to Gizmo."));
		}
	}
}

void ALandscapeGizmoActiveActor::ExportToClipboard()
{
	if (TargetLandscapeInfo.IsValid() && DataType != LGT_None)
	{
		//GWarn->BeginSlowTask( TEXT("Exporting Gizmo Data From Clipboard"), true);

		FString ClipboardString("GizmoData=");

		ClipboardString += FString::Printf(TEXT(" Type=%d,TextureScaleX=%g,TextureScaleY=%g,SampleSizeX=%d,SampleSizeY=%d,CachedWidth=%g,CachedHeight=%g,CachedScaleXY=%g "), 
			(int32)DataType, TextureScale.X, TextureScale.Y, SampleSizeX, SampleSizeY, CachedWidth, CachedHeight, CachedScaleXY);

		for (int32 Y = 0; Y < SampleSizeY; ++Y )
		{
			for (int32 X = 0; X < SampleSizeX; ++X)
			{
				FVector& V = SampledHeight[X + Y * DataTexSize];
				ClipboardString += FString::Printf(TEXT("%d %d %d "), (int32)V.X, (int32)V.Y, *(int32*)(&V.Z) );
			}
		}

		ClipboardString += FString::Printf(TEXT("LayerInfos= "));

		for (ULandscapeLayerInfoObject* LayerInfo : LayerInfos)
		{
			ClipboardString += FString::Printf(TEXT("%s "), *LayerInfo->GetPathName() );
		}

		ClipboardString += FString::Printf(TEXT("Region= "));

		for (const TPair<FIntPoint, FGizmoSelectData>& SelectedDataPair : SelectedData)
		{
			const FIntPoint Key = SelectedDataPair.Key;
			const FGizmoSelectData& Data = SelectedDataPair.Value;
			ClipboardString += FString::Printf(TEXT("%d %d %d %d %d "), Key.X, Key.Y, *(int32*)(&Data.Ratio), *(int32*)(&Data.HeightData), Data.WeightDataMap.Num());

			for (const TPair<ULandscapeLayerInfoObject*, float>& WeightDataPair : Data.WeightDataMap)
			{
				ClipboardString += FString::Printf(TEXT("%d %d "), LayerInfos.Find(WeightDataPair.Key), *(int32*)(&WeightDataPair.Value));
			}
		}

		FPlatformApplicationMisc::ClipboardCopy(*ClipboardString);

		//GWarn->EndSlowTask();
	}
}

#define MAX_GIZMO_PROP_TEXT_LENGTH			1024*1024*8

void ALandscapeGizmoActiveActor::ImportFromClipboard()
{
	FString ClipboardString;
	FPlatformApplicationMisc::ClipboardPaste(ClipboardString);
	const TCHAR* Str = *ClipboardString;
	
	if(FParse::Command(&Str,TEXT("GizmoData=")))
	{
		int32 ClipBoardSize = ClipboardString.Len();
		if (ClipBoardSize > MAX_GIZMO_PROP_TEXT_LENGTH)
		{
			if( EAppReturnType::Yes != FMessageDialog::Open( EAppMsgType::YesNo,
				FText::Format(NSLOCTEXT("UnrealEd", "LandscapeGizmoImport_Warning", "Landscape Gizmo is about to import large amount data ({0}MB) from the clipboard, which will take some time. Do you want to proceed?"),
				FText::AsNumber(ClipBoardSize >> 20) ) ) )
			{
				return;
			}
		}

		GWarn->BeginSlowTask( NSLOCTEXT("Landscape", "BeginImportingGizmoDataFromClipboardTask", "Importing Gizmo Data From Clipboard"), true);

		FParse::Next(&Str);


		int32 ReadNum = 0;

		uint8 Type = 0;
		ReadNum += FParse::Value(Str, TEXT("Type="), Type) ? 1 : 0;
		DataType = (ELandscapeGizmoType)Type;

		ReadNum += FParse::Value(Str, TEXT("TextureScaleX="), TextureScale.X) ? 1 : 0;
		ReadNum += FParse::Value(Str, TEXT("TextureScaleY="), TextureScale.Y) ? 1 : 0;
		ReadNum += FParse::Value(Str, TEXT("SampleSizeX="), SampleSizeX) ? 1 : 0;
		ReadNum += FParse::Value(Str, TEXT("SampleSizeY="), SampleSizeY) ? 1 : 0;
		ReadNum += FParse::Value(Str, TEXT("CachedWidth="), CachedWidth) ? 1 : 0;
		ReadNum += FParse::Value(Str, TEXT("CachedHeight="), CachedHeight) ? 1 : 0;
		ReadNum += FParse::Value(Str, TEXT("CachedScaleXY="), CachedScaleXY) ? 1 : 0;

		if (ReadNum > 0)
		{
			while (!FChar::IsWhitespace(*Str))
			{
				Str++;
			}
			FParse::Next(&Str);

			int32 SquaredDataTex = DataTexSize * DataTexSize;
			if (SampledHeight.Num() != SquaredDataTex)
			{
				SampledHeight.Empty(SquaredDataTex);
				SampledHeight.AddZeroed(SquaredDataTex);
			}

			// For Sample Height...
			TCHAR* StopStr;
			for (int32 Y = 0; Y < SampleSizeY; ++Y )
			{
				for (int32 X = 0; X < SampleSizeX; ++X)
				{
					FVector& V = SampledHeight[X + Y * DataTexSize];
					V.X = FCString::Strtoi(Str, &StopStr, 10);
					while (!FChar::IsWhitespace(*Str))
					{
						Str++;
					}
					FParse::Next(&Str);
					V.Y = FCString::Strtoi(Str, &StopStr, 10);
					while (!FChar::IsWhitespace(*Str))
					{
						Str++;
					}
					FParse::Next(&Str);
					//V.Z = FCString::Atof(Str);
					*((int32*)(&V.Z)) = FCString::Strtoi(Str, &StopStr, 10);
					while (!FChar::IsWhitespace(*Str))
					{
						Str++;
					}
					FParse::Next(&Str);
				}
			}

			CalcNormal();

			TCHAR StrBuf[1024];
			if(FParse::Command(&Str,TEXT("LayerInfos=")))
			{
				while( !FParse::Command(&Str,TEXT("Region=")) )
				{
					FParse::Next(&Str);
					int32 i = 0;
					while (!FChar::IsWhitespace(*Str))
					{
						StrBuf[i++] = *Str;
						Str++;
					}
					StrBuf[i] = 0;
					LayerInfos.Add( LoadObject<ULandscapeLayerInfoObject>(nullptr, StrBuf) );
				}
			}

			//if(FParse::Command(&Str,TEXT("Region=")))
			{
				while (*Str)
				{
					FParse::Next(&Str);
					int32 X, Y, LayerNum;
					FGizmoSelectData Data;
					X = FCString::Strtoi(Str, &StopStr, 10);
					while (!FChar::IsWhitespace(*Str))
					{
						Str++;
					}
					FParse::Next(&Str);
					Y = FCString::Strtoi(Str, &StopStr, 10);
					while (!FChar::IsWhitespace(*Str))
					{
						Str++;
					}
					FParse::Next(&Str);
					*((int32*)(&Data.Ratio)) = FCString::Strtoi(Str, &StopStr, 10);
					while (!FChar::IsWhitespace(*Str))
					{
						Str++;
					}
					FParse::Next(&Str);
					*((int32*)(&Data.HeightData)) = FCString::Strtoi(Str, &StopStr, 10);
					while (!FChar::IsWhitespace(*Str))
					{
						Str++;
					}
					FParse::Next(&Str);
					LayerNum = FCString::Strtoi(Str, &StopStr, 10);
					while (!FChar::IsWhitespace(*Str))
					{
						Str++;
					}
					FParse::Next(&Str);
					for (int32 i = 0; i < LayerNum; ++i)
					{
						int32 LayerIndex = FCString::Strtoi(Str, &StopStr, 10);
						while (!FChar::IsWhitespace(*Str))
						{
							Str++;
						}
						FParse::Next(&Str);
						float Weight;
						*((int32*)(&Weight)) = FCString::Strtoi(Str, &StopStr, 10);
						while (!FChar::IsWhitespace(*Str))
						{
							Str++;
						}
						FParse::Next(&Str);
						Data.WeightDataMap.Add(LayerInfos[LayerIndex], Weight);
					}
					SelectedData.Add(FIntPoint(X, Y), Data);
				}
			}
		}

		GWarn->EndSlowTask();

		ReregisterAllComponents();
	}
}
#endif	//WITH_EDITOR

#if WITH_EDITORONLY_DATA
/** Returns SpriteComponent subobject **/
UBillboardComponent* ALandscapeGizmoActor::GetSpriteComponent() const { return SpriteComponent; }
#endif

