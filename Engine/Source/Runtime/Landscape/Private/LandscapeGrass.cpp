// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "EdGraph/EdGraphNode.h"
#include "GenericPlatform/GenericPlatformStackWalk.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/Guid.h"
#include "Math/RandomStream.h"
#include "Stats/Stats.h"
#include "Async/AsyncWork.h"
#include "HAL/IConsoleManager.h"
#include "Misc/App.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectIterator.h"
#include "EngineDefines.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Actor.h"
#include "ShowFlags.h"
#include "RHI.h"
#include "RenderingThread.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "ShaderParameters.h"
#include "RHIStaticStates.h"
#include "SceneView.h"
#include "Shader.h"
#include "Landscape.h"
#include "LandscapePrivate.h"
#include "LandscapeProxy.h"
#include "LandscapeInfo.h"
#include "LandscapeUtils.h"
#include "LandscapeUtilsPrivate.h"
#include "LightMap.h"
#include "Engine/MapBuildDataRegistry.h"
#include "ShadowMap.h"
#include "LandscapeComponent.h"
#include "LandscapeSubsystem.h"
#include "LandscapeGrassMapsBuilder.h"
#include "LandscapeVersion.h"
#include "MaterialShaderType.h"
#include "MeshMaterialShaderType.h"
#include "MeshMaterialShader.h"
#include "Materials/Material.h"
#include "LandscapeGrassType.h"
#include "Materials/MaterialExpressionLandscapeGrassOutput.h"
#include "Engine/TextureRenderTarget2D.h"
#include "LandscapeDataAccess.h"
#include "StaticMeshComponentLODInfo.h"
#include "StaticMeshResources.h"
#include "LandscapeLight.h"
#include "GrassInstancedStaticMeshComponent.h"
#include "Materials/MaterialInstanceConstant.h"
#include "ShaderParameterUtils.h"
#include "EngineModule.h"
#include "LandscapeRender.h"
#include "LandscapeGrassWeightExporter.h"
#include "MaterialCompiler.h"
#include "Algo/Accumulate.h"
#include "Algo/ForEach.h"
#include "UObject/Package.h"
#include "Engine/StaticMesh.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Engine/InstancedStaticMesh.h"
#include "MeshPassProcessor.h"
#include "MeshPassProcessor.inl"
#include "Math/Halton.h"
#include "EngineUtils.h"
#include "Misc/ScopedSlowTask.h"
#include "TextureCompiler.h"
#include "RenderCaptureInterface.h"
#include "SimpleMeshDrawCommandPass.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "MaterialCachedData.h"
#include "SceneRenderTargetParameters.h"
#include "TextureResource.h"
#include "Serialization/ArchiveCrc32.h"
#include "SceneRendererInterface.h"

#if !UE_BUILD_SHIPPING
#include "DrawDebugHelpers.h"
#endif // !UE_BUILD_SHIPPING

#define LOCTEXT_NAMESPACE "Landscape"

DEFINE_LOG_CATEGORY(LogGrass);

static float GGuardBandMultiplier = 1.3f;
static FAutoConsoleVariableRef CVarGuardBandMultiplier(
	TEXT("grass.GuardBandMultiplier"),
	GGuardBandMultiplier,
	TEXT("Used to control discarding in the grass system. Approximate range, 1-4. Multiplied by the cull distance to control when we add grass components."));

static float GGuardBandDiscardMultiplier = 1.4f;
static FAutoConsoleVariableRef CVarGuardBandDiscardMultiplier(
	TEXT("grass.GuardBandDiscardMultiplier"),
	GGuardBandDiscardMultiplier,
	TEXT("Used to control discarding in the grass system. Approximate range, 1-4. Multiplied by the cull distance to control when we discard grass components."));

static int32 GMinFramesToKeepGrass = 30;
static FAutoConsoleVariableRef CVarMinFramesToKeepGrass(
	TEXT("grass.MinFramesToKeepGrass"),
	GMinFramesToKeepGrass,
	TEXT("Minimum number of frames before cached grass can be discarded; used to prevent thrashing."));

static int32 GGrassTickInterval = 1;
static FAutoConsoleVariableRef CVarGrassTickInterval(
	TEXT("grass.TickInterval"),
	GGrassTickInterval,
	TEXT("Number of frames between grass ticks."));

static float GMinTimeToKeepGrass = 5.0f;
static FAutoConsoleVariableRef CVarMinTimeToKeepGrass(
	TEXT("grass.MinTimeToKeepGrass"),
	GMinTimeToKeepGrass,
	TEXT("Minimum number of seconds before cached grass can be discarded; used to prevent thrashing."));

static int32 GMaxInstancesPerComponent = 65536;
static FAutoConsoleVariableRef CVarMaxInstancesPerComponent(
	TEXT("grass.MaxInstancesPerComponent"),
	GMaxInstancesPerComponent,
	TEXT("Used to control the number of grass components created. More can be more efficient, but can be hitchy as new components come into range"));

static int32 GMaxAsyncTasks = 4;
static FAutoConsoleVariableRef CVarMaxAsyncTasks(
	TEXT("grass.MaxAsyncTasks"),
	GMaxAsyncTasks,
	TEXT("Used to control the number of grass components created at a time."));

static float GGrassDensityScale = 1;
static FAutoConsoleVariableRef CVarGrassDensityScale(
	TEXT("grass.densityScale"),
	GGrassDensityScale,
	TEXT("Multiplier on all grass densities."),
	ECVF_Scalability);

float GGrassCullDistanceScale = 1;
static FAutoConsoleVariableRef CVarGrassCullDistanceScale(
	TEXT("grass.CullDistanceScale"),
	GGrassCullDistanceScale,
	TEXT("Multiplier on all grass cull distances."),
	ECVF_Scalability);

int32 GGrassEnable = 1;
static FAutoConsoleVariableRef CVarGrassEnable(
	TEXT("grass.Enable"),
	GGrassEnable,
	TEXT("1: Enable Grass; 0: Disable Grass"));

static int32 GGrassDiscardDataOnLoad = 0;
static FAutoConsoleVariableRef CVarGrassDiscardDataOnLoad(
	TEXT("grass.DiscardDataOnLoad"),
	GGrassDiscardDataOnLoad,
	TEXT("1: Discard grass data on load (disables grass); 0: Keep grass data (requires reloading level)"),
	ECVF_Scalability);

static int32 GCullSubsections = 1;
static FAutoConsoleVariableRef CVarCullSubsections(
	TEXT("grass.CullSubsections"),
	GCullSubsections,
	TEXT("1: Cull each foliage component; 0: Cull only based on the landscape component."));

static int32 GDisableGPUCull = 0;
static FAutoConsoleVariableRef CVarDisableGPUCull(
	TEXT("grass.DisableGPUCull"),
	GDisableGPUCull,
	TEXT("For debugging. Set this to zero to see where the grass is generated. Useful for tweaking the guard bands."));

static int32 GDisableDynamicShadows = 0;
static FAutoConsoleVariableRef CVarDisableDynamicShadows(
	TEXT("grass.DisableDynamicShadows"),
	GDisableDynamicShadows,
	TEXT("0: Dynamic shadows from grass follow the grass type bCastDynamicShadow flag; 1: Dynamic shadows are disabled for all grass"));

static int32 GIgnoreExcludeBoxes = 0;
static FAutoConsoleVariableRef CVarIgnoreExcludeBoxes(
	TEXT("grass.IgnoreExcludeBoxes"),
	GIgnoreExcludeBoxes,
	TEXT("For debugging. Ignores any exclusion boxes."));

static int32 GGrassMaxCreatePerFrame = 1;
static FAutoConsoleVariableRef CVarGrassMaxCreatePerFrame(
	TEXT("grass.MaxCreatePerFrame"),
	GGrassMaxCreatePerFrame,
	TEXT("Maximum number of Grass components to create per frame"));

static int32 GGrassCreationPrioritizedMultipler = 4;
static FAutoConsoleVariableRef CVarGrassCreationPrioritizedMultipler(
	TEXT("grass.GrassCreationPrioritizedMultipler"),
	GGrassCreationPrioritizedMultipler,
	TEXT("Multiplier applied to MaxCreatePerFrame and MaxAsyncTasks when grass creation is prioritized."));

static TAutoConsoleVariable<int32> CVarRayTracingLandscapeGrass(
	TEXT("r.RayTracing.Geometry.LandscapeGrass"),
	0,
	TEXT("Defines if landscape grass meshes are added to the ray tracing scene.\n")
	TEXT(" 0: raytracing cached MDCs for landscape grass mesh are not created (default).\n")
	TEXT(" 1: raytracing cached MDCs for landscape grass mesh are created and instances are added to the RayTracing scene (default).\n")
	TEXT(" 2: landscape grass meshes are runtime culled from the RayTracing scene, but keeps the cached state for fast toggling."));

const TCHAR* GGrassQualityLevelCVarName = TEXT("r.grass.DensityQualityLevel");
const TCHAR* GGrassQualityLevelScalabilitySection = TEXT("ViewDistanceQuality");

int32 GGrassQualityLevel = -1;
static FAutoConsoleVariableRef CVarGGrassDensityQualityLevelCVar(
	GGrassQualityLevelCVarName,
	GGrassQualityLevel,
	TEXT("The quality level for grass (low, medium, high, epic). \n"),
	ECVF_Scalability);

static FAutoConsoleCommand ConsoleCommandDumpLandscapeGrassData(
	TEXT("grass.DumpGrassData"),
	TEXT("[optional: -csv -detailed -byproxy -bycomponent -bygrasstype -full] - Dumps a report of all grass data being currently used on landscape components. \n"
		"-csv formats the report in a CSV-friendly way. \n"
		"-fullnames displays the listed objects' full names, rather than the user-friendly version. \n"
		"-showempty will dump info even from components with no grass data \n"
		"-detailed shows a detailed report of all grass data, for all grass types, in all landscape components. \n"
		"-byproxy shows a report of grass data per landscape proxy. \n"
		"-bycomponent shows a report of grass data per landscape component. \n"
		"-bygrasstype shows a report of grass data per grass type. \n"
		"-full enables all sub-reports. \n"
		"If no report type option specified, assume full report is requested."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		bool bCSV = false;
		bool bUseFullNames = false;
		bool bIncludeEmpty = false;
		bool bDetailed = false;
		bool bByProxy = false;
		bool bByComponent = false;
		bool bByGrassName = false;
		bool bByGrassType = false;
		bool bFull = false;
		for (const FString& Arg : Args)
		{
			if (FParse::Param(*Arg, TEXT("csv")))
			{
				bCSV = true;
			}
			if (FParse::Param(*Arg, TEXT("fullnames")))
			{
				bUseFullNames = true;
			}
			if (FParse::Param(*Arg, TEXT("showempty")))
			{
				bIncludeEmpty = true;
			}
			if (FParse::Param(*Arg, TEXT("detailed")))
			{
				bDetailed = true;
			}
			if (FParse::Param(*Arg, TEXT("byproxy")))
			{
				bByProxy = true;
			}
			if (FParse::Param(*Arg, TEXT("bycomponent")))
			{
				bByComponent = true;
			}
			if (FParse::Param(*Arg, TEXT("bygrassname")))
			{
				bByGrassName = true;
			}
			if (FParse::Param(*Arg, TEXT("bygrasstype")))
			{
				bByGrassType = true;
			}
			if (FParse::Param(*Arg, TEXT("full")))
			{
				bFull = true;
			}
		}
		if (!bDetailed && !bByProxy && !bByComponent && !bByGrassName && !bByGrassType && !bFull)
		{
			// If nothing specified, assume full report :
			bFull = true;
		}

		struct FSortDataEntry
		{
			FSortDataEntry(const UWorld* InWorld, const ALandscape* InActor, const ALandscapeProxy* InProxy, const ULandscapeComponent* InComponent, const FName& InGrassName, const ULandscapeGrassType* InGrassType, int32 InOffset, bool bInIsHeightData)
				: World(InWorld)
				, Actor(InActor)
				, Proxy(InProxy)
				, Component(InComponent)
				, GrassName(InGrassName)
				, GrassType(InGrassType)
				, Offset(InOffset)
				, bIsHeightData(bInIsHeightData)
			{}
			const UWorld* World = nullptr;
			const ALandscape* Actor = nullptr;
			const ALandscapeProxy* Proxy = nullptr;
			const ULandscapeComponent* Component = nullptr;
			FName GrassName; // Can be NAME_None for height data 
			const ULandscapeGrassType* GrassType = nullptr; // Can be nullptr if this entry corresponds to height data or if we have some grass data in the grass map but no grass type associated with it
			int32 Offset = 0;
			int32 Size = 0;
			bool bIsHeightData = false;
		};
		TArray<FSortDataEntry> SortData;
		TSet<const ULandscapeGrassType*> UniqueGrassTypes;
		TSet<FName> UniqueGrassNames;

		// Gather all data first : 
		for (TObjectIterator<const ULandscapeComponent> It(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage); It; ++It)
		{
			const ULandscapeComponent* Component = *It;
			if (Component->GrassData->HasValidData() && (bIncludeEmpty || !Component->GrassData->HeightWeightData.IsEmpty()))
			{
				if (const UWorld* World = Component->GetWorld())
				{
					if (const ALandscapeProxy* Proxy = Component->GetLandscapeProxy())
					{
						if (const ALandscape* Actor = Proxy->GetLandscapeActor())
						{
							int32 MinWeightOffset = MAX_int32;
							for (auto ItPair : Component->GrassData->WeightOffsets)
							{
								const FName& GrassName = ItPair.Key;
								const ULandscapeGrassType* GrassType = nullptr;
								check(!GrassName.IsNone());
								if (const TObjectPtr<ULandscapeGrassType>* FoundGrassType = Component->GetNamedGrassTypes().Find(GrassName))
								{
									GrassType = *FoundGrassType;
									int32 GrassTypeOffset = ItPair.Value;
									SortData.Add(FSortDataEntry(World, Actor, Proxy, Component, GrassName, GrassType, GrassTypeOffset, /*bInIsHeightData = */false));
									MinWeightOffset = FMath::Min(GrassTypeOffset, MinWeightOffset);

									UniqueGrassNames.Add(GrassName);
									if (GrassType != nullptr)
									{
										UniqueGrassTypes.Add(GrassType);
									}
								}
							}
							// Height data, if any, is at the beginning so if one the weight offsets is at index 0, then there's no height data on this component at all
							int32 HeightOffset = (MinWeightOffset == 0) ? INDEX_NONE : 0;
							// Always add an entry for height, even if no height data was found (makes reporting easier) : 
							SortData.Add(FSortDataEntry(World, Actor, Proxy, Component, /*InGrassName = */NAME_None, /* InGrassType = */nullptr, HeightOffset, /*bInIsHeightData = */true));
						}
					}
				}
			}
		}

		const int32 NumEntries = SortData.Num();
		if (NumEntries == 0)
		{
			GLog->Logf(TEXT("No grass data."));
			return;
		}

		// Then compute the size of each landscape grass type within a component : 
		{
			// Sort by Component then offset: 
			SortData.Sort([](const FSortDataEntry& InLHS, const FSortDataEntry& InRHS)
			{
				if (InLHS.Component == InRHS.Component)
				{
					return InLHS.Offset < InRHS.Offset;
				}
				else
				{
					return InLHS.Component < InRHS.Component;
				}
			});

			// Then iterate over all entries and use the next item's offset to compute the previous item's size, component by component:
			const ULandscapeComponent* CurrentComponent = SortData[0].Component;
			int32 CurrentOffsetInComponent = 0;
			for (int32 NextIndex = 1; NextIndex < NumEntries; ++NextIndex)
			{
				const FSortDataEntry& NextEntry = SortData[NextIndex];
				if (NextEntry.Component != CurrentComponent)
				{
					SortData[NextIndex - 1].Size = CurrentComponent->GrassData->HeightWeightData.Num() - CurrentOffsetInComponent;
					CurrentComponent = NextEntry.Component;
					CurrentOffsetInComponent = 0;
				}
				else
				{
					SortData[NextIndex - 1].Size = NextEntry.Offset - CurrentOffsetInComponent;
					CurrentOffsetInComponent = NextEntry.Offset;
				}
			}

			// Since the loop sets the size at index - 1, the last item's size is not yet computed : 
			SortData[NumEntries - 1].Size = CurrentComponent->GrassData->HeightWeightData.Num() - CurrentOffsetInComponent;
		}

		// Sort by World, then Actor, then Proxy, then Component, then grass name : 
		SortData.Sort([](const FSortDataEntry& InLHS, const FSortDataEntry& InRHS)
		{
			if (InLHS.World == InRHS.World)
			{
				if (InLHS.Actor == InRHS.Actor)
				{
					if (InLHS.Proxy == InRHS.Proxy)
					{
						if (InLHS.Component == InRHS.Component)
						{
							return InLHS.GrassName.LexicalLess(InRHS.GrassName);
						}
						else
						{
							return InLHS.Component < InRHS.Component;
						}
					}
					else
					{
						return InLHS.Proxy < InRHS.Proxy;
					}
				}
				else
				{
					return InLHS.Actor < InRHS.Actor;
				}
			}
			else
			{
				return InLHS.World < InRHS.World;
			}
		});

		auto GetFormattedObjectName = [bUseFullNames](const UObject* InObject) -> FString { return bUseFullNames ? InObject->GetFullName() : InObject->GetName(); };
		auto GetFormattedActorName = [bUseFullNames](const AActor* InActor) -> FString { return bUseFullNames ? InActor->GetFullName() : InActor->GetActorNameOrLabel(); };
		auto GetFormattedGrassName = [](const FName& InGrassName) -> FString { check(!InGrassName.IsNone()); return InGrassName.ToString(); };
		auto GetFormattedGrassTypeName = [bUseFullNames](const ULandscapeGrassType* InGrassType) -> FString { return (InGrassType != nullptr) ? (bUseFullNames ? InGrassType->GetFullName() : InGrassType->GetName()) : FString(TEXT("(null)")); };
		auto LogWorldActorLine = [GetFormattedObjectName, GetFormattedActorName](const UWorld* InWorld, const ALandscape* InActor) { GLog->Logf(TEXT("-- World: %s - Landscape Actor: %s --"), *GetFormattedObjectName(InWorld), *GetFormattedActorName(InActor)); };

		// List of lines to display in a sub-report (one element per entry):
		TArray<TPair<FString, FString>> CurrentSubReportContent; // Key = entry's name (for sorting), Value = line
		auto AddLineToSubReport = [&CurrentSubReportContent](const FString& InEntryName, const FString& InLine) { CurrentSubReportContent.Emplace(InEntryName, InLine); };
		auto FinalizeSubReport = [&CurrentSubReportContent]()
		{
			// Lexicographically sort entries, output them and reset the sub-report content :
			CurrentSubReportContent.Sort([](const TPair<FString, FString>& InLHS, const TPair<FString, FString>& InRHS) { return (InLHS.Key.Compare(InRHS.Key) < 0); });
			Algo::ForEach(CurrentSubReportContent, [](const TPair<FString, FString>& InEntry) { GLog->Log(*InEntry.Value); });
			CurrentSubReportContent.Empty();
		};

		if (bFull || bDetailed)
		{
			// Start a new report :
			GLog->Logf(TEXT("Detailed report:"));
			ON_SCOPE_EXIT{ GLog->Logf(TEXT("--------")); };

			if (bCSV)
			{
				GLog->Logf(TEXT("World,Landscape Actor,Landscape Proxy,Landscape Component,Landscape Grass Name,Landscape Grass Type,Size (KB)"));
				for (const FSortDataEntry& Entry : SortData)
				{
					if (Entry.bIsHeightData)
					{
						FString Line = FString::Printf(TEXT("%s,%s,%s,%s,(HeightData),(HeightData),%.2f"),
							*GetFormattedObjectName(Entry.World),
							*GetFormattedActorName(Entry.Actor),
							*GetFormattedActorName(Entry.Proxy),
							*Entry.Component->GetName(),
							Entry.Size / 1024.0f);
						// Use the line itself as the sort key since it's already formatted the way we want: 
						AddLineToSubReport(Line, Line);
					}
					else if ((Entry.GrassType != nullptr) || (bIncludeEmpty || (Entry.Size > 0)))
					{
						FString Line = FString::Printf(TEXT("%s,%s,%s,%s,%s,%s,%.2f"),
							*GetFormattedObjectName(Entry.World),
							*GetFormattedActorName(Entry.Actor),
							*GetFormattedActorName(Entry.Proxy),
							*Entry.Component->GetName(),
							*GetFormattedGrassName(Entry.GrassName),
							*GetFormattedGrassTypeName(Entry.GrassType),
							Entry.Size / 1024.0f);
						// Use the line itself as the sort key since it's already formatted the way we want: 
						AddLineToSubReport(Line, Line);
					}
				}
				FinalizeSubReport();
			}
			else
			{
				const UWorld* CurrentWorld = SortData[0].World;
				const ALandscape* CurrentActor = SortData[0].Actor;
				const ALandscapeProxy* CurrentProxy = SortData[0].Proxy;
				const ULandscapeComponent* CurrentComponent = SortData[0].Component;
				bool bCurrentWorldHasGrassData = false;
				bool bCurrentActorHasGrassData = false;
				bool bCurrentProxyHasGrassData = false;
				bool bCurrentComponentHasGrassData = false;
				int32 NumWorldsWithGrassData = 0;
				int32 NumActorsWithGrassData = 0;
				int32 NumProxiesWithGrassData = 0;
				int32 NumComponentsWithGrassData = 0;
				int32 TotalHeightDataSize = 0;
				int32 TotalWeightDataSize = 0;
				for (int32 Index = 0; Index < NumEntries; ++Index)
				{
					const FSortDataEntry& Entry = SortData[Index];
					if (CurrentWorld != Entry.World)
					{
						// This world was fully processed, take note of the info and start tracking the new world:
						if (bCurrentWorldHasGrassData)
						{
							++NumWorldsWithGrassData;
						}
						CurrentWorld = Entry.World;
						bCurrentWorldHasGrassData = false;
					}

					if (CurrentActor != Entry.Actor)
					{
						// This actor was fully processed, take note of the info and start tracking the new actor:
						if (bCurrentActorHasGrassData)
						{
							++NumActorsWithGrassData;
						}
						CurrentActor = Entry.Actor;
						bCurrentActorHasGrassData = false;
					}

					if (CurrentProxy != Entry.Proxy)
					{
						// This proxy was fully processed, take note of the info and start tracking the new proxy:
						if (bCurrentProxyHasGrassData)
						{
							++NumProxiesWithGrassData;
						}
						CurrentProxy = Entry.Proxy;
						bCurrentProxyHasGrassData = false;
					}

					// This component was fully processed, take note of the info and start tracking the new component:
					if (CurrentComponent != Entry.Component)
					{
						if (bCurrentComponentHasGrassData)
						{
							++NumComponentsWithGrassData;
						}
						CurrentComponent = Entry.Component;
						bCurrentComponentHasGrassData = false;
					}

					if (Entry.Size > 0)
					{
						bCurrentWorldHasGrassData = true;
						bCurrentActorHasGrassData = true;
						bCurrentProxyHasGrassData = true;
						bCurrentComponentHasGrassData = true;
						if (Entry.bIsHeightData)
						{
							TotalHeightDataSize += Entry.Size;
						}
						else
						{
							TotalWeightDataSize += Entry.Size;
						}
					}
				}

				// And dump one more time, for the last entry, which hasn't been taken into account yet :
				if (bCurrentWorldHasGrassData)
				{
					++NumWorldsWithGrassData;
				}
				if (bCurrentActorHasGrassData)
				{
					++NumActorsWithGrassData;
				}
				if (bCurrentProxyHasGrassData)
				{
					++NumProxiesWithGrassData;
				}
				if (bCurrentComponentHasGrassData)
				{
					++NumComponentsWithGrassData;
				}

				GLog->Logf(TEXT("Num worlds : %i"), NumWorldsWithGrassData);
				GLog->Logf(TEXT("Num landscape actors with grass data: %i"), NumActorsWithGrassData);
				GLog->Logf(TEXT("Num landscape proxies with grass data: %i"), NumProxiesWithGrassData);
				GLog->Logf(TEXT("Num landscape components with grass data: %i"), NumComponentsWithGrassData);
				GLog->Logf(TEXT("Num grass names used: %i"), UniqueGrassNames.Num());
				GLog->Logf(TEXT("Num grass types used: %i"), UniqueGrassTypes.Num());
				GLog->Logf(TEXT("Total height data size (KB): %.2f"), TotalHeightDataSize / 1024.0f);
				GLog->Logf(TEXT("Total weight data size (KB): %.2f"), TotalWeightDataSize / 1024.0f);
				GLog->Logf(TEXT("Total grass data size (KB): %.2f"), (TotalHeightDataSize + TotalWeightDataSize) / 1024.0f);
			}
		}

		// Assumes the list is sorted by world / by actor / by proxy already: 
		if (bFull || bByProxy)
		{
			// Start a new report :
			GLog->Logf(TEXT("By proxy report:"));
			ON_SCOPE_EXIT{ GLog->Logf(TEXT("--------")); };

			// Start a new sub-report per world / per actor:
			LogWorldActorLine(SortData[0].World, SortData[0].Actor);
			const ALandscape* CurrentActor = SortData[0].Actor;

			auto LogCSVHeader = [bCSV]() { if (bCSV) { GLog->Logf(TEXT("Landscape Proxy,Num Grass Names,Num Grass Types,Total Height Data Size (KB), Total Weight Data Size (KB), Total Data Size (KB)")); } };
			auto LogLine = [bCSV, AddLineToSubReport](const TCHAR* InProxyName, int32 InNumGrassNames, int32 InNumGrassTypes, int32 InTotalHeightDataSizeBytes, int32 InTotalWeightDataSizeBytes)
			{
				if (bCSV)
				{
					AddLineToSubReport(FString(InProxyName), FString::Printf(TEXT("%s,%i,%i,%.2f,%.2f,%.2f"),
						InProxyName,
						InNumGrassNames,
						InNumGrassTypes,
						InTotalHeightDataSizeBytes / 1024.0f,
						InTotalWeightDataSizeBytes / 1024.0f,
						(InTotalHeightDataSizeBytes + InTotalWeightDataSizeBytes) / 1024.0f));
				}
				else
				{
					AddLineToSubReport(FString(InProxyName), FString::Printf(TEXT("%s: Num grass names = %i, Num grass types = %i, total height data size = %.2f KB, total weight data size = %.2f KB, total data size = %.2f KB"),
						InProxyName,
						InNumGrassNames,
						InNumGrassTypes,
						InTotalHeightDataSizeBytes / 1024.0f,
						InTotalWeightDataSizeBytes / 1024.0f,
						(InTotalHeightDataSizeBytes + InTotalWeightDataSizeBytes) / 1024.0f));
				}
			};

			LogCSVHeader();
			const ALandscapeProxy* CurrentProxy = SortData[0].Proxy;
			int32 HeightDataSizeForCurrentProxy = 0;
			int32 WeightDataSizeForCurrentProxy = 0;
			TSet<FName> UniqueGrassNamesForCurrentProxy;
			TSet<const ULandscapeGrassType*> UniqueGrassTypesForCurrentProxy;
			for (int32 Index = 0; Index < NumEntries; ++Index)
			{
				const FSortDataEntry& Entry = SortData[Index];
				if (Entry.Proxy != CurrentProxy)
				{
					// The info from CurrentProxy is now complete, we can dump it :
					LogLine(*GetFormattedActorName(CurrentProxy), UniqueGrassNamesForCurrentProxy.Num(), UniqueGrassTypesForCurrentProxy.Num(), HeightDataSizeForCurrentProxy, WeightDataSizeForCurrentProxy);

					// Start gathering stats for the next proxy :
					CurrentProxy = Entry.Proxy;
					HeightDataSizeForCurrentProxy = 0;
					WeightDataSizeForCurrentProxy = 0;
					UniqueGrassNamesForCurrentProxy.Empty();
					UniqueGrassTypesForCurrentProxy.Empty();
				}

				if (Entry.Actor != CurrentActor)
				{
					// Start a new sub-report per world / per actor:
					FinalizeSubReport();
					LogWorldActorLine(Entry.World, Entry.Actor);
					LogCSVHeader();
					CurrentActor = Entry.Actor;
				}

				// Accumulate data about this proxy :
				if (Entry.Size > 0)
				{
					if (Entry.bIsHeightData)
					{
						HeightDataSizeForCurrentProxy += Entry.Size;
					}
					else
					{
						WeightDataSizeForCurrentProxy += Entry.Size;
						UniqueGrassNamesForCurrentProxy.Add(Entry.GrassName);
						if (Entry.GrassType != nullptr)
						{
							UniqueGrassTypesForCurrentProxy.Add(Entry.GrassType);
						}
					}
				}
			}

			// And dump one more time, for the last proxy, which hasn't been reported yet :
			LogLine(*GetFormattedActorName(CurrentProxy), UniqueGrassNamesForCurrentProxy.Num(), UniqueGrassTypesForCurrentProxy.Num(), HeightDataSizeForCurrentProxy, WeightDataSizeForCurrentProxy);
			FinalizeSubReport();
		}

		// Assumes the list is sorted by world / by actor / ... / by component already: 
		if (bFull || bByComponent)
		{
			// Start a new report :
			GLog->Logf(TEXT("By component report:"));
			ON_SCOPE_EXIT{ GLog->Logf(TEXT("--------")); };

			// Start a new sub-report per world / per actor:
			LogWorldActorLine(SortData[0].World, SortData[0].Actor);
			const ALandscape* CurrentActor = SortData[0].Actor;

			auto LogCSVHeader = [bCSV]() { if (bCSV) { GLog->Logf(TEXT("Landscape Proxy,Landscape Component,Num Grass Names,Num Grass Types,Total Height Data Size (KB), Total Weight Data Size (KB), Total Data Size (KB)")); } };
			auto LogLine = [bCSV, AddLineToSubReport](const TCHAR* InProxyName, const TCHAR* InComponentName, int32 InNumGrassNames, int32 InNumGrassTypes, int32 InTotalHeightDataSizeBytes, int32 InTotalWeightDataSizeBytes)
			{
				if (bCSV)
				{
					AddLineToSubReport(FString(InComponentName), FString::Printf(TEXT("%s,%s,%i,%i,%.2f,%.2f,%.2f"),
						InProxyName,
						InComponentName,
						InNumGrassNames,
						InNumGrassTypes,
						InTotalHeightDataSizeBytes / 1024.0f,
						InTotalWeightDataSizeBytes / 1024.0f,
						(InTotalHeightDataSizeBytes + InTotalWeightDataSizeBytes) / 1024.0f));
				}
				else
				{
					AddLineToSubReport(FString(InComponentName), FString::Printf(TEXT("%s (%s): Num grass names = %i, Num grass types = %i, total height data size = %.2f KB, total weight data size = %.2f KB, total data size = %.2f KB"),
						InProxyName,
						InComponentName,
						InNumGrassNames,
						InNumGrassTypes,
						InTotalHeightDataSizeBytes / 1024.0f,
						InTotalWeightDataSizeBytes / 1024.0f,
						(InTotalHeightDataSizeBytes + InTotalWeightDataSizeBytes) / 1024.0f));
				}
			};

			LogCSVHeader();
			const ULandscapeComponent* CurrentComponent = SortData[0].Component;
			int32 HeightDataSizeForCurrentComponent = 0;
			int32 WeightDataSizeForCurrentComponent = 0;
			TSet<FName> UniqueGrassNamesForCurrentComponent;
			TSet<const ULandscapeGrassType*> UniqueGrassTypesForCurrentComponent;
			for (int32 Index = 0; Index < NumEntries; ++Index)
			{
				const FSortDataEntry& Entry = SortData[Index];
				if (Entry.Component != CurrentComponent)
				{
					// The info from CurrentComponent is now complete, we can dump it :
					LogLine(*GetFormattedActorName(CurrentComponent->GetLandscapeProxy()), *CurrentComponent->GetName(), UniqueGrassNamesForCurrentComponent.Num(), UniqueGrassTypesForCurrentComponent.Num(), HeightDataSizeForCurrentComponent, WeightDataSizeForCurrentComponent);

					// Start gathering stats for the next component :
					CurrentComponent = Entry.Component;
					HeightDataSizeForCurrentComponent = 0;
					WeightDataSizeForCurrentComponent = 0;
					UniqueGrassNamesForCurrentComponent.Empty();
					UniqueGrassTypesForCurrentComponent.Empty();
				}

				if (Entry.Actor != CurrentActor)
				{
					// Start a new sub-report per world / per actor:
					FinalizeSubReport();
					LogWorldActorLine(Entry.World, Entry.Actor);
					LogCSVHeader();
					CurrentActor = Entry.Actor;
				}

				// Accumulate data about this component :
				if (Entry.Size > 0)
				{
					if (Entry.bIsHeightData)
					{
						HeightDataSizeForCurrentComponent += Entry.Size;
					}
					else
					{
						WeightDataSizeForCurrentComponent += Entry.Size;
						UniqueGrassNamesForCurrentComponent.Add(Entry.GrassName);
						if (Entry.GrassType != nullptr)
						{
							UniqueGrassTypesForCurrentComponent.Add(Entry.GrassType);
						}
					}
				}
			}

			// And dump one more time, for the last component, which hasn't been reported yet :
			LogLine(*GetFormattedActorName(CurrentComponent->GetLandscapeProxy()), *CurrentComponent->GetName(), UniqueGrassNamesForCurrentComponent.Num(), UniqueGrassTypesForCurrentComponent.Num(), HeightDataSizeForCurrentComponent, WeightDataSizeForCurrentComponent);
			FinalizeSubReport();
		}

		if (bFull || bByGrassName)
		{
			// Sort by Grass Name, then Component: 
			SortData.Sort([](const FSortDataEntry& InLHS, const FSortDataEntry& InRHS)
				{
					if (InLHS.bIsHeightData == InRHS.bIsHeightData)
					{
						if (InLHS.GrassName == InRHS.GrassName)
						{
							return (InLHS.Component < InRHS.Component);
						}
						return InLHS.GrassName.LexicalLess(InRHS.GrassName);
					}
					// Put the height data entries first :
					return InLHS.bIsHeightData;
				});

			// Start a new sub-report :
			GLog->Logf(TEXT("By grass name report:"));
			ON_SCOPE_EXIT{ GLog->Logf(TEXT("--------")); };

			auto LogCSVHeader = [bCSV]() { if (bCSV) { GLog->Logf(TEXT("Grass Name,Num Components,Total Data Size (KB)")); } };
			auto LogLine = [bCSV, AddLineToSubReport](const TCHAR* InGrassName, int32 InNumComponents, int32 InTotalDataSizeBytes)
				{
					if (bCSV)
					{
						AddLineToSubReport(FString(InGrassName), FString::Printf(TEXT("%s,%i,%.2f"),
							InGrassName,
							InNumComponents,
							InTotalDataSizeBytes / 1024.0f));
					}
					else
					{
						AddLineToSubReport(FString(InGrassName), FString::Printf(TEXT("%s: Num components = %i, total data size = %.2f KB"),
							InGrassName,
							InNumComponents,
							InTotalDataSizeBytes / 1024.0f));
					}
				};

			LogCSVHeader();

			int32 Index = 0;
			int32 HeightDataSize = 0;
			int32 NumComponentsWithHeightData = 0;
			// Display all the height data first : 
			for (; (Index < NumEntries) && SortData[Index].bIsHeightData; ++Index)
			{
				const FSortDataEntry& Entry = SortData[Index];
				check(Entry.bIsHeightData);
				HeightDataSize += Entry.Size;
				++NumComponentsWithHeightData;
			}
			if (NumComponentsWithHeightData > 0)
			{
				// Dump info for height :
				LogLine(TEXT("(HeightData)"), NumComponentsWithHeightData, HeightDataSize);
			}

			FName CurrentGrassName = (Index < NumEntries) ? SortData[Index].GrassName : NAME_None;
			int32 DataSizeForCurrentGrassName = 0;
			int32 NumComponentsForCurrentGrassName = 0;
			for (; Index < NumEntries; ++Index)
			{
				const FSortDataEntry& Entry = SortData[Index];
				check(!Entry.bIsHeightData);
				if (Entry.GrassName != CurrentGrassName)
				{
					// The info from CurrentGrassName is now complete, we can dump it :
					LogLine(*GetFormattedGrassName(CurrentGrassName), NumComponentsForCurrentGrassName, DataSizeForCurrentGrassName);

					// Start gathering stats for the next grass type :
					CurrentGrassName = Entry.GrassName;
					DataSizeForCurrentGrassName = 0;
					NumComponentsForCurrentGrassName = 0;
				}

				// Accumulate data about this grass type :
				if (Entry.Size > 0)
				{
					DataSizeForCurrentGrassName += Entry.Size;
					++NumComponentsForCurrentGrassName;
				}
			}

			// And dump one more time, for the last grass type, which hasn't been reported yet :
			if (!CurrentGrassName.IsNone())
			{
				LogLine(*GetFormattedGrassName(CurrentGrassName), NumComponentsForCurrentGrassName, DataSizeForCurrentGrassName);
			}
			FinalizeSubReport();
		}

		if (bFull || bByGrassType)
		{
			// Sort by HeightData first, then Grass Type, then Component: 
			SortData.Sort([](const FSortDataEntry& InLHS, const FSortDataEntry& InRHS)
			{
				if (InLHS.bIsHeightData == InRHS.bIsHeightData)
				{
					if (InLHS.GrassType == InRHS.GrassType)
					{
						return (InLHS.Component < InRHS.Component);
					}
					return (InLHS.GrassType < InRHS.GrassType);
				}
				// Put the height data entries first :
				return InLHS.bIsHeightData;
			});

			// Start a new sub-report :
			GLog->Logf(TEXT("By grass type report:"));
			ON_SCOPE_EXIT{ GLog->Logf(TEXT("--------")); };

			auto LogCSVHeader = [bCSV]() { if (bCSV) { GLog->Logf(TEXT("Grass Type,Num Components,Total Data Size (KB)")); } };
			auto LogLine = [bCSV, AddLineToSubReport](const TCHAR* InGrassTypeName, int32 InNumComponents, int32 InTotalDataSizeBytes)
			{
				if (bCSV)
				{
					AddLineToSubReport(FString(InGrassTypeName), FString::Printf(TEXT("%s,%i,%.2f"),
						InGrassTypeName,
						InNumComponents,
						InTotalDataSizeBytes / 1024.0f));
				}
				else
				{
					AddLineToSubReport(FString(InGrassTypeName), FString::Printf(TEXT("%s: Num components = %i, total data size = %.2f KB"),
						InGrassTypeName,
						InNumComponents,
						InTotalDataSizeBytes / 1024.0f));
				}
			};

			LogCSVHeader();

			int32 Index = 0;
			int32 HeightDataSize = 0;
			int32 NumComponentsWithHeightData = 0;
			// Display all the height data first : 
			for (; (Index < NumEntries) && SortData[Index].bIsHeightData; ++Index)
			{
				const FSortDataEntry& Entry = SortData[Index];
				check(Entry.bIsHeightData);
				HeightDataSize += Entry.Size;
				++NumComponentsWithHeightData;
			}
			if (NumComponentsWithHeightData > 0)
			{
				// Dump info for height :
				LogLine(TEXT("(HeightData)"), NumComponentsWithHeightData, HeightDataSize);
			}
			
			const ULandscapeGrassType* CurrentGrassType = (Index < NumEntries) ? SortData[Index].GrassType : nullptr;
			int32 DataSizeForCurrentGrassType = 0;
			int32 NumComponentsForCurrentGrassType = 0;
			for (; Index < NumEntries; ++Index)
			{
				const FSortDataEntry& Entry = SortData[Index];
				check(!Entry.bIsHeightData);
				if (Entry.GrassType != CurrentGrassType)
				{
					// The info from CurrentGrassType is now complete, we can dump it :
					LogLine(*GetFormattedGrassTypeName(CurrentGrassType), NumComponentsForCurrentGrassType, DataSizeForCurrentGrassType);

					// Start gathering stats for the next grass type :
					CurrentGrassType = Entry.GrassType;
					DataSizeForCurrentGrassType = 0;
					NumComponentsForCurrentGrassType = 0;
				}

				// Accumulate data about this grass type :
				if (Entry.Size > 0)
				{
					DataSizeForCurrentGrassType += Entry.Size;
					++NumComponentsForCurrentGrassType;
				}
			}

			// And dump one more time, for the last grass type, which hasn't been reported yet :
			if (CurrentGrassType != nullptr)
			{
				LogLine(*GetFormattedGrassTypeName(CurrentGrassType), NumComponentsForCurrentGrassType, DataSizeForCurrentGrassType);
			}
			FinalizeSubReport();
		}
	}));

#if !UE_BUILD_SHIPPING
static bool bGGrassDrawExclusionVolumes = false;
static FAutoConsoleVariableRef CVarGrassDrawExclusionVolumes(
	TEXT("grass.DrawExclusionVolumes"),
	bGGrassDrawExclusionVolumes,
	TEXT("Whether we should draw the exclusion volumes or not"));
#endif // !UE_BUILD_SHIPPING

struct FPerQualityLevelInt;
struct FPerQualityLevelFloat;

DECLARE_CYCLE_STAT(TEXT("Grass Async Build Time"), STAT_FoliageGrassAsyncBuildTime, STATGROUP_Foliage);
DECLARE_CYCLE_STAT(TEXT("Grass Start Comp"), STAT_FoliageGrassStartComp, STATGROUP_Foliage);
DECLARE_CYCLE_STAT(TEXT("Grass End Comp"), STAT_FoliageGrassEndComp, STATGROUP_Foliage);
DECLARE_CYCLE_STAT(TEXT("Grass Destroy Comps"), STAT_FoliageGrassDestoryComp, STATGROUP_Foliage);
DECLARE_CYCLE_STAT(TEXT("Grass Update"), STAT_GrassUpdate, STATGROUP_Foliage);
DECLARE_CYCLE_STAT(TEXT("Grass Type Update Summary"), STAT_UpdateGrassTypeSummary, STATGROUP_Foliage);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Grass Exclusion Volumes"), STAT_GrassExclusionVolumes, STATGROUP_Foliage);

int32 ALandscapeProxy::GrassUpdateInterval = 1;

static void GrassCVarSinkFunction()
{
	static float CachedGrassDensityScale = 1.0f;
	float GrassDensityScale = GGrassDensityScale;

	if (FApp::IsGame())
	{
		ALandscapeProxy::SetGrassUpdateInterval(FMath::Clamp<int32>(GGrassTickInterval, 1, 60));
	}

	static float CachedGrassCullDistanceScale = 1.0f;
	float GrassCullDistanceScale = GGrassCullDistanceScale;

	static const IConsoleVariable* DetailModeCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.DetailMode"));
	static int32 CachedDetailMode = DetailModeCVar ? DetailModeCVar->GetInt() : 0;
	int32 DetailMode = DetailModeCVar ? DetailModeCVar->GetInt() : 0;

	static const IConsoleVariable* NaniteEnabledCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Nanite"));
	static int32 CachedNaniteEnabled = NaniteEnabledCVar ? NaniteEnabledCVar->GetInt() : 0;
	int32 NaniteEnabled = NaniteEnabledCVar ? NaniteEnabledCVar->GetInt() : 0;

	if (DetailMode != CachedDetailMode || 
		GrassDensityScale != CachedGrassDensityScale || 
		GrassCullDistanceScale != CachedGrassCullDistanceScale || 
		CachedNaniteEnabled != NaniteEnabled)
	{
		CachedGrassDensityScale = GrassDensityScale;
		CachedGrassCullDistanceScale = GrassCullDistanceScale;
		CachedDetailMode = DetailMode;
		CachedNaniteEnabled = NaniteEnabled;

		for (UWorld* World : TObjectRange<UWorld>(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage))
		{
			if (!World->bIsTearingDown)
			{
				if (ULandscapeSubsystem* LandscapeSubsystem = World->GetSubsystem<ULandscapeSubsystem>())
				{
					LandscapeSubsystem->RegenerateGrass(/*bInFlushGrass = */true, /*bInForceSync = */true);
				}
			}
		}
	}
}

static FAutoConsoleVariableSink CVarGrassSink(FConsoleCommandDelegate::CreateStatic(&GrassCVarSinkFunction));

//
// Grass weightmap rendering
//

FLandscapeComponentGrassData::FLandscapeComponentGrassData(ULandscapeComponent* Component)
{
#if WITH_EDITOR
	GenerationHash = Component->ComputeGrassMapGenerationHash();
#endif // WITH_EDITOR
}

void ULandscapeComponent::InvalidateGrassTypeSummary()
{
	GrassTypeSummary.bInvalid = true;
	GetLandscapeProxy()->InvalidateGrassTypeSummary();
}

bool ULandscapeComponent::UpdateGrassTypes(bool bForceUpdate)
{
	bool bChanged = false;

	auto BuildNamedGrassTypes = [](UMaterialInterface* InMaterial, ALandscapeProxy* InProxy) 
		{
			// Build the final list of grass types for this component : start by using the material's list (we won't ever be able to support anything that is not declared in that list anyway) : 
			TMap<FName, TObjectPtr<ULandscapeGrassType>> LocalNamedGrassTypes = InMaterial->GetMaterial()->GetCachedExpressionData().NamedGrassTypes;
			const TMap<FLandscapeGrassName, TObjectPtr<ULandscapeGrassType>>& GrassTypesOverrides = InProxy->GetGrassTypesOverrides();

			// And apply eventual overrides onto it : 
			for (TPair<FName, TObjectPtr<ULandscapeGrassType>>& ItPair : LocalNamedGrassTypes)
			{
				if (const TObjectPtr<ULandscapeGrassType>* GrassTypeOverride = GrassTypesOverrides.Find(FLandscapeGrassName(ItPair.Key)))
				{
					ItPair.Value = *GrassTypeOverride;
				}
			}

			return LocalNamedGrassTypes;
		};

#if WITH_EDITOR
	if (UMaterialInterface* Material = GetLandscapeMaterial())
	{
		TMap<FName, TObjectPtr<ULandscapeGrassType>> LocalNamedGrassTypes = BuildNamedGrassTypes(Material, GetLandscapeProxy());
		uint32 NewGrassTypesCRC = HashCombine(Material->ComputeAllStateCRC(), UE::Landscape::Grass::Private::ComputeNamedGrassTypesHash(LocalNamedGrassTypes));
		if (bForceUpdate || (GrassTypesCRC != NewGrassTypesCRC))
		{
			GrassTypesCRC = NewGrassTypesCRC;

			SetNamedGrassTypes(LocalNamedGrassTypes);
			bChanged = true;
		}
	}
#else
	// In cooked builds, we don't automatically respond to material changes.
	// The cooked value of the component's GrassTypes array is considered authoritative.
	if (bForceUpdate)
	{
		if (UMaterialInterface* Material = GetLandscapeMaterial())
		{
			TMap<FName, TObjectPtr<ULandscapeGrassType>> LocalNamedGrassTypes = BuildNamedGrassTypes(Material, GetLandscapeProxy());
			SetNamedGrassTypes(LocalNamedGrassTypes);
			bChanged = true;
		}
	}
#endif // WITH_EDITOR
	return bChanged;
}

namespace UE::Landscape::Grass::Private
{
	void AddGrassWeightShaderTypes(FMaterialShaderTypes& InOutShaderTypes);
} // namespace UE::Landscape::Grass::Private

bool ULandscapeComponent::CanRenderGrassMap() const
{
	using namespace UE::Landscape::Grass::Private;

	// Check we can render
	const UWorld* World = GetWorld();
	if (!CanEverRender() || !World || !SceneProxy)
	{
		GRASS_DEBUG_LOG(TEXT("GrassMap No SceneProxy for %s"), *GetName());
		return false;
	}

	// Check we can render the material
	const int32 MatInstCount = GetCurrentRuntimeMaterialInstanceCount();
	if (MatInstCount <= 0)
	{
		GRASS_DEBUG_LOG(TEXT("GrassMap MaterialInstanceCount %d <= 0 for %s (%d dyn:%d inst:%d mob:%d)"),
			MatInstCount,
			*GetName(),
			GetLandscapeProxy()->bUseDynamicMaterialInstance ? 1 : 0,
			MaterialInstancesDynamic.Num(),
			MaterialInstances.Num(),
			MobileMaterialInterfaces.Num());
		return false;
	}

	UMaterialInterface* MaterialInterface = GetCurrentRuntimeMaterialInterface(0);
	if (MaterialInterface == nullptr)
	{
		GRASS_DEBUG_LOG(TEXT("GrassMap MaterialInterface NULL for %s"), *GetName());
		return false;
	}

	const FMaterialResource* MaterialResource = MaterialInterface->GetMaterialResource(GetFeatureLevelShaderPlatform_Checked(World->GetFeatureLevel()));
	if (MaterialResource == nullptr)
	{
		GRASS_DEBUG_LOG(TEXT("GrassMap MaterialResource NULL for %s (%s feature level %d)"), *GetName(), *MaterialInterface->GetName(), World->GetFeatureLevel());
		return false;
	}

	// We only need the GrassWeight shaders on the fixed grid vertex factory to render grass maps : 
	FMaterialShaderTypes ShaderTypes;
	AddGrassWeightShaderTypes(ShaderTypes);

	const FVertexFactoryType* LandscapeGrassVF = FindVertexFactoryType(FName(TEXT("FLandscapeFixedGridVertexFactory"), FNAME_Find));
	if (!MaterialResource->HasShaders(ShaderTypes, LandscapeGrassVF))
	{
		GRASS_DEBUG_LOG(TEXT("GrassMap MaterialResource does not have FixedGridVF for %s"), *GetName());
		return false;
	}
	return true;
}

#if WITH_EDITOR

namespace UE::Landscape::Grass
{
	extern uint32 ComputeGrassMapGenerationHash(const ULandscapeComponent* Component, UMaterialInterface* Material);
} // namespace UE::Landscape::Grass

uint32 ULandscapeComponent::ComputeGrassMapGenerationHash() const
{
	return UE::Landscape::Grass::ComputeGrassMapGenerationHash(this, GetLandscapeMaterial());
}

bool ULandscapeComponent::IsGrassMapOutdated() const
{
	return GrassData->HasValidData() && (ComputeGrassMapGenerationHash() != GrassData->GenerationHash);
}

TArray<uint16> ULandscapeComponent::RenderWPOHeightmap(int32 LOD)
{
	using namespace UE::Landscape::Grass;

	TArray<uint16> Results;
	ALandscape* Landscape = GetLandscapeActor();
	check(Landscape != nullptr);

	if (ensure(SceneProxy))
	{
		if (!CanRenderGrassMap())
		{
			GetMaterialInstance(0)->GetMaterialResource(GetFeatureLevelShaderPlatform_Checked(GetWorld()->GetFeatureLevel()))->FinishCompilation();

			if (!CanRenderGrassMap())
			{
				UE_LOGF(LogGrass, Verbose, "Failed to calculate Landscape WPO height for static lighting. Grass map generation shader could not be compiled.");
				return Results;
			}
		}

		if (LOD == 0)
		{
			EGrassWeightExporterFlags LandscapeGrassWeightExporterFlags = EGrassWeightExporterFlags::NeedsHeightmap | EGrassWeightExporterFlags::RenderImmediately | EGrassWeightExporterFlags::ReadbackToCPU;
			FLandscapeGrassWeightExporter Exporter(Landscape, /*InRequestedComponents = */{ this }, LandscapeGrassWeightExporterFlags);

			while (!Exporter.IsAsyncReadbackComplete())
			{
				UE::Landscape::Private::SubmitGPUCommands(/* bBlockUntilRTComplete =  */ true, /* bBlockRTUntilGPUComplete =  */ true);
				bool bOutRenderCommandsQueued;
				Exporter.CheckAndUpdateAsyncReadback(bOutRenderCommandsQueued, /*bInForceFinish = */true);
			}

			TMap<ULandscapeComponent*, TUniquePtr<FLandscapeComponentGrassData>, TInlineSetAllocator<1>> TempGrassData;
			TempGrassData = Exporter.FetchResults(/* bFreeAsyncReadback= */ true);
			Results = TArray<uint16>(TempGrassData[this]->GetHeightData());
		}
		else
		{
			EGrassWeightExporterFlags LandscapeGrassWeightExporterFlags = EGrassWeightExporterFlags::RenderImmediately | EGrassWeightExporterFlags::ReadbackToCPU;
			TArray<int32> HeightMips;
			HeightMips.Add(LOD);
			FLandscapeGrassWeightExporter Exporter(Landscape, /*InRequestedComponents = */{ this }, LandscapeGrassWeightExporterFlags, /*InRequestedGrassTypes = */{}, MakeConstArrayView(HeightMips));

			while (!Exporter.IsAsyncReadbackComplete())
			{
				UE::Landscape::Private::SubmitGPUCommands(/* bBlockUntilRTComplete =  */ true, /* bBlockRTUntilGPUComplete =  */ true);
				bool bOutRenderCommandsQueued;
				Exporter.CheckAndUpdateAsyncReadback(bOutRenderCommandsQueued, /*bInForceFinish = */true);
			}

			TMap<ULandscapeComponent*, TUniquePtr<FLandscapeComponentGrassData>, TInlineSetAllocator<1>> TempGrassData;
			TempGrassData = Exporter.FetchResults(/* bFreeAsyncReadback= */ true);
			Results = MoveTemp(TempGrassData[this]->HeightMipData[LOD]);
		}
	}

	return Results;
}
#endif // WITH_EDITOR

void ULandscapeComponent::RemoveGrassMap()
{
	// this does a thread safe replacement of the existing grassdata with a newly allocated empty (invalid) one
	// this ensures if anyone else is accessing the old grassdata (while holding a shared ref to it), it won't be modified or deleted
	// this is also important for PIE, which can share grassdatas with editor
	GrassData = MakeShared<FLandscapeComponentGrassData>();
}

void ALandscapeProxy::UpdateGrassTypeSummary()
{
	SCOPE_CYCLE_COUNTER(STAT_UpdateGrassTypeSummary);

	bool bGrassQualityLevelEnabled = GEngine && GEngine->UseGrassVarityPerQualityLevels;

	bool bProxyHasAnyGrass = false;
	double ProxyMaxGrassCullDistance = 0.0;
	for (ULandscapeComponent* Component : LandscapeComponents)
	{
		double ComponentMaxGrassCullDistance;
		int32 ComponentGrassVarietyCount = 0;
		if (Component->GrassTypeSummary.bInvalid)
		{
			// Update the component grass type summary
			ComponentMaxGrassCullDistance = 0.0;
			TMap<FName, TObjectPtr<ULandscapeGrassType>>  ValidNamedGrassTypes = UE::Landscape::Grass::FilterValidNamedGrassTypes(Component->GetNamedGrassTypes());
			for (const TPair<FName, TObjectPtr<ULandscapeGrassType>>& ItPair : ValidNamedGrassTypes)
			{
				check(!ItPair.Key.IsNone() && (ItPair.Value != nullptr));

				for (FGrassVariety& GrassVariety : ItPair.Value->GrassVarieties)
				{
					if (GrassVariety.GrassMesh != nullptr)
					{
						int32 EndCullDistance = bGrassQualityLevelEnabled ? GrassVariety.EndCullDistanceQuality.GetValue(GGrassQualityLevel) : GrassVariety.EndCullDistance.GetValue();
						ComponentMaxGrassCullDistance = FMath::Max(ComponentMaxGrassCullDistance, EndCullDistance);
						ComponentGrassVarietyCount++;
					}
				}
			}
			Component->GrassTypeSummary.MaxInstanceDiscardDistance = ComponentMaxGrassCullDistance;
			Component->GrassTypeSummary.bHasAnyGrass = (ComponentGrassVarietyCount > 0);
			Component->GrassTypeSummary.bInvalid = false;
		}
		else
		{
			ComponentMaxGrassCullDistance = Component->GrassTypeSummary.MaxInstanceDiscardDistance;
		}

		bProxyHasAnyGrass = bProxyHasAnyGrass || Component->GrassTypeSummary.bHasAnyGrass;
		ProxyMaxGrassCullDistance = FMath::Max(ProxyMaxGrassCullDistance, ComponentMaxGrassCullDistance);
	}

	GrassTypeSummary.bHasAnyGrass = bProxyHasAnyGrass;
	GrassTypeSummary.MaxInstanceDiscardDistance = ProxyMaxGrassCullDistance;
	GrassTypeSummary.LandscapeComponentCount = LandscapeComponents.Num();
}

bool ALandscapeProxy::ShouldGenerateGrass() const
{
	if (!GGrassEnable)
	{
		return false;
	}

	// HACK [jonathan.bard] : for hotfix : don't let grass be rendered until all components of the proxy have been registered. That prevents grass maps to be rendered on those components, which 
	//  allows LandscapeInfo to be valid by the time the grass results are fetched
	if (!HasActorRegisteredAllComponents())
	{
		return false;
	}

	if (const ALandscape* LandscapeActor = GetLandscapeActor())
	{
		return LandscapeActor->GetGrassEnabled();
	}

	// No landscape actor available, don't generate grass
	return false;
}

// the purpose of this class is to copy the lightmap from the terrain, and set the CoordinateScale and CoordinateBias to zero.
// we re-use the same texture references, so the memory cost is relatively minimal.
class FLandscapeGrassLightMap : public FLightMap2D
{
public:
	FLandscapeGrassLightMap(const FLightMap2D& InLightMap)
		: FLightMap2D(InLightMap)
	{
		CoordinateScale = FVector2D::ZeroVector;
		CoordinateBias = FVector2D::ZeroVector;
	}
};

// the purpose of this class is to copy the shadowmap from the terrain, and set the CoordinateScale and CoordinateBias to zero.
// we re-use the same texture references, so the memory cost is relatively minimal.
class FLandscapeGrassShadowMap : public FShadowMap2D
{
public:
	FLandscapeGrassShadowMap(const FShadowMap2D& InShadowMap)
		: FShadowMap2D(InShadowMap)
	{
		CoordinateScale = FVector2D::ZeroVector;
		CoordinateBias = FVector2D::ZeroVector;
	}
};

//
// UMaterialExpressionLandscapeGrassOutput
//
FName UMaterialExpressionLandscapeGrassOutput::PinDefaultName = TEXT("Input");

UMaterialExpressionLandscapeGrassOutput::UMaterialExpressionLandscapeGrassOutput(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText STRING_Landscape;
		FName NAME_Grass;
		FConstructorStatics()
			: STRING_Landscape(LOCTEXT("Landscape", "Landscape"))
			, NAME_Grass("Grass")
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.STRING_Landscape);

	// No outputs
	Outputs.Reset();
#endif

	// Default input
	new(GrassTypes)FGrassInput(ConstructorStatics.NAME_Grass);
}

#if WITH_EDITOR
int32 UMaterialExpressionLandscapeGrassOutput::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (GrassTypes.IsValidIndex(OutputIndex))
	{
		// Default input to 0 if not connected.
		int32 CodeInput = GrassTypes[OutputIndex].Input.IsConnected() ? GrassTypes[OutputIndex].Input.Compile(Compiler) : Compiler->Constant(0.f);
		return Compiler->CustomOutput(this, OutputIndex, CodeInput);
	}

	return INDEX_NONE;
}

void UMaterialExpressionLandscapeGrassOutput::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Landscape Grass"));
}

TArrayView<FExpressionInput*> UMaterialExpressionLandscapeGrassOutput::GetInputsView()
{
	CachedInputs.Empty();
	CachedInputs.Reserve(GrassTypes.Num());
	for (auto& GrassType : GrassTypes)
	{
		CachedInputs.Add(&GrassType.Input);
	}
	return CachedInputs;
}

FExpressionInput* UMaterialExpressionLandscapeGrassOutput::GetInput(int32 InputIndex)
{
	return GrassTypes.IsValidIndex(InputIndex) ? &GrassTypes[InputIndex].Input : nullptr;
}

FName UMaterialExpressionLandscapeGrassOutput::GetInputName(int32 InputIndex) const
{
	return GrassTypes[InputIndex].Name;
}
#endif // WITH_EDITOR


#if WITH_EDITOR
void UMaterialExpressionLandscapeGrassOutput::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.MemberProperty)
	{
		const FName PropertyName = PropertyChangedEvent.MemberProperty->GetFName();
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UMaterialExpressionLandscapeGrassOutput, GrassTypes))
		{
			for (FGrassInput& Input : GrassTypes)
			{
				ValidateInputName(Input);
			}

			if (GraphNode)
			{
				GraphNode->ReconstructNode();
			}
		}
	}
}

void UMaterialExpressionLandscapeGrassOutput::ValidateInputName(FGrassInput& InInput) const
{
	if (Material != nullptr)
	{
		int32 NameIndex = 1;
		bool bFoundValidName = false;

		// Parameters cannot be named Name_None, use the default name instead
		FName PotentialName = (InInput.Name == NAME_None) ? UMaterialExpressionLandscapeGrassOutput::PinDefaultName : InInput.Name;

		// Find an available unique name
		while (!bFoundValidName)
		{
			if (NameIndex != 1)
			{
				PotentialName.SetNumber(NameIndex);
			}

			bFoundValidName = true;

			// Make sure the name is unique among others pins of this node
			for (const FGrassInput& OtherInput : GrassTypes)
			{
				if (&OtherInput != &InInput)
				{
					if (OtherInput.Name == PotentialName)
					{
						bFoundValidName = false;
						break;
					}
				}
			}

			++NameIndex;
		}

		InInput.Name = PotentialName;
	}
}
#endif

//
// FGrassVariety
//

FGrassVariety::FGrassVariety()
	: GrassMesh(nullptr)
	, GrassDensity(400)
	, GrassDensityQuality(400.0f)
	, bUseGrid(true)
	, PlacementJitter(1.0f)
	, StartCullDistance(10000)
	, StartCullDistanceQuality(10000)
	, EndCullDistance(10000)
	, EndCullDistanceQuality(10000)
	, MinLOD(-1)
	, AllowedDensityRange(0.0f, 1.0f)
	, Scaling(EGrassScaling::Uniform)
	, ScaleX(1.0f, 1.0f)
	, ScaleY(1.0f, 1.0f)
	, ScaleZ(1.0f, 1.0f)
	, RandomRotation(true)
	, AlignToSurface(true)
	, bAlignToTriangleNormals(true)
	, bUseLandscapeLightmap(false)
	, bReceivesDecals(true)
	, bAffectDistanceFieldLighting(false)
	, bCastDynamicShadow(true)
	, bCastContactShadow(true)
	, bKeepInstanceBufferCPUCopy(false)
	, InstanceWorldPositionOffsetDisableDistance(0)
	, ShadowCacheInvalidationBehavior(EShadowCacheInvalidationBehavior::Auto)
{
	GrassDensityQuality.SetQualityLevelCVarForCooking(GGrassQualityLevelCVarName, GGrassQualityLevelScalabilitySection);
	StartCullDistanceQuality.SetQualityLevelCVarForCooking(GGrassQualityLevelCVarName, GGrassQualityLevelScalabilitySection);
	EndCullDistanceQuality.SetQualityLevelCVarForCooking(GGrassQualityLevelCVarName, GGrassQualityLevelScalabilitySection);
}

bool FGrassVariety::IsGrassQualityLevelEnable() const
{
	return (GEngine && GEngine->UseGrassVarityPerQualityLevels);
}

int32 FGrassVariety::GetStartCullDistance() const
{
	if (IsGrassQualityLevelEnable())
	{
		return StartCullDistanceQuality.GetValue(GGrassQualityLevel);
	}
	else
	{
		return StartCullDistance.GetValue();
	}
}

int32 FGrassVariety::GetEndCullDistance() const
{
	if (IsGrassQualityLevelEnable())
	{
		return EndCullDistanceQuality.GetValue(GGrassQualityLevel);
	}
	else
	{
		return EndCullDistance.GetValue();
	}
}

float FGrassVariety::GetDensity() const
{
	if (IsGrassQualityLevelEnable())
	{
		return GrassDensityQuality.GetValue(GGrassQualityLevel);
	}
	else
	{
		return GrassDensity.GetValue();
	}
}


// ULandscapeGrassType
//

ULandscapeGrassType::ULandscapeGrassType(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	GrassDensity_DEPRECATED = 400;
	StartCullDistance_DEPRECATED = 10000;
	EndCullDistance_DEPRECATED = 10000;
	PlacementJitter_DEPRECATED = 1.0f;
	RandomRotation_DEPRECATED = true;
	AlignToSurface_DEPRECATED = true;
	bEnableDensityScaling = true;
}

void ULandscapeGrassType::PostLoad()
{
	Super::PostLoad();
	if (GrassMesh_DEPRECATED && !GrassVarieties.Num())
	{
		FGrassVariety Grass;
		Grass.GrassMesh = GrassMesh_DEPRECATED;
		Grass.GrassDensity = GrassDensity_DEPRECATED;
		Grass.StartCullDistance = StartCullDistance_DEPRECATED;
		Grass.EndCullDistance = EndCullDistance_DEPRECATED;
		Grass.PlacementJitter = PlacementJitter_DEPRECATED;
		Grass.RandomRotation = RandomRotation_DEPRECATED;
		Grass.AlignToSurface = AlignToSurface_DEPRECATED;

		GrassVarieties.Add(Grass);
		GrassMesh_DEPRECATED = nullptr;
	}

	StateHash = ComputeStateHash();
}

#if WITH_EDITOR
void ULandscapeGrassType::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Look for landscape components using this grass type, and clear their grass type caches
	if (GIsEditor)
	{
		for (TObjectIterator<ALandscapeProxy> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
		{
			ALandscapeProxy* Proxy = *It;
			if (Proxy->GetWorld() && !Proxy->GetWorld()->IsPlayInEditor())
			{
				for (ULandscapeComponent* Component : Proxy->LandscapeComponents)
				{
					if (const FName* FoundGrassNameForType = Component->GetNamedGrassTypes().FindKey(this))
					{
						Component->InvalidateGrassTypeSummary();
					}
				}
			}
		}
	}

	// Update the state hash (so we detect the change and rebuild the relevant grass)
	StateHash = ComputeStateHash();
}
#endif

uint32 ULandscapeGrassType::ComputeStateHash()
{
	uint32 Hash = 0;

	FArchiveCrc32 Ar;

    // Save and nullify previous Hash value (don't want to include it in the new calculated hash)
    uint32 PreviousHash = StateHash;
    StateHash = 0;

	Serialize(Ar);
	Hash = Ar.GetCrc();

    // Restore previous hash value
    StateHash = PreviousHash;

	return Hash;
}


//
// FLandscapeComponentGrassData
//
SIZE_T FLandscapeComponentGrassData::GetAllocatedSize() const
{
	return sizeof(*this) + HeightWeightData.GetAllocatedSize() + WeightOffsets.GetAllocatedSize();
}

bool FLandscapeComponentGrassData::HasWeightData() const
{
	return !!WeightOffsets.Num();
}

bool FLandscapeComponentGrassData::HasValidData() const
{
	// If NumElements < 0, its data wasn't computed. 
	// If == 0, its data was computed (i.e. is valid), but removed (for saving space) because its weight data is all zero : 
	// If > 0, its data was computed and contains non-zero weight data
	return (NumElements >= 0);
}

bool FLandscapeComponentGrassData::HasData() const
{
	if (HasValidData())
	{
		int32 LocalNumElements = HeightWeightData.Num();
#if WITH_EDITORONLY_DATA
		if (LocalNumElements == 0)
		{
			LocalNumElements = HeightMipData.Num();
		}
#endif
		return LocalNumElements > 0;
	}

	return false;
}

TArrayView<uint8> FLandscapeComponentGrassData::GetWeightData(const FName& GrassName)
{
	if (!HeightWeightData.IsEmpty())
	{
		if (int32* OffsetPtr = WeightOffsets.Find(GrassName))
		{
			int32 Offset = *OffsetPtr;
			check(Offset + NumElements <= HeightWeightData.Num());
			check(NumElements);
			return MakeArrayView<uint8>(&HeightWeightData[Offset], NumElements);
		}
	}

	return TArrayView<uint8>();
}

bool FLandscapeComponentGrassData::Contains(const FName& GrassName) const
{
	return WeightOffsets.Contains(GrassName);
}

TArrayView<uint16> FLandscapeComponentGrassData::GetHeightData()
{
	if (HeightWeightData.IsEmpty())
	{
		return TArrayView<uint16>();
	}

	check(NumElements <= HeightWeightData.Num());
	return MakeArrayView<uint16>((uint16*)&HeightWeightData[0], NumElements);
}

#if WITH_EDITOR
// Initialize function meant for deserializing from old data : the new format doesn't take ULandscapeGrassType* but FName :
void FLandscapeComponentGrassData::InitializeFrom(const TArray<uint16>& HeightData, const TMap<ULandscapeGrassType*, TArray<uint8>>& WeightData)
{
	WeightOffsets.Empty(WeightData.Num());

	// If weight data is empty make sure we don't have any memory allocated to grass
	if (WeightData.Num() == 0)
	{
		NumElements = 0;
		HeightWeightData.Empty();
		return;
	}

	NumElements = HeightData.Num();
	HeightWeightData.SetNumUninitialized(NumElements * sizeof(uint16) + NumElements * WeightData.Num() * sizeof(uint8));

	uint8* CopyDest = HeightWeightData.GetData();
	int32 CopyOffset = 0;
	int32 CopySize = HeightData.Num() * sizeof(uint16);

	check((CopyOffset + CopySize) <= HeightWeightData.Num());

	FMemory::Memcpy(&CopyDest[CopyOffset], HeightData.GetData(), CopySize);

	CopyOffset += CopySize;
	CopySize = NumElements * sizeof(uint8);
		
	for (const TPair<ULandscapeGrassType*,TArray<uint8>>& Pair : WeightData)
	{
		// At the time this data was serialized, weights were stored per ULandscapeGrassType. Nowadays, we use FNames instead, but for old data, we 
		//  use the ULandscapeGrassType's name as its grass name : 
		ULandscapeGrassType* GrassType = Pair.Key;
		FName GrassName = GrassType->GetGrassNameFallback();
		WeightOffsets.Add(GrassName, CopyOffset);
		check(Pair.Value.Num() == NumElements);
		check((CopyOffset + CopySize) <= HeightWeightData.Num());

		FMemory::Memcpy(&CopyDest[CopyOffset], Pair.Value.GetData(), CopySize);
		CopyOffset += CopySize;
	}
}

void FLandscapeComponentGrassData::FixupFallbackGrassNames(const TMap<FName, TObjectPtr<ULandscapeGrassType>>& InNamedGrassTypes)
{
	TMap<FName, int32> FixedUpWeightOffsets;
	FixedUpWeightOffsets.Reserve(WeightOffsets.Num());
	for (const TPair<FName, int32>& ItPair : WeightOffsets)
	{
		const FName& GrassNameToFixup = ItPair.Key;
		const int32 Offset = ItPair.Value;
		// If the name is already included in the official list of available grass names, keep it as is (failsafe, in case the function is run on already fixed-up data) :
		if (InNamedGrassTypes.Contains(GrassNameToFixup))
		{
			check(!FixedUpWeightOffsets.Contains(GrassNameToFixup));
			FixedUpWeightOffsets.Add(GrassNameToFixup, Offset);
		}
		else
		{
			if (const TPair<FName, TObjectPtr<ULandscapeGrassType>>* FoundPair = Algo::FindByPredicate(InNamedGrassTypes, [GrassNameToFixup](const TPair<FName, TObjectPtr<ULandscapeGrassType>>& InPair)
				{
					const ULandscapeGrassType* GrassType = InPair.Value;
					return (GrassType != nullptr) && (GrassType->GetGrassNameFallback() == GrassNameToFixup);
				}))
			{ 
				// We've found the new, proper, grass name corresponding to the grass type's fallback name, let's use this
				FixedUpWeightOffsets.Add(FoundPair->Key, Offset);
			}
			else
			{
				UE_LOGF(LogLandscape, Log, "Couldn't fixup grass name %ls : this grass type will be unavailable for spawning grass until generating new grass data", *GrassNameToFixup.ToString());
			}
		}
	}

	// Update the weight offsets with the new names : 
	Swap(FixedUpWeightOffsets, WeightOffsets);
}
#endif // WITH_EDITOR

// Note that this is a relatively expensive operation (in the runtime sense), as it's making a copy of the buffers
void FLandscapeComponentGrassData::InitializeFrom(IBuffer2DView<uint16>* HeightData, TMap<FName, IBuffer2DView<uint8>*>& WeightData, bool bStripEmptyWeights)
{
	const int32 OriginalGrassTypeCount = WeightData.Num();
	WeightOffsets.Empty(OriginalGrassTypeCount);

	// If weight data is empty make sure we don't have any memory allocated to grass
	if (OriginalGrassTypeCount == 0)
	{
		NumElements = 0;
		HeightWeightData.Empty();
		return;
	}

	NumElements = HeightData->Num();

	// reserve space for up to OriginalGrassTypeCount weight elements
	HeightWeightData.SetNumUninitialized(NumElements * sizeof(uint16) + NumElements * OriginalGrassTypeCount * sizeof(uint8));

	uint8* CopyDest = HeightWeightData.GetData();
	int32 CopyOffset = 0;
	int32 CopySize = NumElements * sizeof(uint16);

	check((CopyOffset + CopySize) <= HeightWeightData.Num());

	// copy the height data
	HeightData->CopyTo((uint16*)&CopyDest[CopyOffset], CopySize / sizeof(uint16));
	CopyOffset += CopySize;
	CopySize = NumElements * sizeof(uint8);

	for (const TPair<FName, IBuffer2DView<uint8>*>& Pair : WeightData)
	{
		check(Pair.Value->Num() == NumElements);
		check((CopyOffset + CopySize) <= HeightWeightData.Num());

		const bool bIsAllZero = Pair.Value->CopyToAndCalcIsAllZero(&CopyDest[CopyOffset], CopySize);

		if (bStripEmptyWeights && bIsAllZero)
		{
			// strip all zero weights
		}
		else
		{
			// record the weight data
			WeightOffsets.Add(Pair.Key, CopyOffset);
			CopyOffset += CopySize;
		}
	}

	// shrink if necessary to account for stripped weight data
	HeightWeightData.SetNum(CopyOffset);
}

FArchive& operator<<(FArchive& Ar, FLandscapeComponentGrassData& Data)
{
	Ar.UsingCustomVersion(FLandscapeCustomVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

#if WITH_EDITORONLY_DATA
	if (!Ar.IsFilterEditorOnly())
	{
		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::LandscapeSupportPerComponentGrassTypes)
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (Ar.CustomVer(FLandscapeCustomVersion::GUID) >= FLandscapeCustomVersion::GrassMaterialInstanceFix)
			{
				Ar << Data.MaterialStateIds_DEPRECATED;
			}
			else
			{
				Data.MaterialStateIds_DEPRECATED.Empty(1);
				if (Ar.UEVer() >= VER_UE4_SERIALIZE_LANDSCAPE_GRASS_DATA_MATERIAL_GUID)
				{
					FGuid MaterialStateId;
					Ar << MaterialStateId;
					Data.MaterialStateIds_DEPRECATED.Add(MaterialStateId);
				}
			}

			if (Ar.CustomVer(FLandscapeCustomVersion::GUID) >= FLandscapeCustomVersion::GrassMaterialWPO)
			{
				Ar << Data.RotationForWPO_DEPRECATED;
			}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		else
		{
			// MaterialStateIds and RotationForWPO have been replaced with a simple hash :
			Ar << Data.GenerationHash;
		}
	}

	TArray<uint16> DeprecatedHeightData;
	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::LandscapeGrassSingleArray)
	{
		DeprecatedHeightData.BulkSerialize(Ar);
	}

	if (!Ar.IsFilterEditorOnly())
	{
		if (Ar.CustomVer(FLandscapeCustomVersion::GUID) >= FLandscapeCustomVersion::CollisionMaterialWPO)
		{
			if (Ar.CustomVer(FLandscapeCustomVersion::GUID) >= FLandscapeCustomVersion::LightmassMaterialWPO)
			{
				// todo - BulkSerialize each mip?
				Ar << Data.HeightMipData;
			}
			else
			{
				checkSlow(Ar.IsLoading());

				TArray<uint16> CollisionHeightData;
				CollisionHeightData.BulkSerialize(Ar);
				if (CollisionHeightData.Num())
				{
					const int32 ComponentSizeQuads = static_cast<int32>(FMath::Sqrt(static_cast<float>(Data.GetHeightData().Num())) - 1);
					const int32 CollisionSizeQuads = static_cast<int32>(FMath::Sqrt(static_cast<float>(CollisionHeightData.Num())) - 1);
					const int32 CollisionMip = FMath::FloorLog2(ComponentSizeQuads / CollisionSizeQuads);
					Data.HeightMipData.Add(CollisionMip, MoveTemp(CollisionHeightData));
				}

				TArray<uint16> SimpleCollisionHeightData;
				SimpleCollisionHeightData.BulkSerialize(Ar);
				if (SimpleCollisionHeightData.Num())
				{
					const int32 ComponentSizeQuads = static_cast<int32>(FMath::Sqrt(static_cast<float>(Data.GetHeightData().Num())) - 1);
					const int32 SimpleCollisionSizeQuads = static_cast<int32>(FMath::Sqrt(static_cast<float>(SimpleCollisionHeightData.Num())) - 1);
					const int32 SimpleCollisionMip = FMath::FloorLog2(ComponentSizeQuads / SimpleCollisionSizeQuads);
					Data.HeightMipData.Add(SimpleCollisionMip, MoveTemp(SimpleCollisionHeightData));
				}
			}
		}
	}

	TMap<ULandscapeGrassType*, TArray<uint8>> DeprecatedWeightData;
	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::LandscapeGrassSingleArray)
	{
		Ar << DeprecatedWeightData;

		Data.InitializeFrom(DeprecatedHeightData, DeprecatedWeightData);
	}
	else if ((Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::OverridableLandscapeGrassTypes) && Ar.IsLoading())
	{
		Ar << Data.NumElements;

		TMap<ULandscapeGrassType*, int32> DeprecatedWeightOffsets;
		Ar << DeprecatedWeightOffsets;

		// Convert from ULandscapeGrassType* to fallback FNames for the new format (this will eventually get fixed up afterwards to proper grass names, extracted from the material, in ULandscapeComponent::PostLoad)
		for (const auto& ItPair : DeprecatedWeightOffsets)
		{
			if (ItPair.Key != nullptr)
			{
				Data.WeightOffsets.FindOrAdd(ItPair.Key->GetGrassNameFallback(), ItPair.Value);
			}
		}

		Ar << Data.HeightWeightData;
	}
	else
#endif // WITH_EDITORONLY_DATA
	{
		Ar << Data.NumElements;
		Ar << Data.WeightOffsets;
		Ar << Data.HeightWeightData;
	}

	return Ar;
}

void FLandscapeComponentGrassData::ConditionalDiscardDataOnLoad(const TMap<FName, TObjectPtr<ULandscapeGrassType>>& InNamedGrassTypes)
{
	//check(HasValidData());
	if (!GIsEditor && GGrassDiscardDataOnLoad)
	{
		bool bRemoved = false;
		// Remove data for grass types which have scalability enabled
		for (auto It = WeightOffsets.CreateIterator(); It; ++It)
		{
			const TObjectPtr<ULandscapeGrassType>* FoundGrassType = InNamedGrassTypes.Find(It->Key);
			if ((FoundGrassType == nullptr) || (*FoundGrassType == nullptr) || (*FoundGrassType)->bEnableDensityScaling)
			{
				It.RemoveCurrent();
				bRemoved = true;
			}
		}

		// If all grass types have been removed, discard the height data too.
		if (WeightOffsets.Num() == 0)
		{
			// NOTE: this is NOT thread safe, as we are overwriting the existing grassdata here
			// However if it is only performed on load, then we can assume no async tasks are holding shared refs to this yet.
			*this = FLandscapeComponentGrassData();
			NumElements = 0;
		}
		else if (bRemoved) 
		{
			TMap<FName, int32> PreviousOffsets(ObjectPtrDecay(MoveTemp(WeightOffsets)));
			TArray<uint8> PreviousHeightWeightData(MoveTemp(HeightWeightData));
			HeightWeightData.SetNumUninitialized(NumElements * sizeof(uint16) + NumElements * PreviousOffsets.Num() * sizeof(uint8), EAllowShrinking::Yes);

			uint8* CopyDest = HeightWeightData.GetData();
			uint8* CopySrc = PreviousHeightWeightData.GetData();
			int32 CopyOffset = 0;
			int32 CopySize = NumElements * sizeof(uint16);

			check((CopyOffset + CopySize) <= HeightWeightData.Num());

			FMemory::Memcpy(&CopyDest[CopyOffset], PreviousHeightWeightData.GetData(), CopySize);

			CopyOffset += CopySize;
			CopySize = NumElements * sizeof(uint8);

			for (const TPair<FName, int32>& Pair : PreviousOffsets)
			{
				int32 PreviousOffset = Pair.Value;
				WeightOffsets.Add(Pair.Key, CopyOffset);
				check((CopyOffset + CopySize) <= HeightWeightData.Num());
				check((PreviousOffset + CopySize) <= PreviousHeightWeightData.Num());

				FMemory::Memcpy(&CopyDest[CopyOffset], &CopySrc[PreviousOffset], CopySize);
				CopyOffset += CopySize;
	}
}
	}
}

//
// ALandscapeProxy grass-related functions
//

struct FGrassBuilderBase
{
	bool bHaveValidData;
	float GrassDensity;
	FVector DrawScale;
	FVector DrawLoc;
	FMatrix LandscapeToWorld;

	FIntPoint SectionBase;
	FIntPoint LandscapeSectionOffset;
	int32 ComponentSizeQuads;
	FVector Origin;
	FVector Extent;
	FVector ComponentOrigin;

	int32 SqrtMaxInstances;

	FGrassBuilderBase(ALandscapeProxy* Landscape, ULandscapeComponent* Component, const FGrassVariety& GrassVariety, ERHIFeatureLevel::Type FeatureLevel, int32 SqrtSubsections = 1, int32 SubX = 0, int32 SubY = 0, bool bEnableDensityScaling = true)
	{
		const float DensityScale = bEnableDensityScaling ? GGrassDensityScale : 1.0f;
		GrassDensity = GrassVariety.GetDensity() * DensityScale;

		DrawScale = Landscape->GetRootComponent()->GetRelativeScale3D();
		DrawLoc = Landscape->GetActorLocation();
		LandscapeSectionOffset = Landscape->GetSectionBase();

		SectionBase = Component->GetSectionBase();
		ComponentSizeQuads = Component->ComponentSizeQuads;

		Origin.X = DrawScale.X * SectionBase.X;
		Origin.Y = DrawScale.Y * SectionBase.Y;
		Origin.Z = 0.0f;

		Extent.X = DrawScale.X * ComponentSizeQuads;
		Extent.Y = DrawScale.Y * ComponentSizeQuads;
		Extent.Z = 0.0f;

		ComponentOrigin.X = DrawScale.X * (SectionBase.X - LandscapeSectionOffset.X);
		ComponentOrigin.Y = DrawScale.Y * (SectionBase.Y - LandscapeSectionOffset.Y);
		ComponentOrigin.Z = 0.0f;

		SqrtMaxInstances = FMath::CeilToInt32(FMath::Sqrt(FMath::Abs(Extent.X * Extent.Y * GrassDensity / 1000.0f / 1000.0f)));

		bHaveValidData = SqrtMaxInstances != 0;

		LandscapeToWorld = Landscape->GetRootComponent()->GetComponentTransform().ToMatrixNoScale();

		if (bHaveValidData && SqrtSubsections != 1)
		{
			check(SqrtMaxInstances > 2 * SqrtSubsections);
			SqrtMaxInstances /= SqrtSubsections;
			check(SqrtMaxInstances > 0);

			Extent.X /= SqrtSubsections;
			Extent.Y /= SqrtSubsections;

			Origin.X += Extent.X * SubX;
			Origin.Y += Extent.Y * SubY;
		}
	}
};

// FLandscapeComponentGrassAccess - accessor wrapper for data for one GrassType from one Component
struct FLandscapeComponentGrassAccess
{
	FLandscapeComponentGrassAccess(const ULandscapeComponent* InComponent, const FName& InGrassName)
	: GrassData(InComponent->GrassData)
	, HeightData(InComponent->GrassData->GetHeightData())
	, WeightData(InComponent->GrassData->GetWeightData(InGrassName))
	, Stride(InComponent->ComponentSizeQuads + 1)
	{}

	bool IsValid()
	{
		return WeightData.Num() == FMath::Square(Stride) && HeightData.Num() == FMath::Square(Stride);
	}

	FORCEINLINE float GetHeight(int32 IdxX, int32 IdxY)
	{
		return LandscapeDataAccess::GetLocalHeight(HeightData[IdxX + Stride*IdxY]);
	}
	FORCEINLINE float GetWeight(int32 IdxX, int32 IdxY) // usages
	{
		return ((float)WeightData[IdxX + Stride*IdxY]) / 255.f;
	}

	FORCEINLINE int32 GetStride()
	{
		return Stride;
	}

private:
	TSharedRef<FLandscapeComponentGrassData, ESPMode::ThreadSafe> GrassData;
	TArrayView<uint16> HeightData;
	TArrayView<uint8> WeightData;
	int32 Stride;
};

struct FAsyncGrassBuilder : public FGrassBuilderBase
{
	FLandscapeComponentGrassAccess GrassData;
	EGrassScaling Scaling;
	FFloatInterval ScaleX;
	FFloatInterval ScaleY;
	FFloatInterval ScaleZ;
	FFloatInterval AllowedDensityRange;
	bool bWeightAttenuatesMaxScale;
	float MaxScaleWeightAttenuation ;
	bool bRandomRotation;
	bool bRandomScale;
	bool bAlignToSurface;
	bool bAlignToTriangleNormals;
	float PlacementJitter;
	FRandomStream RandomStream;
	FMatrix XForm;
	FBox MeshBox;
	int32 DesiredInstancesPerLeaf;

	double BuildTime;
	int32 TotalInstances;
	uint32 HaltonBaseIndex;

	bool bUseLandscapeLightmap;
	FVector2D LightmapBaseBias;
	FVector2D LightmapBaseScale;
	FVector2D ShadowmapBaseBias;
	FVector2D ShadowmapBaseScale;
	FVector2D LightMapComponentBias;
	FVector2D LightMapComponentScale;
	bool bRequiresCPUAccess;

	TArray<FBox> ExcludedBoxes;

	// output
	FStaticMeshInstanceData InstanceBuffer;
	TArray<FClusterNode> ClusterTree;
	int32 OutOcclusionLayerNum;

	FAsyncGrassBuilder(ALandscapeProxy* Landscape, ULandscapeComponent* Component, const FName& GrassName, const ULandscapeGrassType* GrassType, const FGrassVariety& GrassVariety, 
		ERHIFeatureLevel::Type FeatureLevel, UGrassInstancedStaticMeshComponent* GrassInstancedStaticMeshComponent, int32 SqrtSubsections, int32 SubX, int32 SubY, uint32 InHaltonBaseIndex, TArray<FBox>& InExcludedBoxes)
		: FGrassBuilderBase(Landscape, Component, GrassVariety, FeatureLevel, SqrtSubsections, SubX, SubY, GrassType->bEnableDensityScaling)
		, GrassData(Component, GrassName)
		, Scaling(GrassVariety.Scaling)
		, ScaleX(GrassVariety.ScaleX)
		, ScaleY(GrassVariety.ScaleY)
		, ScaleZ(GrassVariety.ScaleZ)
		, AllowedDensityRange(GrassVariety.AllowedDensityRange)
		, bWeightAttenuatesMaxScale(GrassVariety.bWeightAttenuatesMaxScale)
		, MaxScaleWeightAttenuation(GrassVariety.MaxScaleWeightAttenuation)
		, bRandomRotation(GrassVariety.RandomRotation)
		, bRandomScale(false)
		, bAlignToSurface(GrassVariety.AlignToSurface)
		, bAlignToTriangleNormals(GrassVariety.bAlignToTriangleNormals)
		, PlacementJitter(GrassVariety.PlacementJitter)
		, RandomStream(GrassInstancedStaticMeshComponent->InstancingRandomSeed)
		, XForm(LandscapeToWorld * GrassInstancedStaticMeshComponent->GetComponentTransform().ToMatrixWithScale().Inverse())
		, MeshBox(GrassVariety.GrassMesh->GetBounds().GetBox())
		, DesiredInstancesPerLeaf(GrassInstancedStaticMeshComponent->DesiredInstancesPerLeaf())

		, BuildTime(0)
		, TotalInstances(0)
		, HaltonBaseIndex(InHaltonBaseIndex)

		, bUseLandscapeLightmap(GrassVariety.bUseLandscapeLightmap)
		, LightmapBaseBias(FVector2D::ZeroVector)
		, LightmapBaseScale(FVector2D::UnitVector)
		, ShadowmapBaseBias(FVector2D::ZeroVector)
		, ShadowmapBaseScale(FVector2D::UnitVector)
		, LightMapComponentBias(FVector2D::ZeroVector)
		, LightMapComponentScale(FVector2D::UnitVector)
		, bRequiresCPUAccess(GrassVariety.bKeepInstanceBufferCPUCopy)
		// output
		, InstanceBuffer(/*bSupportsVertexHalfFloat*/ true)
		, ClusterTree()
		, OutOcclusionLayerNum(0)
	{		

		switch (Scaling)
		{
		case EGrassScaling::Uniform:
			bRandomScale = ScaleX.Size() > 0;
			break;
		case EGrassScaling::Free:
			bRandomScale = ScaleX.Size() > 0 || ScaleY.Size() > 0 || ScaleZ.Size() > 0;
			break;
		case EGrassScaling::LockXY:
			bRandomScale = ScaleX.Size() > 0 || ScaleZ.Size() > 0;
			break;
		default:
			check(0);
		}

		if (InExcludedBoxes.Num())
		{
			ExcludedBoxes.Reserve(InExcludedBoxes.Num());
			FMatrix BoxXForm = GrassInstancedStaticMeshComponent->GetComponentToWorld().ToMatrixWithScale().Inverse() * XForm.Inverse();
			for (const FBox& Box : InExcludedBoxes)
			{
				ExcludedBoxes.Add(Box.TransformBy(BoxXForm));
			}
		}

		bHaveValidData = bHaveValidData && GrassData.IsValid();

		InstanceBuffer.SetAllowCPUAccess(bRequiresCPUAccess);

		check(DesiredInstancesPerLeaf > 0);

		if (bUseLandscapeLightmap)
		{
			InitLandscapeLightmap(Component);
		}
	}

	void InitLandscapeLightmap(ULandscapeComponent* Component)
	{
		const int32 SubsectionSizeQuads = Component->SubsectionSizeQuads;
		const int32 NumSubsections = Component->NumSubsections;
		const int32 LandscapeComponentSizeQuads = Component->ComponentSizeQuads;
	
		const int32 StaticLightingLOD = Component->GetLandscapeProxy()->StaticLightingLOD;
		const int32 ComponentSizeVerts = LandscapeComponentSizeQuads + 1;
		const float LightMapRes = Component->StaticLightingResolution > 0.0f ? Component->StaticLightingResolution : Component->GetLandscapeProxy()->StaticLightingResolution;
		const int32 LightingLOD = Component->GetLandscapeProxy()->StaticLightingLOD;

		// Calculate mapping from landscape to lightmap space for mapping landscape grass to the landscape lightmap
		// Copied from the calculation of FLandscapeUniformShaderParameters::LandscapeLightmapScaleBias in FLandscapeComponentSceneProxy::OnTransformChanged()
		int32 PatchExpandCountX = 0;
		int32 PatchExpandCountY = 0;
		int32 DesiredSize = 1;
		const float LightMapRatio = ::GetTerrainExpandPatchCount(LightMapRes, PatchExpandCountX, PatchExpandCountY, LandscapeComponentSizeQuads, (NumSubsections * (SubsectionSizeQuads + 1)), DesiredSize, LightingLOD);
		const float LightmapLODScaleX = LightMapRatio / ((ComponentSizeVerts >> StaticLightingLOD) + 2 * PatchExpandCountX);
		const float LightmapLODScaleY = LightMapRatio / ((ComponentSizeVerts >> StaticLightingLOD) + 2 * PatchExpandCountY);
		const float LightmapBiasX = PatchExpandCountX * LightmapLODScaleX;
		const float LightmapBiasY = PatchExpandCountY * LightmapLODScaleY;
		const float LightmapScaleX = LightmapLODScaleX * (float)((ComponentSizeVerts >> StaticLightingLOD) - 1) / LandscapeComponentSizeQuads;
		const float LightmapScaleY = LightmapLODScaleY * (float)((ComponentSizeVerts >> StaticLightingLOD) - 1) / LandscapeComponentSizeQuads;

		LightMapComponentScale = FVector2D(LightmapScaleX, LightmapScaleY) / FVector2D(DrawScale);
		LightMapComponentBias = FVector2D(LightmapBiasX, LightmapBiasY);

		const FMeshMapBuildData* MeshMapBuildData = Component->GetMeshMapBuildData();

		if (MeshMapBuildData != nullptr)
		{
			if (MeshMapBuildData->LightMap.IsValid())
			{
				LightmapBaseBias = MeshMapBuildData->LightMap->GetLightMap2D()->GetCoordinateBias();
				LightmapBaseScale = MeshMapBuildData->LightMap->GetLightMap2D()->GetCoordinateScale();
			}

			if (MeshMapBuildData->ShadowMap.IsValid())
			{
				ShadowmapBaseBias = MeshMapBuildData->ShadowMap->GetShadowMap2D()->GetCoordinateBias();
				ShadowmapBaseScale = MeshMapBuildData->ShadowMap->GetShadowMap2D()->GetCoordinateScale();
			}
		}
	}

	void SetInstance(int32 InstanceIndex, const FMatrix& InXForm, float RandomFraction)
	{
		if (bUseLandscapeLightmap)
		{
			FMatrix::FReal InstanceX = InXForm.M[3][0];
			FMatrix::FReal InstanceY = InXForm.M[3][1];

			FVector2D NormalizedGrassCoordinate;
			NormalizedGrassCoordinate.X = (InstanceX - ComponentOrigin.X) * LightMapComponentScale.X + LightMapComponentBias.X;
			NormalizedGrassCoordinate.Y = (InstanceY - ComponentOrigin.Y) * LightMapComponentScale.Y + LightMapComponentBias.Y;

			FVector2D LightMapCoordinate = NormalizedGrassCoordinate * LightmapBaseScale + LightmapBaseBias;
			FVector2D ShadowMapCoordinate = NormalizedGrassCoordinate * ShadowmapBaseScale + ShadowmapBaseBias;

			InstanceBuffer.SetInstance(InstanceIndex, FMatrix44f(InXForm), RandomStream.GetFraction(), LightMapCoordinate, ShadowMapCoordinate);
		}
		else
		{
			InstanceBuffer.SetInstance(InstanceIndex, FMatrix44f(InXForm), RandomStream.GetFraction());
		}
	}

	FVector GetDefaultScale() const
	{
		FVector Result( ScaleX.Min > 0.0f && FMath::IsNearlyZero(ScaleX.Size()) ? ScaleX.Min : 1.0f,
						ScaleY.Min > 0.0f && FMath::IsNearlyZero(ScaleY.Size()) ? ScaleY.Min : 1.0f,
						ScaleZ.Min > 0.0f && FMath::IsNearlyZero(ScaleZ.Size()) ? ScaleZ.Min : 1.0f);
		switch (Scaling)
		{
		case EGrassScaling::Uniform:
			Result.Y = Result.X;
			Result.Z = Result.X;
			break;
		case EGrassScaling::Free:
			break;
		case EGrassScaling::LockXY:
			Result.Y = Result.X;
			break;
		default:
			check(0);
		}
		return Result;
	}

	FVector GetRandomScale(float InWeight) const
	{
		FVector Result(1.0f);

		float WeightAttenuationFactor = 1.0f;		
		if (bWeightAttenuatesMaxScale)
		{
			WeightAttenuationFactor = MaxScaleWeightAttenuation < 1.0f ? FMath::Clamp((InWeight - MaxScaleWeightAttenuation) / (1.0f - MaxScaleWeightAttenuation), 0.0f, 1.0f) : 0.0f;
		}
		
		switch (Scaling)
		{
		case EGrassScaling::Uniform:
			Result.X = ScaleX.Interpolate(RandomStream.GetFraction() * WeightAttenuationFactor);
			Result.Y = Result.X;
			Result.Z = Result.X;
			break;
		case EGrassScaling::Free:
			Result.X = ScaleX.Interpolate(RandomStream.GetFraction() * WeightAttenuationFactor);
			Result.Y = ScaleY.Interpolate(RandomStream.GetFraction() * WeightAttenuationFactor);
			Result.Z = ScaleZ.Interpolate(RandomStream.GetFraction() * WeightAttenuationFactor);
			break;
		case EGrassScaling::LockXY:
			Result.X = ScaleX.Interpolate(RandomStream.GetFraction() * WeightAttenuationFactor);
			Result.Y = Result.X;
			Result.Z = ScaleZ.Interpolate(RandomStream.GetFraction() * WeightAttenuationFactor);
			break;
		default:
			check(0);
		}
	
		return Result;
	}

	bool IsExcluded(const FVector& LocationWithHeight)
	{
		for (const FBox& Box : ExcludedBoxes)
		{
			if (Box.IsInside(LocationWithHeight))
			{
				return true;
			}
		}
		return false;
	}

	void Build()
	{
		SCOPE_CYCLE_COUNTER(STAT_FoliageGrassAsyncBuildTime);
		check(bHaveValidData);
		double StartTime = FPlatformTime::Seconds();
		const FVector DefaultScale = GetDefaultScale();
		float Div = 1.0f / float(SqrtMaxInstances);
		TArray<FMatrix> InstanceTransforms;

		auto ApplyNormalAlignmentTransform = [](const FVector& Normal, FMatrix& InOutXForm)
			{
				// Note: NewZ should never be perpendicular to Z axis, as it is constructed from a heightmap.
				// Therefore it should never have a zero cross product with (0,-1,0)
				const FVector NewZ = Normal * FMath::Sign(Normal.Z);
				const FVector NewX = (FVector(0, -1, 0) ^ NewZ).GetSafeNormal();
				const FVector NewY = NewZ ^ NewX;
				const FMatrix Align = FMatrix(NewX, NewY, NewZ, FVector::ZeroVector);
				InOutXForm = InOutXForm * Align;
			};

		if (HaltonBaseIndex)
		{
			if (Extent.X < 0)
			{
				Origin.X += Extent.X;
				Extent.X *= -1.0f;
			}
			if (Extent.Y < 0)
			{
				Origin.Y += Extent.Y;
				Extent.Y *= -1.0f;
			}
			int32 MaxNum = SqrtMaxInstances * SqrtMaxInstances;
			InstanceTransforms.Reserve(MaxNum);
			FVector DivExtent(Extent * Div);
			for (int32 InstanceIndex = 0; InstanceIndex < MaxNum; InstanceIndex++)
			{
				float HaltonX = Halton(InstanceIndex + HaltonBaseIndex, 2);
				float HaltonY = Halton(InstanceIndex + HaltonBaseIndex, 3);
				FVector Location(Origin.X + HaltonX * Extent.X, Origin.Y + HaltonY * Extent.Y, 0.0f);
				FVector LocationWithHeight;
				FVector ComputedNormal;
				float Weight = 0.f;
				SampleLandscapeAtLocationLocal(Location, LocationWithHeight, Weight, bAlignToSurface ? &ComputedNormal : nullptr);
				bool bKeep = (Weight > AllowedDensityRange.Min) && (Weight <= AllowedDensityRange.Max) 
					&& (Weight >= RandomStream.GetFraction()) 
					&& !IsExcluded(LocationWithHeight);
				if (bKeep)
				{
					const FVector Scale = bRandomScale ? GetRandomScale(Weight) : DefaultScale;
					const float Rot = bRandomRotation ? RandomStream.GetFraction() * 360.0f : 0.0f;
					FMatrix OutXForm = FScaleRotationTranslationMatrix(Scale, FRotator(0.0f, Rot, 0.0f), FVector::ZeroVector);
					if (bAlignToSurface && !ComputedNormal.IsNearlyZero())
					{
						ApplyNormalAlignmentTransform(ComputedNormal, OutXForm);
					}
					OutXForm = OutXForm.ConcatTranslation(LocationWithHeight) * XForm;
					InstanceTransforms.Add(OutXForm);
				}
			}
			if (InstanceTransforms.Num())
			{
				TotalInstances += InstanceTransforms.Num();
				InstanceBuffer.AllocateInstances(InstanceTransforms.Num(), 0, EResizeBufferFlags::AllowSlackOnGrow | EResizeBufferFlags::AllowSlackOnReduce, true);
				for (int32 InstanceIndex = 0; InstanceIndex < InstanceTransforms.Num(); InstanceIndex++)
				{
					const FMatrix& OutXForm = InstanceTransforms[InstanceIndex];

					// NOTE: the random fraction parameter is not used, but we keep it to avoid changing the deterministic random behavior
					SetInstance(InstanceIndex, OutXForm, RandomStream.GetFraction());
				}
			}
		}
		else
		{
			int32 NumKept = 0;
			float MaxJitter1D = FMath::Clamp<float>(PlacementJitter, 0.0f, .99f) * Div * .5f;
			FVector MaxJitter(MaxJitter1D, MaxJitter1D, 0.0f);
			MaxJitter *= Extent;
			Origin += Extent * (Div * 0.5f);
			struct FInstanceLocal
			{
				FVector Pos;
				FVector Normal;
				bool bKeep;
				float Weight;
			};
			TArray<FInstanceLocal> Instances;
			Instances.AddUninitialized(SqrtMaxInstances * SqrtMaxInstances);
			{
				int32 InstanceIndex = 0;
				for (int32 xStart = 0; xStart < SqrtMaxInstances; xStart++)
				{
					for (int32 yStart = 0; yStart < SqrtMaxInstances; yStart++)
					{
						FVector Location(Origin.X + float(xStart) * Div * Extent.X, Origin.Y + float(yStart) * Div * Extent.Y, 0.0f);

						// NOTE: We evaluate the random numbers on the stack and store them in locals rather than inline within the FVector() constructor below, because 
						// the order of evaluation of function arguments in C++ is unspecified.  We really want this to behave consistently on all sorts of
						// different platforms!
						const float FirstRandom = RandomStream.GetFraction();
						const float SecondRandom = RandomStream.GetFraction();
						Location += FVector(FirstRandom * 2.0f - 1.0f, SecondRandom * 2.0f - 1.0f, 0.0f) * MaxJitter;

						FInstanceLocal& Instance = Instances[InstanceIndex];
						float Weight = 0.f;
						SampleLandscapeAtLocationLocal(Location, Instance.Pos, Weight, (bAlignToSurface && bAlignToTriangleNormals) ? &Instance.Normal : nullptr);
						Instance.bKeep = (Weight > AllowedDensityRange.Min) && (Weight <= AllowedDensityRange.Max) 
							&& (Weight >= RandomStream.GetFraction()) 
							&& !IsExcluded(Instance.Pos);
						Instance.Weight = Weight;
						if (Instance.bKeep)
						{
							NumKept++;
						}
						InstanceIndex++;
					}
				}
			}
			
			// A second pass is necessary because we need derivatives to calculate the smooth grid-based normal
			if (NumKept)
			{
				InstanceTransforms.AddUninitialized(NumKept);
				TotalInstances += NumKept;
				{
					InstanceBuffer.AllocateInstances(NumKept, 0, EResizeBufferFlags::AllowSlackOnGrow | EResizeBufferFlags::AllowSlackOnReduce, true);
					int32 InstanceIndex = 0;
					int32 OutInstanceIndex = 0;
					for (int32 xStart = 0; xStart < SqrtMaxInstances; xStart++)
					{
						for (int32 yStart = 0; yStart < SqrtMaxInstances; yStart++)
						{
							const FInstanceLocal& Instance = Instances[InstanceIndex];
							if (Instance.bKeep)
							{
								const FVector Scale = bRandomScale ? GetRandomScale(Instance.Weight) : DefaultScale;
								const float Rot = bRandomRotation ? RandomStream.GetFraction() * 360.0f : 0.0f;
								FMatrix OutXForm = FScaleRotationTranslationMatrix(Scale, FRotator(0.0f, Rot, 0.0f), FVector::ZeroVector);
								if (bAlignToSurface)
								{
									if (bAlignToTriangleNormals)
									{
										ApplyNormalAlignmentTransform(Instance.Normal, OutXForm);
									}
									else
									{
										// NOTE: This calculation of smooth normals uses jittered grid points to compute the derivatives.
										// The jitter is generally ok, though may ocasionally result in a small delta that is sensitive to error.
										// However, the grid may be at any resolution, so the derivative will be estimated from taking steps at that resolution.
										// which may result in normals that don't match the local terrain well, especially for low resolution grids that take large steps.
										FVector PosX1 = xStart ? Instances[InstanceIndex - SqrtMaxInstances].Pos : Instance.Pos;
										FVector PosX2 = (xStart + 1 < SqrtMaxInstances) ? Instances[InstanceIndex + SqrtMaxInstances].Pos : Instance.Pos;
										FVector PosY1 = yStart ? Instances[InstanceIndex - 1].Pos : Instance.Pos;
										FVector PosY2 = (yStart + 1 < SqrtMaxInstances) ? Instances[InstanceIndex + 1].Pos : Instance.Pos;

										// this only fails if your grid is 1x1, or if the jittered offsets of neighboring grids randomly intersected
										// in either case, we can't align, so leave the grass transform alone
										if (PosX1 != PosX2 && PosY1 != PosY2)
										{
											FVector Normal = ((PosX1 - PosX2) ^ (PosY1 - PosY2)).GetSafeNormal();
											ApplyNormalAlignmentTransform(Normal, OutXForm);
										}
									}
								}

								// apply translation and outer transform
								OutXForm = OutXForm.ConcatTranslation(Instance.Pos) * XForm;
								InstanceTransforms[OutInstanceIndex] = OutXForm;
								SetInstance(OutInstanceIndex++, OutXForm, RandomStream.GetFraction());
							}
							InstanceIndex++;
						}
					}
				}
			}
		}

		int32 NumInstances = InstanceTransforms.Num();
		if (NumInstances)
		{
			TArray<int32> SortedInstances;
			TArray<int32> InstanceReorderTable;
			TArray<float> InstanceCustomDataDummy;
			UGrassInstancedStaticMeshComponent::BuildTreeAnyThread(InstanceTransforms, InstanceCustomDataDummy, 0, MeshBox, ClusterTree, SortedInstances, InstanceReorderTable, OutOcclusionLayerNum, DesiredInstancesPerLeaf, false);

			// in-place sort the instances and generate the sorted instance data
			for (int32 FirstUnfixedIndex = 0; FirstUnfixedIndex < NumInstances; FirstUnfixedIndex++)
			{
				int32 LoadFrom = SortedInstances[FirstUnfixedIndex];				

				if (LoadFrom != FirstUnfixedIndex)
				{
					check(LoadFrom > FirstUnfixedIndex);
					InstanceBuffer.SwapInstance(FirstUnfixedIndex, LoadFrom);

					int32 SwapGoesTo = InstanceReorderTable[FirstUnfixedIndex];
					check(SwapGoesTo > FirstUnfixedIndex);
					check(SortedInstances[SwapGoesTo] == FirstUnfixedIndex);
					SortedInstances[SwapGoesTo] = LoadFrom;
					InstanceReorderTable[LoadFrom] = SwapGoesTo;

					InstanceReorderTable[FirstUnfixedIndex] = FirstUnfixedIndex;
					SortedInstances[FirstUnfixedIndex] = FirstUnfixedIndex;
				}
			}
		}
		BuildTime = FPlatformTime::Seconds() - StartTime;
	}

	FORCEINLINE_DEBUGGABLE void SampleLandscapeAtLocationLocal(const FVector& InLocation, FVector& OutLocation, float& OutLayerWeight, FVector* OutNormal = nullptr)
	{
		// Find location
		const float TestX = static_cast<float>(InLocation.X / DrawScale.X - SectionBase.X);
		const float TestY = static_cast<float>(InLocation.Y / DrawScale.Y - SectionBase.Y);

		// Find data
		const int32 X1 = FMath::FloorToInt(TestX);
		const int32 Y1 = FMath::FloorToInt(TestY);
		const int32 X2 = FMath::CeilToInt(TestX);
		const int32 Y2 = FMath::CeilToInt(TestY);

		// Clamp to prevent the sampling of the final columns from overflowing
		const int32 IdxX1 = FMath::Clamp<int32>(X1, 0, GrassData.GetStride() - 1);
		const int32 IdxY1 = FMath::Clamp<int32>(Y1, 0, GrassData.GetStride() - 1);
		const int32 IdxX2 = FMath::Clamp<int32>(X2, 0, GrassData.GetStride() - 1);
		const int32 IdxY2 = FMath::Clamp<int32>(Y2, 0, GrassData.GetStride() - 1);

		const float LerpX = FMath::Fractional(TestX);
		const float LerpY = FMath::Fractional(TestY);

		// Bilinear interpolate sampled weights
		const float SampleWeight11 = GrassData.GetWeight(IdxX1, IdxY1);
		const float SampleWeight21 = GrassData.GetWeight(IdxX2, IdxY1);
		const float SampleWeight12 = GrassData.GetWeight(IdxX1, IdxY2);
		const float SampleWeight22 = GrassData.GetWeight(IdxX2, IdxY2);

		OutLayerWeight = FMath::Lerp(
			FMath::Lerp(SampleWeight11, SampleWeight21, LerpX),
			FMath::Lerp(SampleWeight12, SampleWeight22, LerpX),
			LerpY);

		// Sample quad corner heights
		const float SampleHeight11 = GrassData.GetHeight(IdxX1, IdxY1);
		const float SampleHeight21 = GrassData.GetHeight(IdxX2, IdxY1);
		const float SampleHeight12 = GrassData.GetHeight(IdxX1, IdxY2);
		const float SampleHeight22 = GrassData.GetHeight(IdxX2, IdxY2);

		OutLocation.X = InLocation.X - DrawScale.X * float(LandscapeSectionOffset.X);
		OutLocation.Y = InLocation.Y - DrawScale.Y * float(LandscapeSectionOffset.Y);

		// Compute height using triangle barycentric coordinates to account for quad triangulation
		if (LerpX < LerpY)
		{
			double A = (1.0 - LerpY);
			double B = LerpX;
			double C = 1.0 - A - B;
			OutLocation.Z = DrawScale.Z * (SampleHeight11 * A + SampleHeight22 * B + SampleHeight12 * C);
		}
		else
		{
			double A = (1.0 - LerpX);
			double B = LerpX - LerpY;
			double C = 1.0 - A - B;
			OutLocation.Z = DrawScale.Z * (SampleHeight11 * A + SampleHeight21 * B + SampleHeight22 * C);
		}

		// Compute normal
		if (OutNormal)
		{
			const FVector P11 = FVector((float)X1, (float)Y1, SampleHeight11) * DrawScale;
			const FVector P22 = FVector((float)X2, (float)Y2, SampleHeight22) * DrawScale;
			
			// Choose triangle and compute normal
			if (LerpX > LerpY)
			{
				const FVector P21 = FVector((float)X2, (float)Y1, SampleHeight21) * DrawScale;
				*OutNormal = (P11 != P21 && P22 != P21) ? FVector(((P22 - P21) ^ (P11 - P21)).GetSafeNormal()) : FVector::UpVector;
			}
			else 
			{
				const FVector P12 = FVector((float)X1, (float)Y2, SampleHeight12) * DrawScale;
				*OutNormal = (P11 != P12 && P22 != P12) ? FVector(((P11 - P12) ^ (P22 - P12)).GetSafeNormal()) : FVector::UpVector;
			}
		}
	}
};

void ALandscapeProxy::FlushGrassComponents(const TSet<ULandscapeComponent*>* OnlyForComponents, bool bFlushGrassMaps)
{
	if (OnlyForComponents)
	{
		for (FCachedLandscapeFoliage::TGrassSet::TIterator Iter(FoliageCache.CachedGrassComps); Iter; ++Iter)
		{
			ULandscapeComponent* Component = (*Iter).Key.BasedOn.Get();
			// if the weak pointer in the cache is invalid, we should kill them anyway
			if (Component == nullptr || OnlyForComponents->Contains(Component))
			{
				UHierarchicalInstancedStaticMeshComponent *Used = (*Iter).Foliage.Get();
				if (Used)
				{
					SCOPE_CYCLE_COUNTER(STAT_FoliageGrassDestoryComp);
					Used->ClearInstances();
					Used->DetachFromComponent(FDetachmentTransformRules(EDetachmentRule::KeepRelative, false));
					Used->DestroyComponent();
					FoliageComponents.RemoveSwap(Used);
				}
				Iter.RemoveCurrent();
			}
		}

		if (bFlushGrassMaps)
		{
			for (ULandscapeComponent* Component : *OnlyForComponents)
			{
				Component->RemoveGrassMap();
			}
		}
	}
	else
	{
		// Clear old foliage component containers
		FoliageComponents.Empty();

		// Might as well clear the cache...
		FoliageCache.ClearCache();
		// Destroy any owned foliage components
		TInlineComponentArray<UHierarchicalInstancedStaticMeshComponent*> FoliageComps;
		GetComponents(FoliageComps);
		for (UHierarchicalInstancedStaticMeshComponent* Component : FoliageComps)
		{
			SCOPE_CYCLE_COUNTER(STAT_FoliageGrassDestoryComp);
			Component->ClearInstances();
			Component->DetachFromComponent(FDetachmentTransformRules(EDetachmentRule::KeepRelative, false));
			Component->DestroyComponent();
		}

		check(RootComponent != nullptr);
		TArray<USceneComponent*> AttachedFoliageComponents = RootComponent->GetAttachChildren().FilterByPredicate(
			[](USceneComponent* Component)
			{
				return Cast<UHierarchicalInstancedStaticMeshComponent>(Component);
			});

		// Destroy any attached but un-owned foliage components
		for (USceneComponent* Component : AttachedFoliageComponents)
		{
			SCOPE_CYCLE_COUNTER(STAT_FoliageGrassDestoryComp);
			CastChecked<UHierarchicalInstancedStaticMeshComponent>(Component)->ClearInstances();
			Component->DetachFromComponent(FDetachmentTransformRules(EDetachmentRule::KeepRelative, false));
			Component->DestroyComponent();
		}

#if WITH_EDITOR
		UWorld* World = GetWorld();

		if (bFlushGrassMaps)
		{
			// Clear GrassMaps
			for (UActorComponent* Component : GetComponents())
			{
				if (ULandscapeComponent* LandscapeComp = Cast<ULandscapeComponent>(Component))
				{
					LandscapeComp->RemoveGrassMap();
				}
			}
		}
#endif
	}
}

static uint32 GGrassExclusionChangeTag = 1;
static TMap<FWeakObjectPtr, FBox, FDefaultSetAllocator, TWeakObjectPtrMapKeyFuncs<FWeakObjectPtr, FBox>> GGrassExclusionBoxes;

void ALandscapeProxy::AddExclusionBox(FWeakObjectPtr Owner, const FBox& Box)
{
	INC_DWORD_STAT(STAT_GrassExclusionVolumes);

	GGrassExclusionBoxes.Add(Owner, Box);
	GGrassExclusionChangeTag++;
}
void ALandscapeProxy::RemoveExclusionBox(FWeakObjectPtr Owner)
{
	DEC_DWORD_STAT(STAT_GrassExclusionVolumes);

	GGrassExclusionBoxes.Remove(Owner);
	GGrassExclusionChangeTag++;
}
void ALandscapeProxy::RemoveAllExclusionBoxes()
{
	SET_DWORD_STAT(STAT_GrassExclusionVolumes, 0);

	if (GGrassExclusionBoxes.Num())
	{
		GGrassExclusionBoxes.Empty();
		GGrassExclusionChangeTag++;
	}
}

void ALandscapeProxy::RemoveInvalidExclusionBoxes()
{
	for (auto Iter = GGrassExclusionBoxes.CreateIterator(); Iter; ++Iter)
	{
		if (!Iter->Key.IsValid())
		{
			DEC_DWORD_STAT(STAT_GrassExclusionVolumes);

			Iter.RemoveCurrent();
			GGrassExclusionChangeTag++;
		}
	}

	GGrassExclusionBoxes.Compact();
}

void ALandscapeProxy::DebugDrawExclusionBoxes(const UWorld* InWorld)
{
#if !UE_BUILD_SHIPPING
	if (bGGrassDrawExclusionVolumes)
	{
		for (auto Iter = GGrassExclusionBoxes.CreateIterator(); Iter; ++Iter)
		{
			if (Iter->Key.IsValid())
			{
				const FBox& Box = Iter->Value;
				DrawDebugBox(InWorld, Box.GetCenter(), Box.GetExtent(), FColor::Red);
			}
		}
	}
#endif // !UE_BUILD_SHIPPING
}

#if WITH_EDITOR

void ALandscapeProxy::BuildGrassMaps()
{
	check(!HasAnyFlags(RF_ClassDefaultObject));

	// Update all grass types
	for (ULandscapeComponent* Component : LandscapeComponents)
	{
		if (Component->UpdateGrassTypes())
		{
			Component->MarkPackageDirty();
		}
	}
	UpdateGrassTypeSummary();

	// now ensure that all of our components have up-to-date grass maps
	ULandscapeSubsystem *Subsystem = GetWorld()->GetSubsystem<ULandscapeSubsystem>();
	if (Subsystem)
	{
		const bool bMarkDirty = true;
		Subsystem->GetGrassMapBuilder()->BuildGrassMapsNowForComponents(LandscapeComponents, /*InSlowTask = */nullptr, bMarkDirty);
	}
}


int32 ALandscapeProxy::GetOutdatedGrassMapCount() const
{
	int32 GrassMapsNeedingRender = 0;
	if (ShouldGenerateGrass())
	{
		GrassMapsNeedingRender = GetWorld()->GetSubsystem<ULandscapeSubsystem>()->GetGrassMapBuilder()->CountOutdatedGrassMaps(LandscapeComponents);
	}
	return GrassMapsNeedingRender;
}

#endif

// amortized update path (unless bForceSync is true)
void ALandscapeProxy::UpdateGrass(const TArray<FVector>& Cameras, bool bForceSync /* = false */)
{
	int32 InOutNumCompsCreated = 0;
	UpdateGrass(Cameras, InOutNumCompsCreated, bForceSync);
}

// amortized update path (unless bForceSync is true)
void ALandscapeProxy::UpdateGrass(const TArray<FVector>& Cameras, int32& InOutNumCompsCreated, bool bForceSync /* = false */)
{
	SCOPE_CYCLE_COUNTER(STAT_GrassUpdate);
	TRACE_CPUPROFILER_EVENT_SCOPE(ALandscapeProxy::UpdateGrass);

	// don't let it create grass actors inside a transaction: 
	if (GUndo != nullptr)
	{
		return;
	}

	UWorld* World = GetWorld();
	ULandscapeSubsystem* LandscapeSubsystem = World ? World->GetSubsystem<ULandscapeSubsystem>() : nullptr;
	const bool bIsGrassCreationPrioritized = LandscapeSubsystem ? LandscapeSubsystem->IsGrassCreationPrioritized() : false;
	const bool bWaitAsyncTasks = bForceSync || bIsGrassCreationPrioritized;

	if (ShouldGenerateGrass())
	{
		float GuardBand = GGuardBandMultiplier;
		float DiscardGuardBand = GGuardBandDiscardMultiplier;
		bool bCullSubsections = GCullSubsections > 0;
		bool bDisableGPUCull = GDisableGPUCull > 0;
		bool bDisableDynamicShadows = GDisableDynamicShadows > 0;
		int32 MaxInstancesPerComponent = FMath::Max<int32>(1024, GMaxInstancesPerComponent);
		int32 MaxTasks = GMaxAsyncTasks;
		int32 GrassMaxCreatePerFrame = GGrassMaxCreatePerFrame;
		if (bIsGrassCreationPrioritized && GGrassCreationPrioritizedMultipler > 1)
		{
			MaxTasks *= GGrassCreationPrioritizedMultipler;
			GrassMaxCreatePerFrame *= GGrassCreationPrioritizedMultipler;
		}
			
		const float CullDistanceScale = GGrassCullDistanceScale;

		if (World)
		{
			// Cull grass max distance based on Cull Distance scale factor and on max GuardBand factor.
			// Note this is the proxy combined distance, not the per-component distance...
			float GrassCullDistanceScalar = GGrassCullDistanceScale * FMath::Max(GGuardBandDiscardMultiplier, GGuardBandMultiplier);
			float ProxyGrassMaxCulledDiscardDistance = GrassTypeSummary.MaxInstanceDiscardDistance * GrassCullDistanceScalar;
			float ProxyGrassMaxSquareDiscardDistance = ProxyGrassMaxCulledDiscardDistance * ProxyGrassMaxCulledDiscardDistance;

			ERHIFeatureLevel::Type FeatureLevel = World->Scene->GetFeatureLevel();

			struct SortedLandscapeElement
			{
				SortedLandscapeElement(ULandscapeComponent* InComponent, float InMinDistance, const FBox& InBoundsBox)
				: Component(InComponent)
				, MinDistance(InMinDistance)
				, BoundsBox(InBoundsBox)
				{}

				ULandscapeComponent* Component;
				float MinDistance;
				FBox BoundsBox;
			};

			static TArray<SortedLandscapeElement> SortedLandscapeComponents;
			SortedLandscapeComponents.Reset(LandscapeComponents.Num());
			
			for (ULandscapeComponent* Component : LandscapeComponents)
			{
				// skip if we have no data and no way to generate it
				if (Component == nullptr || !Component->GrassData->HasValidData() || !Component->GrassData->HasWeightData())
				{
					continue;
				}

				FBoxSphereBounds WorldBounds = Component->CalcBounds(Component->GetComponentTransform());			
				float MinSqrDistanceToComponent = Cameras.Num() ? MAX_flt : 0.0f;				
				for (const FVector& CameraPos : Cameras)
				{
					MinSqrDistanceToComponent = FMath::Min<float>(MinSqrDistanceToComponent, static_cast<float>(WorldBounds.ComputeSquaredDistanceFromBoxToPoint(CameraPos)));
				}

				if ((ProxyGrassMaxSquareDiscardDistance > 0.f) && (MinSqrDistanceToComponent > ProxyGrassMaxSquareDiscardDistance))
				{
					continue;
				}

				SortedLandscapeComponents.Emplace(Component, FMath::Sqrt(MinSqrDistanceToComponent), WorldBounds.GetBox());
			}
			
			// Prioritize components that are closer to camera when grass creation is prioritized
			bool bShouldSort = bIsGrassCreationPrioritized;
#if WITH_EDITOR
			// Also when editing landscape (for a more reactive update)
			bShouldSort = bShouldSort || GLandscapeEditModeActive;
#endif
			if (bShouldSort)
			{
				Algo::Sort(SortedLandscapeComponents, [](const SortedLandscapeElement& A, const SortedLandscapeElement& B) { return A.MinDistance < B.MinDistance; });
			}

			// for every component that passes above, we iterate all grass types, grass varieties, and subsections (culling along the way)
			// every subsection gets looked up in the cache and touched (to indicate it is still valid)
			// if the subsection doesn't exist we try to start creating it (hence why we sorted the list in priority order)
			for (const SortedLandscapeElement& SortedLandscapeComponent : SortedLandscapeComponents)
			{
				ULandscapeComponent* Component = SortedLandscapeComponent.Component;
				float MinDistanceToComp = SortedLandscapeComponent.MinDistance;

				// update the component's grass exclusion boxes if necessary
				if (Component->ChangeTag != GGrassExclusionChangeTag)
				{
					Component->ActiveExcludedBoxes.Empty();
					if (GGrassExclusionBoxes.Num() && GIgnoreExcludeBoxes == 0)
					{
						const FBox& WorldBox = SortedLandscapeComponent.BoundsBox;

						for (const TPair<FWeakObjectPtr, FBox>& Pair : GGrassExclusionBoxes)
						{
							if (Pair.Value.Intersect(WorldBox))
							{
								if (Pair.Key.IsValid())
								{
									// this will also filter out boxes that are completely inside of the other boxes
									Component->ActiveExcludedBoxes.AddUnique(ULandscapeComponent::FExcludeBox(Pair.Value));
								}
							}
						}

						Component->ActiveExcludedBoxes.Shrink();

						//Sort exclude boxes by their volume size, as the biggest boxes are more likely to exclude points and early out when building grass
						Component->ActiveExcludedBoxes.Sort([](const ULandscapeComponent::FExcludeBox& A, const ULandscapeComponent::FExcludeBox& B) {
								return A.Box.GetVolume() > B.Box.GetVolume();
							}
						);
					}
					Component->ChangeTag = GGrassExclusionChangeTag;
				}

				const TMap<FName, TObjectPtr<ULandscapeGrassType>> ValidNamedGrassTypes = UE::Landscape::Grass::FilterValidNamedGrassTypes(Component->GetNamedGrassTypes());
				for (const TPair<FName, TObjectPtr<ULandscapeGrassType>>& ItPair : ValidNamedGrassTypes)
				{
					const FName GrassName = ItPair.Key;
					TObjectPtr<ULandscapeGrassType> GrassType = ItPair.Value;
					check(!GrassName.IsNone() && (GrassType != nullptr));

					if (World->IsGameWorld() && !Component->GrassData->Contains(GrassName))		// TODO [chris.tchou] why do we only do this in game worlds?
					{
						continue;
					}

					int32 GrassVarietyIndex = -1;
					uint32 HaltonBaseIndex = 1;
					for (auto& GrassVariety : GrassType->GrassVarieties)
					{
						GrassVarietyIndex++;
						int32 EndCullDistance = GrassVariety.GetEndCullDistance();
						if (GrassVariety.GrassMesh && GrassVariety.GetDensity() > 0.0f && EndCullDistance > 0)
						{
							float MustHaveDistance = GuardBand * (float)EndCullDistance * CullDistanceScale;
							float DiscardDistance = DiscardGuardBand * (float)EndCullDistance * CullDistanceScale;

							bool bUseHalton = !GrassVariety.bUseGrid;

							if (!bUseHalton && MinDistanceToComp > DiscardDistance)
							{
								continue; // we can early out if we're not using halton sequence (otherwise we have to at least update the HaltonBaseIndex)
							}

							FGrassBuilderBase ForSubsectionMath(this, Component, GrassVariety, FeatureLevel);
							int32 SqrtSubsections = 1;
							if (ForSubsectionMath.bHaveValidData && ForSubsectionMath.SqrtMaxInstances > 0)
							{
								SqrtSubsections = FMath::Clamp<int32>(FMath::CeilToInt(float(ForSubsectionMath.SqrtMaxInstances) / FMath::Sqrt((float)MaxInstancesPerComponent)), 1, 16);
							}
							int32 MaxInstancesSub = FMath::Square(ForSubsectionMath.SqrtMaxInstances / SqrtSubsections);

							if (bUseHalton && MinDistanceToComp > DiscardDistance)
							{
								HaltonBaseIndex += MaxInstancesSub * SqrtSubsections * SqrtSubsections;
								continue; // now that we have updated the halton base index, it's ok to skip the rest
							}

							FBox LocalBox = Component->CachedLocalBox;
							FVector LocalExtentDiv = (LocalBox.Max - LocalBox.Min) * FVector(1.0f / float(SqrtSubsections), 1.0f / float(SqrtSubsections), 1.0f);
							for (int32 SubX = 0; SubX < SqrtSubsections; SubX++)
							{
								for (int32 SubY = 0; SubY < SqrtSubsections; SubY++)
								{
									float MinDistanceToSubComp = MinDistanceToComp;

									FBox WorldSubBox;

									if ((bCullSubsections && SqrtSubsections > 1) || Component->ActiveExcludedBoxes.Num())
									{
										FVector BoxMin;
										BoxMin.X = LocalBox.Min.X + LocalExtentDiv.X * float(SubX);
										BoxMin.Y = LocalBox.Min.Y + LocalExtentDiv.Y * float(SubY);
										BoxMin.Z = LocalBox.Min.Z;

										FVector BoxMax;
										BoxMax.X = LocalBox.Min.X + LocalExtentDiv.X * float(SubX + 1);
										BoxMax.Y = LocalBox.Min.Y + LocalExtentDiv.Y * float(SubY + 1);
										BoxMax.Z = LocalBox.Max.Z;

										FBox LocalSubBox(BoxMin, BoxMax);
										WorldSubBox = LocalSubBox.TransformBy(Component->GetComponentTransform());

										if (bCullSubsections && SqrtSubsections > 1)
										{
											MinDistanceToSubComp = Cameras.Num() ? MAX_flt : 0.0f;
											for (auto& Pos : Cameras)
											{
												MinDistanceToSubComp = FMath::Min<float>(MinDistanceToSubComp, static_cast<float>(ComputeSquaredDistanceFromBoxToPoint(WorldSubBox.Min, WorldSubBox.Max, Pos)));
											}
											MinDistanceToSubComp = FMath::Sqrt(MinDistanceToSubComp);
										}
									}

									if (bUseHalton)
									{
										HaltonBaseIndex += MaxInstancesSub;  // we are going to pre-increment this for all of the continues...however we need to subtract later if we actually do this sub
									}

									if (MinDistanceToSubComp > DiscardDistance)
									{
										continue;
									}

									FCachedLandscapeFoliage::FGrassComp NewComp;
									NewComp.Key.BasedOn = Component;
									NewComp.Key.GrassType = GrassType;
									NewComp.Key.GrassName = GrassName;
									NewComp.Key.SqrtSubsections = SqrtSubsections;
									NewComp.Key.CachedMaxInstancesPerComponent = MaxInstancesPerComponent;
									NewComp.Key.SubsectionX = SubX;
									NewComp.Key.SubsectionY = SubY;
									NewComp.Key.NumVarieties = GrassType->GrassVarieties.Num();
									NewComp.Key.VarietyIndex = GrassVarietyIndex;

									bool bRebuildForBoxes = false;

									{
										FCachedLandscapeFoliage::FGrassComp* Existing = FoliageCache.CachedGrassComps.Find(NewComp.Key);
										if (Existing && !Existing->PreviousFoliage.IsValid() && Existing->ExclusionChangeTag != GGrassExclusionChangeTag && !Existing->PendingRemovalRebuild && !Existing->Pending)
										{
											for (const ULandscapeComponent::FExcludeBox& BoxWrapper : Component->ActiveExcludedBoxes)
											{
												if (BoxWrapper.Box.Intersect(WorldSubBox))
												{
													NewComp.ExcludedBoxes.Add(BoxWrapper.Box);
												}
											}
											NewComp.ExcludedBoxes.Shrink();
											if (NewComp.ExcludedBoxes != Existing->ExcludedBoxes)
											{
												bRebuildForBoxes = true;
												NewComp.PreviousFoliage = Existing->Foliage;
												Existing->PendingRemovalRebuild = true;
											}
											else
											{
												Existing->ExclusionChangeTag = GGrassExclusionChangeTag;
											}
										}

										if (Existing || MinDistanceToSubComp > MustHaveDistance)
										{
											if (Existing)
											{
												Existing->Touch();
											}
											if (!bRebuildForBoxes)
											{
												continue;
											}
										}
									}

									if (!bRebuildForBoxes && !bForceSync && (InOutNumCompsCreated >= GrassMaxCreatePerFrame || AsyncFoliageTasks.Num() >= MaxTasks))
									{
										continue; // one per frame, but we still want to touch the existing ones and we must do the rebuilds because we changed the tag
									}
									if (!bRebuildForBoxes)
									{
										for (const ULandscapeComponent::FExcludeBox& BoxWrapper : Component->ActiveExcludedBoxes)
										{
											if (BoxWrapper.Box.Intersect(WorldSubBox))
											{
												NewComp.ExcludedBoxes.Add(BoxWrapper.Box);
											}
										}
										NewComp.ExcludedBoxes.Shrink();
									}
									NewComp.ExclusionChangeTag = GGrassExclusionChangeTag;

									check(Component->GrassData->HasValidData());

									InOutNumCompsCreated++;

									SCOPE_CYCLE_COUNTER(STAT_FoliageGrassStartComp);

									// To guarantee consistency across platforms, we force the string to be lowercase and always treat it as an ANSI string.
									int32 FolSeed = FCrc::StrCrc32(StringCast<ANSICHAR>(*FString::Printf(TEXT("%s_%s_%d_%d_%d_%d_%d"),
										*GrassName.ToString().ToLower(), *GrassType->GetName().ToLower(), Component->GetSectionBase().X, Component->GetSectionBase().Y, SubX, SubY, GrassVarietyIndex)).Get());
									if (FolSeed == 0)
									{
										FolSeed++;
									}

									// Do not record the transaction of creating temp component for visualizations
									ClearFlags(RF_Transactional);
									bool PreviousPackageDirtyFlag = GetOutermost()->IsDirty();

									UGrassInstancedStaticMeshComponent* GrassInstancedStaticMeshComponent;
									{
										QUICK_SCOPE_CYCLE_COUNTER(STAT_GrassCreateComp);
										GrassInstancedStaticMeshComponent = NewObject<UGrassInstancedStaticMeshComponent>(this, NAME_None, RF_Transient);
									}
									NewComp.Foliage = GrassInstancedStaticMeshComponent;
									FoliageCache.CachedGrassComps.Add(NewComp);

									GrassInstancedStaticMeshComponent->Mobility = EComponentMobility::Static;
									GrassInstancedStaticMeshComponent->SetStaticMesh(GrassVariety.GrassMesh);
									GrassInstancedStaticMeshComponent->MinLOD = GrassVariety.MinLOD;
									GrassInstancedStaticMeshComponent->bSelectable = false;
									GrassInstancedStaticMeshComponent->bHasPerInstanceHitProxies = false;
									GrassInstancedStaticMeshComponent->bReceivesDecals = GrassVariety.bReceivesDecals;
									static FName NoCollision("NoCollision");
									GrassInstancedStaticMeshComponent->SetCollisionProfileName(NoCollision);
									GrassInstancedStaticMeshComponent->bDisableCollision = true;
									GrassInstancedStaticMeshComponent->SetCanEverAffectNavigation(false);
									GrassInstancedStaticMeshComponent->InstancingRandomSeed = FolSeed;
									GrassInstancedStaticMeshComponent->LightingChannels = GrassVariety.LightingChannels;
									GrassInstancedStaticMeshComponent->bHoldout = bHoldout;
									GrassInstancedStaticMeshComponent->bCastStaticShadow = false;
									GrassInstancedStaticMeshComponent->CastShadow = (GrassVariety.bCastDynamicShadow || GrassVariety.bCastContactShadow) && !bDisableDynamicShadows;
									GrassInstancedStaticMeshComponent->bAffectDistanceFieldLighting = GrassVariety.bAffectDistanceFieldLighting;
									GrassInstancedStaticMeshComponent->bCastDynamicShadow = GrassVariety.bCastDynamicShadow && !bDisableDynamicShadows;
									GrassInstancedStaticMeshComponent->bCastContactShadow = GrassVariety.bCastContactShadow && !bDisableDynamicShadows;
									GrassInstancedStaticMeshComponent->OverrideMaterials = GrassVariety.OverrideMaterials;
									GrassInstancedStaticMeshComponent->bEvaluateWorldPositionOffset = true;
									GrassInstancedStaticMeshComponent->WorldPositionOffsetDisableDistance = GrassVariety.InstanceWorldPositionOffsetDisableDistance;
									GrassInstancedStaticMeshComponent->ShadowCacheInvalidationBehavior = GrassVariety.ShadowCacheInvalidationBehavior;

									GrassInstancedStaticMeshComponent->PrecachePSOs();

									const FMeshMapBuildData* MeshMapBuildData = Component->GetMeshMapBuildData();

									if (GrassVariety.bUseLandscapeLightmap
										&& GrassVariety.GrassMesh->GetNumLODs() > 0
										&& MeshMapBuildData
										&& MeshMapBuildData->LightMap)
									{
										GrassInstancedStaticMeshComponent->SetLODDataCount(GrassVariety.GrassMesh->GetNumLODs(), GrassVariety.GrassMesh->GetNumLODs());

										FLightMapRef GrassLightMap = new FLandscapeGrassLightMap(*MeshMapBuildData->LightMap->GetLightMap2D());
										FShadowMapRef GrassShadowMap = MeshMapBuildData->ShadowMap ? new FLandscapeGrassShadowMap(*MeshMapBuildData->ShadowMap->GetShadowMap2D()) : nullptr;

										for (auto& LOD : GrassInstancedStaticMeshComponent->LODData)
										{
											// This trasient OverrideMapBuildData will be cleaned up by UMapBuildDataRegistry::CleanupTransientOverrideMapBuildData() if the underlying MeshMapBuildData is gone
											LOD.OverrideMapBuildData = MakeUnique<FMeshMapBuildData>();
											LOD.OverrideMapBuildData->LightMap = GrassLightMap;
											LOD.OverrideMapBuildData->ShadowMap = GrassShadowMap;
											LOD.OverrideMapBuildData->ResourceCluster = MeshMapBuildData->ResourceCluster;
										}
									}

									if (!Cameras.Num() || bDisableGPUCull)
									{
										// if we don't have any cameras, then we are rendering landscape LOD materials or somesuch and we want to disable culling
										GrassInstancedStaticMeshComponent->InstanceStartCullDistance = 0;
										GrassInstancedStaticMeshComponent->InstanceEndCullDistance = 0;
									}
									else
									{
										GrassInstancedStaticMeshComponent->InstanceStartCullDistance = static_cast<int32>(GrassVariety.GetStartCullDistance() * CullDistanceScale);
										GrassInstancedStaticMeshComponent->InstanceEndCullDistance = static_cast<int32>(GrassVariety.GetEndCullDistance() * CullDistanceScale);
									}

									{
										QUICK_SCOPE_CYCLE_COUNTER(STAT_GrassAttachComp);

										GrassInstancedStaticMeshComponent->AttachToComponent(GetRootComponent(), FAttachmentTransformRules::KeepRelativeTransform);
										FTransform DesiredTransform = GetRootComponent()->GetComponentTransform();
										DesiredTransform.RemoveScaling();
										GrassInstancedStaticMeshComponent->SetWorldTransform(DesiredTransform);

										FoliageComponents.Add(GrassInstancedStaticMeshComponent);
									}

									FAsyncGrassBuilder* Builder;

									{
										QUICK_SCOPE_CYCLE_COUNTER(STAT_GrassCreateBuilder);

										uint32 HaltonIndexForSub = 0;
										if (bUseHalton)
										{
											check(HaltonBaseIndex > (uint32)MaxInstancesSub);
											HaltonIndexForSub = HaltonBaseIndex - (uint32)MaxInstancesSub;
										}
										Builder = new FAsyncGrassBuilder(this, Component, GrassName, GrassType, GrassVariety, FeatureLevel, GrassInstancedStaticMeshComponent, SqrtSubsections, SubX, SubY, HaltonIndexForSub, NewComp.ExcludedBoxes);
									}

									if (Builder->bHaveValidData)
									{
										FAsyncTask<FAsyncGrassTask>* Task = new FAsyncTask<FAsyncGrassTask>(Builder, NewComp.Key, GrassInstancedStaticMeshComponent);

										Task->StartBackgroundTask();

										AsyncFoliageTasks.Add(Task);
									}
									else
									{
										delete Builder;
									}
									{
										QUICK_SCOPE_CYCLE_COUNTER(STAT_GrassRegisterComp);

										GrassInstancedStaticMeshComponent->RegisterComponent();
									}

									SetFlags(RF_Transactional);
									GetOutermost()->SetDirtyFlag(PreviousPackageDirtyFlag);
								}
							}
						}
					}
				}
			}
		}
	}

	TSet<UHierarchicalInstancedStaticMeshComponent *> StillUsed;
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_Grass_StillUsed);

		// trim cached items based on time, pending and emptiness
		double OldestToKeepTime = FPlatformTime::Seconds() - GMinTimeToKeepGrass;
		uint32 OldestToKeepFrame = GFrameNumber - GMinFramesToKeepGrass * GetGrassUpdateInterval();
		for (FCachedLandscapeFoliage::TGrassSet::TIterator Iter(FoliageCache.CachedGrassComps); Iter; ++Iter)
		{
			const FCachedLandscapeFoliage::FGrassComp& GrassItem = *Iter;
			UHierarchicalInstancedStaticMeshComponent *Used = GrassItem.Foliage.Get();
			UHierarchicalInstancedStaticMeshComponent *UsedPrev = GrassItem.PreviousFoliage.Get();
			bool bOld =
				!GrassItem.Pending &&
				(
					!GrassItem.Key.BasedOn.Get() ||
					!GrassItem.Key.GrassType.Get() ||
					!Used ||
				(GrassItem.LastUsedFrameNumber < OldestToKeepFrame && GrassItem.LastUsedTime < OldestToKeepTime)
				);
			if (bOld)
			{
				Iter.RemoveCurrent();
			}
			else if (Used || UsedPrev)
			{
				if (!StillUsed.Num())
				{
					StillUsed.Reserve(FoliageCache.CachedGrassComps.Num());
				}
				if (Used)
				{
					StillUsed.Add(Used);
				}
				if (UsedPrev)
				{
					StillUsed.Add(UsedPrev);
				}
			}
		}
	}

	// remove components that are no longer used from the components list
	if (StillUsed.Num() < FoliageComponents.Num())
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_Grass_DelComps);

		// delete components that are no longer used
		for (int32 Index = 0; Index < FoliageComponents.Num(); Index++)
		{
			UHierarchicalInstancedStaticMeshComponent* HComponent = FoliageComponents[Index];
			if (!StillUsed.Contains(HComponent))
			{
				{
					SCOPE_CYCLE_COUNTER(STAT_FoliageGrassDestoryComp);
					if (HComponent)
					{
						HComponent->ClearInstances();
						HComponent->DetachFromComponent(FDetachmentTransformRules(EDetachmentRule::KeepRelative, false));
						HComponent->DestroyComponent();
					}
					FoliageComponents.RemoveAtSwap(Index--);
				}
				if (!bForceSync)
				{
					break; // one per frame is fine
				}
			}
		}
	}
	
	// check if any async tasks have completed, and finalize them
	ProcessAsyncGrassInstanceTasks(bWaitAsyncTasks, bForceSync, StillUsed);
}

void ALandscapeProxy::ProcessAsyncGrassInstanceTasks(bool bWaitAsyncTasks, bool bForceSync, const TSet<UHierarchicalInstancedStaticMeshComponent*>& StillUsed)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_Grass_FinishAsync);
	// finish async tasks
	for (int32 Index = 0; Index < AsyncFoliageTasks.Num(); Index++)
	{
		FAsyncTask<FAsyncGrassTask>* Task = AsyncFoliageTasks[Index];
		if (bWaitAsyncTasks)
		{
			Task->EnsureCompletion();
		}
		if (Task->IsDone())
		{
			SCOPE_CYCLE_COUNTER(STAT_FoliageGrassEndComp);
			FAsyncGrassTask& Inner = Task->GetTask();
			// We need to preserve the order here, otherwise we'll have new jobs that are added to the end jumping up in front of the queue and an original second job would be updated the last
			AsyncFoliageTasks.RemoveAt(Index--);
			UGrassInstancedStaticMeshComponent* GrassISMComponent = Cast<UGrassInstancedStaticMeshComponent>(Inner.Foliage.Get());
			int32 NumBuiltRenderInstances = Inner.Builder->InstanceBuffer.GetNumInstances();
			//UE_LOGF(LogCore, Display, "%d instances in %4.0fms     %6.0f instances / sec", NumBuiltRenderInstances, 1000.0f * float(Inner.Builder->BuildTime), float(NumBuiltRenderInstances) / float(Inner.Builder->BuildTime));

			FCachedLandscapeFoliage::FGrassCompKey& InnerKey = Inner.Key;

			UE_LOGF(LogGrass, Verbose, "ASYNC GRASS INSTANCES COMPLETE %ls %ls %d %d (%d)",
				*InnerKey.BasedOn->GetName(),
				*(InnerKey.GrassType->GrassVarieties[InnerKey.VarietyIndex].GrassMesh->GetName()),
				InnerKey.SubsectionX,
				InnerKey.SubsectionY,
				NumBuiltRenderInstances);

			if (GrassISMComponent && StillUsed.Contains(GrassISMComponent))
			{
				if (NumBuiltRenderInstances > 0)
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_FoliageGrassEndComp_AcceptPrebuiltTree);
					GrassISMComponent->AcceptPrebuiltTree(Inner.Builder->ClusterTree, Inner.Builder->OutOcclusionLayerNum, NumBuiltRenderInstances, &Inner.Builder->InstanceBuffer);
					if (bForceSync && GetWorld())
					{
						QUICK_SCOPE_CYCLE_COUNTER(STAT_FoliageGrassEndComp_SyncUpdate);
						GrassISMComponent->RecreateRenderState_Concurrent();
					}
				}
			}
			FCachedLandscapeFoliage::FGrassComp* Existing = FoliageCache.CachedGrassComps.Find(Inner.Key);
			if (Existing)
			{
				Existing->Pending = false;
				if (Existing->PreviousFoliage.IsValid())
				{
					SCOPE_CYCLE_COUNTER(STAT_FoliageGrassDestoryComp);
					UHierarchicalInstancedStaticMeshComponent* HComponent = Existing->PreviousFoliage.Get();
					if (HComponent)
					{
						HComponent->ClearInstances();
						HComponent->DetachFromComponent(FDetachmentTransformRules(EDetachmentRule::KeepRelative, false));
						HComponent->DestroyComponent();
					}
					FoliageComponents.RemoveSwap(HComponent);
					Existing->PreviousFoliage = nullptr;
				}

				Existing->Touch();
			}
			delete Task;
			if (!bWaitAsyncTasks)
			{
				break; // one per frame is fine
			}
		}
	}
}

FAsyncGrassTask::FAsyncGrassTask(FAsyncGrassBuilder* InBuilder, const FCachedLandscapeFoliage::FGrassCompKey& InKey, UHierarchicalInstancedStaticMeshComponent* InFoliage)
	: Builder(InBuilder)
	, Key(InKey)
	, Foliage(InFoliage)
{
}

void FAsyncGrassTask::DoWork()
{
	Builder->Build();
}

FAsyncGrassTask::~FAsyncGrassTask()
{
	delete Builder;
}

static void FlushGrass(const TArray<FString>& Args)
{
	for (ALandscapeProxy* Landscape : TObjectRange<ALandscapeProxy>(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage))
	{
		Landscape->FlushGrassComponents();
	}
}

static void FlushGrassPIE(const TArray<FString>& Args)
{
	for (ALandscapeProxy* Landscape : TObjectRange<ALandscapeProxy>(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage))
	{
		Landscape->FlushGrassComponents(nullptr, false);
	}
}

static void DumpExclusionBoxes(const TArray<FString>& Args)
{
	for (const TPair<FWeakObjectPtr, FBox>& Pair : GGrassExclusionBoxes)
	{
		UObject* Owner = Pair.Key.Get();
		UE_LOGF(LogCore, Warning, "%f %f %f   %f %f %f   %ls",
			Pair.Value.Min.X,
			Pair.Value.Min.Y,
			Pair.Value.Min.Z,
			Pair.Value.Max.X,
			Pair.Value.Max.Y,
			Pair.Value.Max.Z,
			Owner ? *Owner->GetFullName() : TEXT("[stale]")
		);
	}
}

static FAutoConsoleCommand FlushGrassCmd(
	TEXT("grass.FlushCache"),
	TEXT("Flush the grass cache, debugging."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&FlushGrass)
	);

static FAutoConsoleCommand FlushGrassCmdPIE(
	TEXT("grass.FlushCachePIE"),
	TEXT("Flush the grass cache, debugging."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&FlushGrassPIE)
	);

static FAutoConsoleCommand DumpExclusionBoxesCmd(
	TEXT("grass.DumpExclusionBoxes"),
	TEXT("Print the exclusion boxes, debugging."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&DumpExclusionBoxes)
);


#undef LOCTEXT_NAMESPACE
