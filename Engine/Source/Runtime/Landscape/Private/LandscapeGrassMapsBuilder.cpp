// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeGrassMapsBuilder.h"
#include "Misc/ScopedSlowTask.h"
#include "Landscape.h"
#include "LandscapeAsyncTextureReadback.h"
#include "LandscapePrivate.h"
#include "LandscapeProxy.h"
#include "LandscapeUtils.h"
#include "LandscapeUtilsPrivate.h"
#include "LandscapeGrassWeightExporter.h"
#include "MaterialCachedData.h"
#include "LandscapeEditTypes.h"
#include "LandscapeGrassType.h"
#include "LandscapeSubsystem.h"
#include "Logging/StructuredLog.h"
#include "EngineUtils.h"
#include "Materials/MaterialInstance.h"
#include "Stats/Stats.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "LandscapeTextureHash.h"
#include "SceneInterface.h"

#define LOCTEXT_NAMESPACE "Landscape"

#define GRASS_DEBUG_TRANSITION(StateRef, StageBefore, StageAfter) \
	GRASS_DEBUG_LOG(TEXT("%s %s -> %s (after %d ticks) Pend:%d Strm:%d Rend:%d Fetch:%d Pop:%d NR:%d Total:%d"), \
		StateRef.Component ? *StateRef.Component->GetName() : TEXT("<REMOVED>"), \
		TEXT(#StageBefore), \
		TEXT(#StageAfter), \
		StateRef.TickCount, \
		PendingCount, \
		StreamingCount, \
		RenderingCount, \
		AsyncFetchCount, \
		PopulatedCount, \
		NotReadyCount, \
		ComponentStates.Num())

extern int32 GGrassEnable;
extern float GGrassCullDistanceScale;

int32 GGrassMapUseRuntimeGeneration = 0;
FAutoConsoleVariableRef CVarGrassMapUseRuntimeGeneration(
	TEXT("grass.GrassMap.UseRuntimeGeneration"),
	GGrassMapUseRuntimeGeneration,
	TEXT("Enable runtime grass map generation to save disk space and runtime memory.  When enabled the grass density maps are not serialized and are built on the fly at runtime."));

int32 GGrassMapEvictAll = 0;
FAutoConsoleVariableRef CVarGrassMapEvictAll(
	TEXT("grass.GrassMap.EvictAll"),
	GGrassMapEvictAll,
	TEXT("Remove all grass maps. In editor, or if runtime generation is enabled, the grass maps will repopulate automatically."));

#if !UE_BUILD_SHIPPING
FString GGrassMapRenderEveryFrame;
FAutoConsoleVariableRef CVarGrassMapRenderEveryFrame(
	TEXT("grass.GrassMap.RenderEveryFrame"),
	GGrassMapRenderEveryFrame,
	TEXT("Forces re-rendering the grassmap for the landscape component(s) on any actors with a name containing the given string."));

int32 GGrassMapRenderEveryFrameDelay = 0;
FAutoConsoleVariableRef CVarGrassMapRenderEveryFrameDelay(
	TEXT("grass.GrassMap.RenderEveryFrameDelay"),
	GGrassMapRenderEveryFrameDelay,
	TEXT("When using grass.GrassMap.RenderEveryFrame, delays this many frames in between renders.  Default 0."));
#endif // !UE_BUILD_SHIPPING

int32 GGrassMapUseAsyncFetch = 0;
FAutoConsoleVariableRef CVarGrassMapUseAsyncFetch(
	TEXT("grass.GrassMap.UseAsyncFetch"),
	GGrassMapUseAsyncFetch,
	TEXT("Enable async fetch tasks to readback the runtime grass maps from the GPU.  When disabled, it the fetch is performed on the game thread, when enabled it uses an async task instead."));

int32 GGrassMapAlwaysBuildRuntimeGenerationResources = 0;
static FAutoConsoleVariableRef CVarGrassMapAlwaysBuildRuntimeGenerationResources(
	TEXT("grass.GrassMap.AlwaysBuildRuntimeGenerationResources"),
	GGrassMapAlwaysBuildRuntimeGenerationResources,
	TEXT("By default we only compile shaders and build resources for runtime generation when runtime generation is enabled.  Set this to 1 to always build them for all platforms, allowing you to toggle runtime generation in a cooked build."));

static int32 GGrassMapMaxComponentsStreaming = 1;
static FAutoConsoleVariableRef CVarGrassMapMaxComponentsStreaming(
	TEXT("grass.GrassMap.MaxComponentsStreaming"),
	GGrassMapMaxComponentsStreaming,
	TEXT("How many landscape components can be streaming their textures at once for grass map renders, when using amortized runtime generation."));

// Rendering readback takes ~3 frames on average to complete, while streaming usually takes 1 frame.
// By setting rendering limit higher in editor we can achieve the same average throughput for both streaming and rendering at 1 per frame.
#if WITH_EDITOR
static int32 GGrassMapMaxComponentsRendering = 3;
#else
static int32 GGrassMapMaxComponentsRendering = 1;
#endif // WITH_EDITOR
static FAutoConsoleVariableRef CVarGrassMapMaxComponentsRendering(
	TEXT("grass.GrassMap.MaxComponentsRendering"),
	GGrassMapMaxComponentsRendering,
	TEXT("How many landscape components can be rendering grass maps at once, when using amortized runtime generation."));

static int32 GGrassMapMaxComponentsForBlockingUpdate = 6;
static FAutoConsoleVariableRef CVarGrassMapMaxComponentsForBlockingUpdate(
	TEXT("grass.GrassMap.MaxComponentsForBlockingUpdate"),
	GGrassMapMaxComponentsForBlockingUpdate,
	TEXT("How many landscape components can update simultaneously when running a blocking grass map update (i.e. on editor save)."));

static int32 GGrassMapMaxDiscardChecksPerFrame = 25;
static FAutoConsoleVariableRef CVarGrassMapMaxDiscardChecksPerFrame(
	TEXT("grass.GrassMap.MaxDiscardChecksPerFrame"),
	GGrassMapMaxDiscardChecksPerFrame,
	TEXT("How many landscape components are checked if they should discard their grass maps each frame."));

static int32 GGrassMapPrioritizedMultiplier = 4;
static FAutoConsoleVariableRef CVarGrassMapCreationPrioritizedMultiplier(
	TEXT("grass.GrassMap.PrioritizedMultiplier"),
	GGrassMapPrioritizedMultiplier,
	TEXT("Multiplier applied to MaxComponentsStreaming and MaxComponentsRendering when grass creation is prioritized."));

static float GGrassMapGuardBandMultiplier = 1.5f;
static FAutoConsoleVariableRef CVarGrassMapGuardBandMultiplier(
	TEXT("grass.GrassMap.GuardBandMultiplier"),
	GGrassMapGuardBandMultiplier,
	TEXT("Used to control discarding in the grass map runtime generation system. Approximate range, 1-4. Multiplied by the cull distance to control when we add grass maps."));

static float GGrassMapGuardBandDiscardMultiplier = 1.6f;
static FAutoConsoleVariableRef CVarGrassMapGuardBandDiscardMultiplier(
	TEXT("grass.GrassMap.GuardBandDiscardMultiplier"),
	GGrassMapGuardBandDiscardMultiplier,
	TEXT("Used to control discarding in the grass map runtime generation system. Approximate range, 1-4. Multiplied by the cull distance to control when we discard grass maps."));

static float GGrassMapCameraCutTranslationThreshold = 10000.0f;
static FAutoConsoleVariableRef CVarCameraCutTranslationThreshold(
	TEXT("grass.GrassMap.CameraCutTranslationThreshold"),
	GGrassMapCameraCutTranslationThreshold,
	TEXT("The maximum camera translation distance in centimeters allowed between two frames before we consider it a camera cut and grass map priorities are immediately recalculated."));

DECLARE_CYCLE_STAT(TEXT("Update Component GrassMap "), STAT_UpdateComponentGrassMaps, STATGROUP_Foliage);
DECLARE_CYCLE_STAT(TEXT("Prioritize Pending GrassMaps"), STAT_PrioritizePendingGrassMaps, STATGROUP_Foliage);
DECLARE_CYCLE_STAT(TEXT("Render GrassMap"), STAT_RenderGrassMap, STATGROUP_Foliage);
DECLARE_CYCLE_STAT(TEXT("Populate GrassMap"), STAT_PopulateGrassMap, STATGROUP_Foliage);
DECLARE_CYCLE_STAT(TEXT("Remove Grass Instances"), STAT_RemoveGrassInstances, STATGROUP_Foliage);

FLandscapeGrassMapsBuilder::FLandscapeGrassMapsBuilder(UWorld* InOwner, FLandscapeTextureStreamingManager& InTextureStreamingManager)
	: World(InOwner)
	, TextureStreamingManager(InTextureStreamingManager)
	, PreviousCameraHashGrid(GGrassMapCameraCutTranslationThreshold, FVector(TNumericLimits<double>::Lowest()))
#if WITH_EDITOR
	, OutdatedGrassMapCount(0)
	, GrassMapsLastCheckTime(0)
#endif // WITH_EDITOR
{
}

namespace UE::Landscape::Grass
{

uint32 ComputeGrassMapGenerationHash(const ULandscapeComponent* Component, UMaterialInterface* Material)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ComputeGrassMapGenerationHash);
	uint32 Hash = 0;

	// Change this hash key to invalidate all cached grass density maps, if the generation functions change.
	static uint32 InitialHash = GetTypeHash(FGuid("216D95C7651D4095ADC6A8459B4F181D"));

	Hash = InitialHash;

	// we only include material and texture hashes in editor (there is no automatic detection of changes in non-editor builds)
#if WITH_EDITOR
	// Take into account any material state change : (excluding texture state)
	Hash = FCrc::TypeCrc32(Material->ComputeAllStateCRC(), Hash);

	// hash the heightmap texture (we use the component's heightmap texture hash that ignores normals)
	Hash = FCrc::TypeCrc32(ULandscapeTextureHash::GetHash(Component->GetHeightmap()), Hash);

	// hash the weightmap textures
	for (UTexture2D* Weightmap : Component->GetWeightmapTextures())
	{
		check(Weightmap->Source.IsValid());
		Hash = FCrc::TypeCrc32(ULandscapeTextureHash::GetHash(Weightmap), Hash);
	}
#endif // WITH_EDITOR

	return Hash;
}

uint32 ComputeGrassInstanceGenerationHash(uint32 GrassMapGenerationHash, const TMap<FName, TObjectPtr<ULandscapeGrassType>>& NamedGrassTypes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ComputeGrassInstanceGenerationHash);

	// grass maps are input to grass instances
	return HashCombine(GrassMapGenerationHash, UE::Landscape::Grass::Private::ComputeNamedGrassTypesHash(NamedGrassTypes));
}

// Deprecated
bool CanRenderGrassMap(ULandscapeComponent *Component)
{
	return Component->CanRenderGrassMap();
}

} // namespace UE::Landscape::Grass

namespace UE::Landscape::Grass::Private
{

#if WITH_EDITOR
	void CompileGrassMapShader(const ULandscapeComponent* Component)
	{
		if (Component->GetMaterialInstanceCount(false) > 0)
		{
			if (UMaterialInstance* MaterialInstance = Component->GetMaterialInstance(0))
			{
				if (FMaterialResource* MaterialResource = MaterialInstance->GetMaterialResource(GetFeatureLevelShaderPlatform_Checked(Component->GetWorld()->GetFeatureLevel())))
				{
					MaterialResource->FinishCompilation();
				}
			}
		}
	}
#endif // WITH_EDITOR

// calculates the minimum distance between any cameras and the specified Worldbounds.
static inline double CalculateMinDistanceToCamerasSquared(const TArray<FVector>& Cameras, const FBoxSphereBounds& WorldBounds)
{
	if (!Cameras.Num())
	{
		return 0.0f;
	}
	double MinSqrDistance = MAX_dbl;
	for (const FVector& CameraPos : Cameras)
	{
		MinSqrDistance = FMath::Min<double>(MinSqrDistance, WorldBounds.ComputeSquaredDistanceFromBoxToPoint(CameraPos));
	}
	return MinSqrDistance;
}

} // namespace UE::Landscape::Grass::Private

namespace UE::Landscape
{

// Deprecated
bool CanRenderGrassMap(ULandscapeComponent *Component)
{
	return Component->CanRenderGrassMap();
}
// Deprecated
bool IsRuntimeGrassMapGenerationSupported()
{
	return UE::Landscape::SupportsRuntimeGrassMapGeneration();
}

} // namespace UE::Landscape

FLandscapeGrassMapsBuilder::~FLandscapeGrassMapsBuilder()
{
	// make sure all components were unregistered, so that state cleanup and deletion is triggered
	for (auto It = ComponentStates.CreateIterator(); It; ++It)
	{
		FComponentState* State = It.Value();
		ULandscapeComponent* Component = State->Component;
		if (Component != nullptr)
		{
			// This happens when deleting a level, the components are not unregistered before the world is destroyed.
			UnregisterComponent(Component);
		}
	}

	// update component state until they all delete themselves
	// (this should happen on the first update, unless a GPU readback is active.
	// And it shouldn't take more than 3 update if there is a GPU readback.

	double LastFlush = FPlatformTime::Seconds();

	int32 Iterations = 0;
	while (Iterations < 3 && ComponentStates.Num() > 0)
	{
		TArray<FVector> EmptyCamerasArray;
		int32 UpdateAllComponentCount = ComponentStates.Num();

		const bool bCancelAndEvictAllImmediately = true;
		const bool bEvictWhenBeyondEvictionRange = false;
		const int32 DeltaTicks = 0;
		UpdateTrackedComponents(EmptyCamerasArray, 0, UpdateAllComponentCount, bCancelAndEvictAllImmediately, bEvictWhenBeyondEvictionRange, DeltaTicks);

		ensure(NotReadyCount == 0 && StreamingCount == 0 && RenderingCount == 0 && AsyncFetchCount == 0 && PopulatedCount == 0);

		Iterations++;
	}

	if (ComponentStates.Num() != 0)
	{
		// somehow we failed to free the components the right way (either a GPU readback or async fetch is stuck, or state transition logic is broken)
		for (auto It = ComponentStates.CreateIterator(); It; ++It)
		{
			FComponentState* State = It.Value();
			UE_LOGF(LogGrass, Warning, "Failed to clear grass data state after %d iterations (stage:%d ticks:%d), forcing deletion of the state.", Iterations, EnumToUnderlyingType(State->Stage), State->TickCount);

			if (State->Stage == EComponentStage::Rendering)
			{
				if ((State->ActiveRender != nullptr) && (State->ActiveRender->GameThreadAsyncReadbackPtr != nullptr))
				{
					UE_LOGF(LogGrass, Warning, "  %ls", *State->ActiveRender->GameThreadAsyncReadbackPtr->ToString());
				}
			}
			else if (State->Stage == EComponentStage::AsyncFetch)
			{
				if (State->AsyncFetchTask.Get() != nullptr)
				{
					UE_LOGF(LogGrass, Warning, "  AsyncFetchTask: %p", State->AsyncFetchTask.Get());
				}
			}
		}

		// report the error so we capture the callstack and the log warnings above
		ensure(ComponentStates.Num() == 0);

		// force free the states anyways and hope for the best.  If crashes ensue the logs above should indicate why.
		for (auto It = ComponentStates.CreateIterator(); It; ++It)
		{
			FComponentState* State = It.Value();
			State->~FComponentState();
			StatePoolAllocator.Free(State);
			It.RemoveCurrent();
		}
	}
}

FLandscapeGrassMapsBuilder::FComponentState::FComponentState(ULandscapeComponent* Component)
	: Component(Component)
{
#if WITH_EDITOR
	UMaterialInterface* Material = Component->GetLandscapeMaterial();
	if (Material)
	{
		GrassMapGenerationHash = UE::Landscape::Grass::ComputeGrassMapGenerationHash(Component, Material);
		GrassInstanceGenerationHash = UE::Landscape::Grass::ComputeGrassInstanceGenerationHash(GrassMapGenerationHash, Material->GetCachedExpressionData().NamedGrassTypes);
	}
#endif // WITH_EDITOR
}

void FLandscapeGrassMapsBuilder::FPendingComponent::UpdatePriorityDistance(const TArray<FVector>&Cameras, float MustHaveDistanceScale)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeGrassMapsBuilder::FPendingComponent::UpdatePriorityDistance);
	ULandscapeComponent* Component = State->Component;
	FBoxSphereBounds WorldBounds = Component->CalcBounds(Component->GetComponentTransform());
	const double MinSqrDistanceToComponent = UE::Landscape::Grass::Private::CalculateMinDistanceToCamerasSquared(Cameras, WorldBounds);
	const double ThresholdDistance = Component->GrassTypeSummary.MaxInstanceDiscardDistance * MustHaveDistanceScale;
	PriorityKey = MinSqrDistanceToComponent - ThresholdDistance * ThresholdDistance;
}

void FAsyncFetchTask::DoWork()
{
	// do not delete the async readback resources (it must be done on the game thread, after the task completes)
	constexpr bool bFreeAsyncReadback = false;
	Results = ActiveRender->FetchResults(bFreeAsyncReadback);
}

bool FLandscapeGrassMapsBuilder::UpdateTrackedComponents(const TArray<FVector>& Cameras, int32 LocalMaxRendering, int32 MaxExpensiveUpdateChecksToPerform, bool bCancelAndEvictAllImmediately, bool bEvictWhenBeyondEvictionRange, int32 DeltaTicks)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeGrassMapsBuilder::UpdateTrackedComponents);
	SCOPE_CYCLE_COUNTER(STAT_UpdateComponentGrassMaps);

	bool bChanged = false;
	bRenderCommandsQueuedByLastUpdate = false;

	// Array to store components that are updated after the initial update pass (should never be more than max streaming components)
	static TArray<FComponentState*> StreamingStatesToProcess;
	StreamingStatesToProcess.Reset();

	// Set to store and components that need foliage instances removed
	static TSet<ULandscapeComponent*> ComponentsToRemoveFoliageInstances;
	ComponentsToRemoveFoliageInstances.Reset();

	AmortizedUpdate.StartUpdateTick(ComponentStates.Num(), MaxExpensiveUpdateChecksToPerform);

	const bool bIsGameWorld = World->IsGameWorld();

	// Iterate our components, removing invalid ones and counting how many are in each state.
	// We can also immediately process any components in the populated or rendering states
	// (those states don't need to consider throttling when moving to the next state, and processing them early frees up slots)
	int32 ComponentStateIndex = 0;
	for (auto It = ComponentStates.CreateIterator(); It; ++It, ++ComponentStateIndex)
	{
		FComponentState* State = It.Value();
		ULandscapeComponent* Component = State->Component;
		State->TickCount += DeltaTicks;

		if (bCancelAndEvictAllImmediately || (Component == nullptr))
		{
			// fallthrough to CancelAndEvict below
		}
		else
		{
			switch (State->Stage)
			{
			case EComponentStage::Pending:
				continue; // next!
			case EComponentStage::NotReady:
				// check to see if the conditions changed.
				if (AmortizedUpdate.ShouldUpdate(ComponentStateIndex))
				{
					if (Component->GetLandscapeProxy()->ShouldGenerateGrass() && Component->CanRenderGrassMap())
					{
						break; // cancel and evict to restart build process
					}
				}
				continue; // next!
			case EComponentStage::TextureStreaming:
				if (!Component->GetLandscapeProxy()->ShouldGenerateGrass())
				{
					break; // cancel and evict to restart build process
				}

				// don't process streaming states yet -- first process rendering states to free up slots
				StreamingStatesToProcess.Add(State);
				continue; // next!

			case EComponentStage::Rendering:
				if (!Component->GetLandscapeProxy()->ShouldGenerateGrass())
				{
					break; // cancel and evict to restart build process
				}

				check(State->ActiveRender != nullptr);
				{
					// NOTE: on RHI platforms that don't support fences (D3D11), the IsReady() async check will never return true within a single frame (i.e. BuildGrassMapsNow)
					// So it will never signal complete, unless we force it to finish after a number of ticks
					// in general 3 ticks should be the max latency we should ever see in an amortized use case
					const bool bForceFinish = (State->TickCount > 4);
					const bool bComplete = State->ActiveRender->CheckAndUpdateAsyncReadback(bRenderCommandsQueuedByLastUpdate, bForceFinish);

					if (bComplete)
					{
						if (GGrassMapUseAsyncFetch != 0)
						{
							LaunchAsyncFetchTask(*State);
						}
						else
						{
							PopulateGrassDataFromReadback(*State);
						}
						bChanged = true;
					}
				}
				continue; // next!

			case EComponentStage::AsyncFetch:
				if (!Component->GetLandscapeProxy()->ShouldGenerateGrass())
				{
					break; // cancel and evict to restart build process
				}

				{
					FAsyncTask<FAsyncFetchTask>* Task = State->AsyncFetchTask.Get();
					check(Task);
					if (Task->IsDone())
					{
						PopulateGrassDataFromAsyncFetchTask(*State);
					}
				}
				continue; // next!

			case EComponentStage::GrassMapsPopulated:
				// only check for invalidation of populated grass maps once in a while
				if (AmortizedUpdate.ShouldUpdate(ComponentStateIndex))
				{
					if (!Component->GetLandscapeProxy()->ShouldGenerateGrass()) // TODO put above the ShouldUpdate()?
					{
						break; // cancel and evict to restart build process
					}

					// detect if grass data has been cleared by someone manually calling Flush (i.e. when landscape edits are made)
					if (!Component->GrassData->HasValidData())
					{
						break; // cancel and evict to restart process
					}

					// check if the component is too far from the camera and we can reclaim the grass data
					if (bEvictWhenBeyondEvictionRange &&
						State->IsBeyondEvictionRange(Cameras))
					{
						GRASS_DEBUG_LOG(TEXT("Evicting for being beyond eviction range"));
						break; // cancel and evict
					}

#if WITH_EDITOR
					if (!bIsGameWorld)
					{
						UMaterialInterface* Material = Component->GetLandscapeMaterial();
						if (Material == nullptr)
						{
							break; // cancel and evict
						}

						// check if any dependencies changed
						const uint32 CurGrassMapGenerationHash = UE::Landscape::Grass::ComputeGrassMapGenerationHash(Component, Material);
						if (State->GrassMapGenerationHash != CurGrassMapGenerationHash)
						{
							break; // cancel and evict to restart process
						}

						// check if any grass types have changed -- this invalidates foliage instances but not the grass maps
						const uint32 CurGrassInstanceGenerationHash = UE::Landscape::Grass::ComputeGrassInstanceGenerationHash(CurGrassMapGenerationHash, Component->GetNamedGrassTypes());
						if (State->GrassInstanceGenerationHash != CurGrassInstanceGenerationHash)
						{
							ComponentsToRemoveFoliageInstances.Add(Component);
							Component->InvalidateGrassTypeSummary();
							State->GrassInstanceGenerationHash = CurGrassInstanceGenerationHash;
						}
					}
#endif // WITH_EDITOR
				}
				continue;	// next!

			default:
				check(false);	// unreachable
				break;			// cancel and evict
			}
		}

		// we only fall through to this statement if the code above didn't invoke `continue` (or if the component was unregistered)
		if (CancelAndEvict(*State, bCancelAndEvictAllImmediately))
		{
			if (State->Component == nullptr)
			{
				RemoveFromPendingComponentHeap(State);

				// destruct and free the state (return to our pool)
				State->~FComponentState();
				StatePoolAllocator.Free(State);
				State = nullptr;

				It.RemoveCurrent();
				AmortizedUpdate.HandleDeletion(ComponentStateIndex);
				ComponentStateIndex--;
				check(PendingCount > 0);
				PendingCount--;
			}
			else
			{
				// component is still registered, but has been invalidated.  Remove foliage instances.
				ComponentsToRemoveFoliageInstances.Add(Component);
			}
			bChanged = true;
		}
	}

	// kick off any deferred rendering
	if (StreamingStatesToProcess.Num() > 0)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(StreamingStatesToProcess);
		for (int32 i = 0; (RenderingCount < LocalMaxRendering) && (i < StreamingStatesToProcess.Num()); i++)
		{
			FComponentState* State = StreamingStatesToProcess[i];
			check(State->Stage == EComponentStage::TextureStreaming);

			if (State->AreTexturesStreamedIn())
			{
#if WITH_EDITOR
				// in editor the renderability can change unexpectedly -- so check one last time just before we actually render
				// (in game, we only check at the beginning of kicking off the generation process)
				if (!State->Component->CanRenderGrassMap())
				{
					// can't render, move to NotReady state, which will monitor until it is renderable
					StreamingToNotReady(*State);
					bChanged= true;
					continue;
				}
#endif // WITH_EDITOR
				KickOffRenderAndReadback(*State);
				bRenderCommandsQueuedByLastUpdate = true;
				bChanged = true;
			}
		}
		StreamingStatesToProcess.Reset();
	}

	if (ComponentsToRemoveFoliageInstances.Num() > 0)
	{
		SCOPE_CYCLE_COUNTER(STAT_RemoveGrassInstances);
		World->GetSubsystem<ULandscapeSubsystem>()->RemoveGrassInstances(&ComponentsToRemoveFoliageInstances);
	}

	return bChanged;
}

void FLandscapeGrassMapsBuilder::StartPrioritizedGrassMapGeneration(const TArray<FVector>& Cameras, int32 MaxComponentsToStart, bool bOnlyWhenCloserThanEvictionRange)
{
	SCOPE_CYCLE_COUNTER(STAT_PrioritizePendingGrassMaps);

	// no point in calling this if there are no cameras -- we can't calculate priority
	check(Cameras.Num() > 0);

	// update pending component priorities (distances)
	check(PendingComponentsHeap.Num() == PendingCount);
	if (PendingCount)
	{
		// determine if we need to recalculate all the pending component heap priorities
		bool bRecalculateAllPriorities = (FirstNewPendingComponent == 0);

		// check if scales have changed
		const float MustHaveDistanceScale = GGrassMapGuardBandMultiplier * GGrassCullDistanceScale;
		if (MustHaveDistanceScale != LastMustHaveDistanceScale)
		{
			bRecalculateAllPriorities = true;
			LastMustHaveDistanceScale = MustHaveDistanceScale;
		}

		if (!bRecalculateAllPriorities)
		{
			// check if cameras jumped suddenly
			// TODO [chris.tchou] : This is calculated in FViewInfo already - See GCameraCutTranslationThreshold
			// however, we would need to somehow pass that info through to the FStreamingViewInfos, (and ideally flag position-only cuts as we don't care about angle)
			float ThresholdSquared = GGrassMapCameraCutTranslationThreshold * GGrassMapCameraCutTranslationThreshold;
			for (const FVector& CameraPos : Cameras)
			{
				TPair<FVector, double> Found = PreviousCameraHashGrid.FindAnyInRadius(CameraPos, GGrassMapCameraCutTranslationThreshold, [CameraPos](const FVector& OtherDistanceSqFunc) { return FVector::DistSquared(CameraPos, OtherDistanceSqFunc); });
				if (Found.Value > ThresholdSquared)
				{
					// did not find any previous cameras within the threshold distance, implies the camera must have jumped farther than that distance
					bRecalculateAllPriorities = true;
					break;
				}
			}
		}

		// record camera positions for next frame
		PreviousCameraHashGrid.Reset(GGrassMapCameraCutTranslationThreshold);
		for (const FVector& CameraPos : Cameras)
		{
			PreviousCameraHashGrid.InsertPointUnsafe(CameraPos, CameraPos);
		}

		if (bRecalculateAllPriorities)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(RecalculateAllPriorities)
			for (FPendingComponent& Pending : PendingComponentsHeap)
			{
				Pending.UpdatePriorityDistance(Cameras, MustHaveDistanceScale);
			}
		}
		else
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AmortizedPriorityUpdate)

			// immediately update any new pending components (at the end of the array)
			if (FirstNewPendingComponent < PendingComponentsHeap.Num())
			{
				check(FirstNewPendingComponent >= 0);
				for (int32 Index = FirstNewPendingComponent; Index < PendingComponentsHeap.Num(); Index++)
				{
					FPendingComponent& Pending = PendingComponentsHeap[Index];
					Pending.UpdatePriorityDistance(Cameras, MustHaveDistanceScale);
				}
			}
			else
			{
				// set temporarily so we can use it as the upper bound in the amortized update below
				FirstNewPendingComponent = PendingComponentsHeap.Num();
			}

			// We update the priority of one element in each heap level.
			// Because the heap is ordered by distance, this approximately updates
			// closer elements more often than distant elements.
			// The closest element is updated every frame, the second and third every other frame, 4-7 every fourth frame, etc.
			// This way we update at most Log2(N) elements each frame.
			for (int32 LevelSize = 1; LevelSize <= 65536; LevelSize += LevelSize)
			{
				// this should select successive elements on the given Level as the counter is incremented
				int32 UpdateIndex = (PendingUpdateAmortizationCounter & (LevelSize - 1)) + (LevelSize - 1);
				if (UpdateIndex >= FirstNewPendingComponent)
				{
					break;
				}
				PendingComponentsHeap[UpdateIndex].UpdatePriorityDistance(Cameras, MustHaveDistanceScale);
			}
			PendingUpdateAmortizationCounter++;
		}
	}

	FirstNewPendingComponent = TNumericLimits<int32>::Max();

	// re-heapify to reflect updated priorities
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(Heapify)
		PendingComponentsHeap.Heapify();
	}

	// now pull as many elements off of the heap as we need
	while ((MaxComponentsToStart > 0) && PendingComponentsHeap.Num() > 0)
	{
		const FPendingComponent Pending = PendingComponentsHeap[0];
		FComponentState* State = Pending.State;
		const ULandscapeComponent* Component = State->Component;
		check(State->Stage == EComponentStage::Pending);
		check(Pending.PriorityKey != TNumericLimits<double>::Lowest()); // default value, indicates the priority was never calculated

		// if threshold is enabled, check if the nearest pending component is close enough
		if (bOnlyWhenCloserThanEvictionRange)
		{
			if (Pending.PriorityKey > 0.0)
			{
				// the closest component is not close enough to generate yet
				break;
			}
		}

		PendingComponentsHeap.HeapPopDiscard(EAllowShrinking::No);
		State->bInPendingHeap = false;

		if (StartGrassMapGeneration(*State, false))
		{
			MaxComponentsToStart--;
		}
	}

	check(PendingComponentsHeap.Num() == PendingCount);

	TotalComponentsWaitingCount = PendingComponentsHeap.Num();
}

// called when components are registered to the world	
void FLandscapeGrassMapsBuilder::RegisterComponent(ULandscapeComponent* Component)
{
	check(Component);
	if (FComponentState* State = ComponentStates.FindRef(Component))
	{
		GRASS_DEBUG_LOG(TEXT("Re-Register %s (%d total)"), *Component->GetName(), ComponentStates.Num());
		State->Component = Component;
		State->TickCount = 0;
	}
	else
	{
		FComponentState* NewState = new(StatePoolAllocator.Allocate()) FComponentState(Component);
		// we defer adding NewState to the pending component heap - to save the work of removing it if a fastpath is taken
		ComponentStates.Add(Component, NewState);
		PendingCount++;

		GRASS_DEBUG_LOG(TEXT("Register %s (%d total)"), *Component->GetName(), ComponentStates.Num());

		// immediately after a new registration, check if fast paths apply
		if (!TryFastpathsFromPending(*NewState, /* bRecalculateHashes = */ false))
		{
			// no fast path was taken, so add it to the pending component heap
			check(NewState->Stage == EComponentStage::Pending);
			AddToPendingComponentHeap(NewState);
		}
	}
}

void FLandscapeGrassMapsBuilder::UnregisterComponent(const ULandscapeComponent* Component)
{
	check(Component);
	if (FComponentState* State = ComponentStates.FindRef(Component))
	{
		GRASS_DEBUG_LOG(TEXT("Unregister %s"), *Component->GetName());
		State->Component = nullptr;				// we should no longer access the component, it may disappear
		State->TickCount = 0;					// track how long since unregistered
		// After ~2 ticks, Update will CancelAndEvict to clean up the remaining state
	}
	else
	{
		GRASS_DEBUG_LOG(TEXT("Unregister %s - NOT REGISTERED"), *Component->GetName());
	}
}

// false if the world can not currently render the grass (but this may change later, for example if preview modes are modified)
bool FLandscapeGrassMapsBuilder::CanCurrentlyRender() const
{
	if (FApp::CanEverRender())
	{
		check(World && World->Scene);
		// GPU scene is required by landscape fixed grid vertex factory
		EShaderPlatform ShaderPlatform = World->Scene->GetShaderPlatform();
		ERHIFeatureLevel::Type FeatureLevel = World->GetFeatureLevel();
		return UseGPUScene(ShaderPlatform);
	}
	return false;
}


void FLandscapeGrassMapsBuilder::AmortizedUpdateGrassMaps(
	const TArray<FVector>& Cameras,
	bool bPrioritizeCreation,
	bool bAllowStartGrassMapGeneration,
	bool bAnyLandscapeActorHasGrassEnabled)
{
#if !WITH_EDITOR
	if (!GGrassMapUseRuntimeGeneration) // in cooked builds, we don't run any updates at all unless runtime generation is enabled
	{
		return;
	}
#endif // WITH_EDITOR

	if (!bAnyLandscapeActorHasGrassEnabled && !StreamingCount && !RenderingCount && !AsyncFetchCount && !PopulatedCount)
	{
		return;
	}

	if (!FApp::CanEverRender())
	{
		return; // if we can never ever render, don't bother to do anything here
	}

#if !UE_BUILD_SHIPPING
	// determine if we have a debug render component
	if (!GGrassMapRenderEveryFrame.IsEmpty())
	{
		bool bRenderThisFrame = true;
		static int32 DelayTick = 0;
		if (GGrassMapRenderEveryFrameDelay > 0)
		{
			DelayTick++;
			bRenderThisFrame = false;
			if (DelayTick > GGrassMapRenderEveryFrameDelay)
			{
				bRenderThisFrame = true;
				DelayTick = 0;
			}
		}

		if (bRenderThisFrame)
		{
			// trigger a debug render of the grass maps for the specified components
			TArray<TObjectPtr<ULandscapeComponent>> DebugComponents;
			for (auto It = ComponentStates.CreateIterator(); It; ++It)
			{
				FComponentState* State = It.Value();
				if (ULandscapeComponent* Component = State->Component)
				{
					bool bDebugThisComponent = Component->GetName().Contains(GGrassMapRenderEveryFrame);
					if (!bDebugThisComponent)
					{
						ALandscapeProxy* Proxy = Component->GetLandscapeProxy();
						if (Proxy && Proxy->GetName().Contains(GGrassMapRenderEveryFrame))
						{
							bDebugThisComponent = true;
						}
					}

					if (bDebugThisComponent)
					{
						UE_LOGF(LogGrass, Display, "Issuing debug render for the grassmap on component'%ls'", *State->Component->GetName());
						DebugComponents.Add(Component);;
					}
				}
			}

			if (DebugComponents.Num() > 0)
			{
				DebugRenderComponents(DebugComponents);
			}
		}
	}
#endif // !UE_BUILD_SHIPPING

	int32 AmortizedMaxStreaming = GGrassMapMaxComponentsStreaming;
	int32 AmortizedMaxRendering = GGrassMapMaxComponentsRendering;

	if (bPrioritizeCreation && (GGrassMapPrioritizedMultiplier > 1))
	{
		AmortizedMaxStreaming *= GGrassMapPrioritizedMultiplier;
		AmortizedMaxRendering *= GGrassMapPrioritizedMultiplier;
	}

#if WITH_EDITOR
	// In editor we want to build all of the grass maps, and generally keep them around so we don't have to rebuild them
	// because we will want to serialize all of them to disk on save.
	const bool bCancelAndEvictAllImmediately = (GGrassMapEvictAll != 0);
	const bool bEvictWhenBeyondEvictionRange = false;
#else
	// At runtime, if we are using runtime generation then we evict anything beyond the eviction range
	// And we also evict everything if grass is disabled generally
	const bool bCancelAndEvictAllImmediately = (GGrassMapEvictAll != 0) || (GGrassEnable == 0);
	const bool bEvictWhenBeyondEvictionRange = (GGrassMapUseRuntimeGeneration != 0);
#endif // WITH_EDITOR
	GGrassMapEvictAll = 0;
	const int32 DeltaTicks = 1;
	UpdateTrackedComponents(Cameras, AmortizedMaxRendering, GGrassMapMaxDiscardChecksPerFrame, bCancelAndEvictAllImmediately, bEvictWhenBeyondEvictionRange, DeltaTicks);

	// no point in looking to start new grass map generation if nothing is pending, if grass is disabled or there are no cameras
	if (bAllowStartGrassMapGeneration && PendingCount > 0 && GGrassEnable && Cameras.Num() > 0)
	{
		// check our pipeline limits to make sure we have room to start components
		const int32 AvailableStreamingSlots = AmortizedMaxStreaming - StreamingCount;
		// do not build grass maps beyond eviction range if runtime generation is enabled
		const bool bOnlyWhenCloserThanEvictionRange = !!GGrassMapUseRuntimeGeneration;
		StartPrioritizedGrassMapGeneration(Cameras, AvailableStreamingSlots, bOnlyWhenCloserThanEvictionRange);
	}
}

#if !UE_BUILD_SHIPPING
void FLandscapeGrassMapsBuilder::DebugRenderComponents(
	TArrayView<TObjectPtr<ULandscapeComponent>> LandscapeComponents)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeGrassMapsBuilder::DebugRenderComponents);
	const int32 MaxStreamingRendering = FMath::Max(GGrassMapMaxComponentsForBlockingUpdate, 1);

	// the goal of this function is to get every input Landscape Component to transition from the TextureStreaming to the Rendering state
	// (i.e. issue GPU render commands for those components)

	// the structure of this code is similar to BuildGrassMapsNow, except:
	//    1) we don't need to complete the full process, we just need to ensure we trigger a GPU render of each LandscapeComponent
	//    2) we need to evict any component that is already past the TextureStreaming->Rendering transition
	//    3) we should skip fast paths, as we don't want to use pre-existing data, we want to render from scratch

	if (LandscapeComponents.IsEmpty())
	{
		return;
	}

	if (!CanCurrentlyRender())
	{
		return;
	}

	// using an empty cameras array causes all distance checks to return 0, so grass maps won't be evicted
	const TArray<FVector> EmptyCamerasArray;

	TSet<ULandscapeComponent*> PreRenderComponents;			// components that have been in a pre-Rendering state inside this function (to know we did the transition)
	TSet<ULandscapeComponent*> RenderedComponents;			// components that we have issued GPU renders for inside this function
	TSet<ULandscapeComponent*> FailedComponents;			// components that have failed to render (for any reason)
	TSet<ULandscapeComponent*> RemainingComponents;			// components that are not yet reached Rendered or Failed state, and still need to be checked
	TArray<ULandscapeComponent*> ComponentsCompletedThisIteration;

	RemainingComponents.Append(LandscapeComponents);
	
	while (RemainingComponents.Num() > 0)					// RenderedComponents.Num() + FailedComponents.Num() < LandscapeComponents.Num())
	{
		int32 AvailableStreamingSlots = MaxStreamingRendering - StreamingCount; // here we don't limit by overall population count
		for (ULandscapeComponent* Component : RemainingComponents)
		{
			FComponentState* State = ComponentStates.FindRef(Component);
			if (State == nullptr)
			{
				FailedComponents.Add(Component);
				ComponentsCompletedThisIteration.Add(Component);
				continue;
			}
			check(State->Component != nullptr);

			switch(State->Stage)
			{
				case EComponentStage::Pending:
					{
						if (AvailableStreamingSlots > 0)
						{
							constexpr bool bForceCompileShaders = true;
							constexpr bool bTryFastPaths = false;		// we want to force a render, so skip the fast paths
							PreRenderComponents.Add(Component);
							StartGrassMapGeneration(*State, bForceCompileShaders, bTryFastPaths);
							if (State->Stage != EComponentStage::NotReady &&
								State->Stage != EComponentStage::Pending)
							{
								AvailableStreamingSlots--;
							}
						}
					}
					break;

				case EComponentStage::NotReady:
					{
						// if it's not ready because of shader reasons
						if (!FailedComponents.Contains(Component) && !Component->CanRenderGrassMap())
						{
	#if WITH_EDITOR
							// in editor, try to force compilation to complete
							UE::Landscape::Grass::Private::CompileGrassMapShader(State->Component);
							if (!Component->CanRenderGrassMap())
	#endif // WITH_EDITOR
							{
								// failed to compile shaders... we won't be able to build the grass map for this component
								FailedComponents.Add(Component);
								ComponentsCompletedThisIteration.Add(Component);
							}
						}
					}
					break;

				case EComponentStage::TextureStreaming:
					PreRenderComponents.Add(Component);
					break;

				case EComponentStage::Rendering:
				case EComponentStage::AsyncFetch:
				case EComponentStage::GrassMapsPopulated:
					if (!PreRenderComponents.Contains(Component))
					{
						// this was not yet in the streaming state in this function, which means the render happened outside of this function.
						// evict to restart the process
						constexpr bool bCancelImmediately = true;
						CancelAndEvict(*State, bCancelImmediately);
						PreRenderComponents.Add(Component);
					}
					else
					{
						if (State->Stage == EComponentStage::Rendering)
						{
							// yay we issued the render commands for this component
							RenderedComponents.Add(Component);
							ComponentsCompletedThisIteration.Add(Component);
						}
					}
					break;
			}			
		}

		for (ULandscapeComponent* Component : ComponentsCompletedThisIteration)
		{
			RemainingComponents.Remove(Component);
		}
		ComponentsCompletedThisIteration.Reset();

		if (RemainingComponents.Num() <= 0)
		{
			break;
		}

		// update all components that are tracked (without evicting)
		const int32 UpdateAllComponentCount = ComponentStates.Num();
		const bool bCancelAndEvictAllImmediately = false;
		const bool bEvictWhenBeyondEvictionRange = false;
		const int32 DeltaTicks = 0;
		bool bChanged = UpdateTrackedComponents(EmptyCamerasArray, MaxStreamingRendering, UpdateAllComponentCount, bCancelAndEvictAllImmediately, bEvictWhenBeyondEvictionRange, DeltaTicks);

		// no need to tick SubmitGPUCommands or wait for AsyncFetch tasks to complete
		// because we don't need to block on them (we can let the async stuff just pile up til we exit this function)

		// If any streaming is in flight, we do need to ensure that completes, via a blocking texture streaming update.
		// TODO [chris.tchou] : ideally this would be a non-blocking streaming update tick, so we can react to other updates finishing
		if (StreamingCount > 0)
		{
			TextureStreamingManager.WaitForTextureStreaming();
		}
	}
}
#endif // !UE_BUILD_SHIPPING

bool FLandscapeGrassMapsBuilder::BuildGrassMapsNowForComponents(
	TArrayView<TObjectPtr<ULandscapeComponent>> LandscapeComponents, FScopedSlowTask* SlowTask, bool bMarkDirty)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeGrassMapsBuilder::BuildGrassMapsNowForComponents);
	const int32 MaxStreamingRendering = FMath::Max(GGrassMapMaxComponentsForBlockingUpdate, 1);

	if (LandscapeComponents.IsEmpty())
	{
		return true;
	}

	if (!CanCurrentlyRender())
	{
		return false; // can't build grass maps without rendering, unfortunately
	}

	auto UpdateProgress = [SlowTask](int Increment)
	{
		if (SlowTask && Increment && ((SlowTask->CompletedWork + Increment) <= SlowTask->TotalAmountOfWork))
		{
			SlowTask->EnterProgressFrame(static_cast<float>(Increment), FText::Format(LOCTEXT("GrassMaps_BuildGrassMapsProgress", "Building Grass Map {0} of {1})"), FText::AsNumber(SlowTask->CompletedWork), FText::AsNumber(SlowTask->TotalAmountOfWork)));
		}
	};

	// using an empty cameras array causes all distance checks to return 0, so grass maps won't be evicted
	const TArray<FVector> EmptyCamerasArray;
	int32 LastUpToDateCount = 0;
	int32 UpToDateCount = 0;

	// track any components that have failed to build
	TSet<ULandscapeComponent*> FailedComponents;

	const double StartTime = FPlatformTime::Seconds();
	double LastFlush = StartTime;
	double LastChangeTime = StartTime;
	while (UpToDateCount + FailedComponents.Num() != LandscapeComponents.Num())
	{
		// ensure we are making progress within a reasonable amount of time TODO [chris.tchou] there should be a better way to detect non-progress here
		const double CurTime = FPlatformTime::Seconds();
		if (CurTime > LastChangeTime + 30.0)
		{
			UE_LOGF(LogGrass, Error, "ERROR: BuildGrassMapsNowForComponents() took too long, grass maps are not up to date");
			break;
		}

		// update all components that are tracked (without evicting)
		const int32 UpdateAllComponentCount = ComponentStates.Num();
		const bool bCancelAndEvictAllImmediately = false;
		const bool bEvictWhenBeyondEvictionRange = false;
		const int32 DeltaTicks = 1; // we need a positive delta tick to force completion of GPU readbacks on D3D11
		bool bChanged = UpdateTrackedComponents(EmptyCamerasArray, MaxStreamingRendering, UpdateAllComponentCount, bCancelAndEvictAllImmediately, bEvictWhenBeyondEvictionRange, DeltaTicks);

		UpToDateCount = 0;
		int32 AvailableStreamingSlots = MaxStreamingRendering - StreamingCount; // here we don't limit by overall population count
		for (ULandscapeComponent* Component : LandscapeComponents)
		{
			FComponentState* State = ComponentStates.FindRef(Component);
			if ((State == nullptr) || (State->Component == nullptr))
			{
				// untracked or unregistered components will not be built
				FailedComponents.Add(Component);
				continue;
			}

			if (State->Stage == EComponentStage::Pending)
			{
				// Start tracking to kick off the build process
				constexpr bool bForceCompileShaders = true;
				if (AvailableStreamingSlots > 0)
				{
					constexpr bool bTryFastPaths = true;
					StartGrassMapGeneration(*State, bForceCompileShaders, bTryFastPaths);
					if (State->Stage != EComponentStage::NotReady &&
						State->Stage != EComponentStage::Pending)
					{
						// modification isn't complete yet, but convenient to dirty the package when starting the process here
						if (bMarkDirty)
						{
							Component->MarkPackageDirty();
						}
						AvailableStreamingSlots--;
						bChanged = true;
					}
				}
			}

			if (State->Stage == EComponentStage::GrassMapsPopulated)
			{
				check(Component->GrassData->HasValidData()); // guaranteed by UpdateTrackedComponents(), as long as we update all of the components
#if WITH_EDITOR
				// this should be guaranteed by UpdateTrackedComponents(), as long as we update all of the components
				// if not, then issue a report to try to gather more information as to what is causing this.
				uint32 CurrentGrassMapGenHash = Component->ComputeGrassMapGenerationHash();
				if (CurrentGrassMapGenHash != Component->GrassData->GenerationHash)
				{
					UE_LOGF(LogGrass, Warning, "Unexpected content change while generating grass maps synchronously (Component:%p %p %ls Ticks:%d WorldType:%d OldHash:%x NewHash:%x StateHash:%x NumElems:%d)",
						Component, State->Component, *Component->GetPathName(),
						State->TickCount,
						World->WorldType, Component->GrassData->GenerationHash, CurrentGrassMapGenHash, State->GrassMapGenerationHash,
						Component->GrassData->NumElements);

					UE_LOGF(LogGrass, Warning, "  GrassBuilder State: Pend:%d (Heap:%d) Strm:%d Rend:%d Fetch:%d Pop:%d NR:%d Total:%d",
						PendingCount, PendingComponentsHeap.Num(), StreamingCount, RenderingCount, AsyncFetchCount, PopulatedCount, NotReadyCount, ComponentStates.Num());

					check(State->Component == Component);

					// report the error
					ensure(CurrentGrassMapGenHash == Component->GrassData->GenerationHash);

					// don't count this as an UpToDate component -- the hash mismatch should be picked up by Update and evicted in the next iteration
					continue;
				}
#endif // WITH_EDITOR
				if (FailedComponents.Contains(Component))
				{
					FailedComponents.Remove(Component);
				}
				UpToDateCount++;
			}

			if (State->Stage == EComponentStage::NotReady)
			{
				if (!Component->GetLandscapeActor()->ShouldGenerateGrass())
				{
					// nothing to do for this component
					++UpToDateCount;
				}
				else if (!FailedComponents.Contains(Component))
				{
#if WITH_EDITOR
					// if it's not ready because the shaders are not ready
					if (!Component->CanRenderGrassMap())
					{
						// try to force the shaders to compile
						UE::Landscape::Grass::Private::CompileGrassMapShader(State->Component);
						if (Component->CanRenderGrassMap())
						{
							// succeeded, let it try to complete the grass (the next UpdateTrackedComponents should pull it out of NotReady state if it can)
						}
						else
						{
							// failed to compile shaders, it's definitely failed for real
							FailedComponents.Add(Component);
						}
					}
					else
#endif // WITH_EDITOR
					{
						// failed for other unspecified reasons
						FailedComponents.Add(Component);
					}
				}
			}
		}

		if (LastUpToDateCount != UpToDateCount)
		{
			UpdateProgress(UpToDateCount - LastUpToDateCount);
			LastUpToDateCount = UpToDateCount;
		}

		// If any rendering is in flight, queue up the gpu commands on the render thread, so the GPU can start working on them.
		if (bRenderCommandsQueuedByLastUpdate || (RenderingCount > 0 && (CurTime - LastFlush > (1.0 / 60.0))))
		{
			// TODO [chris.tchou] it currently seems to be faster to block here; otherwise it takes a long time to complete the readback
			// not sure why this is, something must be getting starved in the non-blocking path.
			UE::Landscape::Private::SubmitGPUCommands(/* bBlockUntilRTComplete =  */ true, /* bBlockRTUntilGPUComplete =  */ false);
			LastFlush = CurTime;
		}
		
		// If any streaming is in flight, do a blocking texture streaming update.
		// TODO [chris.tchou] : ideally this would be a non-blocking streaming update tick, so we can react to other updates finishing
		if (StreamingCount > 0)
		{
			TextureStreamingManager.WaitForTextureStreaming();
		}

		if (AsyncFetchCount > 0)
		{
			CompleteAllAsyncTasksNow();
		}

		if (bChanged)
		{
			LastChangeTime = FPlatformTime::Seconds();
		}
	}

	UE_LOGF(LogGrass, Verbose, "BuildGrassMapsNowForComponents() updated %d/%d components in %f seconds", UpToDateCount, LandscapeComponents.Num(), FPlatformTime::Seconds() - StartTime);
	
	// warn if we failed to build grass maps, except when there are no registered states (which happens when we migrate levels from project to project - because it doesn't register before saving)
	if ((UpToDateCount != LandscapeComponents.Num()) && (ComponentStates.Num() > 0))
	{
		UE_LOGF(LogGrass, Warning, "Failed to build grass maps for %d/%d landscape components, check if you are using a render preview mode, or a non-SM5 capable render device.  (%d pending, %d streaming, %d rendering, %d fetching, %d built)",
			LandscapeComponents.Num() - UpToDateCount,
			LandscapeComponents.Num(),
			PendingCount, StreamingCount, RenderingCount, AsyncFetchCount, PopulatedCount);
	}

	return (UpToDateCount == LandscapeComponents.Num());
}

void FLandscapeGrassMapsBuilder::CompleteAllAsyncTasksNow()
{
	for (auto It = ComponentStates.CreateIterator(); It; ++It)
	{
		FComponentState* State = It.Value();
		if (State->Stage == EComponentStage::AsyncFetch)
		{
			check(State->AsyncFetchTask.Get());
			State->AsyncFetchTask->EnsureCompletion(/* bDoWorkOnThisThreadIfNotStarted= */ true, /* bIsLatencySensitive= */ true);
		}
	}
}

bool FLandscapeGrassMapsBuilder::CancelAndEvict(FComponentState& State, bool bCancelImmediately)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeGrassMapsBuilder::CancelAndEvict);

	// handle transitioning from any stage to EComponentState::Pending
	switch (State.Stage)
	{
	case EComponentStage::Pending:
		check(PendingCount > 0);
		break;
	case EComponentStage::NotReady:
		{
			check(NotReadyCount > 0);
			NotReadyCount--;
			PendingCount++;
			GRASS_DEBUG_TRANSITION(State, NotReady, Pending);
		}
		break;
	case EComponentStage::TextureStreaming:
		{
			RemoveTextureStreamingRequests(State);
			check(StreamingCount > 0);
			StreamingCount--;
			PendingCount++;
			GRASS_DEBUG_TRANSITION(State, TextureStreaming, Pending);
		}
		break;
	case EComponentStage::Rendering:
		{
			RemoveTextureStreamingRequests(State);
			if (State.ActiveRender != nullptr)
			{
				// calling update ensures it is pushed forward if there is still a readback in progress
				bool bNewRenderCommands = false;
				if (bCancelImmediately)
				{
					// release active render from the state, and cancel and self destruct it
					// (this may cause a stall in the render thread waiting for GPU completion)
					FLandscapeGrassWeightExporter* Render = State.ActiveRender.Release();
					Render->CancelAndSelfDestruct();
				}
				else
				{
					// generally we arrive here when the Component was unregistered,
					// but we must wait for the readback to complete before exiting the rendering state.
					// Because Components can be re-registered and pick up where they left off, we don't want to cancel the operation outright just yet.
					// To avoid potential render thread stalls we let the operation continue asynchronously for up to four ticks,
					// before forcing completion (this also gives us more time for re-register)
					const bool bForceFinish = (State.TickCount > 4);
					if (!State.ActiveRender->CheckAndUpdateAsyncReadback(bNewRenderCommands, bForceFinish))
					{
						// we can't evict yet.. must wait for the readback to complete
						return false;
					}
					else
					{
						// readback is complete, we can delete the active render and move to the pending state
						// (under the hood this queues deletion on render thread)
						State.ActiveRender.Reset();
					}
				}
			}
			check(RenderingCount > 0);
			RenderingCount--;
			PendingCount++;
			GRASS_DEBUG_TRANSITION(State, Rendering, Pending);
		}
		break;
	case EComponentStage::AsyncFetch:
		{
			FAsyncTask<FAsyncFetchTask>* Task = State.AsyncFetchTask.Get();
			if (Task != nullptr)
			{
				if (bCancelImmediately)
				{
					Task->EnsureCompletion(/* bDoWorkOnThisThreadIfNotStarted= */ true, /* bIsLatencySensitive= */ true);
				}
				else if (!Task->IsDone())
				{
					// can't cancel, async task is still in flight
					return false;
				}

				State.ActiveRender->FreeAsyncReadback(); // TODO [chris.tchou] : I think this is redundant with Reset() below
				State.ActiveRender.Reset();
				State.AsyncFetchTask.Reset();
			}
			check(AsyncFetchCount > 0);
			AsyncFetchCount--;
			PendingCount++;
			GRASS_DEBUG_TRANSITION(State, AsyncFetch, Pending);
		}
		break;
	case EComponentStage::GrassMapsPopulated:
		{
			ULandscapeComponent* Component = State.Component;
		
			if (!bCancelImmediately && (Component == nullptr))
			{
				// component was unregistered. Wait a few ticks to see if it comes back before fully evicting.
				if (State.TickCount < 2)
				{
					return false;
				}
			}

			// if the component is still around, clear any existing grass data from it
			if (Component && Component->GrassData->HasValidData())
			{
				Component->RemoveGrassMap();
			}
			check(PopulatedCount > 0);
			PopulatedCount--;
			PendingCount++;
			GRASS_DEBUG_TRANSITION(State, Populated, Pending);
		}
		break;
	default:
		check(false);	// unreachable
		break;
	}

	check(State.ActiveRender == nullptr);
	check(State.TexturesToStream.IsEmpty());

	if (State.Stage != EComponentStage::Pending)
	{
		// back to pending state with you!
		State.Stage = EComponentStage::Pending;
		State.TickCount = 0;
		if (State.Component != nullptr)
		{
			// don't bother to add if component is null as we will just have to remove it immediately in the deallocate
			AddToPendingComponentHeap(&State);
		}
	}

	return true;
}

bool FLandscapeGrassMapsBuilder::TryFastpathsFromPending(FComponentState& State, bool bRecalculateHashes)
{
	check(State.Stage == EComponentStage::Pending);
	ULandscapeComponent* Component = State.Component;

	if (!Component->GetLandscapeProxy()->ShouldGenerateGrass())
	{
		GRASS_DEBUG_LOG(TEXT("Grass generation disabled for %s"), *Component->GetName());
		PendingToNotReady(State);
		return true;
	}

	if (World->IsGameWorld()) // including PIE
	{
		// if runtime grass generation is disabled, go straight to not ready
		if (Component->GetLandscapeProxy()->GetDisableRuntimeGrassMapGeneration())
		{
			GRASS_DEBUG_LOG(TEXT("GrassMap Proxy DisableRuntimeGeneration for %s"), *Component->GetName());
			PendingToNotReady(State);
			return true;
		}
	}
	else
	{
#if WITH_EDITOR
		// recalculate hashes
		if (bRecalculateHashes)
		{
			if (UMaterialInterface* Material = Component->GetLandscapeMaterial())
			{
				State.GrassMapGenerationHash = UE::Landscape::Grass::ComputeGrassMapGenerationHash(Component, Material);
				State.GrassInstanceGenerationHash = UE::Landscape::Grass::ComputeGrassInstanceGenerationHash(State.GrassMapGenerationHash, Material->GetCachedExpressionData().NamedGrassTypes);
			}
		}

		// in editor, if the existing grass data is valid and has a matching hash, then we can skip straight to Populated
		if (Component->GrassData->HasValidData() &&
			Component->GrassData->GenerationHash == State.GrassMapGenerationHash)
		{
			PendingToPopulatedFastPathAlreadyHasData(State);
			return true;
		}
#endif // WITH_EDITOR
	}

	// handle the easy case of empty grass types; skip directly to the populated state without running the pipeline
	if (!Component->GrassTypeSummary.bHasAnyGrass)
	{
		PendingToPopulatedFastPathNoGrass(State);
		return true;
	}

	return false;
}

bool FLandscapeGrassMapsBuilder::StartGrassMapGeneration(FComponentState& State, bool bForceCompileShaders, bool bTryFastpaths)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeGrassMapsBuilder::StartGrassMapGeneration);
	check(State.Stage == EComponentStage::Pending);

	ULandscapeComponent* Component = State.Component;
	check(Component); // must be registered before starting grass map generation

	if (bTryFastpaths && TryFastpathsFromPending(State, /* bRecalculateHashes = */ true))
	{
		return false;
	}

	// if we can't currently render, it's not ready
	if (!Component->CanRenderGrassMap())
	{
#if WITH_EDITOR
		if (bForceCompileShaders)
		{
			UE::Landscape::Grass::Private::CompileGrassMapShader(Component);
		}

		if (!Component->CanRenderGrassMap())
#endif // WITH_EDITOR
		{
			PendingToNotReady(State);
			return false;
		}
	}

	PendingToStreaming(State);
	return true;
}

void FLandscapeGrassMapsBuilder::StreamingToNotReady(FComponentState& State)
{
	check(StreamingCount > 0);
	StreamingCount--;
	RemoveTextureStreamingRequests(State);
	State.Stage = EComponentStage::NotReady;
	NotReadyCount++;

	GRASS_DEBUG_TRANSITION(State, Streaming, NotReady);
	State.TickCount = 0;
}

void FLandscapeGrassMapsBuilder::PendingToNotReady(FComponentState& State)
{
	check(PendingCount > 0);
	PendingCount--;
	State.Stage = EComponentStage::NotReady;
	NotReadyCount++;

	RemoveFromPendingComponentHeap(&State);

	GRASS_DEBUG_TRANSITION(State, Pending, NotReady);
	State.TickCount = 0;
}

void FLandscapeGrassMapsBuilder::PendingToPopulatedFastPathAlreadyHasData(FComponentState& State)
{
	check(PendingCount > 0);
	PendingCount--;
	State.Stage = EComponentStage::GrassMapsPopulated;
	PopulatedCount++;

	RemoveFromPendingComponentHeap(&State);

	GRASS_DEBUG_TRANSITION(State, Pending, Populated_Existing);
	State.TickCount = 0;
}

void FLandscapeGrassMapsBuilder::PendingToPopulatedFastPathNoGrass(FComponentState& State)
{
	ULandscapeComponent* Component = State.Component;

	TUniquePtr<FLandscapeComponentGrassData> NewGrassData = MakeUnique<FLandscapeComponentGrassData>();
	Component->GrassData = MakeShareable(NewGrassData.Release());
#if WITH_EDITOR
	Component->GrassData->GenerationHash = State.GrassMapGenerationHash;
#endif // WITH_EDITOR
	Component->GrassData->NumElements = 0;

	check(PendingCount > 0);
	PendingCount--;
	State.Stage = EComponentStage::GrassMapsPopulated;
	PopulatedCount++;

	RemoveFromPendingComponentHeap(&State);

	GRASS_DEBUG_TRANSITION(State, Pending, Populated_Empty);
	State.TickCount = 0;
}

void FLandscapeGrassMapsBuilder::PendingToStreaming(FComponentState& State)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeGrassMapsBuilder::PendingToStreaming);
	ULandscapeComponent* Component = State.Component;

	// determine which textures we need to stream by inspecting the material, kick off streaming requests for them
	// TODO [chris.tchou] : also grab asset textures from the material.
	State.TexturesToStream.Add(Component->GetHeightmap());

	const ERHIFeatureLevel::Type FeatureLevel = World->GetFeatureLevel();
	for (UTexture2D* WeightmapTexture : Component->GetRenderedWeightmapTexturesForFeatureLevel(FeatureLevel))
	{
		State.TexturesToStream.Add(WeightmapTexture);
	}

	// Request streaming for our textures
	for (const TWeakObjectPtr<UTexture>& TexturePtr : State.TexturesToStream)
	{
		if (UTexture* Texture = TexturePtr.Get())
		{
			TextureStreamingManager.RequestTextureFullyStreamedIn(Texture, /* bWaitForStreaming= */ false);
		}
	}

	check(PendingCount > 0);
	PendingCount--;
	State.Stage = EComponentStage::TextureStreaming;
	StreamingCount++;

	RemoveFromPendingComponentHeap(&State);

	GRASS_DEBUG_TRANSITION(State, Pending, Streaming);
	State.TickCount = 0;
}

void FLandscapeGrassMapsBuilder::AddToPendingComponentHeap(FComponentState* State)
{
	check(!State->bInPendingHeap);
	int32 NewIndex = PendingComponentsHeap.Add(FPendingComponent(State));
	State->bInPendingHeap = true;
	FirstNewPendingComponent = FMath::Min(FirstNewPendingComponent, NewIndex);
}

void FLandscapeGrassMapsBuilder::RemoveFromPendingComponentHeap(FComponentState* State)
{
	// may be removed more than once
	if (State->bInPendingHeap)
	{
		for (int32 Index = 0; Index < PendingComponentsHeap.Num(); Index++)
		{
			if (PendingComponentsHeap[Index].State == State)
			{
				PendingComponentsHeap.RemoveAtSwap(Index, EAllowShrinking::No);

				if ((FirstNewPendingComponent <= PendingComponentsHeap.Num()) && (Index < FirstNewPendingComponent))
				{
					// this should be fairly rare, as we generally process new pending components before removing anything
					// but if a new pending component does get swapped by the remove, we need to update the new range
					FirstNewPendingComponent = Index;
				}
				break;
			}
		}
		State->bInPendingHeap = false;
	}
}

void FLandscapeGrassMapsBuilder::KickOffRenderAndReadback(FComponentState& State)
{
	SCOPE_CYCLE_COUNTER(STAT_RenderGrassMap);

	check(State.Stage == EComponentStage::TextureStreaming);
	check(StreamingCount > 0);
	StreamingCount--;

	TArray<int32> HeightMips;

	ULandscapeComponent* Component = State.Component;

#if WITH_EDITORONLY_DATA
	// the height mips data is discarded unless we are in editor, don't bother to request it otherwise
	const bool bBakeMaterialPositionOffsetIntoCollision = (Component->GetLandscapeProxy() && Component->GetLandscapeProxy()->bBakeMaterialPositionOffsetIntoCollision);
	if (bBakeMaterialPositionOffsetIntoCollision)
	{
		if (Component->CollisionMipLevel > 0)
		{
			HeightMips.Add(Component->CollisionMipLevel);
		}
		if (Component->SimpleCollisionMipLevel > Component->CollisionMipLevel)
		{
			HeightMips.Add(Component->SimpleCollisionMipLevel);
		}
	}
#endif // WITH_EDITORONLY_DATA

	check(State.ActiveRender == nullptr);
	constexpr bool bInNeedsGrassmap = true;
	constexpr bool bInNeedsHeightmap = true;
	State.ActiveRender.Reset(new FLandscapeGrassWeightExporter(Component->GetLandscapeActor(), { Component }, UE::Landscape::Grass::EGrassWeightExporterFlags::Default, /*InRequestedGrassNames = */{}, MakeConstArrayView(HeightMips)));
	check(State.ActiveRender != nullptr);

	State.Stage = EComponentStage::Rendering;
	RenderingCount++;

	GRASS_DEBUG_TRANSITION(State, Streaming, Rendering);
	State.TickCount = 0;
}

void FLandscapeGrassMapsBuilder::LaunchAsyncFetchTask(FComponentState& State)
{
	SCOPE_CYCLE_COUNTER(STAT_PopulateGrassMap);

	check(State.Stage == EComponentStage::Rendering);
	check(RenderingCount > 0);
	RenderingCount--;

	// now that render is complete, we can drop the texture streaming requests and allow textures to stream out
	RemoveTextureStreamingRequests(State);

	check(State.ActiveRender != nullptr);

	State.AsyncFetchTask.Reset(new FAsyncTask<FAsyncFetchTask>(State.ActiveRender.Get()));
	State.AsyncFetchTask->StartBackgroundTask();
	State.Stage = EComponentStage::AsyncFetch;
	AsyncFetchCount++;

	GRASS_DEBUG_TRANSITION(State, Rendering, AsyncFetch);
	State.TickCount = 0;
}

void FLandscapeGrassMapsBuilder::PopulateGrassDataFromAsyncFetchTask(FComponentState& State)
{
	check(AsyncFetchCount > 0);
	AsyncFetchCount--;

	State.ActiveRender->FreeAsyncReadback();

	FAsyncFetchTask& Inner = State.AsyncFetchTask->GetTask();
	FLandscapeGrassWeightExporter::ApplyResults(Inner.Results);

	State.ActiveRender.Reset();
	State.AsyncFetchTask.Reset();

	State.Stage = EComponentStage::GrassMapsPopulated;
	PopulatedCount++;

	GRASS_DEBUG_TRANSITION(State, AsyncFetch, Populated);
	State.TickCount = 0;
}

void FLandscapeGrassMapsBuilder::PopulateGrassDataFromReadback(FComponentState& State)
{
	SCOPE_CYCLE_COUNTER(STAT_PopulateGrassMap);

	check(State.Stage == EComponentStage::Rendering);
	check(RenderingCount > 0);
	RenderingCount--;

	// now that render is complete, we can drop the texture streaming requests and allow textures to stream out
	RemoveTextureStreamingRequests(State);

	check(State.ActiveRender != nullptr);
	State.ActiveRender->ApplyResults();
	State.ActiveRender.Reset();

	State.Stage = EComponentStage::GrassMapsPopulated;
	PopulatedCount++;

	GRASS_DEBUG_TRANSITION(State, Rendering, Populated);
	State.TickCount = 0;
}

void FLandscapeGrassMapsBuilder::RemoveTextureStreamingRequests(FComponentState& State)
{
	for (const TWeakObjectPtr<UTexture>& TexturePtr : State.TexturesToStream)
	{
		if (UTexture* Texture = TexturePtr.Get())
		{
			TextureStreamingManager.UnrequestTextureFullyStreamedIn(Texture);
		}
	}
	State.TexturesToStream.Empty();
}

bool FLandscapeGrassMapsBuilder::FComponentState::AreTexturesStreamedIn()
{
	for (auto It = TexturesToStream.CreateIterator(); It; It++)
	{
		if (UTexture* Texture = It->Get())
		{
			if (!FLandscapeTextureStreamingManager::IsTextureFullyStreamedIn(Texture))
				return false;
		}
		else
		{
			// Texture was garbage collected unexpectedly, remove from our list
			It.RemoveCurrentSwap();
		}
	}
	return true;
}

bool FLandscapeGrassMapsBuilder::FComponentState::IsBeyondEvictionRange(const TArray<FVector>& Cameras) const
{
	check(Stage == EComponentStage::GrassMapsPopulated);
	const FBoxSphereBounds WorldBounds = Component->CalcBounds(Component->GetComponentTransform());
	const float MinSqrDistanceToComponent = UE::Landscape::Grass::Private::CalculateMinDistanceToCamerasSquared(Cameras, WorldBounds);
	const float DiscardDistanceScale = GGrassMapGuardBandDiscardMultiplier * GGrassCullDistanceScale;
	const float MinEvictDistance = Component->GrassTypeSummary.MaxInstanceDiscardDistance * DiscardDistanceScale;
	return (MinSqrDistanceToComponent > MinEvictDistance * MinEvictDistance);
}


#if WITH_EDITOR

void FLandscapeGrassMapsBuilder::Build(UE::Landscape::EBuildFlags InBuildFlags)
{
	if (World)
	{
		int32 ValidCount = 0;
		int32 TotalCount = 0;

		// iterate proxies, update grass types and create the list of components to build
		TArray<TObjectPtr<ULandscapeComponent>> LandscapeComponentsToBuild;
		TSet<ALandscapeProxy*> LandscapeProxiesToBuild;
		for (TActorIterator<ALandscapeProxy> ProxyIt(World); ProxyIt; ++ProxyIt)
		{
			ALandscapeProxy* Proxy = *ProxyIt;
			check(!Proxy->HasAnyFlags(RF_ClassDefaultObject));

			for (ULandscapeComponent* Component : Proxy->LandscapeComponents)
			{
				FComponentState* State = ComponentStates.FindRef(Component);

				ValidCount += Component->GrassData->HasValidData() ? 1 : 0;
				TotalCount ++;

				bool bBuildThisComponent =
					Component->UpdateGrassTypes() ||							// grass types changed
					(State->Stage != EComponentStage::GrassMapsPopulated);		// not in a populated state yet

				if (bBuildThisComponent)
				{
					Component->MarkPackageDirty();
					LandscapeComponentsToBuild.Add(Component);
					LandscapeProxiesToBuild.Add(Proxy);
				}
			}

			Proxy->UpdateGrassTypeSummary();
		}

		UE_LOGF(LogGrass, Verbose, "FLandscapeGrassMapsBuilder::Build() building %d component grass maps (%d / %d valid)",
			LandscapeComponentsToBuild.Num(), ValidCount, TotalCount);

		// build the grass maps
		if (LandscapeComponentsToBuild.Num() > 0)
		{
			FScopedSlowTask SlowTask(static_cast<float>(LandscapeComponentsToBuild.Num()), (LOCTEXT("GrassMaps_BuildGrassMaps", "Building Grass maps")));
			SlowTask.MakeDialog();

			constexpr bool bMarkDirty = true;
			BuildGrassMapsNowForComponents(LandscapeComponentsToBuild, &SlowTask, bMarkDirty);
		}

		if (EnumHasAnyFlags(InBuildFlags, UE::Landscape::EBuildFlags::WriteFinalLog))
		{
			UE_LOGFMT_LOC(LogLandscape, Log, "BuildGrassFinalLog", "Build Grass: {NumProxies} landscape {NumProxies}|plural(one=proxy,other=proxies) built.", ("NumProxies", LandscapeProxiesToBuild.Num()));
		}
	}
}

int32 FLandscapeGrassMapsBuilder::CountOutdatedGrassMaps(const TArray<TObjectPtr<ULandscapeComponent>>& LandscapeComponents) const
{
	int32 ProxyOutdatedGrassMapCount = 0;
	for (ULandscapeComponent* Component : LandscapeComponents)
	{
	#if WITH_EDITOR
		if (Component->IsGrassMapOutdated())
	#else
		if (!Component->GrassData->HasValidData())
	#endif // WITH_EDITOR
		{
			ProxyOutdatedGrassMapCount++;
		}
	}
	return ProxyOutdatedGrassMapCount;
}

int32 FLandscapeGrassMapsBuilder::GetOutdatedGrassMapCount(bool bInForceUpdate) const
{
	if (World)
	{
		bool bUpdate = bInForceUpdate || GLandscapeEditModeActive;
		if (!bUpdate)
		{
			const double GrassMapsTimeNow = FPlatformTime::Seconds();
			// Recheck every 20 secs to handle the case where levels may have been Streamed in/out
			if ((GrassMapsTimeNow - GrassMapsLastCheckTime) > 20)
			{
				GrassMapsLastCheckTime = GrassMapsTimeNow;
				bUpdate = true;
			}
		}

		if (bUpdate)
		{
			OutdatedGrassMapCount = 0;
			for (TActorIterator<ALandscapeProxy> ProxyIt(World); ProxyIt; ++ProxyIt)
			{
				OutdatedGrassMapCount += CountOutdatedGrassMaps(ProxyIt->LandscapeComponents);
			}
		}
	}
	return OutdatedGrassMapCount;
}

#endif // WITH_EDITOR
#undef LOCTEXT_NAMESPACE
#undef GRASS_DEBUG_TRANSITION