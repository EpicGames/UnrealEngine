// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Async/AsyncWork.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "LandscapeComponent.h"
#include "LandscapeTextureStreamingManager.h"
#include "Containers/AllocatorFixedSizeFreeList.h"
#include "Spatial/PointHashGrid3.h"

class FLandscapeGrassWeightExporter;
struct FScopedSlowTask;

namespace UE::Landscape
{
	enum class EBuildFlags : uint8;
} // namespace UE::Landscape

class FAsyncFetchTask : public FNonAbandonableTask
{
public:
	// non-owned pointer. The lifetime is managed externally and must guarantee this pointer is valid while the task is in flight
	TNonNullPtr< FLandscapeGrassWeightExporter> ActiveRender;
	TMap<ULandscapeComponent*, TUniquePtr<FLandscapeComponentGrassData>, TInlineSetAllocator<1>> Results;

	FAsyncFetchTask(FLandscapeGrassWeightExporter* ActiveRender)
		: ActiveRender(ActiveRender)
	{
	}

	void DoWork();

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FAsyncFetchTask, STATGROUP_ThreadPoolAsyncTasks);
	}
};

/**
 * Helper class used to Build or monitor outdated Grass maps of a world
 */
class FLandscapeGrassMapsBuilder
{
public:
	LANDSCAPE_API FLandscapeGrassMapsBuilder(UWorld* InWorld, FLandscapeTextureStreamingManager& InTextureStreamingManager);
	~FLandscapeGrassMapsBuilder();

#if WITH_EDITOR
	/** Synchronously build all grass maps on all proxies in the world */
	LANDSCAPE_API void Build(UE::Landscape::EBuildFlags InBuildFlags);
	
	/** Return the number of landscape components in the world that have grass map data that is not up to date (NOT including components missing grass map data altogether) */
	LANDSCAPE_API int32 GetOutdatedGrassMapCount(bool bInForceUpdate = true) const;
#endif // WITH_EDITOR

	// count components that have valid data but whose generation hash does not match
	int32 CountOutdatedGrassMaps(const TArray<TObjectPtr<ULandscapeComponent>>& LandscapeComponents) const;

	// called when components are registered to the world
	void RegisterComponent(ULandscapeComponent* Component);
	// called when components are unregistered from the world.
	void UnregisterComponent(const ULandscapeComponent* Component);

	// get the number of grass maps that are still waiting to render, as of the last AmortizedUpdateGrassMaps()
	int32 GetTotalGrassMapsWaitingToRender() const { return TotalComponentsWaitingCount; }

	// Amortized Update of Grass Maps near the specified cameras.
	// if Cameras is empty, it considers all distances to be zero for update purposes:
	//   - it won't evict anything for distance
	//   - it will not start tracking (building grass maps) for any components
	void AmortizedUpdateGrassMaps(const TArray<FVector>& Cameras, bool bPrioritizeCreation, bool bAllowStartGrassMapGeneration, bool bAnyLandscapeActorHasGrassEnabled);

	// synchronous build of grassmaps for a specific set of components
	// returns true if it successfully makes all LandscapeComponents have up to date Grass Maps.
	bool BuildGrassMapsNowForComponents(TArrayView<TObjectPtr<ULandscapeComponent>> LandscapeComponents, FScopedSlowTask* SlowTask, bool bMarkDirty);

#if !UE_BUILD_SHIPPING
	// Force stream textures and issue draw calls for the specified components -- mainly for capturing the rendering behavior and debugging issues
	// (this will evict existing populated grass maps on those components)
	void DebugRenderComponents(TArrayView<TObjectPtr<ULandscapeComponent>> LandscapeComponents);
#endif // !UE_BUILD_SHIPPING

private:

	// false if the world can not currently render the grass (but this may change later, for example if preview modes are modified)
	bool CanCurrentlyRender() const;

	// Update all of the non-pending components.  Returns true if any components changed states.
	// If passed an empty Cameras array, distances are calculated as zero (i.e. it won't evict for distance)
	// MaxExpensiveUpdateChecksToPerform controls how many expensive component updates are performed (pass ComponentStates.Num() to process all)
	// bCancelAndEvictAllImmediately will move all tracked component states to the pending state and block until any in-flight processing is cancelled and cleaned up
	// bEvictWhenBeyondEvictionRange will evict any populated components that are beyond the eviction range distance (used to reclaim memory when using runtime generation)
	// DeltaTicks specifies how many ticks to advance the tick counter on each tracked component.  In general 1 tick, though in some cases where we are iterating within a single frame, and don't need to trigger delayed-tick events (i.e. D3D11 readback), we can pass 0.
	bool UpdateTrackedComponents(const TArray<FVector>& Cameras, int32 LocalMaxRendering, int32 MaxExpensiveUpdateChecksToPerform, bool bCancelAndEvictAllImmediately, bool bEvictWhenBeyondEvictionRange, int32 DeltaTicks);

	// Start the grass map generation process on pending components in priority order
	// (based on distance from the given Camera set) -- Cameras must not be empty.
	void StartPrioritizedGrassMapGeneration(const TArray<FVector>& Cameras, int32 MaxComponentsToStart, bool bOnlyWhenCloserThanEvictionRange);
	
	enum class EComponentStage : uint8
	{
		Pending,							// initial state; the component is waiting to start the generation process.
		NotReady,							// tried to start generation process, but either no grass types exist or the material is not ready. wait for that to change.
		TextureStreaming,					// texture streaming was requested.  wait for the mips to be available
		Rendering,							// GPU render commands were sent -- waiting for async readback to complete
		AsyncFetch,							// Waiting for the async fetch task to complete
		GrassMapsPopulated,					// grass maps are built and are ready to create instances
	};

	struct FComponentState
	{
	public:
		// this pointer is valid as long as the Component is registered
		// (the FComponentState itself may continue to exist after unregistration, until all resources are cleaned up)
		ULandscapeComponent* Component = nullptr;

		EComponentStage Stage = EComponentStage::Pending;

		bool bInPendingHeap = false;

		// counts the number of ticks this component has remained in the current stage
		int32 TickCount = 0;

		#if WITH_EDITOR
		// the hashes of dependencies when this component's grass map was built, for tracking automatic invalidation
		uint32 GrassMapGenerationHash = 0;
		uint32 GrassInstanceGenerationHash = 0;
		#endif // WITH_EDITOR

		// list of textures to stream prior to rendering this component (valid only in TextureSreaming and Rendering stages)
		TArray<TWeakObjectPtr<UTexture>> TexturesToStream;
		
		// the active render (valid only in Rendering stage)
		TUniquePtr<FLandscapeGrassWeightExporter> ActiveRender;

		// when in AsyncFetch stage, this is async task that we are waiting for
		TUniquePtr<FAsyncTask<FAsyncFetchTask>> AsyncFetchTask;

		FComponentState(ULandscapeComponent* Component);

		bool AreTexturesStreamedIn();
		bool IsBeyondEvictionRange(const TArray<FVector>& Cameras) const;
	};

	// components in the Pending state are also tracked in a priority heap, using this structure
	struct FPendingComponent
	{
		FPendingComponent(FComponentState* InComponentState)
			: State(InComponentState)
		{}

		// comparison operator used for Heap functions -- smaller keys have priority over larger keys
		bool operator<(const FPendingComponent& Other) const
		{
			return PriorityKey < Other.PriorityKey;
		}

		void UpdatePriorityDistance(const TArray<FVector>& Cameras, float MustHaveDistanceScale);

		FComponentState* State = nullptr;
		double PriorityKey = TNumericLimits<double>::Lowest();
	};

	// structure to do an amortized update of a set of update elements in a (conceptual) array
	struct FAmortizedUpdate
	{
		// the first index to update this tick (inclusive)
		int32 FirstIndex = 0;

		// the last index to update this tick (exclusive)
		int32 LastIndex = 0;

		// Call at the beginning of the update tick to set up how many updates you want to run this tick
		// UpdateElementCount is the total number of update elements (to wrap when we hit the end of the possibility space)
		void StartUpdateTick(int32 UpdateElementCount, int32 MaxUpdatesThisFrame)
		{
			check(MaxUpdatesThisFrame >= 0);
			if (MaxUpdatesThisFrame >= UpdateElementCount)
			{
				// if we are updating more than what we have, then just update everything
				FirstIndex = 0;
				LastIndex = UpdateElementCount;
			}
			else
			{
				FirstIndex = LastIndex;
				if (FirstIndex >= UpdateElementCount)
				{
					FirstIndex = 0;
				}
				check(FirstIndex >= 0);
				LastIndex = FirstIndex + MaxUpdatesThisFrame;
			}
		}

		// return true if the given element should update
		bool ShouldUpdate(int32 UpdateElementIndex)
		{
			if ((UpdateElementIndex >= FirstIndex) && (UpdateElementIndex < LastIndex))
			{
				return true;
			}
			return false;
		}

		// call if an update element is deleted from the array, to update the valid ranges
		void HandleDeletion(int32 DeletedUpdateElementIndex)
		{
			if (DeletedUpdateElementIndex < FirstIndex)
			{
				FirstIndex--;
			}
			if (DeletedUpdateElementIndex < LastIndex)
			{
				LastIndex--;
			}

		}
	};

	UWorld* World = nullptr;

	// counts of how many components are currently in each stage
	int32 PendingCount = 0;
	int32 NotReadyCount = 0;
	int32 StreamingCount = 0;
	int32 RenderingCount = 0;
	int32 AsyncFetchCount = 0;
	int32 PopulatedCount = 0;

	// number of components that need to render but are waiting (as of the last call to StartTrackingComponents())
	int32 TotalComponentsWaitingCount = 0;

	// true if any render thread commands were queued by the last call to UpdateTrackedComponents()
	bool bRenderCommandsQueuedByLastUpdate = false;

	TAllocatorFixedSizeFreeList<sizeof(FComponentState), 32> StatePoolAllocator;

	// store the grass map state of each registered (or recently unregistered) component
	TMap<ULandscapeComponent*, FComponentState*> ComponentStates;

	// Pending components, in a min heap by distance to streaming cameras
	TArray<FPendingComponent> PendingComponentsHeap;
	int32 FirstNewPendingComponent = TNumericLimits<int32>::Max();
	int32 PendingUpdateAmortizationCounter = 0;
	float LastMustHaveDistanceScale = 0.0f;			// used to detect spawn distance scales changing

	// state to amortize the update of components
	FAmortizedUpdate AmortizedUpdate;

	// system to manage texture streaming requests
	FLandscapeTextureStreamingManager& TextureStreamingManager;

	// hash grid storing previously used camera locations (to determine when cameras have jumped)
	UE::Geometry::TPointHashGrid3<FVector, double> PreviousCameraHashGrid;

	// tries to cancel any in flight operations and transition back to the Pending state
	// returns true when the state has been successfully transitioned back to Pending
	// if bCancelImmediately is true, it will ensure the component reaches Pending state before returning (possibly blocking on async or gpu tasks)
	bool CancelAndEvict(FComponentState& State, bool bCancelImmediately);

	// try to kick off the grass map generation pipeline -- returns true if it started the amortized update path, false otherwise.
	bool StartGrassMapGeneration(FComponentState& State, bool bForceCompileShaders, bool bTryFastPaths = true);

	// try to apply fast path transitions to a pending component -- returns true if a fastpath was taken, and the component is no longer pending.
	bool TryFastpathsFromPending(FComponentState& State, bool bRecalculateHashes);

	// once textures are streamed, this kicks off the grass data render, and async GPU readback
	void KickOffRenderAndReadback(FComponentState& State);

	// once the GPU readback is complete, this starts processing the data to generate a GrassData structure
	void LaunchAsyncFetchTask(FComponentState& State);

	void PopulateGrassDataFromAsyncFetchTask(FComponentState& State);

	// once the GPU readback is complete, this populates the grass data on the component (and cancels texture stream requests)
	void PopulateGrassDataFromReadback(FComponentState& State);
		void RemoveTextureStreamingRequests(FComponentState& State);

	// state transition helpers
	void StreamingToNotReady(FComponentState& State);
	void PendingToNotReady(FComponentState& State);
	void PendingToPopulatedFastPathAlreadyHasData(FComponentState& State);
	void PendingToPopulatedFastPathNoGrass(FComponentState& State);
	void PendingToStreaming(FComponentState& State);

	// pending component heap add/remove
	void AddToPendingComponentHeap(FComponentState* State);
	void RemoveFromPendingComponentHeap(FComponentState* State);

	void CompleteAllAsyncTasksNow();

#if WITH_EDITOR
	// cached count of how many grass maps are outdated, and the last time we calculated that value
	mutable int32 OutdatedGrassMapCount = 0;
	mutable double GrassMapsLastCheckTime = 0.0;
#endif // WITH_EDITOR
};


