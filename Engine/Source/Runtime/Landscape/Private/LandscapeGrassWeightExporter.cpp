// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeGrassWeightExporter.h"
#include "SceneRendererInterface.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Materials/Material.h"
#include "LandscapeGrassType.h"
#include "Engine/TextureRenderTarget2D.h"
#include "EngineModule.h"
#include "LandscapeRender.h"
#include "MeshPassProcessor.h"
#include "MeshPassProcessor.inl"
#include "SimpleMeshDrawCommandPass.h"
#include "TextureResource.h"
#include "RenderCaptureInterface.h"
#include "ShaderPlatformCachedIniValue.h"
#include "Landscape.h"
#include "LandscapePrivate.h"
#include "LandscapeSubsystem.h"
#include "LandscapeAsyncTextureReadback.h"
#include "Materials/MaterialExpressionLandscapeGrassOutput.h"
#include "PSOPrecacheSettings.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "RenderGraph.h"
#include "RHICommandList.h"

#if WITH_EDITOR
#include "Misc/FileHelper.h"
#endif // WITH_EDITOR

class FLandscapeGrassWeightVS;
class FLandscapeGrassWeightPS;

// TODO [jonathan.bard] : Remove this after we've let this run for a while
#define GRASS_ALLOW_EXTRAVALIDATION 1

#define GRASS_ENABLE_EXTRAVALIDATION (!UE_BUILD_SHIPPING && GRASS_ALLOW_EXTRAVALIDATION)

// ----------------------------------------------------------------------------------

int32 GRenderCaptureNextGrassmapDraws = 0;
static FAutoConsoleVariableRef CVarRenderCaptureNextGrassmapDraws(
	TEXT("grass.GrassMap.RenderCaptureNextDraws"),
	GRenderCaptureNextGrassmapDraws,
	TEXT("Trigger render captures during the next N grassmap draw calls."));

static TAutoConsoleVariable<int32> CVarDumpGrassmaps(
	TEXT("grass.GrassMap.Dump"),
	0,
	TEXT("Dumps every rendered grass map to disk."));


// ----------------------------------------------------------------------------------

extern int32 GGrassMapAlwaysBuildRuntimeGenerationResources;
extern int32 GGrassMapUseRuntimeGeneration;
extern int32 GGrassEnable;


// ----------------------------------------------------------------------------------

DECLARE_GPU_STAT(LandscapeGrassMaps);


// ----------------------------------------------------------------------------------

namespace UE::Landscape::Grass::Private
{
	static constexpr int32 GrassWeightInvalidIndex = -3;
	static constexpr int32 HeightGrassWeightIndex0 = -2;
	static constexpr int32 HeightGrassWeightIndex1 = -1;
} // namespace UE::Landscape::Grass::Private


// ----------------------------------------------------------------------------------

BEGIN_SHADER_PARAMETER_STRUCT(FLandscapeGrassPassParameters, )
	SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
	SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
	SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
	RENDER_TARGET_BINDING_SLOTS()
END_SHADER_PARAMETER_STRUCT()


// ----------------------------------------------------------------------------------

class FLandscapeGrassWeightShaderElementData : public FMeshMaterialShaderElementData
{
public:
	uint32 HeightGrassWeightChannelMask = 0;
	FUint32Vector4 PerGrassWeightChannelMask = FUint32Vector4(0, 0, 0, 0);
	FVector2f RenderOffset = FVector2f::ZeroVector;
};


// ----------------------------------------------------------------------------------

static bool ShouldCacheLandscapeGrassShaders(const FMeshMaterialShaderPermutationParameters& Parameters)
{
	const bool bIsEditorPlatform = 
		IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) &&
		EnumHasAllFlags(Parameters.Flags, EShaderPermutationFlags::HasEditorOnlyData);

#if WITH_EDITOR
	static FShaderPlatformCachedIniValue<int32> GrassMapsUseRuntimeGenerationPerPlatform(TEXT("grass.GrassMap.UseRuntimeGeneration"));
	const bool bPlatformUsesRuntimeGen = (GrassMapsUseRuntimeGenerationPerPlatform.Get(Parameters.Platform) != 0);
#else
	const bool bPlatformUsesRuntimeGen = (GGrassMapUseRuntimeGeneration != 0);
#endif // WITH_EDITOR

#if WITH_EDITOR
	static FShaderPlatformCachedIniValue<int32> GrassEnabledPerPlatform(TEXT("grass.Enable"));
	const bool bGrassEnable = (GrassEnabledPerPlatform.Get(Parameters.Platform) != 0);
#else 
	const bool bGrassEnable = GGrassEnable != 0;
#endif // WITH_EDITOR

	const bool bShouldBuildForPlatform = GGrassMapAlwaysBuildRuntimeGenerationResources || (
		bGrassEnable && (bIsEditorPlatform || bPlatformUsesRuntimeGen));

	const bool bIsFixedGridVertexFactory =
		Parameters.VertexFactoryType == FindVertexFactoryType(FName(TEXT("FLandscapeFixedGridVertexFactory"), FNAME_Find));

	// We only need grass weight shaders for Landscape fixed grid vertex factories
	// And only for platforms that have runtime generation enabled or are editor platforms (or if we are always building resources)
	const bool bIsLandscapeRelated = (Parameters.MaterialParameters.bIsUsedWithLandscape || Parameters.MaterialParameters.bIsSpecialEngineMaterial);

	const bool bShouldCache =
		bIsLandscapeRelated &&
		bIsFixedGridVertexFactory &&
		bShouldBuildForPlatform;

	return bShouldCache;
}


// ----------------------------------------------------------------------------------

class FLandscapeGrassWeightVS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FLandscapeGrassWeightVS, MeshMaterial);

	LAYOUT_FIELD(FShaderParameter, RenderOffsetParameter);

protected:

	FLandscapeGrassWeightVS()
	{}

	FLandscapeGrassWeightVS(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{
		RenderOffsetParameter.Bind(Initializer.ParameterMap, TEXT("RenderOffset"));
		PassUniformBuffer.Bind(Initializer.ParameterMap, FSceneTextureUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName());
	}

public:

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return ShouldCacheLandscapeGrassShaders(Parameters);
	}

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const FLandscapeGrassWeightShaderElementData& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);

		ShaderBindings.Add(RenderOffsetParameter, ShaderElementData.RenderOffset);
	}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FLandscapeGrassWeightVS, TEXT("/Engine/Private/LandscapeGrassWeight.usf"), TEXT("VSMain"), SF_Vertex);


// ----------------------------------------------------------------------------------

class FLandscapeGrassWeightPS : public FMeshMaterialShader
{
	DECLARE_SHADER_TYPE(FLandscapeGrassWeightPS, MeshMaterial);
	LAYOUT_FIELD(FShaderParameter, HeightGrassWeightChannelMaskParameter);
	LAYOUT_FIELD(FShaderParameter, PerGrassWeightChannelMaskParameter);
public:

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return ShouldCacheLandscapeGrassShaders(Parameters);
	}

	FLandscapeGrassWeightPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{
		HeightGrassWeightChannelMaskParameter.Bind(Initializer.ParameterMap, TEXT("HeightGrassWeightChannelMask"));
		PerGrassWeightChannelMaskParameter.Bind(Initializer.ParameterMap, TEXT("PerGrassWeightChannelMask"));
		PassUniformBuffer.Bind(Initializer.ParameterMap, FSceneTextureUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName());
	}

	FLandscapeGrassWeightPS()
	{}

	void GetShaderBindings(
		const FScene* Scene,
		ERHIFeatureLevel::Type FeatureLevel,
		const FPrimitiveSceneProxy* PrimitiveSceneProxy,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& Material,
		const FLandscapeGrassWeightShaderElementData& ShaderElementData,
		FMeshDrawSingleShaderBindings& ShaderBindings) const
	{
		FMeshMaterialShader::GetShaderBindings(Scene, FeatureLevel, PrimitiveSceneProxy, MaterialRenderProxy, Material, ShaderElementData, ShaderBindings);

		ShaderBindings.Add(HeightGrassWeightChannelMaskParameter, ShaderElementData.HeightGrassWeightChannelMask);
		ShaderBindings.Add(PerGrassWeightChannelMaskParameter, ShaderElementData.PerGrassWeightChannelMask);
	}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FLandscapeGrassWeightPS, TEXT("/Engine/Private/LandscapeGrassWeight.usf"), TEXT("PSMain"), SF_Pixel);


// ----------------------------------------------------------------------------------

namespace UE::Landscape::Grass::Private
{

void AddGrassWeightShaderTypes(FMaterialShaderTypes& InOutShaderTypes)
{
	InOutShaderTypes.AddShaderType<FLandscapeGrassWeightVS>();
	InOutShaderTypes.AddShaderType<FLandscapeGrassWeightPS>();
}

} // namespace UE::Landscape::Grass::Private

// ----------------------------------------------------------------------------------

class FLandscapeGrassWeightMeshProcessor : public FMeshPassProcessor
{
public:
	FLandscapeGrassWeightMeshProcessor(const FScene* Scene, ERHIFeatureLevel::Type InFeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext);

	void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		TConstArrayView<FIntVector4> PerPassPerChannelGrassWeightIndices,
		FVector2D ViewOffset,
		float PassOffsetX,
		int32 FirstHeightMipsPassIndex,
		const TArray<int32>& HeightMips,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy);

	virtual void AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch, uint64 BatchElementMask, const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy, int32 StaticMeshId = -1) override final
	{
		checkf(false, TEXT("Default AddMeshBatch can't be used as rendering requires extra parameters per pass."));
	}
	
	virtual void CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, 
		const FMaterial& Material, 
		const FPSOPrecacheVertexFactoryData& VertexFactoryData, 
		const FPSOPrecacheParams& PreCacheParams, 
		TArray<FPSOPrecacheData>& PSOInitializers) override final;

private:
	bool TryAddMeshBatch(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId,
		const FMaterialRenderProxy& MaterialRenderProxy,
		const FMaterial& MaterialResource,
		TConstArrayView<FIntVector4> PerPassPerChannelGrassWeightIndices,
		FVector2D ViewOffset,
		float PassOffsetX,
		int32 FirstHeightMipsPassIndex,
		const TArray<int32>& HeightMips);

	bool Process(
		const FMeshBatch& MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
		const FMaterial& RESTRICT MaterialResource,
		TConstArrayView<FIntVector4> PerPassPerChannelGrassWeightIndices,
		FVector2D ViewOffset,
		float PassOffsetX,
		int32 FirstHeightMipsPassIndex,
		const TArray<int32>& HeightMips);

	FMeshPassProcessorRenderState PassDrawRenderState;
};

static const TCHAR* LandscapeGrassWeightMeshPassName = TEXT("LandscapeGrassWeight");

FLandscapeGrassWeightMeshProcessor::FLandscapeGrassWeightMeshProcessor(const FScene* Scene, ERHIFeatureLevel::Type InFeatureLevel, const FSceneView* InViewIfDynamicMeshCommand, FMeshPassDrawListContext* InDrawListContext)
	: FMeshPassProcessor(LandscapeGrassWeightMeshPassName, Scene, InFeatureLevel, InViewIfDynamicMeshCommand, InDrawListContext)
{
	PassDrawRenderState.SetBlendState(TStaticBlendState<>::GetRHI());
	PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Always>::GetRHI());
}

void FLandscapeGrassWeightMeshProcessor::AddMeshBatch(const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	TConstArrayView<FIntVector4> PerPassPerChannelGrassWeightIndices,
	FVector2D ViewOffset,
	float PassOffsetX,
	int32 FirstHeightMipsPassIndex,
	const TArray<int32>& HeightMips,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy)
{
	const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
	while (MaterialRenderProxy)
	{
		const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
		if (Material)
		{
			if (TryAddMeshBatch(MeshBatch, BatchElementMask, PrimitiveSceneProxy, -1, *MaterialRenderProxy, *Material, PerPassPerChannelGrassWeightIndices, ViewOffset, PassOffsetX, FirstHeightMipsPassIndex, HeightMips))
			{
				break;
			}
		}

		MaterialRenderProxy = MaterialRenderProxy->GetFallback(FeatureLevel);
	}
}

bool FLandscapeGrassWeightMeshProcessor::TryAddMeshBatch(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId,
	const FMaterialRenderProxy& MaterialRenderProxy,
	const FMaterial& MaterialResource,
	TConstArrayView<FIntVector4> PerPassPerChannelGrassWeightIndices,
	FVector2D ViewOffset,
	float PassOffsetX,
	int32 FirstHeightMipsPassIndex,
	const TArray<int32>& HeightMips)
{
	check(MeshBatch.VertexFactory != nullptr);
	return Process(MeshBatch, BatchElementMask, PrimitiveSceneProxy, MaterialRenderProxy, MaterialResource, PerPassPerChannelGrassWeightIndices, ViewOffset, PassOffsetX, FirstHeightMipsPassIndex, HeightMips);
}

bool FLandscapeGrassWeightMeshProcessor::Process(
	const FMeshBatch& MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	const FMaterialRenderProxy& RESTRICT MaterialRenderProxy,
	const FMaterial& RESTRICT MaterialResource,
	TConstArrayView<FIntVector4> PerPassPerChannelGrassWeightIndices,
	FVector2D ViewOffset,
	float PassOffsetX,
	int32 FirstHeightMipsPassIndex,
	const TArray<int32>& HeightMips)
{
	using namespace UE::Landscape::Grass::Private;

	const FVertexFactory* VertexFactory = MeshBatch.VertexFactory;

	FMaterialShaderTypes ShaderTypes;
	ShaderTypes.AddShaderType<FLandscapeGrassWeightVS>();
	ShaderTypes.AddShaderType<FLandscapeGrassWeightPS>();

	FMaterialShaders Shaders;
	if (!MaterialResource.TryGetShaders(ShaderTypes, VertexFactory->GetType(), Shaders))
	{
		return false;
	}

	TMeshProcessorShaders<
		FLandscapeGrassWeightVS,
		FLandscapeGrassWeightPS> PassShaders;
	Shaders.TryGetVertexShader(PassShaders.VertexShader);
	Shaders.TryGetPixelShader(PassShaders.PixelShader);

	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
	const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(MaterialResource, OverrideSettings);
	const ERasterizerCullMode MeshCullMode = CM_None;

	FLandscapeGrassWeightShaderElementData ShaderElementData;
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, -1, true);

	const FMeshDrawCommandSortKey SortKey = CalculateMeshStaticSortKey(PassShaders.VertexShader, PassShaders.PixelShader);
	
	const int32 NumPasses = PerPassPerChannelGrassWeightIndices.Num();
	for (int32 PassIndex = 0; PassIndex < NumPasses; ++PassIndex)
	{
		ShaderElementData.HeightGrassWeightChannelMask = 0;
		ShaderElementData.PerGrassWeightChannelMask = FUint32Vector4(0, 0, 0, 0);
		// Convert the grass weight indices into a format that's more appropriate to the shader compiler : 
		for (uint32 ChannelIndex = 0; ChannelIndex < 4; ++ChannelIndex)
		{
			int32 GrassWeightIndex = PerPassPerChannelGrassWeightIndices[PassIndex][ChannelIndex];
			// 4 bits per output (one per RGBA channel)
			if (GrassWeightIndex == HeightGrassWeightIndex0)
			{
				// The first 4 bits of HeightGrassWeightChannelMask are for HeightGrassWeightIndex0
				ShaderElementData.HeightGrassWeightChannelMask |= (1u << (ChannelIndex + 0));
			}
			else if (GrassWeightIndex == HeightGrassWeightIndex1)
			{
				// The next 4 bits of HeightGrassWeightChannelMask are for HeightGrassWeightIndex1
				ShaderElementData.HeightGrassWeightChannelMask |= (1u << (ChannelIndex + 4));
			}
			// Grass weights : max 32 (MaxGrassTypes), 4 bits per grass weight (one per RGBA channel) = 128 bits, which fit in a uint4 (i.e. 8 grass weights per uint)
			else if (GrassWeightIndex != GrassWeightInvalidIndex)
			{
				ShaderElementData.PerGrassWeightChannelMask[GrassWeightIndex / 8] |= (1u << (ChannelIndex + (GrassWeightIndex % 8) * 4u));
			}
		}

		ShaderElementData.RenderOffset = FVector2f(ViewOffset) + FVector2f(PassOffsetX * PassIndex, 0);	// LWC_TODO: Precision loss

		uint64 Mask = (PassIndex >= FirstHeightMipsPassIndex) ? HeightMips[PassIndex - FirstHeightMipsPassIndex] : BatchElementMask;

		BuildMeshDrawCommands(
			MeshBatch,
			Mask,
			PrimitiveSceneProxy,
			MaterialRenderProxy,
			MaterialResource,
			PassDrawRenderState,
			PassShaders,
			MeshFillMode,
			MeshCullMode,
			SortKey,
			EMeshPassFeatures::Default,
			ShaderElementData);
	}

	return true;
}


void FLandscapeGrassWeightMeshProcessor::CollectPSOInitializers(const FSceneTexturesConfig& SceneTexturesConfig, const FMaterial& Material, const FPSOPrecacheVertexFactoryData& VertexFactoryData, const FPSOPrecacheParams& PreCacheParams, TArray<FPSOPrecacheData>& PSOInitializers)
{
	// Early out if disabled
	if (!UPSOPrecacheSettingsManager::GetSettings().bLandscapeGrassWeight)
	{
		return;
	}

	// Only support the Landscape fixed grid vertex factory type.
	if (VertexFactoryData.VertexFactoryType != &FLandscapeFixedGridVertexFactory::StaticType)
	{
		return;
	}

	FMaterialShaderTypes ShaderTypes;
	ShaderTypes.AddShaderType<FLandscapeGrassWeightVS>();
	ShaderTypes.AddShaderType<FLandscapeGrassWeightPS>();

	FMaterialShaders Shaders;
	if (!Material.TryGetShaders(ShaderTypes, VertexFactoryData.VertexFactoryType, Shaders))
	{
		return;
	}

	TMeshProcessorShaders<
		FLandscapeGrassWeightVS,
		FLandscapeGrassWeightPS> PassShaders;
	Shaders.TryGetVertexShader(PassShaders.VertexShader);
	Shaders.TryGetPixelShader(PassShaders.PixelShader);

	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(PreCacheParams);
	const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(Material, OverrideSettings);
	const ERasterizerCullMode MeshCullMode = CM_None;

	FGraphicsPipelineRenderTargetsInfo RenderTargetsInfo;
	RenderTargetsInfo.NumSamples = 1;
	AddRenderTargetInfo(PF_B8G8R8A8, ETextureCreateFlags::RenderTargetable, RenderTargetsInfo);
	
	AddGraphicsPipelineStateInitializer(
		VertexFactoryData,
		Material,
		PassDrawRenderState,
		RenderTargetsInfo,
		PassShaders,
		MeshFillMode,
		MeshCullMode,
		PT_PointList,
		EMeshPassFeatures::Default,
		true /*bRequired*/,
		PSOInitializers);
}

IPSOCollector* CreateLandscapeGrassWeightPSOCollector(ERHIFeatureLevel::Type FeatureLevel)
{
	return new FLandscapeGrassWeightMeshProcessor(nullptr, FeatureLevel, nullptr, nullptr);
}
FRegisterPSOCollectorCreateFunction RegisterLandscapeGrassWeightPSOCollector(&CreateLandscapeGrassWeightPSOCollector, EShadingPath::Deferred, LandscapeGrassWeightMeshPassName);
FRegisterPSOCollectorCreateFunction RegisterMobileLandscapeGrassWeightPSOCollector(&CreateLandscapeGrassWeightPSOCollector, EShadingPath::Mobile, LandscapeGrassWeightMeshPassName);


// ----------------------------------------------------------------------------------

FLandscapeGrassWeightExporter_RenderThread::FLandscapeGrassWeightExporter_RenderThread(int32 InComponentSizeVerts, int32 InSubsectionSizeQuads, int32 InNumSubsections, 
	TConstArrayView<int32> InHeightMips, UE::Landscape::Grass::EGrassWeightExporterFlags InFlags)
	: ComponentSizeVerts(InComponentSizeVerts)
	, SubsectionSizeQuads(InSubsectionSizeQuads)
	, NumSubsections(InNumSubsections)
	, HeightMips(InHeightMips)
	, Flags(InFlags)
{
	if (EnumHasAnyFlags(InFlags, UE::Landscape::Grass::EGrassWeightExporterFlags::ReadbackToCPU))
	{
		// even when doing a synchronous readback, we use the async readback structure
		GameThreadAsyncReadbackPtr = new FLandscapeAsyncTextureReadback();
	}
}

FLandscapeGrassWeightExporter_RenderThread::~FLandscapeGrassWeightExporter_RenderThread()
{
	if (GameThreadAsyncReadbackPtr != nullptr)
	{
		GameThreadAsyncReadbackPtr->QueueDeletionFromGameThread();
		GameThreadAsyncReadbackPtr = nullptr;
	}
}

void FLandscapeGrassWeightExporter_RenderThread::RenderLandscapeComponentToTexture_RenderThread(FRHICommandListImmediate& RHICmdList, FLandscapeAsyncTextureReadback* AsyncReadbackPtr)
{
	FRDGBuilder GraphBuilder(RHICmdList);

	RHI_BREADCRUMB_EVENT_STAT(RHICmdList, LandscapeGrassMaps, "LandscapeGrassMaps");

	FRDGTextureDesc TextureDesc = FRDGTextureDesc::Create2D(
		TargetSize,
		PF_B8G8R8A8,
		FClearValueBinding(),
		ETextureCreateFlags::RenderTargetable);

	FRDGTextureRef OutputTexture = GraphBuilder.CreateTexture(TextureDesc, TEXT("LandscapeGrassMapRenderTarget"), ERDGTextureFlags::None);

	RenderLandscapeComponentToTexture_RenderThread(GraphBuilder, OutputTexture);

	if (AsyncReadbackPtr != nullptr)
	{
		AsyncReadbackPtr->StartReadback_RenderThread(GraphBuilder, OutputTexture);
	}

	GraphBuilder.Execute();
}

void FLandscapeGrassWeightExporter_RenderThread::RenderLandscapeComponentToTexture_RenderThread(FRDGBuilder& GraphBuilder, FRDGTextureRef OutputTexture)
{
	RDG_EVENT_SCOPE(GraphBuilder, "RenderLandscapeComponentToTexture");

	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(nullptr, SceneInterface, FEngineShowFlags(ESFIM_Game))
		.SetTime(FGameTime::GetTimeSinceAppStart()));

	ViewFamily.LandscapeLODOverride = 0; // Force LOD render

	// Ensure scene primitive rendering is valid (added primitives comitted, GPU-Scene updated, push/pop dynamic culling context).
	FScenePrimitiveRenderingContextScopeHelper ScenePrimitiveRenderingContextScopeHelper(GetRendererModule().BeginScenePrimitiveRendering(GraphBuilder, &ViewFamily));

	FSceneViewInitOptions ViewInitOptions;
	ViewInitOptions.SetViewRectangle(FIntRect(0, 0, TargetSize.X, TargetSize.Y));
	ViewInitOptions.ViewOrigin = ViewOrigin;
	ViewInitOptions.ViewRotationMatrix = ViewRotationMatrix;
	ViewInitOptions.ProjectionMatrix = ProjectionMatrix;
	ViewInitOptions.ViewFamily = &ViewFamily;

	GetRendererModule().CreateAndInitSingleView(GraphBuilder.RHICmdList, &ViewFamily, &ViewInitOptions);

	const FSceneView* View = ViewFamily.Views[0];
	FLandscapeGrassPassParameters* PassParameters = GraphBuilder.AllocParameters<FLandscapeGrassPassParameters>();
	PassParameters->View = View->ViewUniformBuffer;
	PassParameters->Scene = GetSceneUniformBufferRef(GraphBuilder, *View);
	PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputTexture, ERenderTargetLoadAction::EClear);

	AddSimpleMeshPass(GraphBuilder, PassParameters, SceneInterface->GetRenderScene(), *View, nullptr, RDG_EVENT_NAME("LandscapeGrass"), View->UnscaledViewRect,
		[&View, this](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
		{
			FLandscapeGrassWeightMeshProcessor PassMeshProcessor(
				nullptr,
				View->GetFeatureLevel(),
				View,
				DynamicMeshPassContext);

			const uint64 DefaultBatchElementMask = 1 << 0; // LOD 0 only

			for (FComponentInfo& ComponentInfo : ComponentInfos)
			{
				if (ensure(ComponentInfo.SceneProxy))
				{
					const FMeshBatch& Mesh = ComponentInfo.SceneProxy->GetGrassMeshBatch();
					Mesh.MaterialRenderProxy->UpdateUniformExpressionCacheIfNeeded(View->GetFeatureLevel());

					PassMeshProcessor.AddMeshBatch(Mesh, DefaultBatchElementMask, ComponentInfo.PerPassPerChannelGrassWeightIndices, ComponentInfo.ViewOffset, PassOffsetX, ComponentInfo.FirstHeightMipsPassIndex, HeightMips, ComponentInfo.SceneProxy);
				}
			}
		});
}

TValueOrError<FIntRect /*OutputRect*/, FLandscapeGrassWeightExporter_RenderThread::EHeightmapTextureInfoError> FLandscapeGrassWeightExporter_RenderThread::GetTextureInfoForHeight(const FIntPoint& InComponentKey, int32 InMipIndex)
{
	using namespace UE::Landscape::Grass;
	using namespace UE::Landscape::Grass::Private;

	const FComponentInfo* FoundComponentInfo = ComponentInfos.FindByPredicate([InComponentKey](const FComponentInfo& InComponentInfo) { return (InComponentInfo.ComponentKey == InComponentKey); });
	if (FoundComponentInfo == nullptr)
	{
		return MakeError(EHeightmapTextureInfoError::InvalidComponentKey);
	}

	// This means : get the height (mip 0), even if not in the requested mips (requires EGrassWeightExporterFlags::NeedsHeightmap)
	if (InMipIndex < 0)
	{
		if (!EnumHasAnyFlags(Flags, EGrassWeightExporterFlags::NeedsHeightmap))
		{
			return MakeError(EHeightmapTextureInfoError::InvalidFlags);
		}

		// Height is always in the first pass if present :
		check(!FoundComponentInfo->PerPassPerChannelGrassWeightIndices.IsEmpty());
		// Height is always in the 2 first channels :
		check((FoundComponentInfo->PerPassPerChannelGrassWeightIndices[0][0] == HeightGrassWeightIndex0)
			&& (FoundComponentInfo->PerPassPerChannelGrassWeightIndices[0][1] == HeightGrassWeightIndex1));
		FIntPoint PixelOffset(FoundComponentInfo->PixelOffsetX, 0);
		FIntRect OutputRect(PixelOffset, PixelOffset + FIntPoint(GetTileSize(), GetTileSize()));
		return MakeValue(OutputRect);
	}
	
	// Other mips :
	if (!HeightMips.Contains(InMipIndex))
	{
		return MakeError(EHeightmapTextureInfoError::InvalidMipIndex);
	}

	int32 MipOffset = HeightMips.IndexOfByKey(InMipIndex);
	check(MipOffset != INDEX_NONE);
	int32 PassIndexForMip = FoundComponentInfo->FirstHeightMipsPassIndex + MipOffset;
	check(PassIndexForMip < FoundComponentInfo->GetNumPasses());

	check(FoundComponentInfo->PerPassPerChannelGrassWeightIndices.IsValidIndex(PassIndexForMip));
	// Height is always in the 2 first channels :
	check((FoundComponentInfo->PerPassPerChannelGrassWeightIndices[PassIndexForMip][0] == HeightGrassWeightIndex0)
		&& (FoundComponentInfo->PerPassPerChannelGrassWeightIndices[PassIndexForMip][1] == HeightGrassWeightIndex1));

	// Passes don't care about mip size so the offset is always a multiple of GetTileSize() :
	FIntPoint PixelOffset(FoundComponentInfo->PixelOffsetX + PassIndexForMip * GetTileSize(), 0);
	
	// Compute the number of vertices of that mip (i.e. without subsection duplicates)
	int32 MipSize = NumSubsections * (SubsectionSizeQuads >> InMipIndex) + 1;

	FIntRect OutputRect(PixelOffset, PixelOffset + FIntPoint(MipSize, MipSize));
	return MakeValue(OutputRect);
}

TValueOrError<TPair<FIntRect /*OutputRect*/, uint8 /*ChannelIndex*/>, FLandscapeGrassWeightExporter_RenderThread::EGrassmapTextureInfoError> FLandscapeGrassWeightExporter_RenderThread::GetTextureInfoForGrass(const FIntPoint& InComponentKey, const FName& InGrassName)
{
	using namespace UE::Landscape::Grass;
	using namespace UE::Landscape::Grass::Private;

	const FComponentInfo* FoundComponentInfo = ComponentInfos.FindByPredicate([InComponentKey](const FComponentInfo& InComponentInfo) { return (InComponentInfo.ComponentKey == InComponentKey); });
	if (FoundComponentInfo == nullptr)
	{
		return MakeError(EGrassmapTextureInfoError::InvalidComponentKey);
	}

	if (!EnumHasAnyFlags(Flags, EGrassWeightExporterFlags::NeedsGrassmap))
	{
		return MakeError(EGrassmapTextureInfoError::InvalidFlags);
	}

	const FComponentInfo::FPassChannelIndex* FoundPassChannelIndex = FoundComponentInfo->PerGrassNamePassChannelIndices.Find(InGrassName);
	if (FoundPassChannelIndex == nullptr)
	{
		return MakeError(EGrassmapTextureInfoError::InvalidGrassName);
	}

	check(FoundComponentInfo->PerPassPerChannelGrassWeightIndices.IsValidIndex(FoundPassChannelIndex->PassIndex));
	check(FoundPassChannelIndex->IsValidChannelIndex());
	check(FoundComponentInfo->PerPassPerChannelGrassWeightIndices[FoundPassChannelIndex->PassIndex][FoundPassChannelIndex->ChannelIndex] != GrassWeightInvalidIndex);

	FIntPoint PixelOffset(FoundComponentInfo->PixelOffsetX + FoundPassChannelIndex->PassIndex * GetTileSize(), 0);
	FIntRect OutputRect(PixelOffset, PixelOffset + FIntPoint(GetTileSize(), GetTileSize()));
	return MakeValue(MakeTuple(OutputRect, FoundPassChannelIndex->ChannelIndex));
}

namespace UE::Landscape::Private
{
// Remove once the old FLandscapeGrassWeightExporter constructor is deprecated : 
UE::Landscape::Grass::EGrassWeightExporterFlags BuildLandscapeGrassWeightExporterConstructionFlags(bool bInNeedsGrassmap, bool bInNeedsHeightmap, bool bInRenderImmediately, bool bInReadbackToCPU)
{
	using namespace UE::Landscape::Grass;

	EGrassWeightExporterFlags LandscapeGrassWeightExporterFlags = EGrassWeightExporterFlags::None;
	if (bInNeedsGrassmap)
	{
		LandscapeGrassWeightExporterFlags |= EGrassWeightExporterFlags::NeedsGrassmap;
	}
	if (bInNeedsHeightmap)
	{
		LandscapeGrassWeightExporterFlags |= EGrassWeightExporterFlags::NeedsHeightmap;
	}
	if (bInRenderImmediately)
	{
		LandscapeGrassWeightExporterFlags |= EGrassWeightExporterFlags::RenderImmediately;
	}
	if (bInReadbackToCPU)
	{
		LandscapeGrassWeightExporterFlags |= EGrassWeightExporterFlags::ReadbackToCPU;
	}
	return LandscapeGrassWeightExporterFlags;
}
} // namespace UE::Landscape::Private


// Deprecated
FLandscapeGrassWeightExporter::FLandscapeGrassWeightExporter(ALandscapeProxy* InLandscapeProxy, TArrayView<ULandscapeComponent* const> InLandscapeComponents, bool bInNeedsGrassmap, bool bInNeedsHeightmap, const TArray<int32>& InHeightMips, bool bInRenderImmediately, bool bInReadbackToCPU)
	: FLandscapeGrassWeightExporter(
		InLandscapeProxy->GetLandscapeActor(),
		InLandscapeComponents,
		UE::Landscape::Private::BuildLandscapeGrassWeightExporterConstructionFlags(bInNeedsGrassmap, bInNeedsHeightmap, bInRenderImmediately, bInReadbackToCPU),
		/*InRequestedGrassNames = */{},
		MakeArrayView(InHeightMips))
{
}

FLandscapeGrassWeightExporter::FLandscapeGrassWeightExporter(ALandscape* InLandscape, TArrayView<ULandscapeComponent* const> InRequestedComponents, UE::Landscape::Grass::EGrassWeightExporterFlags InFlags,
	TConstArrayView<FName> InRequestedGrassNames, TConstArrayView<int32> InRequestedHeightMips)
	: FLandscapeGrassWeightExporter_RenderThread(/*ComponentSizeVerts = */InLandscape->ComponentSizeQuads + 1, InLandscape->SubsectionSizeQuads, InLandscape->NumSubsections, InRequestedHeightMips, InFlags)
	, Landscape(InLandscape)
{
	check(InLandscape != nullptr);
	check(InRequestedComponents.Num() > 0);
	SceneInterface = InRequestedComponents[0]->GetScene();

	// todo: use a 2d target?
	const int32 SingleTileWidth = ComponentSizeVerts;
	TargetSize = FIntPoint(0, ComponentSizeVerts);

	// First compute the total render target size and prepare ComponentInfos (each component has its own number of needed passes because some might have different materials, thus different grass types) :
	ComponentInfos.Reserve(InRequestedComponents.Num());
	int32 CurrentPixelOffsetX = 0;
	FIntPoint MinSectionBase(MAX_int32, MAX_int32);
	for (ULandscapeComponent* Component : InRequestedComponents)
	{
		check(InLandscape == Component->GetLandscapeActor());
		MinSectionBase = MinSectionBase.ComponentMin(Component->GetSectionBase());
		ensure(Component->SceneProxy);

		FComponentInfo& ComponentInfo = ComponentInfos.Emplace_GetRef(Component, InFlags, InRequestedGrassNames, InRequestedHeightMips);
		ComponentInfo.PixelOffsetX = CurrentPixelOffsetX;

		CurrentPixelOffsetX += ComponentInfo.GetNumPasses() * SingleTileWidth;
	}
	TargetSize.X = CurrentPixelOffsetX;

	FIntPoint TargetSizeMinusOne(TargetSize - FIntPoint(1, 1));
	PassOffsetX = 2.0f * (float)SingleTileWidth / (float)TargetSize.X;

	// Then compute FComponentInfo's ViewOffset with the knowledge of the total render target size : 
	const int32 NumComponents = ComponentInfos.Num();
	check(NumComponents == InRequestedComponents.Num());
	for (int32 ComponentIndex = 0; ComponentIndex < NumComponents; ++ComponentIndex)
	{
		FComponentInfo& ComponentInfo = ComponentInfos[ComponentIndex];
		ULandscapeComponent* Component = InRequestedComponents[ComponentIndex];
		FIntPoint ComponentOffset = (Component->GetSectionBase() - InLandscape->GetSectionBase());
		FVector2D ViewOffset(-ComponentOffset.X, ComponentOffset.Y);
		ViewOffset.X += ComponentInfo.PixelOffsetX;
		ViewOffset /= (FVector2D(TargetSize) * 0.5f);
		ComponentInfo.ViewOffset = ViewOffset;
	}

	// center of target area in world
	FVector TargetCenter = InLandscape->GetTransform().TransformPosition(FVector(TargetSizeMinusOne, 0.f) * 0.5f);

	// extent of target in world space
	FVector TargetExtent = FVector(TargetSize, 0.0f) * InLandscape->GetActorScale() * 0.5f;

	ViewOrigin = TargetCenter;
	ViewRotationMatrix = FInverseRotationMatrix(InLandscape->GetActorRotation());
	ViewRotationMatrix *= FMatrix(FPlane(1.0f, 0.0f, 0.0f, 0.0f),
		FPlane(0.0f, -1.0f, 0.0f, 0.0f),
		FPlane(0.0f, 0.0f, -1.0f, 0.0f),
		FPlane(0.0f, 0.0f, 0.0f, 1.0f));

	const float ZOffset = UE_OLD_WORLD_MAX;
	ProjectionMatrix = FReversedZOrthoMatrix(
		TargetExtent.X,
		TargetExtent.Y,
		0.5f / ZOffset,
		ZOffset);

	if (EnumHasAnyFlags(InFlags, UE::Landscape::Grass::EGrassWeightExporterFlags::RenderImmediately))
	{
		UE::RenderCommandPipe::FSyncScope SyncScope;

		RenderCaptureInterface::FScopedCapture RenderCapture((GRenderCaptureNextGrassmapDraws != 0), TEXT("LandscapeGrassmapCapture"));
		GRenderCaptureNextGrassmapDraws = FMath::Max(0, GRenderCaptureNextGrassmapDraws - 1);

		// render
		FLandscapeGrassWeightExporter_RenderThread* Exporter = this;
		ENQUEUE_RENDER_COMMAND(FDrawSceneCommand)(
			[Exporter, AsyncReadbackPtr = GameThreadAsyncReadbackPtr](FRHICommandListImmediate& RHICmdList)
			{
				Exporter->RenderLandscapeComponentToTexture_RenderThread(RHICmdList, AsyncReadbackPtr);
				RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);
			});
	}
}

bool FLandscapeGrassWeightExporter::CheckAndUpdateAsyncReadback(bool& bOutRenderCommandsQueued, const bool bInForceFinish)
{
	check(GameThreadAsyncReadbackPtr != nullptr);
	return GameThreadAsyncReadbackPtr->CheckAndUpdate(bOutRenderCommandsQueued, bInForceFinish);
}

bool FLandscapeGrassWeightExporter::IsAsyncReadbackComplete()
{
	check(GameThreadAsyncReadbackPtr != nullptr);
	return GameThreadAsyncReadbackPtr->IsComplete();
}

struct FByteBuffer2DView : public IBuffer2DView<uint8>
{
	uint8* BufferStart = nullptr;
	int32 ByteStrideX = 0;
	int32 ByteStrideY = 0;
	int32 NumX = 0;
	int32 NumY = 0;

	FByteBuffer2DView(int32 InByteStrideX, int32 InByteStrideY, int32 InNumX, int32 InNumY)
		: ByteStrideX(InByteStrideX)
		, ByteStrideY(InByteStrideY)
		, NumX(InNumX)
		, NumY(InNumY)
	{}

	// copy elements from buffer to Dest, in X then Y order
	virtual void CopyTo(uint8* Dest, int32 SizeInBytes) const override
	{
		for (int Y = 0; SizeInBytes > 0 && Y < NumY; Y++)
		{
			uint8* Src = BufferStart + Y * ByteStrideY;
			int32 CopyCountX = FMath::Min(SizeInBytes, NumX);
			// we can't use memcpy because of the ByteStride
			while (CopyCountX--)
			{
				*Dest = *Src;
				Dest++;
				Src += ByteStrideX;
			}
			SizeInBytes -= NumX;
		}
	}

	// copy elements from buffer to Dest, in X then Y order, return true if the copied data is all zero
	virtual bool CopyToAndCalcIsAllZero(uint8* Dest, int32 SizeInBytes) const override
	{
		uint8 MaxBits = 0;
		for (int Y = 0; SizeInBytes > 0 && Y < NumY; Y++)
		{
			uint8* Src = BufferStart + Y * ByteStrideY;
			int32 CopyCountX = FMath::Min(SizeInBytes, NumX);
			// we can't use memcpy because of the ByteStride
			while (CopyCountX--)
			{
				uint8 Value = *Src;
				MaxBits = MaxBits | Value;
				*Dest = Value;
				Dest++;
				Src += ByteStrideX;
			}
			SizeInBytes -= NumX;
		}
		return (MaxBits == 0);
	}

	virtual int32 Num() const override 
	{ 
		return NumX * NumY; 
	}
};

struct FHeightBuffer2DView : public IBuffer2DView<uint16>
{
	FColor* BufferStart = nullptr;
	int32 StrideY = 0;
	int32 NumX = 0;
	int32 NumY = 0;

	// copy elements from buffer to Dest, in X then Y order
	virtual void CopyTo(uint16* Dest, int32 Count) const override
	{
		static_assert(PLATFORM_LITTLE_ENDIAN, "Big-endian is not supported anymore");
		for (int y = 0; Count > 0 && y < NumY; y++)
		{
			FColor* Src = BufferStart + y * StrideY;
			int32 CopyCountX = FMath::Min(Count, NumX);
			while (CopyCountX--)
			{
				*Dest = (((uint16)Src->R) << 8) + (uint16)(Src->G);
				Dest++;
				Src++;
			}
			Count -= NumX;
		}
	}

	// copy elements from buffer to Dest, in X then Y order
	virtual bool CopyToAndCalcIsAllZero(uint16* Dest, int32 Count) const override
	{
		unimplemented()
		return true;
	}

	virtual int32 Num() const override { return NumX * NumY; }
};


void FLandscapeGrassWeightExporter::FreeAsyncReadback()
{
	check(GameThreadAsyncReadbackPtr != nullptr);
	GameThreadAsyncReadbackPtr->QueueDeletionFromGameThread();
	GameThreadAsyncReadbackPtr = nullptr;
}

void FLandscapeGrassWeightExporter::CancelAndSelfDestruct()
{
	check(GameThreadAsyncReadbackPtr != nullptr);

	// Cancel the readback, and queue destruction on the render thread
	GameThreadAsyncReadbackPtr->CancelAndSelfDestruct();
	GameThreadAsyncReadbackPtr = nullptr;

	// Queue destruction of FLandscapeGrassWeightExporter, also on the render thread
	FLandscapeGrassWeightExporter* Exporter = this;
	ENQUEUE_RENDER_COMMAND(FCancelAndDestructCommand)(
		[Exporter](FRHICommandListImmediate& RHICmdList)
		{
			delete Exporter;
		});
}

TMap<ULandscapeComponent*, TUniquePtr<FLandscapeComponentGrassData>, TInlineSetAllocator<1>> FLandscapeGrassWeightExporter::FetchResults(bool bFreeAsyncReadback)
{
	using namespace UE::Landscape::Grass;
	using namespace UE::Landscape::Grass::Private;

	TRACE_CPUPROFILER_EVENT_SCOPE(FetchResults);

	TMap<ULandscapeComponent*, TUniquePtr<FLandscapeComponentGrassData>, TInlineSetAllocator<1>> Results;
	TArray<FColor> Samples;

	check(GameThreadAsyncReadbackPtr != nullptr);

	{
		FIntPoint Size;
		Samples = GameThreadAsyncReadbackPtr->TakeResults(&Size);
		if (bFreeAsyncReadback)
		{
			FreeAsyncReadback();
		}
		check(Size == TargetSize);
	}

	Results.Reserve(ComponentInfos.Num());
	FHeightBuffer2DView HeightData;
	TMap<FName, FByteBuffer2DView> WeightData;

	ULandscapeInfo* LandscapeInfo = Landscape->GetLandscapeInfo();
	// HACK [jonathan.bard] : this should be changed to a check 
	ensure(LandscapeInfo != nullptr);
	if (LandscapeInfo == nullptr)
	{
		return {};
	}

	for (FComponentInfo& ComponentInfo : ComponentInfos)
	{
		ULandscapeComponent** ComponentPtr = LandscapeInfo->XYtoComponentMap.Find(ComponentInfo.ComponentKey);
		if ((ComponentPtr == nullptr) || (*ComponentPtr == nullptr))
		{
			continue;
		}
		ULandscapeComponent* Component = *ComponentPtr;

		TUniquePtr<FLandscapeComponentGrassData> NewGrassData = MakeUnique<FLandscapeComponentGrassData>(Component);

		HeightData.NumX = ComponentSizeVerts;
		HeightData.NumY = ComponentSizeVerts;
		HeightData.StrideY = TargetSize.X;

#if WITH_EDITORONLY_DATA
		NewGrassData->HeightMipData.Empty(HeightMips.Num());
#endif // WITH_EDITORONLY_DATA

		WeightData.Empty();
		WeightData.Reserve(ComponentInfo.PerGrassNamePassChannelIndices.Num());

		for (auto& ItPair : ComponentInfo.PerGrassNamePassChannelIndices)
		{
			// Note: WeightData points directly at the elements of GrassWeightArrays (DO NOT REALLOCATE GRASSWEIGHTARRAYS)
			WeightData.Add(ItPair.Key, FByteBuffer2DView(/*InByteStrideX = */4, /*InByteStrideY = */TargetSize.X * 4, /*InNumX = */ComponentSizeVerts, /*InNumY = */ComponentSizeVerts));
		}

		auto FindGrassNameForPassAndGrassWeight = [&](const FComponentInfo::FPassChannelIndex& InPassChannelIndex) -> FName
		{
			for (auto& ItPair : ComponentInfo.PerGrassNamePassChannelIndices)
			{
				if (ItPair.Value == InPassChannelIndex)
				{
					return ItPair.Key;
				}
			}
			return NAME_None;
		};

#if WITH_EDITOR
		// output debug bitmap
		if (CVarDumpGrassmaps.GetValueOnGameThread())
		{
			FString WorldName = Component->GetWorld()->GetName();
			ULandscapeSubsystem* LandscapeSubsystem = Component->GetWorld()->GetSubsystem<ULandscapeSubsystem>();
			check(LandscapeSubsystem != nullptr);
			const FDateTime CurrentTime = LandscapeSubsystem->GetAppCurrentDateTime();
			FString ParentLandscapeActorName = Landscape->GetActorLabel();
			ALandscapeProxy* Proxy = Cast<ALandscapeProxy>(Component->GetOwner());
			check(Proxy != nullptr);
			FString ActorName = Proxy->GetActorLabel();
			FString FilePath = FString::Format(TEXT("{0}/LandscapeGrass/{1}/{2}/{3}/{4}-{5}"), 
				{ FPaths::ProjectSavedDir(), CurrentTime.ToString(), WorldName, ParentLandscapeActorName, ActorName, Component->GetDebugName() });
			TStringBuilder<1024> StrBuilder;
			for (int32 PassIndex = 0; PassIndex < ComponentInfo.GetNumPasses(); ++PassIndex)
			{
				auto GetChannelString = [&](const FComponentInfo::FPassChannelIndex& InPassChannelIndex) -> FString
					{
						FString Result(TEXT("<empty>"));
						int32 GrassWeightIndex = ComponentInfo.PerPassPerChannelGrassWeightIndices[InPassChannelIndex.PassIndex][InPassChannelIndex.ChannelIndex];
						if (GrassWeightIndex != GrassWeightInvalidIndex)
						{
							if (InPassChannelIndex.PassIndex < ComponentInfo.FirstHeightMipsPassIndex)
							{
								if (GrassWeightIndex == HeightGrassWeightIndex0)
								{
									Result = FString(TEXT("[Mip=0]Height0"));
								}
								else if (GrassWeightIndex == HeightGrassWeightIndex1)
								{
									Result = FString(TEXT("[Mip=0]Height1"));
								}
								else 
								{
									const FName GrassName = FindGrassNameForPassAndGrassWeight(InPassChannelIndex);
									check(!GrassName.IsNone());
									Result = GrassName.ToString();
								}
							}
							else
							{
								int32 HeightMipIndex = InPassChannelIndex.PassIndex - ComponentInfo.FirstHeightMipsPassIndex;
								check(HeightMips.IsValidIndex(HeightMipIndex));
								if (GrassWeightIndex == HeightGrassWeightIndex0)
								{
									Result = FString::Printf(TEXT("[Mip=%i]Height0"), HeightMips[HeightMipIndex]);
								}
								else if (GrassWeightIndex == HeightGrassWeightIndex1)
								{
									Result = FString::Printf(TEXT("[Mip=%i]Height1"), HeightMips[HeightMipIndex]);
								}
							}
						}
						return Result;
					};

				int32 OffsetX = ComponentInfo.PixelOffsetX + PassIndex * ComponentSizeVerts;
				StrBuilder.Appendf(TEXT("- OffsetX = %i :\n"), OffsetX);
				StrBuilder.Appendf(TEXT("R : %s\n"), *GetChannelString(FComponentInfo::FPassChannelIndex { PassIndex, 0 }));
				StrBuilder.Appendf(TEXT("G : %s\n"), *GetChannelString(FComponentInfo::FPassChannelIndex { PassIndex, 1 }));
				StrBuilder.Appendf(TEXT("B : %s\n"), *GetChannelString(FComponentInfo::FPassChannelIndex { PassIndex, 2 }));
				StrBuilder.Appendf(TEXT("A : %s\n"), *GetChannelString(FComponentInfo::FPassChannelIndex { PassIndex, 3 }));
			}
			FFileHelper::SaveStringToFile(StrBuilder, *(FilePath + ".desc.txt"));
			FFileHelper::CreateBitmap(*(FilePath + ".bmp"), TargetSize.X, TargetSize.Y, Samples.GetData(), /*SubRegion = */nullptr, &IFileManager::Get(), /*OutFilename = */nullptr, /*bInWriteAlpha = */!ComponentInfo.PerGrassNamePassChannelIndices.IsEmpty());
		}
#endif // WITH_EDITOR
	
#if GRASS_ENABLE_EXTRAVALIDATION
		FIntRect HeightRect;
		if (EnumHasAnyFlags(Flags, EGrassWeightExporterFlags::NeedsHeightmap))
		{
			TValueOrError<FIntRect /*OutputRect*/, EHeightmapTextureInfoError> TextureInfo = GetTextureInfoForHeight(ComponentInfo.ComponentKey);
			checkf(!TextureInfo.HasError(), TEXT("GetTextureInfoForHeight failed (error : %i)"), static_cast<int32>(TextureInfo.GetError()));
			HeightRect = TextureInfo.GetValue();
		}

		TMap<int32, FIntRect> HeightMipsRects;
		HeightMipsRects.Reserve(HeightMips.Num());
		for (int32 MipIndex : HeightMips)
		{
			TValueOrError<FIntRect /*OutputRect*/, EHeightmapTextureInfoError> TextureInfo = GetTextureInfoForHeight(ComponentInfo.ComponentKey, MipIndex);
			checkf(!TextureInfo.HasError(), TEXT("GetTextureInfoForHeight failed for mip %i (error : %i)"), MipIndex, static_cast<int32>(TextureInfo.GetError()));
			HeightMipsRects.Add(MipIndex, TextureInfo.GetValue());
		}

		TMap<FName, TPair<FIntRect, int32>> GrassRectsAndChannelIndices;
		for (auto& ItPair : ComponentInfo.PerGrassNamePassChannelIndices)
		{
			TValueOrError<TPair<FIntRect /*OutputRect*/, uint8 /*ChannelIndex*/>, EGrassmapTextureInfoError> TextureInfo = GetTextureInfoForGrass(ComponentInfo.ComponentKey, ItPair.Key);
			checkf(!TextureInfo.HasError(), TEXT("GetTextureInfoForGrass failed for grass name %s (error : %i)"), *ItPair.Key.ToString(), static_cast<int32>(TextureInfo.GetError()));
			GrassRectsAndChannelIndices.Add(ItPair.Key, TextureInfo.GetValue());
		}
#endif // GRASS_ENABLE_EXTRAVALIDATION

		for (int32 PassIdx = 0; PassIdx < ComponentInfo.GetNumPasses(); PassIdx++)
		{
			FColor* SampleData = &Samples[ComponentInfo.PixelOffsetX + PassIdx * ComponentSizeVerts];
			const FIntVector4& PerChannelGrassWeightIndices = ComponentInfo.PerPassPerChannelGrassWeightIndices[PassIdx];

			if (PassIdx < ComponentInfo.FirstHeightMipsPassIndex)
			{
				int32 ChannelIndex = 0;
				if (PassIdx == 0)	// height in RG, grass weights in BA
				{
#if GRASS_ENABLE_EXTRAVALIDATION
					FColor* HeightSampleData = &Samples[HeightRect.Min.X];
					check(HeightSampleData == SampleData);
#endif // GRASS_ENABLE_EXTRAVALIDATION

					check((PerChannelGrassWeightIndices[0] == HeightGrassWeightIndex0) && (PerChannelGrassWeightIndices[1] == HeightGrassWeightIndex1));
					HeightData.BufferStart = SampleData;
					ChannelIndex += 2;
				}

				for (; ChannelIndex < 4; ++ChannelIndex)
				{
					if (PerChannelGrassWeightIndices[ChannelIndex] != GrassWeightInvalidIndex)
					{
						const FName GrassName = FindGrassNameForPassAndGrassWeight(FComponentInfo::FPassChannelIndex { PassIdx, ChannelIndex });
						check(!GrassName.IsNone());

#if GRASS_ENABLE_EXTRAVALIDATION
						TPair<FIntRect, int32>* RectAndIndex = GrassRectsAndChannelIndices.Find(GrassName);
						check(RectAndIndex != nullptr);
						check(ChannelIndex == RectAndIndex->Value);
						FColor* GrassSampleData = &Samples[RectAndIndex->Key.Min.X];
						check(GrassSampleData == SampleData);
#endif // GRASS_ENABLE_EXTRAVALIDATION

						FByteBuffer2DView* ByteBufferView = WeightData.Find(GrassName);
						check(ByteBufferView != nullptr);
						static_assert(PLATFORM_LITTLE_ENDIAN, "Big-endian is not supported anymore");
						ByteBufferView->BufferStart = reinterpret_cast<uint8*>((ChannelIndex == 0) ? &SampleData->R : (ChannelIndex == 1) ? &SampleData->G : (ChannelIndex == 2) ? &SampleData->B : &SampleData->A);
					}
				}
			}
			else // PassIdx >= FirstHeightMipsPassIndex
			{
#if WITH_EDITOR
				const int32 Mip = HeightMips[PassIdx - ComponentInfo.FirstHeightMipsPassIndex];
				int32 MipSizeVerts = NumSubsections * (SubsectionSizeQuads >> Mip);
				TArray<uint16>& MipHeightData = NewGrassData->HeightMipData.Add(Mip);
				MipHeightData.SetNumUninitialized(MipSizeVerts* MipSizeVerts);
				uint16* DstMipHeight = MipHeightData.GetData();
				for (int32 y = 0; y < MipSizeVerts; y++)
				{
					FColor* SrcSample = &SampleData[y * TargetSize.X];
					for (int32 x = 0; x < MipSizeVerts; x++)
					{
						static_assert(PLATFORM_LITTLE_ENDIAN, "Big-endian is not supported anymore");
						*DstMipHeight++ = (((uint16)SrcSample->R) << 8) + (uint16)(SrcSample->G);
						SrcSample++;
					}
				}

#if GRASS_ENABLE_EXTRAVALIDATION
				FIntRect* MipRect = HeightMipsRects.Find(Mip);
				check(MipRect != nullptr);
				FColor* MipSampleData = &Samples[MipRect->Min.X];
				check(MipSampleData == SampleData);
#endif // GRASS_ENABLE_EXTRAVALIDATION

#endif // WITH_EDITOR
			}
		}
		
		TMap<FName, IBuffer2DView<uint8>*> WeightDataPtrs;
		WeightDataPtrs.Reserve(WeightData.Num());
		for (auto& ItPair : WeightData)
		{
			WeightDataPtrs.Add(ItPair.Key, static_cast<IBuffer2DView<uint8>*>(&ItPair.Value));
		}
		NewGrassData->InitializeFrom(&HeightData, WeightDataPtrs, /* bStripEmptyWeights = */ true);
		Results.Add(Component, MoveTemp(NewGrassData));
	}

	return Results;
}

void FLandscapeGrassWeightExporter::ApplyResults()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ApplyResults);

	TMap<ULandscapeComponent*, TUniquePtr<FLandscapeComponentGrassData>, TInlineSetAllocator<1>> NewGrassData = FetchResults(/* bFreeAsyncReadback = */ true);
	ApplyResults(NewGrassData);
}

void FLandscapeGrassWeightExporter::ApplyResults(TMap<ULandscapeComponent*, TUniquePtr<FLandscapeComponentGrassData>, TInlineSetAllocator<1>>& Results)
{
	for (auto&& GrassDataPair : Results)
	{
		ULandscapeComponent* Component = GrassDataPair.Key;
		FLandscapeComponentGrassData* ComponentGrassData = GrassDataPair.Value.Release();
		ALandscapeProxy* Proxy = Component->GetLandscapeProxy();

		UE_LOGF(LogGrass, Verbose, "Populating component %ls with grass data, size: %d", *Component->GetName(), ComponentGrassData->NumElements);

		// Assign the new data (thread-safe)
		Component->GrassData = MakeShareable(ComponentGrassData);

#if WITH_EDITOR
		if (Proxy->bBakeMaterialPositionOffsetIntoCollision)
		{
			Component->DestroyCollisionData();
			Component->UpdateCollisionData();
		}
#endif // WITH_EDITOR
	}
}

FLandscapeGrassWeightExporter_RenderThread::FComponentInfo::FComponentInfo(ULandscapeComponent* InComponent, UE::Landscape::Grass::EGrassWeightExporterFlags InFlags,
	TConstArrayView<FName> InRequestedGrassNames, TConstArrayView<int32> InRequestedHeightMips) 
	: ComponentKey(InComponent->GetComponentKey())
	, SceneProxy((FLandscapeComponentSceneProxy*)InComponent->SceneProxy)
	, Flags(InFlags)
{
	using namespace UE::Landscape::Grass;
	using namespace UE::Landscape::Grass::Private;

	int32 CurrentPassIndex = -1;
	int32 CurrentChannelIndex = -1;

	auto RequestNewPass = [&]() 
		{
			// Setting the channel index to -1 will ensure that a new pass is created in the next call to FindOrAddChannel :
			CurrentChannelIndex = -1;
		};

	auto FindOrAddChannel = [&]() -> FPassChannelIndex
		{
			if ((CurrentChannelIndex < 0) || (CurrentChannelIndex >= 3))
			{
				// Add a new pass :
				PerPassPerChannelGrassWeightIndices.Emplace(GrassWeightInvalidIndex, GrassWeightInvalidIndex, GrassWeightInvalidIndex, GrassWeightInvalidIndex);
				CurrentChannelIndex = -1;
				++CurrentPassIndex;
			}
			return FPassChannelIndex{ CurrentPassIndex, ++CurrentChannelIndex };
		};

	auto SetValueAtPassIndex = [&](FPassChannelIndex InPassChannelIndex, int32 InValue)
		{
			check(PerPassPerChannelGrassWeightIndices.IsValidIndex(InPassChannelIndex.PassIndex));
			check(InPassChannelIndex.IsValidChannelIndex());
			PerPassPerChannelGrassWeightIndices[InPassChannelIndex.PassIndex][InPassChannelIndex.ChannelIndex] = InValue;
		};

	// Start with 2 channels for heightmaps if needed : 
	if (EnumHasAnyFlags(InFlags, EGrassWeightExporterFlags::NeedsHeightmap))
	{
		SetValueAtPassIndex(FindOrAddChannel(), HeightGrassWeightIndex0);
		SetValueAtPassIndex(FindOrAddChannel(), HeightGrassWeightIndex1);
	}

	if (EnumHasAnyFlags(InFlags, EGrassWeightExporterFlags::NeedsGrassmap))
	{
		TArray<FName> RenderableGrassNames;
		InComponent->GetNamedGrassTypes().GetKeys(RenderableGrassNames);
		TArray<FName> RequestedGrassNames = InRequestedGrassNames.IsEmpty() ? RenderableGrassNames : TArray<FName>(InRequestedGrassNames);
		for (FName RequestedGrassName : RequestedGrassNames)
		{
			if (!RequestedGrassName.IsNone())
			{
				if (int32 GrassOutputIndex = RenderableGrassNames.IndexOfByKey(RequestedGrassName);
					GrassOutputIndex != INDEX_NONE)
				{
					check(GrassOutputIndex < UMaterialExpressionLandscapeGrassOutput::MaxGrassTypes);
					FPassChannelIndex PassChannelIndex = FindOrAddChannel();
					SetValueAtPassIndex(PassChannelIndex, GrassOutputIndex);
					PerGrassNamePassChannelIndices.Add(RequestedGrassName, PassChannelIndex);
				}
			}
		}
	}

	// since we don't read HeightMips unless we are in editor, there's no reason to add passes for it unless we are in editor
#if WITH_EDITOR
	// Add a new pass for each height mip, for each pass we'll use just 2 channels :  
	if (!InRequestedHeightMips.IsEmpty())
	{
		FirstHeightMipsPassIndex = CurrentPassIndex + 1;

		for (int32 MipIndex : InRequestedHeightMips)
		{
			// Request a new pass for each height mip :
			RequestNewPass();
			SetValueAtPassIndex(FindOrAddChannel(), HeightGrassWeightIndex0);
			SetValueAtPassIndex(FindOrAddChannel(), HeightGrassWeightIndex1);
		}
	}
#endif // WITH_EDITOR
}

#undef GRASS_ENABLE_EXTRAVALIDATION