// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeGroup.h"
#include "CoreMinimal.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Landscape.h"
#include "LandscapeStreamingProxy.h"
#include "LandscapeEdgeFixup.h"
#include "LandscapePrivate.h"
#include "LandscapeSubsystem.h"
#include "LandscapeEditReadback.h"

// enables verbose debug spew
//#define ENABLE_LANDSCAPE_EDGE_FIXUP_DEBUG_SPEW 1
#ifdef ENABLE_LANDSCAPE_EDGE_FIXUP_DEBUG_SPEW
#define GROUP_DEBUG_LOG(...) UE_LOG(LogLandscape, Warning, __VA_ARGS__)
#define GROUP_DEBUG_LOG_DETAIL(...) UE_LOG(LogLandscape, Warning, __VA_ARGS__)
#else
#define GROUP_DEBUG_LOG(...) UE_LOG(LogLandscape, Verbose, __VA_ARGS__)
#define GROUP_DEBUG_LOG_DETAIL(...) do {} while(0)
#endif // ENABLE_LANDSCAPE_EDGE_FIXUP_DEBUG_SPEW

TMap<TObjectPtr<UTexture2D>, TObjectPtr<ULandscapeComponent>> FLandscapeGroup::HeightmapTextureToActiveComponent;

namespace UE::Landscape
{
	int32 GInstallEdgeFixup = 1;
	static FAutoConsoleVariableRef CVarInstallEdgeFixup(
		TEXT("landscape.InstallEdgeFixup"),
		GInstallEdgeFixup,
		TEXT("Controls whether edge fixup tracking is installed on landscape heightmap textures.  Default enabled (1)."));

	bool ShouldInstallEdgeFixup()
	{
		return GInstallEdgeFixup != 0;
	}

	int32 GPatchEdges = 1;
	static FAutoConsoleVariableRef CVarPatchEdges(
		TEXT("landscape.PatchEdges"),
		GPatchEdges,
		TEXT("Controls whether landscape heightmap texture edges are patched to match neighboring heightmaps.  Default enabled (1)."));

	int32 GPatchStreamingMipEdges = 1;
	static FAutoConsoleVariableRef CVarPatchStreamingMipEdges(
		TEXT("landscape.PatchStreamingMipEdges"),
		GPatchStreamingMipEdges,
		TEXT("Controls whether landscape heightmap texture MIP edges are patched when they stream in.  Default enabled (1)."));

	bool ShouldPatchStreamingMipEdges()
	{
		return GPatchStreamingMipEdges != 0;
	}

	int32 GForcePatchAllEdges = 0;
	static FAutoConsoleVariableRef CVarForcePatchAllEdges(
		TEXT("landscape.ForcePatchAllEdges"),
		GForcePatchAllEdges,
		TEXT("Forces landscape edge patching to patch the edges of every registered landscape component on the next tick.  Default disabled (0)."));

	bool ShouldPatchAllLandscapeComponentEdges(bool bResetForNext)
	{
		bool bPatchAll = (GForcePatchAllEdges != 0);
		if (bResetForNext)
		{
			GForcePatchAllEdges = 0;
		}
		return bPatchAll;
	}

	int32 GAmortizedGroupValidation = 1;
	static FAutoConsoleVariableRef CVarAmortizedGroupValidation(
		TEXT("landscape.AmortizedGroupValidation"),
		GAmortizedGroupValidation,
		TEXT("Enables amortized validation of landscape group registrations, to verify that components are registered properly with their landscape group.  Default enabled (1)."));
}

void FLandscapeGroup::RegisterComponent(ULandscapeComponent* Component)
{
	if (Component->RegisteredEdgeFixup != nullptr)
	{
		// multiple registrations can happen in editor, ignore the extras
		GROUP_DEBUG_LOG_DETAIL(TEXT("Group %p RegisterComponent %p %d (%d,%d) -- AlreadyRegistered"), this, Component, Component->RegisteredEdgeFixup->bMapped, Component->RegisteredEdgeFixup->GroupCoord.X, Component->RegisteredEdgeFixup->GroupCoord.Y);
		return;
	}

	ALandscapeProxy* LandscapeProxy = Component->GetLandscapeProxy();
	check(LandscapeProxy != nullptr);

	const UWorld* ComponentWorld = Component->GetWorld();
	check(ComponentWorld != nullptr);

	// TODO [jonathan.bard] : Remove this once we remove shared heightmaps, so we can enable edge fixup on non-WP landscapes (and world composition levels too).
	if (!LandscapeProxy->IsA<ALandscapeStreamingProxy>() || (ComponentWorld->WorldComposition != nullptr))
	{
		return;
	}
	
	if (!LandscapeProxy->bIsLandscapeActorRegisteredWithLandscapeInfo)
	{
		// the landscape group only registers components on streaming proxies (we don't handle non-WP cases)
		// and we can't register until the landscape actor has been registered and shared settings are fixed
		// (we will re-register all streaming proxy components when the landscape actor gets registered,
		// to handle any components that were skipped here)
		GROUP_DEBUG_LOG_DETAIL(TEXT("Group %p RegisterComponent %p -- WaitForActor (%d RegisteredFixups, %d NeedPatching, %d Mapped, %d Moved)"),
			this, Component, AllRegisteredFixups.Num(), HeightmapsNeedingEdgeTexturePatching.Num(), XYToEdgeFixupMap.Num(), HeightmapsMoved.Num());
		return;
	}

	// Early-out if the proxy hasn't got a proper root component. To our knowledge, this only ever happens when cooking where landscape groups are useless : 
	if (USceneComponent* RootComponent = LandscapeProxy->GetRootComponent(); (RootComponent == nullptr) || !RootComponent->IsRegistered())
	{
		UE_LOGF(LogLandscape, Verbose, "Skipped registration of landscape component to its proxy %ls : the proxy's root component isn't properly set or registered", *LandscapeProxy->GetFullName())
		return;
	}

	UTexture2D* Heightmap = Component->GetHeightmap();
	check(Heightmap);

	// grab an exclusive lock on the group registration data, as registration modifies the neighbor Mapping
	FWriteScopeLock ScopeWriteLock(RWLock);

	ULandscapeHeightmapTextureEdgeFixup* EdgeFixup = nullptr;
	bool bIsDisabled = false;
	{
		// check if there is an active component handling this heightmap texture
		if (TObjectPtr<ULandscapeComponent>* FoundComponent = HeightmapTextureToActiveComponent.Find(Heightmap))
		{
			ULandscapeComponent* ActiveComponent = FoundComponent->Get();
			check(ActiveComponent != Component);
			check(ActiveComponent->RegisteredLandscapeGroup != nullptr);

			const UWorld* ActiveComponentWorld = ActiveComponent->GetWorld();
			check(ComponentWorld != ActiveComponentWorld);
			
			// since there is already an active component, edge fixup is already created and registered, we can just grab it
			EdgeFixup = ActiveComponent->RegisteredEdgeFixup;
			check(EdgeFixup != nullptr);
			check(EdgeFixup->HeightmapTexture == Heightmap);
			check(EdgeFixup->ActiveComponent == ActiveComponent);
			check(EdgeFixup->ActiveGroup == ActiveComponent->RegisteredLandscapeGroup);

			bIsDisabled = EdgeFixup->DisabledComponents.Contains(Component);

			FLandscapeGroup* ActiveGroup = EdgeFixup->ActiveGroup;
			check(ActiveGroup != nullptr);

			GROUP_DEBUG_LOG_DETAIL(TEXT("Group %p RegisterComponent %p -- Use Existing EdgeFixup (Active Component %p, new component disabled: %d)"), this, Component, ActiveComponent, bIsDisabled);
		}
		else
		{
			// ensure texture user datas are installed
			GROUP_DEBUG_LOG_DETAIL(TEXT("Group %p RegisterComponent %p -- Activate EdgeFixup (%d RegisteredFixups, %d NeedPatching, %d Mapped, %d Moved)"),
				this, Component, AllRegisteredFixups.Num(), HeightmapsNeedingEdgeTexturePatching.Num(), XYToEdgeFixupMap.Num(), HeightmapsMoved.Num());

			const bool bShouldCompressHeightmap = false; // compress only happens at cook time, and once compressed it remains compressed
			const bool bUseEdgeFixup = true;
			const bool bUpdateSnapshotNow = false; // by default we wait until Tick to update the snapshot
			EdgeFixup = Component->InstallOrUpdateTextureUserDatas(bUseEdgeFixup, bShouldCompressHeightmap, bUpdateSnapshotNow);
			
			if (EdgeFixup == nullptr)
			{
				// failed to install, or InstallEdgeFixup is disabled
				return;
			}

			// there should be no disabled components (that requires a collision)
			check(EdgeFixup->DisabledComponents.Num() == 0);
			bIsDisabled = false;

			// there should be no active component yet
			check(EdgeFixup->ActiveGroup == nullptr);
			check(EdgeFixup->ActiveComponent == nullptr);
		}
	}

	check(EdgeFixup);
	AllRegisteredFixups.Add(EdgeFixup);
	Component->RegisteredEdgeFixup = EdgeFixup;
	Component->RegisteredLandscapeGroup = this;

	if (bIsDisabled)
	{
		// a disabled component is not set active on register, it must wait for the conflicting component to unregister first
		GROUP_DEBUG_LOG_DETAIL(TEXT("  -- EdgeFixup Not Active (Disabled)"));
		check(EdgeFixup->HeightmapTexture == Heightmap);
	}
	else
	{
		// set active will also map the fixup within the group
		EdgeFixup->SetActiveComponent(Component, this);
		HeightmapTextureToActiveComponent.FindOrAdd(Heightmap) = Component;

		GROUP_DEBUG_LOG_DETAIL(TEXT("  -- EdgeFixup Set %x Active At Coord (%d, %d)"), Component, EdgeFixup->GroupCoord.X, EdgeFixup->GroupCoord.Y);

#if WITH_EDITOR
		// Request an edge snapshot update, and initialize the GPU edge hashes from that.
		// Also disable patching until this happens.  This is because we may have loaded out of date hashes (if versions changed for instance)
		// and we should check if we need to recapture new snapshots before doing anything else.
		HeightmapsNeedingEdgeSnapshotCapture.Add(EdgeFixup);
		EdgeFixup->bUpdateGPUEdgeHashes = true;
		EdgeFixup->bDoNotPatchUntilGPUEdgeHashesUpdated = true;
#endif // WITH_EDITOR

		// Mark it, and all neighbors, as needing edge texture patching.
		// (the appearance of this new component may cause any of them to need patching)
		HeightmapsNeedingEdgeTexturePatching.Add(EdgeFixup);
		for (UE::Landscape::ENeighborIndex NeighborIndex : TEnumRange<UE::Landscape::ENeighborIndex>())
		{
			FIntPoint NeighborCoord = EdgeFixup->GroupCoord + GetNeighborRelativePosition(NeighborIndex);
			if (TObjectPtr<ULandscapeHeightmapTextureEdgeFixup>* NeighborPtrPtr = XYToEdgeFixupMap.Find(NeighborCoord))
			{
				HeightmapsNeedingEdgeTexturePatching.Add(NeighborPtrPtr->Get());
			}
		}
	}
}

void FLandscapeGroup::UnregisterComponent(ULandscapeComponent* Component)
{
	ULandscapeHeightmapTextureEdgeFixup* EdgeFixup = Component->RegisteredEdgeFixup;
	if (EdgeFixup == nullptr)
	{
		GROUP_DEBUG_LOG_DETAIL(TEXT("Group %p UnregisterComponent %p -- Already Unregistered"), this, Component);
		return;
	}
	check(Component->RegisteredLandscapeGroup == this);

	// grab an exclusive lock on the group registration data, as registration modifies the neighbor Mapping
	FWriteScopeLock ScopeWriteLock(RWLock);

	Component->RegisteredEdgeFixup = nullptr;
	Component->RegisteredLandscapeGroup = nullptr;

	int32 RemovedFixups = AllRegisteredFixups.Remove(EdgeFixup);
	check(RemovedFixups == 1);

	bool bIsDisabled = EdgeFixup->DisabledComponents.Contains(Component);

	if (bIsDisabled)
	{
		// unregistering a disabled component does not change the active component
		// the disabled component is just removed from the AllRegisteredFixups list.
		// the disabled component remains on the DisabledComponents list,
		// so that if it is re-registered it stay disabled if there is another active component
		GROUP_DEBUG_LOG_DETAIL(TEXT("Group %p UnregisterComponent %p %d (%d,%d) -- Was Disabled"), this, Component, EdgeFixup->bMapped, EdgeFixup->GroupCoord.X, EdgeFixup->GroupCoord.Y);
		check(EdgeFixup->ActiveComponent != Component);
	}
	else
	{
		check(EdgeFixup->ActiveComponent == Component);

		// first let's see if there is a replacement disabled component we can activate in place of the one being unregistered
		bool bReactivatedDisabled = false;
		while (EdgeFixup->DisabledComponents.Num() > 0)
		{
			// if a disabled duplicate exists (and is still registered), then reactivate it!
			// any disabled components that are no longer registered we can just drop
			if (ULandscapeComponent* DisabledComponent = EdgeFixup->DisabledComponents.Pop().Get())
			{
				if (DisabledComponent->RegisteredEdgeFixup != nullptr)
				{
					check(DisabledComponent->RegisteredEdgeFixup == EdgeFixup);	// sanity check
					FLandscapeGroup* DisabledGroup = DisabledComponent->RegisteredLandscapeGroup;
					check(DisabledGroup != nullptr);

					GROUP_DEBUG_LOG_DETAIL(TEXT("Group %p UnregisterComponent %p %d (%d,%d) -- Reactivate Disabled Component %p"), this, Component, EdgeFixup->bMapped, EdgeFixup->GroupCoord.X, EdgeFixup->GroupCoord.Y, DisabledComponent);

					const bool bDisableCurrentActive = false; // just unmap it
					EdgeFixup->SetActiveComponent(DisabledComponent, DisabledGroup, bDisableCurrentActive);
					HeightmapTextureToActiveComponent[EdgeFixup->HeightmapTexture] = DisabledComponent;

					bReactivatedDisabled = true;
					break;
				}
			}
		}

		if (!bReactivatedDisabled)
		{
			GROUP_DEBUG_LOG_DETAIL(TEXT("Group %p UnregisterComponent %p %d (%d,%d) -- Deactivating (%d RegisteredFixups, %d NeedSnapshot, %d Mapped, %d Moved)"),
				this, Component, EdgeFixup->bMapped, EdgeFixup->GroupCoord.X, EdgeFixup->GroupCoord.Y,
				AllRegisteredFixups.Num(), HeightmapsNeedingEdgeTexturePatching.Num(), XYToEdgeFixupMap.Num(), HeightmapsMoved.Num());

			// no disabled available to replace the active component -- just set NO active component
			const bool bDisableCurrentActive = false; // just unmap it
			EdgeFixup->SetActiveComponent(nullptr, nullptr, bDisableCurrentActive);
			HeightmapTextureToActiveComponent.Remove(EdgeFixup->HeightmapTexture);
		}
	}
}

void FLandscapeGroup::OnTransformUpdated(ULandscapeComponent* Component)
{
	ULandscapeHeightmapTextureEdgeFixup* EdgeFixup = Component->RegisteredEdgeFixup;
	check(EdgeFixup != nullptr);
	check(Component->RegisteredLandscapeGroup == this);

	if (!EdgeFixup->IsComponentActive(Component))
	{
		// a disabled component is not mapped, so we don't need to fix anything
	}
	else
	{
		// flag it as moved, so we can fix it on the next tick
		// (this must be deferred so we can move all components simultaneously, and redefine the grid if necessary)
		GROUP_DEBUG_LOG_DETAIL(TEXT("Group %x MOVE component %x"), this, Component);
		HeightmapsMoved.Add(EdgeFixup);
	}
}

void FLandscapeGroup::DisableAndUnmap(ULandscapeHeightmapTextureEdgeFixup* Fixup)
{
	// should be registered before it can be disabled
	check(AllRegisteredFixups.Contains(Fixup));

	// you can't disable the same component twice
	check(!Fixup->DisabledComponents.Contains(Fixup->ActiveComponent));
	Fixup->DisabledComponents.Add(Fixup->ActiveComponent);

	Unmap(Fixup);
}

#if WITH_EDITOR
namespace UE::Landscape::Private
{
	bool FixupProxyTransform(ALandscapeProxy* Proxy)
	{
		// make sure we've run shared property sync before moving proxies
		if (!Proxy->bIsRegisteredWithLandscapeInfo)
		{
			return false;
		}

		ALandscape* Landscape = Proxy->GetLandscapeActor();
		check(Landscape);

		// Make sure section offset of all proxies is multiple of ALandscapeProxy::ComponentSizeQuads
		{
			FIntPoint LandscapeSectionOffset = Proxy->GetSectionBase() - Landscape->GetSectionBase();
			FIntPoint LandscapeSectionOffsetRem(
				LandscapeSectionOffset.X % Proxy->ComponentSizeQuads,
				LandscapeSectionOffset.Y % Proxy->ComponentSizeQuads);
			if (LandscapeSectionOffsetRem.X != 0 || LandscapeSectionOffsetRem.Y != 0)
			{
				FIntPoint NewLandscapeSectionOffset = Proxy->GetSectionBase() - LandscapeSectionOffsetRem;

				UE_LOGF(LogLandscape, Warning, "Landscape section base is not multiple of component size, attempted automated fix: '%ls', %d,%d vs %d,%d.",
					*Proxy->GetFullName(), Proxy->GetSectionBase().X, Proxy->GetSectionBase().Y, NewLandscapeSectionOffset.X, NewLandscapeSectionOffset.Y);

				Proxy->SetSectionBase(NewLandscapeSectionOffset);
			}
		}

		// Update transformations of all linked landscape proxies
		FTransform LandscapeTM = Landscape->LandscapeActorToWorld();
		{
			FTransform ProxyRelativeTM(FVector(Proxy->GetSectionBase()));
			FTransform ProxyTransform = ProxyRelativeTM * LandscapeTM;

			if (!Proxy->GetTransform().Equals(ProxyTransform))
			{
				Proxy->SetActorTransform(ProxyTransform);

				// Let other systems know that an actor was moved
				GEngine->BroadcastOnActorMoved(Proxy);
			}
		}

		return true;
	}
}
#endif // WITH_EDITOR

FIntPoint FLandscapeGroup::Map(ULandscapeHeightmapTextureEdgeFixup* Fixup, ULandscapeComponent* Component)
{
	check(Fixup->bMapped == false);

	ALandscapeProxy* Proxy = Component->GetLandscapeProxy();

	FTransform ComponentLocalToWorld = Component->GetComponentTransform();
	FBoxSphereBounds ComponentLocalBounds = Component->CalcBounds(FTransform::Identity);
	FVector ComponentCenterWorldSpace = ComponentLocalToWorld.TransformPosition(ComponentLocalBounds.Origin);
	FVector ComponentXVectorWorldSpace = ComponentLocalToWorld.TransformVector(FVector::XAxisVector);
	FVector ComponentYVectorWorldSpace = ComponentLocalToWorld.TransformVector(FVector::YAxisVector);
	FVector ComponentLandscapeGridScale = Proxy->GetRootComponent()->GetRelativeScale3D();

	// in general, the first registered section gets to set up our RenderCoord grid so it is located at the origin
	// however, there is also the case where we change the landscape transform, which should cause the entire grid to be remapped
	// it's a bit tricky to handle properly though, as all proxies are moved, then each proxy's components are mapped and unmapped.
	// meaning there's no time when there is nothing in the fixup map.  Instead, we check if all remaining items in the fixup map are moved.
	// in this case we can assume it's ok to redefine the grid.
	bool bCanRedefineGrid = false;
	if ((XYToEdgeFixupMap.Num() == 0) || HeightmapsMoved.Num() >= XYToEdgeFixupMap.Num())
	{
		bCanRedefineGrid = true;

		// check if every mapped fixup is moved
		for (auto MappedFixupPair : XYToEdgeFixupMap)
		{
			ULandscapeHeightmapTextureEdgeFixup* MappedFixup = MappedFixupPair.Value;
			if (!HeightmapsMoved.Contains(MappedFixup))
			{
				// there is still a mapped fixup that wasn't moved, don't update the coords
				bCanRedefineGrid = false;
				break;
			}
		}
	}

	if (bCanRedefineGrid)
	{
		GROUP_DEBUG_LOG_DETAIL(TEXT("Map REDEFINE GRID: %d existing, %d moved"), XYToEdgeFixupMap.Num(), HeightmapsMoved.Num());

		// we will remap everything below, which will handle the moves.  Clear this early so recursive calls to Map don't trigger grid redefinition again
		HeightmapsMoved.Empty();

		// first unmap ALL mapped fixups prior to changing the rendercoord grid
		TArray<TObjectPtr<ULandscapeHeightmapTextureEdgeFixup>> AllMappedFixups;
		XYToEdgeFixupMap.GenerateValueArray(AllMappedFixups);
		for (ULandscapeHeightmapTextureEdgeFixup* MappedFixup : AllMappedFixups)
		{
			const bool bRemoveFromLists = false; // don't bother to remove from the lists, as we will just add them again below
			Unmap(MappedFixup, bRemoveFromLists);
		}

		// redefine the grid
		ComponentResolution = Component->ComponentSizeQuads;
		GroupCoordOrigin = ComponentCenterWorldSpace;
		GroupCoordXVector = ComponentXVectorWorldSpace;
		GroupCoordYVector = ComponentYVectorWorldSpace;
		LandscapeGridScale = ComponentLandscapeGridScale;

		// remap all of the fixups that used to be mapped
		for (ULandscapeHeightmapTextureEdgeFixup* MappedFixup : AllMappedFixups)
		{
			check(MappedFixup != Fixup); // we shouldn't ever be remapping ourselves recursively
			Map(MappedFixup, MappedFixup->ActiveComponent);

			// Mark it, and all neighbors, as needing edge texture patching after the remap.
			// (may or may not need it, depending on what kind of move was done)
			HeightmapsNeedingEdgeTexturePatching.Add(MappedFixup);
			for (UE::Landscape::ENeighborIndex NeighborIndex : TEnumRange<UE::Landscape::ENeighborIndex>())
			{
				FIntPoint NeighborCoord = MappedFixup->GroupCoord + GetNeighborRelativePosition(NeighborIndex);
				if (TObjectPtr<ULandscapeHeightmapTextureEdgeFixup>* NeighborPtrPtr = XYToEdgeFixupMap.Find(NeighborCoord))
				{
					HeightmapsNeedingEdgeTexturePatching.Add(NeighborPtrPtr->Get());
				}
			}
		}
	}
	else
	{
		// validate each additional section has a matching resolution, scale and orientation
		bool bResolutionMatches = (ComponentResolution == Component->ComponentSizeQuads);
		bool bXVectorMatches = (ComponentXVectorWorldSpace - GroupCoordXVector).IsNearlyZero();
		bool bYVectorMatches = (ComponentYVectorWorldSpace - GroupCoordYVector).IsNearlyZero();
		if (!(bResolutionMatches && bXVectorMatches && bYVectorMatches))
		{
#if WITH_EDITOR
			// mismatch: first check if this component just needs its transform updated (it may have been recently loaded and not yet fixed)
			if (UE::Landscape::Private::FixupProxyTransform(Proxy))
			{
				// fixup success: grab the new transform and check again
				ComponentLocalToWorld = Component->GetComponentTransform();
				ComponentLocalBounds = Component->CalcBounds(FTransform::Identity);
				ComponentCenterWorldSpace = ComponentLocalToWorld.TransformPosition(ComponentLocalBounds.Origin);
				ComponentXVectorWorldSpace = ComponentLocalToWorld.TransformVector(FVector::XAxisVector);
				ComponentYVectorWorldSpace = ComponentLocalToWorld.TransformVector(FVector::YAxisVector);
				ComponentLandscapeGridScale = Proxy->GetRootComponent()->GetRelativeScale3D();
				
				bResolutionMatches = (ComponentResolution == Component->ComponentSizeQuads);
				bXVectorMatches = (ComponentXVectorWorldSpace - GroupCoordXVector).IsNearlyZero();
				bYVectorMatches = (ComponentYVectorWorldSpace - GroupCoordYVector).IsNearlyZero();
			}
#endif // WITH_EDITOR

			if (!(bResolutionMatches && bXVectorMatches && bYVectorMatches))
			{
				UE_LOGF(LogLandscape, Warning, "Landscapes in LOD Group with Key %u do not have matching resolution (%d == %d), scale (%f == %f, %f == %f) and/or rotation; seam artifacts may appear.",
					LandscapeGroupKey,
					ComponentResolution, Component->ComponentSizeQuads,
					GroupCoordXVector.Length(), ComponentXVectorWorldSpace.Length(),
					GroupCoordYVector.Length(), ComponentYVectorWorldSpace.Length());
			}
		}

		if (!(LandscapeGridScale - ComponentLandscapeGridScale).IsNearlyZero())
		{
			UE_LOGF(LogLandscape, Warning, "Landscapes in LOD Group with Key %u do not have matching grid scale (%ls == %ls); seam artifacts may appear.",
				LandscapeGroupKey,
				*LandscapeGridScale.ToString(), *ComponentLandscapeGridScale.ToString());
		}
	}

	// project onto the Component X/Y plane to calculate the group coordinates
	FVector Delta = ComponentCenterWorldSpace - GroupCoordOrigin;
	double GroupCoordX = Delta.Dot(GroupCoordXVector) / (GroupCoordXVector.SquaredLength() * ComponentResolution);
	double GroupCoordY = Delta.Dot(GroupCoordYVector) / (GroupCoordYVector.SquaredLength() * ComponentResolution);

	FIntPoint GroupCoord;
	GroupCoord.X = FMath::RoundToInt32(GroupCoordX);
	GroupCoord.Y = FMath::RoundToInt32(GroupCoordY);

	if ((FMath::Abs(GroupCoordX - GroupCoord.X) > 0.01) || (FMath::Abs(GroupCoordY - GroupCoord.Y) > 0.01))
	{
		UE_LOGF(LogLandscape, Warning, "Landscape component %ls is not spatially aligned with the group grid (LOD Group with Key %u), seam artifacts may appear.", *Component->GetPathName(), LandscapeGroupKey);
	}

	// Add to the Map
	while (true)
	{
		TObjectPtr<ULandscapeHeightmapTextureEdgeFixup>* MapEntryPtrPtr = XYToEdgeFixupMap.Find(GroupCoord);

		if (MapEntryPtrPtr == nullptr)
		{
			XYToEdgeFixupMap.Add(GroupCoord) = Fixup;
			break;
		}
		else if (*MapEntryPtrPtr == Fixup)
		{
			break;
		}
		else
		{
			UE_LOGF(LogLandscape, Warning, "Two landscape components in group (LOD Group with Key %u) occupy the same group grid cell: %ls and %ls, artifacts may appear. Please move one.",
				LandscapeGroupKey, *Component->GetPathName(), *(*MapEntryPtrPtr)->ActiveComponent->GetPathName());

			// attempt to give a temporary location to the overlapped component
			// We could potentially use a MultiMap instead so we don't have to hack the GroupCoord.. 
			// TODO [chris.tchou] : when overlapped components move, we should also re-register the non-moved one.
			GroupCoord.X += 100000;
		}
	}

	Fixup->bMapped = true;
	Fixup->GroupCoord = GroupCoord;

	return GroupCoord;
}

void FLandscapeGroup::Unmap(ULandscapeHeightmapTextureEdgeFixup* Fixup, const bool bRemoveFromLists)
{
	if (!Fixup->bMapped)
	{
		return;
	}
	check(Fixup->ActiveGroup == this);

	FIntPoint GroupCoord = Fixup->GroupCoord;
	TObjectPtr<ULandscapeHeightmapTextureEdgeFixup> RemovedEdgeFixup = nullptr;
	XYToEdgeFixupMap.RemoveAndCopyValue(GroupCoord, RemovedEdgeFixup);
	check(Fixup == RemovedEdgeFixup);

	if (bRemoveFromLists)
	{
#if WITH_EDITOR
		HeightmapsNeedingEdgeSnapshotCapture.Remove(Fixup);
#endif // WITH_EDITOR
		HeightmapsNeedingEdgeTexturePatching.Remove(Fixup);
		HeightmapsMoved.Remove(Fixup);
	}

	Fixup->bMapped = false;
}

void FLandscapeGroup::RegisterAllComponentsOnStreamingProxy(ALandscapeStreamingProxy* StreamingProxy)
{
	check(StreamingProxy->bIsLandscapeActorRegisteredWithLandscapeInfo);

	UWorld* World = StreamingProxy->GetWorld();
	check(World);

	GROUP_DEBUG_LOG_DETAIL(TEXT("Registering components for World %p %s (Scene: %p WorldType: %d NetMode: %d)"),
		World, *World->GetName(), World->Scene, World->WorldType, World->GetNetMode());

	// if world is not renderable, we don't need the landscape groups / edge patching
	if (World->Scene == nullptr)
	{
		return;
	}

	if (World->GetNetMode() == ENetMode::NM_DedicatedServer)
	{
		return;
	}

	if (ULandscapeSubsystem* LandscapeSubsystem = World->GetSubsystem<ULandscapeSubsystem>())
	{
		if (FLandscapeGroup* Group = LandscapeSubsystem->GetLandscapeGroupForProxy(StreamingProxy))
		{
			for (ULandscapeComponent* Component : StreamingProxy->LandscapeComponents)
			{
				if (Component != nullptr)
				{
					Group->RegisterComponent(Component);
				}
			}
		}
	}
}

void FLandscapeGroup::UnregisterAllComponentsOnStreamingProxy(ALandscapeStreamingProxy* StreamingProxy)
{
	UWorld* World = StreamingProxy->GetWorld();
	check(World);

	if (ULandscapeSubsystem* LandscapeSubsystem = World->GetSubsystem<ULandscapeSubsystem>())
	{
		for (ULandscapeComponent* Component : StreamingProxy->LandscapeComponents)
		{
			if (Component && Component->RegisteredLandscapeGroup)
			{
				Component->RegisteredLandscapeGroup->UnregisterComponent(Component);
			}
		}
	}
}

void FLandscapeGroup::AddReferencedObjects(FLandscapeGroup* InThis, FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(InThis->AllRegisteredFixups);
#if WITH_EDITOR
	check(InThis->AllRegisteredFixups.Includes(InThis->HeightmapsNeedingEdgeSnapshotCapture));
	check(InThis->AllRegisteredFixups.Includes(InThis->HeightmapsNeedingEdgeTexturePatching));
#endif // WITH_EDITOR
	Collector.AddReferencedObjects(InThis->XYToEdgeFixupMap);
}


FString DirectionFlagsToString(UE::Landscape::EDirectionFlags Flags)
{
	using namespace UE::Landscape;
	if (Flags == EDirectionFlags::None)
	{
		return TEXT("None");
	}
	else
	{
		TStringBuilder<64> Builder;
		bool bFirst = true;
		for (EDirectionIndex Index : TEnumRange<EDirectionIndex>())
		{
			if (EnumHasAnyFlags(Flags, ToFlag(Index)))
			{
				if (bFirst)
				{
					bFirst = false;
				}
				else
				{
					Builder.Append(",", 1);
				}
				Builder.Append(GetDirectionString(Index));
			}
		}
		return Builder.ToString();
	}
}

void FLandscapeGroup::TickEdgeFixup(ULandscapeSubsystem* LandscapeSubsystem, bool bForcePatchAll)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeGroup::TickEdgeFixup);

	using namespace UE::Landscape;

	if (XYToEdgeFixupMap.Num() == 0)
	{
		return;
	}

	if (HeightmapsMoved.Num() > 0)
	{
		// unmap the moved components
		for (ULandscapeHeightmapTextureEdgeFixup* EdgeFixup : HeightmapsMoved)
		{
			check(EdgeFixup);
			const bool bRemoveFromLists = false; // everything will be remapped below, no need to remove
			Unmap(EdgeFixup, bRemoveFromLists);
		}

		// remap the moved components
		for (ULandscapeHeightmapTextureEdgeFixup* EdgeFixup : HeightmapsMoved)
		{
			check(EdgeFixup);
			Map(EdgeFixup, EdgeFixup->ActiveComponent);

			// Mark it, and all neighbors, as needing edge texture patching.
			HeightmapsNeedingEdgeTexturePatching.Add(EdgeFixup);
			for (UE::Landscape::ENeighborIndex NeighborIndex : TEnumRange<UE::Landscape::ENeighborIndex>())
			{
				FIntPoint NeighborCoord = EdgeFixup->GroupCoord + GetNeighborRelativePosition(NeighborIndex);
				if (TObjectPtr<ULandscapeHeightmapTextureEdgeFixup>* NeighborPtrPtr = XYToEdgeFixupMap.Find(NeighborCoord))
				{
					HeightmapsNeedingEdgeTexturePatching.Add(NeighborPtrPtr->Get());
				}
			}
		}

		HeightmapsMoved.Empty();
	}

#if !UE_BUILD_SHIPPING
	if (GAmortizedGroupValidation)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AmortizedGroupValidation);

		// perform an amortized check on all MAPPED fixups to catch any discrepancies
		// this shouldn't be necessary, but is a good double check that nothing
		// fell through the cracks.

		// Grab the Nth entry in the fixup map, if it exists, and check it
		AmortizeIndex = FSetElementId::FromInteger(AmortizeIndex.AsInteger() + 1);
		if (AmortizeIndex.AsInteger() > XYToEdgeFixupMap.GetMaxIndex())
		{
			AmortizeIndex = FSetElementId::FromInteger(0);
		}
		if (XYToEdgeFixupMap.IsValidId(AmortizeIndex))
		{
			auto Pair = XYToEdgeFixupMap.Get(AmortizeIndex);
			ULandscapeHeightmapTextureEdgeFixup* Fixup = Pair.Value;

			// any entry in the group's map should be mapped, and this group should be the active one
			check(Fixup->bMapped);
			check(Fixup->ActiveGroup == this);
			ULandscapeComponent* Component = Fixup->ActiveComponent;
			check(Component);
			check(Component->RegisteredLandscapeGroup == this);
			check(Component->RegisteredEdgeFixup == Fixup);

			FVector CurrentLandscapeGridScale = Component->GetLandscapeProxy()->GetRootComponent()->GetRelativeScale3D();
			if (!ensure((CurrentLandscapeGridScale - LandscapeGridScale).IsNearlyZero()))
			{
				UE_LOGF(LogLandscape, Warning, "Landscape component %p (%ls) has a different scale than other components in the Landscape Group with Key %u, normal seam artifacts may occur", Component, *Component->GetName(), this->LandscapeGroupKey);
			}

			FLandscapeGroup* NewGroup = LandscapeSubsystem->GetLandscapeGroupForComponent(Component);
			if (!ensure(NewGroup == this))
			{
				UE_LOGF(LogLandscape, Warning, "Landscape component %p (%ls) changed groups unexpectedly (%u ==> %u) - attempting to fix automatically by re-registering it", Component, *Component->GetName(), this->LandscapeGroupKey, NewGroup->LandscapeGroupKey);
				this->UnregisterComponent(Component);
				NewGroup->RegisterComponent(Component);
			}
		}
	}
#endif // !UE_BUILD_SHIPPING

	if (bForcePatchAll)
	{
		for (ULandscapeHeightmapTextureEdgeFixup* Fixup : AllRegisteredFixups)
		{
			if (Fixup->ActiveGroup == this)
			{
				// mark all edges modified, but with the wrong hash, so they will all patch
				check(Fixup->bMapped);
				Fixup->GPUEdgeModifiedFlags = EEdgeFlags::All;
				memset(Fixup->GPUEdgeHashes.GetData(), 0, sizeof(uint32) * 8);
				HeightmapsNeedingEdgeTexturePatching.Add(Fixup);
			}
		}
	}

	int32 SnapshotsCaptured = 0;
	int32 SnapshotEdgesChanged = 0;
	int32 EdgesPatched = 0;
	int32 TexturesPatched = 0;

	// grab an exclusive lock on the group registration data, as we may modify snapshots and/or patch hash tracking
	FWriteScopeLock ScopeWriteLock(RWLock);

#if WITH_EDITOR
	// first we check if any edge data needs to be updated
	if (HeightmapsNeedingEdgeSnapshotCapture.Num() > 0)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(HeightmapsNeedingEdgeSnapshotCapture);

		for (auto It = HeightmapsNeedingEdgeSnapshotCapture.CreateIterator(); It; ++It)
		{
			ULandscapeHeightmapTextureEdgeFixup* EdgeFixup = *It;
			check(EdgeFixup->ActiveGroup == this); // should never be in HeightmapsNeedingEdgeSnapshotCapture unless active in this group
			check(AllRegisteredFixups.Contains(EdgeFixup));
			check(EdgeFixup->bMapped);

#if WITH_EDITORONLY_DATA
			ALandscape* Landscape = EdgeFixup->ActiveComponent->GetLandscapeActor();
			if (!Landscape || !Landscape->bGrassUpdateEnabled)
			{
				GROUP_DEBUG_LOG_DETAIL(TEXT("-- Snap Component %p (%d,%d) -------- PAUSED (editing or landscape unregistered)"), EdgeFixup->ActiveComponent.Get(), EdgeFixup->GroupCoord.X, EdgeFixup->GroupCoord.Y);
				continue;
			}
#endif // WITH_EDITORONLY_DATA

			bool bSuccess = false;
			EEdgeFlags ChangedEdges = EdgeFixup->UpdateEdgeSnapshotFromHeightmapSource(LandscapeGridScale, false, &bSuccess);
			if (bSuccess)
			{
				SnapshotsCaptured++;
				SnapshotEdgesChanged += FPlatformMath::CountBits64((uint64)ChangedEdges);

				GROUP_DEBUG_LOG_DETAIL(TEXT("-- Snap Component %p (%d,%d) -------- UPDATED, Changed edges: %s"), EdgeFixup->ActiveComponent.Get(), EdgeFixup->GroupCoord.X, EdgeFixup->GroupCoord.Y, *DirectionFlagsToString(ChangedEdges));
				if (ChangedEdges != EEdgeFlags::None)
				{
					// if any edges were changed -- queue it for texture edge patching
					HeightmapsNeedingEdgeTexturePatching.Add(EdgeFixup);

					ENeighborFlags AffectedNeighbors = EdgesToAffectedNeighbors(ChangedEdges);

					// and also mark the affected neighbors for potential texture edge patching
					EdgeFixup->RequestEdgeTexturePatchingForNeighbors(AffectedNeighbors);
				}
				It.RemoveCurrent();
			}
		}
	}
#endif // WITH_EDITOR

	// now we try to fix up the heightmap GPU texture to address any edges that need to be patched
	if ((bForcePatchAll || GPatchEdges) && (HeightmapsNeedingEdgeTexturePatching.Num() > 0))
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(HeightmapsNeedingEdgeTexturePatching);


#if WITH_EDITORONLY_DATA
		if (FLandscapeEditLayerReadback::HasAnyGlobalWork())
		{
			GROUP_DEBUG_LOG_DETAIL(TEXT("-- Patch (All Components) -------- PAUSED (deferred while any readbacks pending)"));
		}
		else
#endif
		{
			for (auto It = HeightmapsNeedingEdgeTexturePatching.CreateIterator(); It; ++It)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(CheckHeightmap);

				ULandscapeHeightmapTextureEdgeFixup* EdgeFixup = *It;
				check(EdgeFixup->ActiveGroup == this); // should never be in HeightmapsNeedingEdgeTexturePatching unless active in this group
				check(AllRegisteredFixups.Contains(EdgeFixup));
				check(EdgeFixup->bMapped);

#if WITH_EDITORONLY_DATA
				ALandscape* Landscape = EdgeFixup->ActiveComponent->GetLandscapeActor();
				if (!Landscape || !Landscape->bGrassUpdateEnabled)
				{
					GROUP_DEBUG_LOG_DETAIL(TEXT("-- Patch Component %p (%d,%d) -------- PAUSED (editing or landscape unregistered)"), EdgeFixup->ActiveComponent.Get(), EdgeFixup->GroupCoord.X, EdgeFixup->GroupCoord.Y);
					continue;
				}
#endif // WITH_EDITORONLY_DATA

				if (EdgeFixup->HeightmapTexture->HasPendingInitOrStreaming())
				{
					GROUP_DEBUG_LOG_DETAIL(TEXT("-- Patch Component %p (%d,%d) -------- PAUSED (pending)"), EdgeFixup->ActiveComponent.Get(), EdgeFixup->GroupCoord.X, EdgeFixup->GroupCoord.Y);
					continue;
				}

#if WITH_EDITOR
				if (EdgeFixup->IsTextureEdgePatchingPaused())
				{
					GROUP_DEBUG_LOG_DETAIL(TEXT("-- Patch Component %p (%d,%d) -------- PAUSED (readback)"), EdgeFixup->ActiveComponent.Get(), EdgeFixup->GroupCoord.X, EdgeFixup->GroupCoord.Y);
					continue;
				}

				// wait til we are not using the default texture to apply edge patching
				if (EdgeFixup->GetHeightmapTexture()->IsDefaultTexture())
				{
					GROUP_DEBUG_LOG_DETAIL(TEXT("-- Patch Component %p (%d,%d) -------- WAITING (texture load)"), EdgeFixup->ActiveComponent.Get(), EdgeFixup->GroupCoord.X, EdgeFixup->GroupCoord.Y);
					continue;
				}
#endif // WITH_EDITOR

				check(EdgeFixup->ActiveGroup == this);
				int32 PatchedEdgeCount = EdgeFixup->CheckAndPatchTextureEdgesFromEdgeSnapshots();
				if (PatchedEdgeCount > 0)
				{
					TexturesPatched++;
				}
				EdgesPatched += PatchedEdgeCount;

				It.RemoveCurrent();
				GROUP_DEBUG_LOG_DETAIL(TEXT("-- Patch Component %p (%d,%d) -------- PATCHED %d edges"), EdgeFixup->ActiveComponent.Get(), EdgeFixup->GroupCoord.X, EdgeFixup->GroupCoord.Y, PatchedEdgeCount);
			}
		}
	}

	if ((SnapshotsCaptured > 0) || (EdgesPatched > 0))
	{
		GROUP_DEBUG_LOG(TEXT("LandscapeGroup Tick [%d Snapshots Captured, %d edges changed) (%d edges patched on %d textures)"), SnapshotsCaptured, SnapshotEdgesChanged, EdgesPatched, TexturesPatched);
	}
}


FLandscapeGroup::~FLandscapeGroup()
{
	if (AllRegisteredFixups.Num() > 0)
	{
		UE_LOGF(LogLandscape, Warning, "Landscape Group with Key %u had registered components at destruction, this indicates some components were not properly unregistered", this->LandscapeGroupKey);
		
		// Avoid UnregisterComponent removing from AllRegisteredFixups while iterating through it
		TSet<TObjectPtr<ULandscapeHeightmapTextureEdgeFixup>> LocalAllRegisteredFixups(AllRegisteredFixups);

		for (ULandscapeHeightmapTextureEdgeFixup* Fixup : LocalAllRegisteredFixups)
		{
			if (Fixup->ActiveGroup == this)
			{
				// if we're the active group, the corresponding component is the active one
				UnregisterComponent(Fixup->ActiveComponent);
			}
			else
			{
				// find our group's component by searching through the disabled list
				for (TWeakObjectPtr<ULandscapeComponent> DisabledComponentPtr : Fixup->DisabledComponents)
				{
					if (ULandscapeComponent* DisabledComponent = DisabledComponentPtr.Get())
					{
						if (DisabledComponent->RegisteredLandscapeGroup == this)
						{
							// found it!						
							UnregisterComponent(DisabledComponent);
							break;
						}
					}
				}
			}
		}
		check(AllRegisteredFixups.Num() == 0);
	}

#if WITH_EDITOR
	check(HeightmapsNeedingEdgeSnapshotCapture.IsEmpty());
#endif // WITH_EDITOR
	check(HeightmapsNeedingEdgeTexturePatching.IsEmpty());
}

#undef GROUP_DEBUG_LOG
#undef GROUP_DEBUG_LOG_DETAIL
