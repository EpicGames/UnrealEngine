// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LandscapeEdgeFixup.h"

class ULandscapeHeightmapTextureEdgeFixup;
class ULandscapeComponent;
class ULandscapeSubsystem;
class UTexture2D;
class ALandscapeStreamingProxy;

namespace UE::Landscape
{
	bool ShouldInstallEdgeFixup();
	bool ShouldPatchStreamingMipEdges();
	bool ShouldPatchAllLandscapeComponentEdges(bool bResetForNext = false);
}

struct FLandscapeGroup
{
	friend class ULandscapeHeightmapTextureEdgeFixup;

	// read-write lock that protect access to XYToEdgeFixupMap, and registered ULandscapeHeightmapTextureEdgeFixup's Snapshots, EdgeModifiedFlags and GPUEdgeHashes
	FRWLock RWLock;

	uint32 LandscapeGroupKey = 0;

	// Resolution, Origin and Size, defines the group coordinate system
	// (the first component registered gets to be the origin)
	int32 ComponentResolution = -1;
	FVector GroupCoordOrigin = FVector::ZeroVector;		// world space position of the center of the origin component (render coord 0,0)
	FVector GroupCoordXVector = FVector::ZeroVector;	// world space vector in the direction of component local X
	FVector GroupCoordYVector = FVector::ZeroVector;	// world space vector in the direction of component local Y	
	FVector LandscapeGridScale = FVector::ZeroVector;	// scale used to calculate normals, value set at the same time as above
	
	// map of fixups (key is group coordinate)
	TMap<FIntPoint, TObjectPtr<ULandscapeHeightmapTextureEdgeFixup>> XYToEdgeFixupMap;

	// global map of heightmap textures to the landscape active component
	// (this is used to detect and handle shared heightmaps)
	static TMap<TObjectPtr<UTexture2D>, TObjectPtr<ULandscapeComponent>> HeightmapTextureToActiveComponent;

	// set of fixups that are registered with this group (some of which may not be mapped)
	// this is only used to double check correct behavior
	TSet<TObjectPtr<ULandscapeHeightmapTextureEdgeFixup>> AllRegisteredFixups;

	// set of fixups that were moved and may need to modify their mapped position (or if all are moved, we may need to reset the map grid)
	TSet<TObjectPtr<ULandscapeHeightmapTextureEdgeFixup>> HeightmapsMoved;

#if WITH_EDITOR
	// fixups that need to capture new edge snapshots
	TSet<TObjectPtr<ULandscapeHeightmapTextureEdgeFixup>> HeightmapsNeedingEdgeSnapshotCapture;
#endif // WITH_EDITOR

	// fixups that may need to GPU edge patch their heightmap textures
	TSet<TObjectPtr<ULandscapeHeightmapTextureEdgeFixup>> HeightmapsNeedingEdgeTexturePatching;

	// amortization state for amortized slow checks
	FSetElementId AmortizeIndex;

	FLandscapeGroup(uint32 InLandscapeGroupKey)
		: LandscapeGroupKey(InLandscapeGroupKey)
	{
	}

	~FLandscapeGroup();

	void RegisterComponent(ULandscapeComponent* Component);
	void UnregisterComponent(ULandscapeComponent* Component);
	void OnTransformUpdated(ULandscapeComponent* Component);

	static void RegisterAllComponentsOnStreamingProxy(ALandscapeStreamingProxy* StreamingProxy);
	static void UnregisterAllComponentsOnStreamingProxy(ALandscapeStreamingProxy* StreamingProxy);
	static void AddReferencedObjects(FLandscapeGroup* InThis, FReferenceCollector& Collector);

	ULandscapeHeightmapTextureEdgeFixup* GetEdgeFixupAtCoord(FIntPoint Coord)
	{
		TObjectPtr<ULandscapeHeightmapTextureEdgeFixup>* Result = XYToEdgeFixupMap.Find(Coord);
		if (Result != nullptr)
		{
			return Result->Get();
		}
		return nullptr;
	}

	ULandscapeHeightmapTextureEdgeFixup* GetNeighborEdgeFixup(FIntPoint Coord, UE::Landscape::ENeighborIndex NeighborIndex)
	{
		FIntPoint NeighborCoord = Coord + UE::Landscape::GetNeighborRelativePosition(NeighborIndex);
		return GetEdgeFixupAtCoord(NeighborCoord);
	}

	void TickEdgeFixup(ULandscapeSubsystem* LandscapeSubsystem, bool bForcePatchAll);

private:
	void DisableAndUnmap(ULandscapeHeightmapTextureEdgeFixup* Fixup);
	FIntPoint Map(ULandscapeHeightmapTextureEdgeFixup* Fixup, ULandscapeComponent* Component);
	void Unmap(ULandscapeHeightmapTextureEdgeFixup* Fixup, const bool bRemoveFromLists = true);
};
