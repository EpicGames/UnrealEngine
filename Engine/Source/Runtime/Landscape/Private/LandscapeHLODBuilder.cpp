// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeHLODBuilder.h"
#include "Materials/Material.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapeHLODBuilder)

#if WITH_EDITOR

#include "Landscape.h"
#include "LandscapeComponent.h"
#include "LandscapeMeshProxyComponent.h"
#include "LandscapeProxy.h"
#include "LandscapeSettings.h"

#include "MeshDescription.h"
#include "PhysicsEngine/BodySetup.h"
#include "StaticMeshAttributes.h"
#include "TriangleTypes.h"

#include "Materials/MaterialInstanceConstant.h"
#include "MaterialUtilities.h"

#include "Engine/Engine.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshSourceData.h"

#include "Algo/ForEach.h"

#include "Serialization/ArchiveCrc32.h"
#include "Engine/HLODProxy.h"

#include "WorldPartition/HLOD/HLODHashBuilder.h"

#endif // WITH_EDITOR

ULandscapeHLODBuilder::ULandscapeHLODBuilder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR

bool ULandscapeHLODBuilder::ComputeHLODHash(FHLODHashBuilder& HashBuilder, const UActorComponent* InSourceComponent) const
{
	const ULandscapeComponent* LSComponent = Cast<ULandscapeComponent>(InSourceComponent);
	if (LSComponent == nullptr)
	{
		return false;
	}
	
	// Base landscape HLOD key, changing this will force a rebuild of all landscape HLODs
	FString HLODBaseKey = "51EDA4343E26409DA6ABEACE14C138D1";
	HashBuilder.HashField(HLODBaseKey, TEXT("LandscapeHLODBuilderBaseKey"));

	const ALandscapeProxy* LSProxy = LSComponent->GetLandscapeProxy();

	// LS LOD setup
	HashBuilder.HashField(LSProxy->GetLODScreenSizeArray(), TEXT("LODScreenSize"));
		
	// LS Transform
	HashBuilder.HashField(LSComponent->GetComponentTransform(), TEXT("Transform"));

	// LS Content - Heightmap & Weightmaps
	uint32 LSContentHash = LSComponent->ComputeLayerHash(/*InReturnEditingHash=*/ false);
	HashBuilder.HashField(LSContentHash, TEXT("LSContentHash"));

	TSet<UTexture*> UsedTextures;

	// LS Materials
	{
		if (LSProxy->HLODMaterialOverride)
		{
			HashBuilder << LSProxy->HLODMaterialOverride;
		}
		else
		{
			if (LSProxy->bUseDynamicMaterialInstance)
			{
				HashBuilder << LSComponent->MaterialInstancesDynamic;
			}
			else
			{
				HashBuilder << LSComponent->MaterialInstances;
			}

			HashBuilder << LSComponent->OverrideMaterial;
		}

		HashBuilder << LSComponent->OverrideHoleMaterial;  //Hole material matters regardless of which base material is used.
	}

	// Nanite enabled?
	bool bNaniteEnabled = LSProxy->IsNaniteEnabled() || GetDefault<UStaticMesh>()->IsNaniteForceEnabled();
	HashBuilder.HashField(bNaniteEnabled, TEXT("NaniteEnabled"));
	if (bNaniteEnabled)
	{
		HashBuilder.HashField(LSProxy->GetNaniteLODIndex(), TEXT("NaniteLODIndex"));
		HashBuilder.HashField(LSProxy->GetNanitePositionPrecision(), TEXT("NanitePositionPrecision"));
		HashBuilder.HashField(LSProxy->GetNaniteMaxEdgeLengthFactor(), TEXT("NaniteMaxEdgeLengthFactor"));

		HashBuilder.HashField(LSProxy->IsNaniteSkirtEnabled(), TEXT("NaniteSkirtEnabled"));
		if (LSProxy->IsNaniteSkirtEnabled())
		{
			HashBuilder.HashField(LSProxy->GetNaniteSkirtDepth(), TEXT("NaniteSkirtDepth"));
		}
	}

	// HLODTextureSize
	HashBuilder.HashField(LSProxy->HLODTextureSizePolicy, TEXT("HLODTextureSizePolicy"));
	if (LSProxy->HLODTextureSizePolicy == ELandscapeHLODTextureSizePolicy::SpecificSize)
	{
		HashBuilder.HashField(LSProxy->HLODTextureSize, TEXT("HLODTextureSize"));
	}

	// HLODMeshSourceLOD
	HashBuilder.HashField(LSProxy->HLODMeshSourceLODPolicy, TEXT("HLODMeshSourceLODPolicy"));
	if (LSProxy->HLODMeshSourceLODPolicy == ELandscapeHLODMeshSourceLODPolicy::SpecificLOD)
	{
		HashBuilder.HashField(LSProxy->HLODMeshSourceLOD, TEXT("HLODMeshSourceLOD"));
	}

	// Project max texture size for landscape HLOD textures
	HashBuilder.HashField(GetDefault<ULandscapeSettings>()->GetHLODMaxTextureSize(), TEXT("ProjectHLODMaxTextureSize"));

	return true;
}

static int32 GetMeshTextureSizeFromTargetTexelDensity(const FMeshDescription& InMesh, float InTargetTexelDensity)
{
	FStaticMeshConstAttributes Attributes(InMesh);

	TVertexAttributesConstRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
	
	double Mesh3DArea = 0;
	for (const FTriangleID TriangleID : InMesh.Triangles().GetElementIDs())
	{
		TArrayView<const FVertexID> TriVertices = InMesh.GetTriangleVertices(TriangleID);

		// World space area
		Mesh3DArea += UE::Geometry::VectorUtil::Area(VertexPositions[TriVertices[0]],
													 VertexPositions[TriVertices[1]],
													 VertexPositions[TriVertices[2]]);
	}
	double TexelRatio = FMath::Sqrt(1.0 / Mesh3DArea) * 100;

	// Compute the perfect texture size that would get us to our texture density
	// Also compute the nearest power of two sizes (below and above our target)
	const int32 SizePerfect = FMath::CeilToInt32(InTargetTexelDensity / TexelRatio);
	const int32 SizeHi = FMath::RoundUpToPowerOfTwo(SizePerfect);
	const int32 SizeLo = SizeHi >> 1;

	// Compute the texel density we achieve with these two texture sizes
	const double TexelDensityLo = SizeLo * TexelRatio;
	const double TexelDensityHi = SizeHi * TexelRatio;

	// Select best match between low & high res textures.
	const double TexelDensityLoDiff = InTargetTexelDensity - TexelDensityLo;
	const double TexelDensityHiDiff = TexelDensityHi - InTargetTexelDensity;
	const int32 BestTextureSize = TexelDensityLoDiff < TexelDensityHiDiff ? SizeLo : SizeHi;

	return BestTextureSize;
}

static int32 ComputeRequiredLandscapeLOD(const ALandscapeProxy* InLandscapeProxy, const float InViewDistance)
{
	check(InLandscapeProxy && !InLandscapeProxy->LandscapeComponents.IsEmpty());

	const TArray<float> LODScreenSizes = InLandscapeProxy->GetLODScreenSizeArray();
	int32 RequiredLOD = 0;	

	switch (InLandscapeProxy->HLODMeshSourceLODPolicy)
	{
		case ELandscapeHLODMeshSourceLODPolicy::AutomaticLOD:
		{
			// These constants are showing up a lot in the screen size computation for Level HLODs. This should be configurable per project.
			const float HalfFOV = PI * 0.25f;
			const float ScreenWidth = 1920.0f;
			const float ScreenHeight = 1080.0f;
			const FPerspectiveMatrix ProjMatrix(HalfFOV, ScreenWidth, ScreenHeight, 1.0f);

			const ULandscapeComponent* LSComponent = InLandscapeProxy->LandscapeComponents[0];
			const float ComponentRadiusScaled = static_cast<float>(LSComponent->GetLocalBounds().SphereRadius * LSComponent->GetComponentTransform().GetScale3D().GetAbsMax());
			const float ExpectedScreenSize = ComputeBoundsScreenSize(FVector::ZeroVector, ComponentRadiusScaled, FVector(0.0f, 0.0f, InViewDistance), ProjMatrix);

			// Find the matching LOD for the screen size. No need to test the last LOD screen size if we get to it.
			for (RequiredLOD = 0; RequiredLOD < LODScreenSizes.Num() - 1; ++RequiredLOD)
			{
				if (ExpectedScreenSize > LODScreenSizes[RequiredLOD])
				{
					break;
				}
			}
		} break;

		case ELandscapeHLODMeshSourceLODPolicy::SpecificLOD:
		{
			RequiredLOD = FMath::Clamp(InLandscapeProxy->HLODMeshSourceLOD, 0, LODScreenSizes.Num() - 1);
		} break;

		case ELandscapeHLODMeshSourceLODPolicy::LowestDetailLOD:
		{
			RequiredLOD = LODScreenSizes.Num() - 1;
		} break;
	}

	return RequiredLOD;
}

static int32 ComputeRequiredTextureSize(const ALandscapeProxy* InLandscapeProxy, const float InViewDistance, const FMeshDescription* InMeshDescription)
{
	int32 RequiredTextureSize = 0;

	switch (InLandscapeProxy->HLODTextureSizePolicy)
	{
		case ELandscapeHLODTextureSizePolicy::AutomaticSize:
		{
			const float TargetTexelDensityPerMeter = FMaterialUtilities::ComputeRequiredTexelDensityFromDrawDistance(InViewDistance, static_cast<float>(InMeshDescription->GetBounds().SphereRadius));
			RequiredTextureSize = GetMeshTextureSizeFromTargetTexelDensity(*InMeshDescription, TargetTexelDensityPerMeter);
		} break;

		case ELandscapeHLODTextureSizePolicy::SpecificSize:
		{
			RequiredTextureSize = InLandscapeProxy->HLODTextureSize;
		} break;
	}

	// Clamp to a sane minimum value
	const int32 MinLandscapeHLODTextureSize = 16;
	RequiredTextureSize = FMath::Max(RequiredTextureSize, MinLandscapeHLODTextureSize);

	// Clamp to the project's max texture size for landscape HLODs
	const ULandscapeSettings* LandscapeSettings = GetDefault<ULandscapeSettings>();
	RequiredTextureSize = FMath::Min(RequiredTextureSize, LandscapeSettings->GetHLODMaxTextureSize());

	// Clamp to the maximum possible texture size for safety
	RequiredTextureSize = FMath::Min(RequiredTextureSize, (int32)GetMax2DTextureDimension());

	return RequiredTextureSize;
}


static UMaterialInterface* BakeLandscapeMaterial(const FHLODBuildContext& InHLODBuildContext, const FMeshDescription& InMeshDescription, const ALandscapeProxy* InLandscapeProxy, int32 InTextureSize)
{
	// Build landscape material
	FFlattenMaterial LandscapeFlattenMaterial;
	LandscapeFlattenMaterial.SetPropertySize(EFlattenMaterialProperties::Diffuse, InTextureSize);
	LandscapeFlattenMaterial.SetPropertySize(EFlattenMaterialProperties::Normal, InTextureSize);
	LandscapeFlattenMaterial.SetPropertySize(EFlattenMaterialProperties::Metallic, InTextureSize);
	LandscapeFlattenMaterial.SetPropertySize(EFlattenMaterialProperties::Roughness, InTextureSize);
	LandscapeFlattenMaterial.SetPropertySize(EFlattenMaterialProperties::Specular, InTextureSize);
	FMaterialUtilities::ExportLandscapeMaterial(InLandscapeProxy, LandscapeFlattenMaterial);

	// Optimize flattened material
	FMaterialUtilities::OptimizeFlattenMaterial(LandscapeFlattenMaterial);

	UMaterial* LandscapeMaterial = GEngine->DefaultLandscapeFlattenMaterial;

	// Validate that the flatten material expects world space normals
	if (LandscapeMaterial->bTangentSpaceNormal)
	{
		UE_LOGF(LogHLODBuilder, Error, "Landscape flatten material %ls should use world space normals rather than tangent space normals.", *LandscapeMaterial->GetName());
	}

	FMaterialProxySettings MaterialProxySettings;
	MaterialProxySettings.TextureSizingType = ETextureSizingType::TextureSizingType_UseSingleTextureSize;
	MaterialProxySettings.TextureSize = InTextureSize;
	MaterialProxySettings.bNormalMap = true;
	MaterialProxySettings.bMetallicMap = true;
	MaterialProxySettings.bRoughnessMap = true;
	MaterialProxySettings.bSpecularMap = true;

	// Create a new proxy material instance
	TArray<UObject*> GeneratedAssets;
	UMaterialInstanceConstant* LandscapeMaterialInstance = FMaterialUtilities::CreateFlattenMaterialInstance(InHLODBuildContext.AssetsOuter->GetPackage(), MaterialProxySettings, LandscapeMaterial, LandscapeFlattenMaterial, InHLODBuildContext.AssetsBaseName, InLandscapeProxy->GetName(), GeneratedAssets);

	Algo::ForEach(GeneratedAssets, [&InHLODBuildContext](UObject* Asset) 
	{
		// We don't want any of the generate HLOD assets to be public, unless required by HLODBuildContext
		Asset->ClearFlags(RF_Public | RF_Standalone);
		Asset->SetFlags(InHLODBuildContext.AssetsObjectFlags);

		// Use clamp texture addressing to avoid artifacts between tiles
		if (UTexture2D* Texture = Cast<UTexture2D>(Asset))
		{
			Texture->PreEditChange(nullptr);
			Texture->AddressX = TA_Clamp;
			Texture->AddressY = TA_Clamp;
			Texture->PostEditChange();
		}
	});

	return LandscapeMaterialInstance;
}

// Multiple improvements could be done
// * Currently, for each referenced landscape proxy, we generate individual HLOD meshes & textures. This should output a single mesh for all proxies
TArray<UActorComponent*> ULandscapeHLODBuilder::Build(const FHLODBuildContext& InHLODBuildContext, const TArray<UActorComponent*>& InSourceComponents) const
{
	TArray<ULandscapeComponent*> SourceLandscapeComponents = FilterComponents<ULandscapeComponent>(InSourceComponents);
	
	TSet<ALandscapeProxy*> LandscapeProxies;
	Algo::Transform(SourceLandscapeComponents, LandscapeProxies, [](ULandscapeComponent* SourceComponent) { return SourceComponent->GetLandscapeProxy(); });

	// This code assume all components of a proxy are included in the same build... validate this
	checkCode
	(
		for (ALandscapeProxy* LandscapeProxy : LandscapeProxies)
		{
			for (ULandscapeComponent* LandscapeComponent : LandscapeProxy->LandscapeComponents)
			{
				check(SourceLandscapeComponents.Contains(LandscapeComponent));
			}
		}
	);

	TArray<UActorComponent*> HLODComponents;
	TArray<UStaticMesh*> StaticMeshes;

	for (ALandscapeProxy* LandscapeProxy : LandscapeProxies)
	{
		UStaticMesh* StaticMesh = NewObject<UStaticMesh>(InHLODBuildContext.AssetsOuter);
		StaticMesh->SetFlags(InHLODBuildContext.AssetsObjectFlags);

		const bool bExportNaniteEnabled = LandscapeProxy->IsNaniteEnabled() || GetDefault<UStaticMesh>()->IsNaniteForceEnabled();

		FMeshDescription* MeshDescription = nullptr;

		// Compute source landscape LOD
		const int32 LandscapeLOD = ComputeRequiredLandscapeLOD(LandscapeProxy, static_cast<float>(InHLODBuildContext.MinVisibleDistance));

		// Mesh
		{
			FStaticMeshSourceModel& SrcModel = StaticMesh->AddSourceModel();
			
			// Don't allow the engine to recalculate normals
			SrcModel.BuildSettings.bRecomputeNormals = false;
			SrcModel.BuildSettings.bRecomputeTangents = false;
			SrcModel.BuildSettings.bRemoveDegenerates = false;
			SrcModel.BuildSettings.bUseHighPrecisionTangentBasis = false;
			SrcModel.BuildSettings.bUseFullPrecisionUVs = false;

			// Disable distance field generation
			SrcModel.BuildSettings.DistanceFieldResolutionScale = 0;

			MeshDescription = StaticMesh->CreateMeshDescription(0);
	
			ALandscapeProxy::FRawMeshExportParams ExportParams;
			ExportParams.ExportLOD = LandscapeLOD;

			// Always add a skirt when dealing with a Nanite landscape, as we'll not be able to avoid a gap when dealing with a lower landscape LOD in HLOD
			if (bExportNaniteEnabled)
			{
				// Use a full tile size (at the ExportLOD LOD) as the skirt depth, this will cover all possible gap scenario
				// and avoid the skirt clipping through neighborhood tiles/HLODs
				const int32 ComponentSizeVerts = (LandscapeProxy->ComponentSizeQuads + 1) >> LandscapeLOD;
				const float ScaleFactor = (float)LandscapeProxy->ComponentSizeQuads / (float)(ComponentSizeVerts - 1);
				ExportParams.SkirtDepth = ScaleFactor;
			}

			// It's possible for landscape proxies to have no mesh data, in this case the export will fail...
			if (!LandscapeProxy->ExportToRawMesh(ExportParams, *MeshDescription))
			{
				UE_LOGF(LogHLODBuilder, Display, "Skipping HLOD builder for landscape proxy '%ls' as it failed to export a mesh!", *LandscapeProxy->GetFullName());
				continue;
			}

			StaticMesh->CommitMeshDescription(0);

			// Nanite settings
		    const FVector3d Scale = LandscapeProxy->GetTransform().GetScale3D();
			StaticMesh->GetNaniteSettings().bEnabled = bExportNaniteEnabled;
		    StaticMesh->GetNaniteSettings().PositionPrecision = FMath::Log2(Scale.GetAbsMax()) + LandscapeProxy->GetNanitePositionPrecision();
		    StaticMesh->GetNaniteSettings().MaxEdgeLengthFactor = LandscapeProxy->GetNaniteMaxEdgeLengthFactor();

			StaticMesh->SetImportVersion(EImportStaticMeshVersion::LastVersion);
		}

		// Material
		{
			UMaterialInterface* LandscapeMaterial;
			if (LandscapeProxy->HLODMaterialOverride)
			{
				LandscapeMaterial = LandscapeProxy->HLODMaterialOverride.Get();
			}
			else
			{
				int32 TextureSize = ComputeRequiredTextureSize(LandscapeProxy, static_cast<float>(InHLODBuildContext.MinVisibleDistance), MeshDescription);
				LandscapeMaterial = BakeLandscapeMaterial(InHLODBuildContext, *MeshDescription, LandscapeProxy, TextureSize);
			}

			//Assign the proxy material to the static mesh
			StaticMesh->GetStaticMaterials().Add(FStaticMaterial(LandscapeMaterial));
		}

		StaticMeshes.Add(StaticMesh);

		// In case we are dealing with a Nanite LS, simply create a static mesh component
		if (bExportNaniteEnabled)
		{
			UStaticMeshComponent* StaticMeshComponent = NewObject<UStaticMeshComponent>();
			StaticMeshComponent->SetStaticMesh(StaticMesh);
			HLODComponents.Add(StaticMeshComponent);
		}
		// Otherwise, we use a ULandscapeMeshProxyComponent, which will ensure the landscape proxies surrounding 
		// the HLOD tiles blends properly to avoid any visible gaps.
		else
		{
			ULandscapeMeshProxyComponent* LandcapeMeshProxyComponent = NewObject<ULandscapeMeshProxyComponent>();
			LandcapeMeshProxyComponent->InitializeForLandscape(LandscapeProxy, static_cast<int8>(LandscapeLOD));
			LandcapeMeshProxyComponent->SetStaticMesh(StaticMesh);
			HLODComponents.Add(LandcapeMeshProxyComponent);
		}
	}

	UStaticMesh::BatchBuild(StaticMeshes);

	return HLODComponents;
}

#endif // WITH_EDITOR

