// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeInfo.h"
#include "LandscapeLayerInfoObject.h"
#include "LandscapeProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapeInfo)

LANDSCAPE_API FLandscapeInfoLayerSettings::FLandscapeInfoLayerSettings(ULandscapeLayerInfoObject* InLayerInfo, class ALandscapeProxy* InProxy)
	: LayerInfoObj(InLayerInfo)
	, LayerName((InLayerInfo != nullptr) ? InLayerInfo->GetLayerName() : NAME_None)
#if WITH_EDITORONLY_DATA
	, ThumbnailMIC(nullptr)
	, Owner(InProxy)
	, DebugColorChannel(0)
#endif
{}

LANDSCAPE_API FLandscapeInfoLayerSettings::FLandscapeInfoLayerSettings(FName InPlaceholderLayerName, ALandscapeProxy* InProxy)
		: LayerInfoObj(nullptr)
		, LayerName(InPlaceholderLayerName)
#if WITH_EDITORONLY_DATA
		, ThumbnailMIC(nullptr)
		, Owner(InProxy)
		, DebugColorChannel(0)
#endif
	{
	}