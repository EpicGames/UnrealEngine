// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeInfoMap.h"
#include "Engine/World.h"
#include "LandscapeInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapeInfoMap)

ULandscapeInfoMap::ULandscapeInfoMap(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, World(nullptr)
{
}

void ULandscapeInfoMap::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	check(Map.Num() == 0);
}

void ULandscapeInfoMap::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsTransacting() || Ar.IsObjectReferenceCollector())
	{
		Ar << Map;
	}
}

void ULandscapeInfoMap::BeginDestroy()
{
	if (World.IsValid())
	{
		World->PerModuleDataObjects.Remove(this);
	}

	Super::BeginDestroy();
}

void ULandscapeInfoMap::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	ULandscapeInfoMap* This = CastChecked<ULandscapeInfoMap>(InThis);
	Collector.AddReferencedObjects(This->Map, This);
}

ULandscapeInfoMap& ULandscapeInfoMap::GetLandscapeInfoMap(const UWorld* World)
{
	ULandscapeInfoMap *FoundObject = nullptr;
	World->PerModuleDataObjects.FindItemByClass(&FoundObject);

	checkf(FoundObject, TEXT("ULandscapInfoMap object was not created for this UWorld."));

	return *FoundObject;
}

ULandscapeInfoMap* ULandscapeInfoMap::FindLandscapeInfoMap(const UWorld* World)
{
	ULandscapeInfoMap* FoundObject = nullptr;
	World->PerModuleDataObjects.FindItemByClass(&FoundObject);
	return FoundObject;
}

