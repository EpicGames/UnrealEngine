// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeLayerInfoObject.h"

#include "Landscape.h"
#include "LandscapeInfo.h"
#include "LandscapeProxy.h"
#include "LandscapeSettings.h"
#include "UObject/UObjectIterator.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapeLayerInfoObject)


ULandscapeLayerInfoObject::ULandscapeLayerInfoObject(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (!IsTemplate())
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Assign initial random LayerUsageDebugColor
		LayerUsageDebugColor = GenerateLayerUsageDebugColor();
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// The default blend method comes from the settings :
		BlendMethod = GetDefault<ULandscapeSettings>()->GetTargetLayerDefaultBlendMethod();
	}
}

FLinearColor ULandscapeLayerInfoObject::GenerateLayerUsageDebugColor() const
{
	uint8 Hash[20];
	FString PathNameString = GetPathName();
	FSHA1::HashBuffer(*PathNameString, PathNameString.Len() * sizeof(PathNameString[0]), Hash);

	return FLinearColor(float(Hash[0]) / 255.f, float(Hash[1]) / 255.f, float(Hash[2]) / 255.f, 1.f);
}

void ULandscapeLayerInfoObject::SetBlendMethod(ELandscapeTargetLayerBlendMethod InBlendMethod, bool bInModify)
{
	const bool bHasValueChanged = (InBlendMethod != BlendMethod);
	if (bHasValueChanged)
	{
		if (bInModify)
		{
			Modify();
		}
		BlendMethod = InBlendMethod;
	}

#if WITH_EDITOR
	BroadcastOnLayerInfoObjectDataChanged(GET_MEMBER_NAME_CHECKED(ULandscapeLayerInfoObject, BlendMethod), /*bInUserTriggered = */true, /*bInRequiresLandscapeUpdate =*/ true, bHasValueChanged, /*InChangeType = */EPropertyChangeType::ValueSet);
#endif // WITH_EDITOR
}

FName ULandscapeLayerInfoObject::GetBlendGroup() const
{
	return (BlendMethod == ELandscapeTargetLayerBlendMethod::PremultipliedAlphaBlending) ? BlendGroup : NAME_None;
}

void ULandscapeLayerInfoObject::SetBlendGroup(const FName& InBlendGroup, bool bInModify)
{
	const bool bHasValueChanged = (InBlendGroup != BlendGroup);
	if (bHasValueChanged)
	{
		if (bInModify)
		{
			Modify();
		}
		BlendGroup = InBlendGroup;
	}

#if WITH_EDITOR
	BroadcastOnLayerInfoObjectDataChanged(GET_MEMBER_NAME_CHECKED(ULandscapeLayerInfoObject, BlendGroup), /*bInUserTriggered = */true, /*bInRequiresLandscapeUpdate =*/ true, bHasValueChanged, /*InChangeType = */EPropertyChangeType::ValueSet);
#endif // WITH_EDITOR
}

// TODO [jared.ritchie] when deprecated public properties are made private, re-enable deprecation warnings
PRAGMA_DISABLE_DEPRECATION_WARNINGS
void ULandscapeLayerInfoObject::SetLayerName(const FName& InLayerName, bool bInModify)
{
	const bool bHasValueChanged = (InLayerName != LayerName);
	if (bHasValueChanged)
	{
		if (bInModify)
		{
			Modify();
		}
		LayerName = InLayerName;
	}

#if WITH_EDITOR
	BroadcastOnLayerInfoObjectDataChanged(GET_MEMBER_NAME_CHECKED(ULandscapeLayerInfoObject, LayerName), /*bInUserTriggered = */true, /*bInRequiresLandscapeUpdate =*/ true, bHasValueChanged, /*InChangeType = */EPropertyChangeType::ValueSet);
#endif // WITH_EDITOR
}

void ULandscapeLayerInfoObject::SetPhysicalMaterial(TObjectPtr<UPhysicalMaterial> InPhysicalMaterial, bool bInModify)
{
	const bool bHasValueChanged = (InPhysicalMaterial != PhysMaterial);
	if (bHasValueChanged)
	{
		if (bInModify)
		{
			Modify();
		}
		PhysMaterial = InPhysicalMaterial;
	}

#if WITH_EDITOR
	BroadcastOnLayerInfoObjectDataChanged(GET_MEMBER_NAME_CHECKED(ULandscapeLayerInfoObject, PhysMaterial), /*bInUserTriggered = */true, /*bInRequiresLandscapeUpdate =*/ true, bHasValueChanged, /*InChangeType = */EPropertyChangeType::ValueSet);
#endif // WITH_EDITOR
}

void ULandscapeLayerInfoObject::SetHardness(float InHardness, bool bInModify, EPropertyChangeType::Type InChangeType)
{
	const bool bHasValueChanged = (InHardness != Hardness);
	if (bHasValueChanged)
	{
		if (bInModify)
		{
			Modify();
		}
		Hardness = InHardness;
	}

#if WITH_EDITOR
	BroadcastOnLayerInfoObjectDataChanged(GET_MEMBER_NAME_CHECKED(ULandscapeLayerInfoObject, Hardness), /*bInUserTriggered = */true, /*bInRequiresLandscapeUpdate =*/ false, bHasValueChanged, InChangeType);
#endif // WITH_EDITOR
}

void ULandscapeLayerInfoObject::SetLayerUsageDebugColor(const FLinearColor& InLayerUsageDebugColor, bool bInModify, EPropertyChangeType::Type InChangeType)
{
	LayerUsageDebugColor.A = 1;
	const bool bHasValueChanged = (InLayerUsageDebugColor != LayerUsageDebugColor);

	if (bHasValueChanged)
	{
		if (bInModify)
		{
			Modify();
		}
		LayerUsageDebugColor = InLayerUsageDebugColor;
	}

#if WITH_EDITOR
	BroadcastOnLayerInfoObjectDataChanged(GET_MEMBER_NAME_CHECKED(ULandscapeLayerInfoObject, LayerUsageDebugColor), /*bInUserTriggered = */true, /*bInRequiresLandscapeUpdate =*/false, bHasValueChanged, InChangeType);
#endif // WITH_EDITOR
}

#if WITH_EDITORONLY_DATA
void ULandscapeLayerInfoObject::SetMinimumCollisionRelevanceWeight(float InMinimumCollisionRelevanceWeight, bool bInModify, EPropertyChangeType::Type InChangeType)
{
	const bool bHasValueChanged = (InMinimumCollisionRelevanceWeight != MinimumCollisionRelevanceWeight);
	if (bHasValueChanged)
	{
		if (bInModify)
		{
			Modify();
		}
		MinimumCollisionRelevanceWeight = InMinimumCollisionRelevanceWeight;
	}

	BroadcastOnLayerInfoObjectDataChanged(GET_MEMBER_NAME_CHECKED(ULandscapeLayerInfoObject, MinimumCollisionRelevanceWeight), /*bInUserTriggered = */true, /*bInRequiresLandscapeUpdate =*/ true, bHasValueChanged, InChangeType);
}

void ULandscapeLayerInfoObject::SetSplineFalloffModulationTexture(TObjectPtr<UTexture2D> InSplineFalloffModulationTexture, bool bInModify)
{
	const bool bHasValueChanged = (InSplineFalloffModulationTexture != SplineFalloffModulationTexture);
	if (bHasValueChanged)
	{
		if (bInModify)
		{
			Modify();
		}
		SplineFalloffModulationTexture = InSplineFalloffModulationTexture;
	}

	BroadcastOnLayerInfoObjectDataChanged(GET_MEMBER_NAME_CHECKED(ULandscapeLayerInfoObject, SplineFalloffModulationTexture), /*bInUserTriggered = */true, /*bInRequiresLandscapeUpdate =*/ true, bHasValueChanged, /*InChangeType = */EPropertyChangeType::ValueSet);
}

void ULandscapeLayerInfoObject::SetSplineFalloffModulationColorMask(ESplineModulationColorMask InSplineFalloffModulationColorMask, bool bInModify)
{
	const bool bHasValueChanged = (InSplineFalloffModulationColorMask != SplineFalloffModulationColorMask);
	if (bHasValueChanged)
	{
		if (bInModify)
		{
			Modify();
		}
		SplineFalloffModulationColorMask = InSplineFalloffModulationColorMask;
	}

	BroadcastOnLayerInfoObjectDataChanged(GET_MEMBER_NAME_CHECKED(ULandscapeLayerInfoObject, SplineFalloffModulationColorMask), /*bInUserTriggered = */true, /*bInRequiresLandscapeUpdate =*/ true, bHasValueChanged, /*InChangeType = */EPropertyChangeType::ValueSet);
}

void ULandscapeLayerInfoObject::SetSplineFalloffModulationTiling(float InSplineFalloffModulationTiling, bool bInModify, EPropertyChangeType::Type InChangeType)
{
	const bool bHasValueChanged = (InSplineFalloffModulationTiling != SplineFalloffModulationTiling);
	if (bHasValueChanged)
	{
		if (bInModify)
		{
			Modify();
		}
		SplineFalloffModulationTiling = InSplineFalloffModulationTiling;
	}

	BroadcastOnLayerInfoObjectDataChanged(GET_MEMBER_NAME_CHECKED(ULandscapeLayerInfoObject, SplineFalloffModulationTiling), /*bInUserTriggered = */true, /*bInRequiresLandscapeUpdate =*/ true, bHasValueChanged, InChangeType);
}

void ULandscapeLayerInfoObject::SetSplineFalloffModulationBias(float InSplineFalloffModulationBias, bool bInModify, EPropertyChangeType::Type InChangeType)
{
	const bool bHasValueChanged = (InSplineFalloffModulationBias != SplineFalloffModulationBias);
	if (bHasValueChanged)
	{
		if (bInModify)
		{
			Modify();
		}
		SplineFalloffModulationBias = InSplineFalloffModulationBias;
	}

	BroadcastOnLayerInfoObjectDataChanged(GET_MEMBER_NAME_CHECKED(ULandscapeLayerInfoObject, SplineFalloffModulationBias), /*bInUserTriggered = */true, /*bInRequiresLandscapeUpdate =*/ true, bHasValueChanged, InChangeType);
}

void ULandscapeLayerInfoObject::SetSplineFalloffModulationScale(float InSplineFalloffModulationScale, bool bInModify, EPropertyChangeType::Type InChangeType)
{
	const bool bHasValueChanged = (InSplineFalloffModulationScale != SplineFalloffModulationScale);
	if (bHasValueChanged)
	{
		if (bInModify)
		{
			Modify();
		}
		SplineFalloffModulationScale = InSplineFalloffModulationScale;
	}

	BroadcastOnLayerInfoObjectDataChanged(GET_MEMBER_NAME_CHECKED(ULandscapeLayerInfoObject, SplineFalloffModulationScale), /*bInUserTriggered = */true, /*bInRequiresLandscapeUpdate =*/ true, bHasValueChanged, InChangeType);
}
#endif // WITH_EDITORONLY_DATA
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void ULandscapeLayerInfoObject::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
}

#if WITH_EDITOR

void ULandscapeLayerInfoObject::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberPropertyName = PropertyChangedEvent.MemberProperty ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;

	bool bRequiresLandscapeUpdate = true;
	// Some properties don't need the landscape to be updated :
	if (MemberPropertyName == GetHardnessMemberName() || MemberPropertyName == GetLayerUsageDebugColorMemberName())
	{
		bRequiresLandscapeUpdate = false;
	}

	BroadcastOnLayerInfoObjectDataChanged(MemberPropertyName, /*bInUserTriggered = */true, bRequiresLandscapeUpdate, /*bHasValueChanged =*/true, PropertyChangedEvent.ChangeType);
}

void ULandscapeLayerInfoObject::PostLoad()
{
	Super::PostLoad();

	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::LandscapeAdvancedWeightBlending)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		BlendMethod = bNoWeightBlend_DEPRECATED ? ELandscapeTargetLayerBlendMethod::None : ELandscapeTargetLayerBlendMethod::FinalWeightBlending;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
	if (GIsEditor)
	{
		if (!HasAnyFlags(RF_Standalone))
		{
			SetFlags(RF_Standalone);
		}
		SetHardness(FMath::Clamp<float>(GetHardness(), 0.0f, 1.0f), /*bInModify =*/false, /*InChangeType =*/ EPropertyChangeType::ValueSet);
	}
}

void ULandscapeLayerInfoObject::BroadcastOnLayerInfoObjectDataChanged(FName InPropertyName, bool bInUserTriggered, bool bInRequiresLandscapeUpdate, bool bInHasValueChanged, EPropertyChangeType::Type InChangeType)
{
	FProperty* Property = nullptr;
	if (InPropertyName != NAME_None)
	{
		Property = FindFProperty<FProperty>(StaticClass(), InPropertyName);
		check(Property != nullptr);
	}

	FOnLandscapeLayerInfoDataChangedParams OnLayerInfoChangedParams(*this, FPropertyChangedEvent(Property, InChangeType));
	OnLayerInfoChangedParams.bUserTriggered = bInUserTriggered;
	OnLayerInfoChangedParams.bRequiresLandscapeUpdate = bInRequiresLandscapeUpdate;
	OnLayerInfoChangedParams.bHasValueChanged = bInHasValueChanged;
	OnLayerInfoObjectChangedDelegate.Broadcast(OnLayerInfoChangedParams);
}

#endif // WITH_EDITOR
