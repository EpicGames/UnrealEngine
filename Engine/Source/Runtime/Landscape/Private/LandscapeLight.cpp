// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
LandscapeLight.cpp: Static lighting for LandscapeComponents
=============================================================================*/

#include "LandscapeLight.h"
#include "Engine/EngineTypes.h"
#include "CollisionQueryParams.h"
#include "LandscapeProxy.h"
#include "LandscapeInfo.h"
#include "LightMap.h"
#include "Engine/Level.h"
#include "Engine/MapBuildDataRegistry.h"
#include "StaticLightingBuildContext.h"
#include "Components/LightComponent.h"
#include "ShadowMap.h"
#include "LandscapeComponent.h"
#include "LandscapeDataAccess.h"
#include "LandscapeGrassType.h"
#include "UnrealEngine.h"
#include "Materials/Material.h"

#if WITH_EDITOR

#include "ComponentReregisterContext.h"

#define LANDSCAPE_LIGHTMAP_UV_INDEX 1

TMap<FIntPoint, FColor> FLandscapeStaticLightingMesh::LandscapeUpscaleHeightDataCache;

/** A texture mapping for landscapes */
/** Initialization constructor. */
FLandscapeStaticLightingTextureMapping::FLandscapeStaticLightingTextureMapping(
	ULandscapeComponent* InComponent,FStaticLightingMesh* InMesh,int32 InLightMapWidth,int32 InLightMapHeight,bool bPerformFullQualityRebuild) :
FStaticLightingTextureMapping(
							  InMesh,
							  InComponent,
							  InLightMapWidth,
							  InLightMapHeight,
							  LANDSCAPE_LIGHTMAP_UV_INDEX
							  ),
							  LandscapeComponent(InComponent)
{
}

FLandscapeStaticLightingGlobalVolumeMapping::FLandscapeStaticLightingGlobalVolumeMapping(ULandscapeComponent* InComponent,FStaticLightingMesh* InMesh,int32 InLightMapWidth,int32 InLightMapHeight,bool bPerformFullQualityRebuild) :	
	FLandscapeStaticLightingTextureMapping(InComponent, InMesh, InLightMapWidth, InLightMapHeight, bPerformFullQualityRebuild)
{}

FLandscapeStaticLightingTextureMapping::FLandscapeStaticLightingTextureMapping(const FArchive& Ar)
	: FStaticLightingTextureMapping(Ar), 
	  LandscapeComponent(nullptr)
{
}

void FLandscapeStaticLightingTextureMapping::Serialize(FArchive& Ar)
{
	FStaticLightingTextureMapping::Serialize(Ar);

	FSoftObjectPath LandscapePath;
	if (LandscapeComponent)
	{
		LandscapePath = FSoftObjectPath(LandscapeComponent);
	}
	Ar << LandscapePath;
	LandscapeComponent = Cast<ULandscapeComponent>(LandscapePath.ResolveObject());
	
}

void FLandscapeStaticLightingTextureMapping::Apply(FQuantizedLightmapData* QuantizedData, const TMap<ULightComponent*,FShadowMapData2D*>& ShadowMapData, const FStaticLightingBuildContext* LightingContext)
{
	const bool bUseVirtualTextures = UseVirtualTextureLightmap(GMaxRHIShaderPlatform);

	//ELightMapPaddingType PaddingType = GAllowLightmapPadding ? LMPT_NormalPadding : LMPT_NoPadding;
	ELightMapPaddingType PaddingType = LMPT_NoPadding;

	UMapBuildDataRegistry* Registry = LightingContext->GetOrCreateRegistryForActor(LandscapeComponent->GetOwner());
	FMeshMapBuildData& MeshBuildData = Registry->AllocateMeshBuildData(LandscapeComponent->MapBuildDataId, true);

	const bool bHasNonZeroData = QuantizedData != NULL && QuantizedData->HasNonZeroData();

	// We always create a light map if the surface either has any non-zero lighting data, or if the surface has a shadow map.  The runtime
	// shaders are always expecting a light map in the case of a shadow map, even if the lighting is entirely zero.  This is simply to reduce
	// the number of shader permutations to support in the very unlikely case of a unshadowed surfaces that has lighting values of zero.
	const bool bNeedsLightMap = bHasNonZeroData || ShadowMapData.Num() > 0 || Mesh->RelevantLightsGuid.Num() > 0 || (QuantizedData != NULL && QuantizedData->bHasSkyShadowing);
	if (bNeedsLightMap)
	{
		// Create a light-map for the primitive.
		TMap<ULightComponent*, FShadowMapData2D*> EmptyShadowMapData;
		MeshBuildData.LightMap = FLightMap2D::AllocateLightMap(
			Registry,
			QuantizedData,
			bUseVirtualTextures ? ShadowMapData : EmptyShadowMapData,
			LandscapeComponent->Bounds,
			PaddingType,
			LMF_Streamed
			);
	}
	else
	{
		MeshBuildData.LightMap = NULL;
	}

	if (ShadowMapData.Num() > 0 && !bUseVirtualTextures)
	{
		MeshBuildData.ShadowMap = FShadowMap2D::AllocateShadowMap(
			Registry,
			ShadowMapData,
			LandscapeComponent->Bounds,
			PaddingType,
			SMF_Streamed
			);
	}
	else
	{
		MeshBuildData.ShadowMap = NULL;
	}

	// Flush Grass 
	if (ALandscapeProxy* Proxy = Cast<ALandscapeProxy>(LandscapeComponent->GetOuter()))
	{
		TSet<ULandscapeComponent*> Components;
		Components.Add(LandscapeComponent);
		Proxy->FlushGrassComponents(&Components, false);
	}
	
	// Build the list of statically irrelevant lights.
	// TODO: This should be stored per LOD.
	for (int32 LightIndex = 0; LightIndex < Mesh->RelevantLightsGuid.Num(); LightIndex++)
	{
		FGuid LightGuid = Mesh->RelevantLightsGuid[LightIndex];

		// Check if the light is stored in the light-map.
		const bool bIsInLightMap = MeshBuildData.LightMap && MeshBuildData.LightMap->LightGuids.Contains(LightGuid);

		// Add the light to the statically irrelevant light list if it is in the potentially relevant light list, but didn't contribute to the light-map.
		if(!bIsInLightMap)
		{
			MeshBuildData.IrrelevantLights.AddUnique(LightGuid);
		}
	}
}

namespace
{
	// Calculate Geometric LOD for lighting
	int32 GetLightingLOD(const ULandscapeComponent* InComponent)
	{
		if (InComponent->LightingLODBias < 0)
		{
			return FMath::Clamp<int32>(InComponent->ForcedLOD >= 0 ? InComponent->ForcedLOD : InComponent->LODBias, 0, FMath::CeilLogTwo(InComponent->SubsectionSizeQuads + 1) - 1);
		}
		else
		{
			return InComponent->LightingLODBias;
		}
	}
};

/** Initialization constructor. */
FLandscapeStaticLightingMesh::FLandscapeStaticLightingMesh(ULandscapeComponent* InComponent, const TArray<ULightComponent*>& InRelevantLights, int32 InExpandQuadsX, int32 InExpandQuadsY, float InLightMapRatio, int32 InLOD)
	: FStaticLightingMesh(
		FMath::Square(((InComponent->ComponentSizeQuads + 1) >> InLOD) - 1 + 2 * InExpandQuadsX) * 2,
		FMath::Square(((InComponent->ComponentSizeQuads + 1) >> InLOD) - 1 + 2 * InExpandQuadsX) * 2,
		FMath::Square(((InComponent->ComponentSizeQuads + 1) >> InLOD) + 2 * InExpandQuadsX),
		FMath::Square(((InComponent->ComponentSizeQuads + 1) >> InLOD) + 2 * InExpandQuadsX),
		0,
		!!(InComponent->CastShadow | InComponent->bCastHiddenShadow),
		false,
		InRelevantLights,
		InComponent,
		InComponent->Bounds.GetBox(),
		InComponent->GetLightingGuid(),
		InComponent->MapBuildDataId
	)
	, LandscapeComponent(InComponent)
	, LightMapRatio(InLightMapRatio)
	, ExpandQuadsX(InExpandQuadsX)
	, ExpandQuadsY(InExpandQuadsY)
{
	const float LODScale = (float)InComponent->ComponentSizeQuads / (((InComponent->ComponentSizeQuads + 1) >> InLOD) - 1);
	LocalToWorld = FTransform(FQuat::Identity, FVector::ZeroVector, FVector(LODScale, LODScale, 1)) * InComponent->GetComponentTransform();
	ComponentSizeQuads = ((InComponent->ComponentSizeQuads + 1) >> InLOD) - 1;
	NumVertices = ComponentSizeQuads + 2*InExpandQuadsX + 1;
	NumQuads = NumVertices - 1;
	UVFactor = LightMapRatio / NumVertices;
	bReverseWinding = (LocalToWorld.GetDeterminant() < 0.0f);

	int32 GeometricLOD = ::GetLightingLOD(InComponent);
	GetHeightmapData(InLOD, FMath::Max(GeometricLOD, InLOD));
}

FLandscapeStaticLightingMesh::~FLandscapeStaticLightingMesh()
{
}

namespace
{
	template <typename T, typename U>
	T MultiLerp(T A, T B, U F1, T C, T D, U F2, U F3)
	{
		return static_cast<T>(FMath::RoundToInt32(
			FMath::Lerp<float>(
				FMath::Lerp<float>(A, B, F1),
				FMath::Lerp<float>(C, D, F2),
				F3
			)
		));
	}

	template <typename U>
	float MultiLerp(int32 A, int32 B, U F1, int32 C, int32 D, U F2, U F3)
	{
		return FMath::Lerp(
			FMath::Lerp(static_cast<float>(A), static_cast<float>(B), static_cast<float>(F1)),
			FMath::Lerp(static_cast<float>(C), static_cast<float>(D), static_cast<float>(F2)),
			static_cast<float>(F3)
		);
	}

	void GetLODData(ULandscapeComponent* LandscapeComponent, int32 X, int32 Y, int32 HeightmapOffsetX, int32 HeightmapOffsetY, int32 LODValue, int32 HeightmapStride, FColor& OutHeight)
	{
		int32 ComponentSize = ((LandscapeComponent->SubsectionSizeQuads + 1) * LandscapeComponent->NumSubsections) >> LODValue;
		int32 LODHeightmapSizeX = LandscapeComponent->GetHeightmap()->Source.GetSizeX() >> LODValue;
		int32 LODHeightmapSizeY = LandscapeComponent->GetHeightmap()->Source.GetSizeY() >> LODValue;
		float Ratio = (float)(LODHeightmapSizeX) / (HeightmapStride);

		int32 CurrentHeightmapOffsetX = FMath::RoundToInt32(LODHeightmapSizeX * LandscapeComponent->HeightmapScaleBias.Z);
		int32 CurrentHeightmapOffsetY = FMath::RoundToInt32(LODHeightmapSizeY * LandscapeComponent->HeightmapScaleBias.W);

		float XX = FMath::Clamp<float>((X - HeightmapOffsetX) * Ratio, 0.f, ComponentSize - 1.f) + CurrentHeightmapOffsetX;
		int32 XI = (int32)XX;
		float XF = XX - XI;

		float YY = FMath::Clamp<float>((Y - HeightmapOffsetY) * Ratio, 0.f, ComponentSize - 1.f) + CurrentHeightmapOffsetY;
		int32 YI = (int32)YY;
		float YF = YY - YI;

		FLandscapeComponentDataInterface DataInterface(LandscapeComponent, LODValue);
		FColor* HeightMipData = DataInterface.GetRawHeightData();

		FColor H1 = HeightMipData[XI + YI * LODHeightmapSizeX];
		FColor H2 = HeightMipData[FMath::Min(XI + 1, LODHeightmapSizeX - 1) + YI * LODHeightmapSizeX];
		FColor H3 = HeightMipData[XI + FMath::Min(YI + 1, LODHeightmapSizeY - 1) * LODHeightmapSizeX];
		FColor H4 = HeightMipData[FMath::Min(XI + 1, LODHeightmapSizeX - 1) + FMath::Min(YI + 1, LODHeightmapSizeY - 1) * LODHeightmapSizeX];

		uint16 Height = MultiLerp<uint16, float>((H1.R << 8) + H1.G, (H2.R << 8) + H2.G, XF, (H3.R << 8) + H3.G, (H4.R << 8) + H4.G, XF, YF);
		uint8 B = MultiLerp<uint8, float>(H1.B, H2.B, XF, H3.B, H4.B, XF, YF);
		uint8 A = MultiLerp<uint8, float>(H1.A, H2.A, XF, H3.A, H4.A, XF, YF);

		OutHeight = FColor((Height >> 8), Height & 255, B, A);
	}

	void InternalUpscaling(FLandscapeComponentDataInterface& DataInterface, ULandscapeComponent* LandscapeComponent, int32 InLOD, int32 GeometryLOD, TArray<FColor>& CompHeightData)
	{
		// Upscaling using Landscape LOD system 
		ULandscapeInfo* const Info = LandscapeComponent->GetLandscapeInfo();
		check(Info);

		FIntPoint ComponentBase = LandscapeComponent->GetSectionBase() / LandscapeComponent->ComponentSizeQuads;
		int32 NeighborLODs[8] = { 0, 0, 0, 0, 0, 0, 0, 0 };

		int32 MaxLOD = FMath::CeilLogTwo(LandscapeComponent->SubsectionSizeQuads + 1) - 1;
		bool bNeedUpscaling = GeometryLOD > InLOD;
		int32 NeighborIdx = 0;

		for (int32 y = -1; y <= 1; y++)
		{
			for (int32 x = -1; x <= 1; x++)
			{
				if (x == 0 && y == 0)
				{
					continue;
				}

				ULandscapeComponent* Neighbor = Info->XYtoComponentMap.FindRef(ComponentBase + FIntPoint(x, y));
				int32 NeighborLOD;

				if (Neighbor)
				{
					NeighborLOD = ::GetLightingLOD(Neighbor);
				}
				else
				{
					NeighborLOD = 0;
					// Sample neighbor components to find maximum LOD
					for (int32 yy = -1; yy <= 1; yy++)
					{
						for (int32 xx = -1; xx <= 1; xx++)
						{
							if (xx == 0 && yy == 0)
							{
								continue;
							}

							ULandscapeComponent* ComponentNeighbor = Info->XYtoComponentMap.FindRef(ComponentBase + FIntPoint(x+xx, y+yy));
							if (ComponentNeighbor)
							{
								NeighborLOD = FMath::Max(::GetLightingLOD(ComponentNeighbor), NeighborLOD);
							}
						}
					}
				}

				bNeedUpscaling |= (NeighborLOD > InLOD);
				NeighborLODs[NeighborIdx++] = NeighborLOD;
			}
		}

		if (bNeedUpscaling)
		{
			check(LandscapeComponent);
			// Need Upscaling
			int32 HeightmapStride = LandscapeComponent->GetHeightmap()->Source.GetSizeX() >> InLOD;
			int32 HeightDataSize = HeightmapStride * LandscapeComponent->GetHeightmap()->Source.GetSizeY() >> InLOD;

			CompHeightData.Empty(HeightDataSize);
			CompHeightData.AddZeroed(HeightDataSize);

			// Update for only component region for performance
			int32 ComponentSize = ((LandscapeComponent->SubsectionSizeQuads + 1) * LandscapeComponent->NumSubsections) >> InLOD;

			for (int32 Y = DataInterface.HeightmapComponentOffsetY; Y < DataInterface.HeightmapComponentOffsetY + ComponentSize; ++Y)
			{
				for (int32 X = DataInterface.HeightmapComponentOffsetX; X < DataInterface.HeightmapComponentOffsetX + ComponentSize; ++X)
				{
					FIntPoint IXY(X - DataInterface.HeightmapComponentOffsetX, Y - DataInterface.HeightmapComponentOffsetY);
					IXY += ComponentBase * (ComponentSize - 1);

					FColor* CachedHeight = FLandscapeStaticLightingMesh::LandscapeUpscaleHeightDataCache.Find(IXY);

					if (CachedHeight)
					{
						CompHeightData[X + Y * HeightmapStride] = *CachedHeight;
					}
					else
					{
						// LOD System similar to the shader
						FVector2D XY(float(X - DataInterface.HeightmapComponentOffsetX) / (ComponentSize - 1), float(Y - DataInterface.HeightmapComponentOffsetY) / (ComponentSize - 1));
						XY = XY - 0.5f;

						float RealLOD;

						if (XY.X < 0.f)
						{
							if (XY.Y < 0.f)
							{
								RealLOD = MultiLerp(NeighborLODs[0], NeighborLODs[1], XY.X + 1.f, NeighborLODs[3], GeometryLOD, XY.X + 1.f, XY.Y + 1.f); // 0
							}
							else
							{
								RealLOD = MultiLerp(NeighborLODs[3], GeometryLOD, XY.X + 1.f, NeighborLODs[5], NeighborLODs[6], XY.X + 1.f, XY.Y); // 2
							}
						}
						else
						{
							if (XY.Y < 0.f)
							{
								RealLOD = MultiLerp(NeighborLODs[1], NeighborLODs[2], XY.X, GeometryLOD, NeighborLODs[4], XY.X, XY.Y + 1.f); // 1
							}
							else
							{
								RealLOD = MultiLerp(GeometryLOD, NeighborLODs[4], XY.X, NeighborLODs[6], NeighborLODs[7], XY.X, XY.Y); // 3
							}
						}

						RealLOD = FMath::Min(RealLOD, (float)MaxLOD);

						int32 LODValue = (int32)RealLOD;
						float MorphAlpha = FMath::Fractional(RealLOD);

						FColor Height[2];
						::GetLODData(LandscapeComponent, X, Y, DataInterface.HeightmapComponentOffsetX, DataInterface.HeightmapComponentOffsetY,
							FMath::Min(MaxLOD, LODValue), HeightmapStride, Height[0]);

						// Interpolation between two LOD
						if ((RealLOD > InLOD) && (LODValue + 1 <= MaxLOD) && MorphAlpha != 0.f)
						{
							::GetLODData(LandscapeComponent, X, Y, DataInterface.HeightmapComponentOffsetX, DataInterface.HeightmapComponentOffsetY,
								FMath::Min(MaxLOD, LODValue + 1), HeightmapStride, Height[1]);

							// Need interpolation
							const uint16 Height0 = (Height[0].R << 8) + Height[0].G;
							const uint16 Height1 = (Height[1].R << 8) + Height[1].G;
							const uint16 LerpHeight = static_cast<uint16>(FMath::RoundToInt32(FMath::Lerp<float>(Height0, Height1, MorphAlpha)));

							CompHeightData[X + Y * HeightmapStride] =
								FColor((LerpHeight >> 8), LerpHeight & 255,
								static_cast<uint8>(FMath::RoundToInt32(FMath::Lerp<float>(Height[0].B, Height[1].B, MorphAlpha))),
								static_cast<uint8>(FMath::RoundToInt32(FMath::Lerp<float>(Height[0].A, Height[1].A, MorphAlpha))));
						}
						else
						{
							CompHeightData[X + Y * HeightmapStride] = Height[0];
						}

						// Caching current calculated value
						FLandscapeStaticLightingMesh::LandscapeUpscaleHeightDataCache.Add(IXY, CompHeightData[X + Y * HeightmapStride]);
					}
				}
			}

			DataInterface.SetRawHeightData(&CompHeightData[0]);
		}
	}
};

void FLandscapeStaticLightingMesh::GetHeightmapData(int32 InLOD, int32 GeometryLOD)
{
	ULandscapeInfo* const Info = LandscapeComponent->GetLandscapeInfo();
	check(Info);

	bool bUseRenderedWPO = LandscapeComponent->GetLandscapeProxy()->bUseMaterialPositionOffsetInStaticLighting &&
		LandscapeComponent->GetLandscapeMaterial()->GetMaterial()->IsPropertyConnected(MP_WorldPositionOffset);

	HeightData.Empty(FMath::Square(NumVertices));
	HeightData.AddUninitialized(FMath::Square(NumVertices));

	const int32 NumSubsections = LandscapeComponent->NumSubsections;
	const int32 SubsectionSizeVerts = (LandscapeComponent->SubsectionSizeQuads + 1) >> InLOD;
	const int32 SubsectionSizeQuads = SubsectionSizeVerts - 1;
	FIntPoint ComponentBase = LandscapeComponent->GetSectionBase()/LandscapeComponent->ComponentSizeQuads;

	// assume that ExpandQuad size <= SubsectionSizeQuads...
	check(ExpandQuadsX <= SubsectionSizeQuads);
	check(ExpandQuadsY <= SubsectionSizeQuads);

	// copy heightmap data for this component...
	{
		// Data array for upscaling case
		TArray<FColor> CompHeightData;
		FLandscapeComponentDataInterface DataInterface(LandscapeComponent, InLOD);
		::InternalUpscaling(DataInterface, LandscapeComponent, InLOD, GeometryLOD, CompHeightData);

		TArray<uint16> RenderedWPOData;
		if (bUseRenderedWPO)
		{
			// todo - do we need to invalidate the landscape mesh version?
			RenderedWPOData = LandscapeComponent->RenderWPOHeightmap(InLOD);
		}

		const int32 ComponentSizeVerts = ComponentSizeQuads + 1;
		for (int32 Y = 0; Y < ComponentSizeVerts; Y++)
		{
			const FColor* const Data = DataInterface.GetHeightData(0, Y);

			for (int32 SubsectionX = 0; SubsectionX < NumSubsections; SubsectionX++)
			{
				const int32 X = SubsectionSizeQuads * SubsectionX;
				const int32 TexX = X + FMath::Min(X/SubsectionSizeQuads, NumSubsections - 1);
				const FColor* const SubsectionData = &Data[TexX];

				// Copy the data
				FMemory::Memcpy(&HeightData[X + ExpandQuadsX + (Y + ExpandQuadsY) * NumVertices], SubsectionData, SubsectionSizeVerts * sizeof(FColor));
			}

			if (bUseRenderedWPO && RenderedWPOData.Num())
			{
				const int32 HeightDataOffset = ExpandQuadsX + (Y + ExpandQuadsY) * NumVertices;
				const int32 WPODataOffset    = Y * ComponentSizeVerts;
				for (int32 X = 0; X < ComponentSizeVerts; ++X)
				{
					const uint16 WPOHeight = RenderedWPOData[X + WPODataOffset];
					FColor& Height = HeightData[X + HeightDataOffset];
					Height.R = (WPOHeight >> 8);
					Height.G = (WPOHeight & 0xFF);
				}
			}
		}
	}

	// copy surrounding heightmaps...
	for (int32 ComponentY = -1; ComponentY <= 1; ++ComponentY)
	{
		for (int32 ComponentX = -1; ComponentX <= 1; ++ComponentX)
		{
			if (ComponentX == 0 && ComponentY == 0)
			{
				// Ourself
				continue;
			}

			// Coordinates and Num are all in component-space not tex-space
			// Note: This means they don't include the duped vert when NumSubsections == 2
			const int32 XSource = (ComponentX == -1) ? (ComponentSizeQuads - ExpandQuadsX) : ((ComponentX == 0) ? 0 : 1);
			const int32 YSource = (ComponentY == -1) ? (ComponentSizeQuads - ExpandQuadsY) : ((ComponentY == 0) ? 0 : 1);
			const int32 XDest = (ComponentX == -1) ? 0 : ((ComponentX == 0) ? ExpandQuadsX : (ComponentSizeQuads + ExpandQuadsX + 1));
			const int32 YDest = (ComponentY == -1) ? 0 : ((ComponentY == 0) ? ExpandQuadsY : (ComponentSizeQuads + ExpandQuadsY + 1));
			const int32 XNum = (ComponentX == 0) ? (ComponentSizeQuads + 1) : ExpandQuadsX;
			const int32 YNum = (ComponentY == 0) ? (ComponentSizeQuads + 1) : ExpandQuadsY;

			ULandscapeComponent* Neighbor = Info->XYtoComponentMap.FindRef(ComponentBase + FIntPoint(ComponentX, ComponentY));
			if (Neighbor)
			{
				// Data array for upscaling case
				TArray<FColor> CompHeightData;
				int32 NeighborGeometricLOD = ::GetLightingLOD(Neighbor);
				FLandscapeComponentDataInterface DataInterface(Neighbor, InLOD);
				::InternalUpscaling(DataInterface, Neighbor, InLOD, NeighborGeometricLOD, CompHeightData);

				TArray<uint16> RenderedWPOData;
				if (bUseRenderedWPO)
				{
					RenderedWPOData = Neighbor->RenderWPOHeightmap(InLOD);
				}

				for (int32 Y = 0; Y < YNum; Y++)
				{
					const FColor* const Data = DataInterface.GetHeightData(0, YSource + Y);

					const int32 HeightDataOffset = XDest - XSource + (YDest + Y) * NumVertices;

					int32 NextX;
					for (int32 X = XSource; X < XSource + XNum; X = NextX)
					{
						NextX = (X / SubsectionSizeQuads + 1) * SubsectionSizeQuads + 1;

						// Correct for Subsection texel duplication
						const int32 TexX = X + FMath::Min(X/SubsectionSizeQuads, NumSubsections - 1);
						const FColor* const SubsectionData = &Data[TexX];

						// Copy the data
						FMemory::Memcpy(&HeightData[X + HeightDataOffset], SubsectionData, (FMath::Min(NextX, XSource + XNum) - X) * sizeof(FColor));
					}

					if (bUseRenderedWPO && RenderedWPOData.Num())
					{
						// All in component-space, no need to take texel duplication into account :)
						const int32 WPODataOffset    = (YSource + Y) * (ComponentSizeQuads + 1);
						for (int32 X = XSource; X < XSource + XNum; ++X)
						{
							const uint16 WPOHeight = RenderedWPOData[X + WPODataOffset];
							FColor& Height = HeightData[X + HeightDataOffset];
							Height.R = (WPOHeight >> 8);
							Height.G = (WPOHeight & 0xFF);
						}
					}
				}
			}
			else
			{
				const int32 XBackup = (ComponentX == 1) ? (ComponentSizeQuads + ExpandQuadsX) : ExpandQuadsX;
				const int32 YBackup = (ComponentY == 1) ? (ComponentSizeQuads + ExpandQuadsY) : ExpandQuadsY;
				const int32 XBackupNum = (ComponentX == 0) ? (ComponentSizeQuads + 1) : 1;
				const int32 YBackupNum = (ComponentY == 0) ? (ComponentSizeQuads + 1) : 1;

				for (int32 Y = 0; Y < YNum; Y++)
				{
					for (int32 X = 0; X < XNum; X += XBackupNum)
					{
						const FColor* const BackupData = &HeightData[XBackup + (YBackup + (Y % YBackupNum)) * NumVertices];

						// Copy the data
						FMemory::Memcpy( &HeightData[XDest + X + (YDest + Y) * NumVertices], BackupData, XBackupNum * sizeof(FColor));
					}
				}
			}
		}
	}
}

/** Fills in the static lighting vertex data for the Landscape vertex. */
void FLandscapeStaticLightingMesh::GetStaticLightingVertex(int32 VertexIndex, FStaticLightingVertex& OutVertex) const
{
	const int32 X = VertexIndex % NumVertices;
	const int32 Y = VertexIndex / NumVertices;

	const int32 LocalX = X - ExpandQuadsX;
	const int32 LocalY = Y - ExpandQuadsY;

	const FColor* Data = &HeightData[X + Y * NumVertices];
	OutVertex.WorldTangentZ = LandscapeDataAccess::UnpackNormal(*Data);
	OutVertex.WorldTangentX = FVector4(OutVertex.WorldTangentZ.Z, 0.0f, -OutVertex.WorldTangentZ.X);
	OutVertex.WorldTangentY = OutVertex.WorldTangentZ ^ OutVertex.WorldTangentX;

	// Copied from FLandscapeComponentDataInterface::GetWorldPositionTangents to fix bad lighting when rotated
	OutVertex.WorldTangentX = LocalToWorld.TransformVectorNoScale(OutVertex.WorldTangentX);
	OutVertex.WorldTangentY = LocalToWorld.TransformVectorNoScale(OutVertex.WorldTangentY);
	OutVertex.WorldTangentZ = LocalToWorld.TransformVectorNoScale(OutVertex.WorldTangentZ);

	const float Height = LandscapeDataAccess::UnpackHeight(*Data);
	OutVertex.WorldPosition = LocalToWorld.TransformPosition(FVector(LocalX, LocalY, Height));

	OutVertex.TextureCoordinates[0] = FVector2D((float)X / NumVertices, (float)Y / NumVertices); 
	OutVertex.TextureCoordinates[LANDSCAPE_LIGHTMAP_UV_INDEX].X = X * UVFactor;
	OutVertex.TextureCoordinates[LANDSCAPE_LIGHTMAP_UV_INDEX].Y = Y * UVFactor;
}

void FLandscapeStaticLightingMesh::GetTriangle(int32 TriangleIndex,FStaticLightingVertex& OutV0,FStaticLightingVertex& OutV1,FStaticLightingVertex& OutV2) const
{
	int32 I0, I1, I2;
	GetTriangleIndices(TriangleIndex,I0, I1, I2);
	GetStaticLightingVertex(I0,OutV0);
	GetStaticLightingVertex(I1,OutV1);
	GetStaticLightingVertex(I2,OutV2);
}

void FLandscapeStaticLightingMesh::GetTriangleIndices(int32 TriangleIndex,int32& OutI0,int32& OutI1,int32& OutI2) const
{
	int32 QuadIndex = TriangleIndex >> 1;
	int32 QuadTriIndex = TriangleIndex & 1;

	int32 QuadX = QuadIndex % (NumVertices - 1);
	int32 QuadY = QuadIndex / (NumVertices - 1);

	if (QuadTriIndex == 0)
	{
		OutI0 = (QuadX + 0) + (QuadY + 0) * NumVertices;
		OutI1 = (QuadX + 1) + (QuadY + 1) * NumVertices;
		OutI2 = (QuadX + 1) + (QuadY + 0) * NumVertices;
	}
	else // QuadTriIndex == 1
	{
		OutI0 = (QuadX + 0) + (QuadY + 0) * NumVertices;
		OutI1 = (QuadX + 0) + (QuadY + 1) * NumVertices;
		OutI2 = (QuadX + 1) + (QuadY + 1) * NumVertices;
	}

	if (bReverseWinding)
	{
		Swap(OutI1, OutI2);
	}
}



FLightRayIntersection FLandscapeStaticLightingMesh::IntersectLightRay(const FVector& Start,const FVector& End,bool bFindNearestIntersection) const
{
	// Intersect the light ray with the terrain component.
	FHitResult Result(1.0f);

	FCollisionQueryParams NewTraceParams(SCENE_QUERY_STAT(FLandscapeStaticLightingMesh_IntersectLightRay), true );
	
	const bool bIntersects = LandscapeComponent->LineTraceComponent( Result, Start, End, NewTraceParams );

	// Setup a vertex to represent the intersection.
	FStaticLightingVertex IntersectionVertex = {};
	if(bIntersects)
	{
		IntersectionVertex.WorldPosition = Result.Location;
		IntersectionVertex.WorldTangentZ = Result.Normal;
	}
	else
	{
		IntersectionVertex.WorldPosition.Set(0,0,0);
		IntersectionVertex.WorldTangentZ.Set(0,0,1);
	}
	return FLightRayIntersection(bIntersects,IntersectionVertex);
}


void ULandscapeComponent::GetStaticLightingInfo(FStaticLightingPrimitiveInfo& OutPrimitiveInfo,const TArray<ULightComponent*>& InRelevantLights,const FLightingBuildOptions& Options)
{
	if( HasStaticLighting() && GetLandscapeInfo() != nullptr)
	{
		const float LightMapRes = StaticLightingResolution > 0.f ? StaticLightingResolution : GetLandscapeProxy()->StaticLightingResolution;
		const int32 LightingLOD = GetLandscapeProxy()->StaticLightingLOD;

		int32 PatchExpandCountX = 0;
		int32 PatchExpandCountY = 0;
		int32 DesiredSize = 1;
		const float LightMapRatio = ::GetTerrainExpandPatchCount(LightMapRes, PatchExpandCountX, PatchExpandCountY, ComponentSizeQuads, (NumSubsections * (SubsectionSizeQuads+1)), DesiredSize, LightingLOD);

		int32 SizeX = DesiredSize;
		int32 SizeY = DesiredSize;

		if (SizeX > 0 && SizeY > 0)
		{
			FLandscapeStaticLightingMesh* StaticLightingMesh = new FLandscapeStaticLightingMesh(this, InRelevantLights, PatchExpandCountX, PatchExpandCountY, LightMapRatio, LightingLOD);
			OutPrimitiveInfo.Meshes.Add(StaticLightingMesh);
			if (GetLightmapType() == ELightmapType::ForceVolumetric)
			{
				OutPrimitiveInfo.Mappings.Add(new FLandscapeStaticLightingGlobalVolumeMapping(
					this,StaticLightingMesh,SizeX,SizeY,true));
			}
			else
			{
				// Create a static lighting texture mapping
				OutPrimitiveInfo.Mappings.Add(new FLandscapeStaticLightingTextureMapping(
					this,StaticLightingMesh,SizeX,SizeY,true));
			}
		}
	}
}

void ULandscapeComponent::AddMapBuildDataGUIDs(TSet<FGuid>& InGUIDs) const
{
	InGUIDs.Add(MapBuildDataId);
}

bool ULandscapeComponent::GetLightMapResolution( int32& Width, int32& Height ) const
{
	// Assuming DXT_1 compression at the moment...
	float LightMapRes = StaticLightingResolution > 0.f ? StaticLightingResolution : GetLandscapeProxy()->StaticLightingResolution;
	int32 PatchExpandCountX = 1;
	int32 PatchExpandCountY = 1;
	int32 DesiredSize = 1;
	uint32 LightingLOD = GetLandscapeProxy()->StaticLightingLOD;

	GetTerrainExpandPatchCount(LightMapRes, PatchExpandCountX, PatchExpandCountY, ComponentSizeQuads, (NumSubsections * (SubsectionSizeQuads+1)), DesiredSize, LightingLOD);

	Width = DesiredSize;
	Height = DesiredSize;

	return false;
}

int32 ULandscapeComponent::GetStaticLightMapResolution() const
{
	int32 Width, Height;
	GetLightMapResolution(Width, Height);
	return FMath::Max<int32>(Width, Height);
}

void ULandscapeComponent::GetLightAndShadowMapMemoryUsage( int32& LightMapMemoryUsage, int32& ShadowMapMemoryUsage ) const
{
	int32 Width, Height;
	GetLightMapResolution(Width, Height);

	UWorld* World = GetWorld();
	const ERHIFeatureLevel::Type FeatureLevel = World ? World->GetFeatureLevel() : GMaxRHIFeatureLevel;

	if(AllowHighQualityLightmaps(FeatureLevel))
	{
		LightMapMemoryUsage  = NUM_HQ_LIGHTMAP_COEF * (Width * Height * 4 / 3); // assuming DXT5
	}
	else
	{
		LightMapMemoryUsage  = NUM_LQ_LIGHTMAP_COEF * (Width * Height * 4 / 3) / 2; // assuming DXT1
	}

	ShadowMapMemoryUsage = (Width * Height * 4 / 3); // assuming G8
	return;
}

void ULandscapeComponent::InvalidateLightingCacheDetailed(bool bInvalidateBuildEnqueuedLighting, bool bTranslationOnly)
{
	if (ULandscapeInfo* Info = GetLandscapeInfo())
	{
		Info->ModifyObject(this);
	}

	FComponentReregisterContext ReregisterContext(this);

	Super::InvalidateLightingCacheDetailed(bInvalidateBuildEnqueuedLighting, bTranslationOnly);

	// invalidate grass that has bUseLandscapeLightmap so the new lightmap is applied to the grass
	for (auto Iter = GetLandscapeProxy()->FoliageCache.CachedGrassComps.CreateIterator(); Iter; ++Iter)
	{
		const auto& GrassKey = Iter->Key;
		const ULandscapeGrassType* GrassType = GrassKey.GrassType.Get();
		const ULandscapeComponent* BasedOn = GrassKey.BasedOn.Get();
		UHierarchicalInstancedStaticMeshComponent* GrassComponent = Iter->Foliage.Get();

		if (BasedOn == this && GrassType && GrassComponent &&
			GrassType->GrassVarieties.IsValidIndex(GrassKey.VarietyIndex) &&
			GrassType->GrassVarieties[GrassKey.VarietyIndex].bUseLandscapeLightmap)
		{
			// Remove this grass component from the cache, which will cause it to be replaced
			Iter.RemoveCurrent();
		}
	}

	MapBuildDataId = FGuid::NewGuid();
}

#endif

