// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeModule.h"
#include "Containers/Ticker.h"
#include "Serialization/CustomVersion.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectHash.h"
#include "UObject/Package.h"
#include "Engine/World.h"
#include "Materials/MaterialInterface.h"
#include "LandscapeComponent.h"
#include "LandscapeVersion.h"
#include "LandscapeInfoMap.h"
#include "LandscapeHeightfieldCollisionComponent.h"
#include "Materials/MaterialInstance.h"

#include "LandscapeProxy.h"
#include "Landscape.h"
#include "LandscapeRender.h"
#include "LandscapeSplineActor.h"
#include "Engine/Texture2D.h"
#include "EngineUtils.h"

#include "Chaos/PhysicalMaterials.h"

#if WITH_EDITOR
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#endif

#define ALLOW_LANDSCAPE_COLLISION_COMPONENT_DEBUG_DISPLAY !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

// Register the custom version with core
FCustomVersionRegistration GRegisterLandscapeCustomVersion(FLandscapeCustomVersion::GUID, FLandscapeCustomVersion::LatestVersion, TEXT("Landscape"));

class FLandscapeModule : public ILandscapeModule
{
public:
	/** IModuleInterface implementation */
	void StartupModule() override;
	void ShutdownModule() override;

	virtual TSharedPtr<FLandscapeSceneViewExtension, ESPMode::ThreadSafe> GetLandscapeSceneViewExtension() const override { return SceneViewExtension; }

	virtual void SetLandscapeEditorServices(ILandscapeEditorServices* InLandscapeEditorServices) override { LandscapeEditorServices = InLandscapeEditorServices; }
	virtual ILandscapeEditorServices* GetLandscapeEditorServices() const override { return LandscapeEditorServices; }
	virtual void SetCreateLandscapeComponentSceneProxyDelegate(const UE::Landscape::FCreateLandscapeComponentSceneProxyDelegate& InDelegate) override
	{
		// Setting new delegate without reseting previous one is not allowed.
		check(CreateLandscapeComponentSceneProxyDelegate.IsBound() != InDelegate.IsBound());
		CreateLandscapeComponentSceneProxyDelegate = InDelegate;
	}
	virtual const UE::Landscape::FCreateLandscapeComponentSceneProxyDelegate& GetCreateLandscapeComponentSceneProxyDelegate() const override { return CreateLandscapeComponentSceneProxyDelegate; }
	virtual void SetSetupLandscapePSOPrecacheVertexFactoryDataListDelegate(const UE::Landscape::FSetupLandscapePSOPrecacheVertexFactoryDataListDelegate& InDelegate) override
	{
		// Setting new delegate without reseting previous one is not allowed.
		check(SetupLandscapePSOPrecacheVertexFactoryDataListDelegate.IsBound() != InDelegate.IsBound());
		SetupLandscapePSOPrecacheVertexFactoryDataListDelegate = InDelegate;
	}
	virtual const UE::Landscape::FSetupLandscapePSOPrecacheVertexFactoryDataListDelegate& GetSetupLandscapePSOPrecacheVertexFactoryDataListDelegate() const override { return SetupLandscapePSOPrecacheVertexFactoryDataListDelegate; }

private:
	void OnPostEngineInit();
	void OnEnginePreExit();

#if ALLOW_LANDSCAPE_COLLISION_COMPONENT_DEBUG_DISPLAY
	void OnUpdatePhysicalMaterial(Chaos::FMaterialHandle InHandle);
#endif // ALLOW_LANDSCAPE_COLLISION_COMPONENT_DEBUG_DISPLAY

private:
	TSharedPtr<FLandscapeSceneViewExtension, ESPMode::ThreadSafe> SceneViewExtension;
	ILandscapeEditorServices* LandscapeEditorServices = nullptr;
	UE::Landscape::FCreateLandscapeComponentSceneProxyDelegate CreateLandscapeComponentSceneProxyDelegate;
	UE::Landscape::FSetupLandscapePSOPrecacheVertexFactoryDataListDelegate SetupLandscapePSOPrecacheVertexFactoryDataListDelegate;

	/** Event binding handles */
#if ALLOW_LANDSCAPE_COLLISION_COMPONENT_DEBUG_DISPLAY
	FDelegateHandle OnUpdatePhysicalMaterialHandle;
#endif // ALLOW_LANDSCAPE_COLLISION_COMPONENT_DEBUG_DISPLAY
};

/**
 * Add landscape-specific per-world data.
 *
 * @param World A pointer to world that this data should be created for.
 */
void AddPerWorldLandscapeData(UWorld* World)
{
	EObjectFlags NewLandscapeDataFlags = RF_NoFlags;
	if (!World->PerModuleDataObjects.FindItemByClass<ULandscapeInfoMap>())
	{
		if (World->HasAnyFlags(RF_Transactional))
		{
			NewLandscapeDataFlags = RF_Transactional;
		}
		ULandscapeInfoMap* InfoMap = NewObject<ULandscapeInfoMap>(GetTransientPackage(), NAME_None, NewLandscapeDataFlags);
		InfoMap->World = World;
		World->PerModuleDataObjects.Add(InfoMap);
	}
}

/**
 * Function that will fire every time a world is created.
 *
 * @param World A world that was created.
 * @param IVS Initialization values.
 */
void WorldCreationEventFunction(UWorld* World)
{
	AddPerWorldLandscapeData(World);
}

/**
 * Function that will fire every time a world is destroyed.
 *
 * @param World A world that's being destroyed.
 */
void WorldDestroyEventFunction(UWorld* World)
{
	World->PerModuleDataObjects.RemoveAll(
		[](UObject* Object)
		{
			return Object != nullptr && Object->IsA(ULandscapeInfoMap::StaticClass());
		}
	);
}

#if WITH_EDITOR
/**
 * Gets array of Landscape-specific textures and materials connected with given
 * level.
 *
 * @param Level Level to search textures and materials in.
 * @param OutTexturesAndMaterials (Output parameter) Array to fill.
 */
void GetLandscapeTexturesAndMaterials(ULevel* Level, TArray<UObject*>& OutTexturesAndMaterials)
{
	TArray<UObject*> ObjectsInLevel;
	GetObjectsWithOuter(Level, ObjectsInLevel, EGetObjectsFlags::IncludeNestedObjects);
	for (auto* ObjInLevel : ObjectsInLevel)
	{
		ULandscapeComponent* LandscapeComponent = Cast<ULandscapeComponent>(ObjInLevel);
		if (LandscapeComponent)
		{
			LandscapeComponent->GetGeneratedTexturesAndMaterialInstances(OutTexturesAndMaterials);
		}		
	}
}

/**
 * A function that fires everytime a world is renamed.
 *
 * @param World A world that was renamed.
 * @param InName New world name.
 * @param NewOuter New outer of the world after rename.
 * @param Flags Rename flags.
 * @param bShouldFailRename (Output parameter) If you set it to true, then the renaming process should fail.
 */
void WorldRenameEventFunction(UWorld* World, const TCHAR* InName, UObject* NewOuter, ERenameFlags Flags, bool& bShouldFailRename)
{
	// Also rename all textures and materials used by landscape components
	TArray<UObject*> LandscapeTexturesAndMaterials;
	GetLandscapeTexturesAndMaterials(World->PersistentLevel, LandscapeTexturesAndMaterials);
	UPackage* PersistentLevelPackage = World->PersistentLevel->GetOutermost();
	for (auto* OldTexOrMat : LandscapeTexturesAndMaterials)
	{
		// Now that landscape textures and materials are properly parented, this should not be necessary anymore
		if (OldTexOrMat && OldTexOrMat->GetOuter() == PersistentLevelPackage)
		{
			// The names for these objects are not important, just generate a new name to avoid collisions
			if (!OldTexOrMat->Rename(nullptr, NewOuter, Flags))
			{
				bShouldFailRename = true;
			}
		}
	}
}
#endif

/**
 * A function that fires everytime a world is duplicated.
 *
 * If there are some objects duplicated during this event fill out
 * ReplacementMap and ObjectsToFixReferences in order to properly fix
 * references in objects created during this duplication.
 *
 * @param World A world that was duplicated.
 * @param bDuplicateForPIE If this duplication was done for PIE.
 * @param ReplacementMap Replacement map (i.e. old object -> new object).
 * @param ObjectsToFixReferences Array of objects that may contain bad
 *		references to old objects.
 */
void WorldDuplicateEventFunction(UWorld* World, bool bDuplicateForPIE, TMap<UObject*, UObject*>& ReplacementMap, TArray<UObject*>& ObjectsToFixReferences)
{
	int32 Index;
	ULandscapeInfoMap* InfoMap;
	if (World->PerModuleDataObjects.FindItemByClass(&InfoMap, &Index))
	{
		ULandscapeInfoMap* NewInfoMap = Cast<ULandscapeInfoMap>( StaticDuplicateObject(InfoMap, InfoMap->GetOuter()) );
		NewInfoMap->World = World;

		World->PerModuleDataObjects[Index] = NewInfoMap;
	}
	else
	{
		AddPerWorldLandscapeData(World);
	}

#if WITH_EDITOR
	// Fixup LandscapeGuid on World duplication
	if (!bDuplicateForPIE && !IsRunningCommandlet())
	{
		TMap<FGuid, FGuid> NewLandscapeGuids;
		for (ALandscapeProxy* Proxy : TActorRange<ALandscapeProxy>(World, ALandscapeProxy::StaticClass(), EActorIteratorFlags::SkipPendingKill))
		{
			FGuid& NewGuid = NewLandscapeGuids.FindOrAdd(Proxy->GetLandscapeGuid(), FGuid::NewGuid());
			Proxy->SetLandscapeGuid(NewGuid);
		}

		for (ALandscapeSplineActor* SplineActor : TActorRange<ALandscapeSplineActor>(World, ALandscapeSplineActor::StaticClass(), EActorIteratorFlags::SkipPendingKill))
		{
			FGuid& NewGuid = NewLandscapeGuids.FindOrAdd(SplineActor->GetLandscapeGuid(), FGuid::NewGuid());
			SplineActor->SetLandscapeGuid(NewGuid);
		}
	}
#endif
}

void FLandscapeModule::StartupModule()
{
	FWorldDelegates::OnPostWorldCreation.AddStatic(
		&WorldCreationEventFunction
	);
	FWorldDelegates::OnPreWorldFinishDestroy.AddStatic(
		&WorldDestroyEventFunction
	);

#if WITH_EDITOR
	FWorldDelegates::OnPreWorldRename.AddStatic(
		&WorldRenameEventFunction
	);
#endif // WITH_EDITOR

	FWorldDelegates::OnPostDuplicate.AddStatic(
		&WorldDuplicateEventFunction
	);

	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FLandscapeModule::OnPostEngineInit);
	FCoreDelegates::OnEnginePreExit.AddRaw(this, &FLandscapeModule::OnEnginePreExit);

	// Bind material events
#if ALLOW_LANDSCAPE_COLLISION_COMPONENT_DEBUG_DISPLAY
	{
		Chaos::FPhysicalMaterialManager& MaterialManager = Chaos::FPhysicalMaterialManager::Get();
		OnUpdatePhysicalMaterialHandle = MaterialManager.OnMaterialUpdated.Add(Chaos::FMaterialUpdatedDelegate::CreateRaw(this, &FLandscapeModule::OnUpdatePhysicalMaterial));
	}
#endif // ALLOW_LANDSCAPE_COLLISION_COMPONENT_DEBUG_DISPLAY

#if WITH_EDITOR
	// Register LandscapeSplineActorDesc Deprecation
	FWorldPartitionActorDesc::RegisterActorDescDeprecator(ALandscapeSplineActor::StaticClass(), [](FArchive& Ar, FWorldPartitionActorDesc* ActorDesc)
	{
		check(Ar.IsLoading());
		if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::AddedLandscapeSplineActorDesc)
		{
			ActorDesc->AddProperty(ALandscape::AffectsLandscapeActorDescProperty);
		}
		else if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::LandscapeSplineActorDescDeprecation)
		{
			FGuid LandscapeGuid;
			Ar << LandscapeGuid;
			ActorDesc->AddProperty(ALandscape::AffectsLandscapeActorDescProperty, *LandscapeGuid.ToString());
		}
	});
#endif // WITH_EDITOR
}

void FLandscapeModule::OnPostEngineInit()
{
	check(!SceneViewExtension.IsValid());
	SceneViewExtension = FSceneViewExtensions::NewExtension<FLandscapeSceneViewExtension>();
}

void FLandscapeModule::OnEnginePreExit()
{
	check(SceneViewExtension.IsValid());
	SceneViewExtension.Reset();
}

void FLandscapeModule::ShutdownModule()
{
	// Unbind material events
#if ALLOW_LANDSCAPE_COLLISION_COMPONENT_DEBUG_DISPLAY
	{
		Chaos::FPhysicalMaterialManager& MaterialManager = Chaos::FPhysicalMaterialManager::Get();
		MaterialManager.OnMaterialUpdated.Remove(OnUpdatePhysicalMaterialHandle);
	}
#endif // ALLOW_LANDSCAPE_COLLISION_COMPONENT_DEBUG_DISPLAY

	FCoreDelegates::OnEnginePreExit.RemoveAll(this);
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
}

#if ALLOW_LANDSCAPE_COLLISION_COMPONENT_DEBUG_DISPLAY
void FLandscapeModule::OnUpdatePhysicalMaterial(Chaos::FMaterialHandle InHandle)
{
	ExecuteOnGameThread(UE_SOURCE_LOCATION, []
		{
			// Just rebuild all collision components' render states when the Chaos material changes, this is just for debug purposes anyway : 
			for (ULandscapeHeightfieldCollisionComponent* LandscapeHeightfieldCollisionComponent 
				: TObjectRange<ULandscapeHeightfieldCollisionComponent>(RF_ClassDefaultObject | RF_ArchetypeObject, /*bIncludeDerivedClasses = */true, EInternalObjectFlags::Garbage))
			{
				LandscapeHeightfieldCollisionComponent->MarkRenderStateDirty();
			}
		});
}
#endif // ALLOW_LANDSCAPE_COLLISION_COMPONENT_DEBUG_DISPLAY

IMPLEMENT_MODULE(FLandscapeModule, Landscape);

#undef ALLOW_LANDSCAPE_COLLISION_COMPONENT_DEBUG_DISPLAY