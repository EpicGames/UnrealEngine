// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeNotification.h"
#include "Application/SlateApplicationBase.h"
#include "LandscapeSubsystem.h"

#if WITH_EDITOR

#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Algo/Count.h"

#define LOCTEXT_NAMESPACE "Landscape"

// ----------------------------------------------------------------------------------

FLandscapeNotification::FLandscapeNotification(const TWeakObjectPtr<ALandscape>& InLandscape, ELandscapeNotificationType InNotificationType, FConditionCallback InConditionCallback, FUpdateTextCallback InUpdateTextCallback)
	: Landscape(InLandscape)
	, NotificationType(InNotificationType)
	, ConditionCallback(InConditionCallback)
	, UpdateTextCallback(InUpdateTextCallback)
{
	check(!IsRunningCommandlet());
}

bool FLandscapeNotification::operator == (const FLandscapeNotification& Other) const
{
	return (Landscape == Other.Landscape) && (NotificationType == Other.NotificationType);
}

bool FLandscapeNotification::operator < (const FLandscapeNotification& Other) const
{
	if (NotificationType == Other.NotificationType)
	{
		return (Landscape.Get() < Other.Landscape.Get());
	}
	return (NotificationType < Other.NotificationType);
}

bool FLandscapeNotification::ShouldShowNotification() const 
{
	return ConditionCallback() && FSlateApplicationBase::IsInitialized() && (FSlateApplicationBase::Get().GetCurrentTime() >= NotificationStartTime || NotificationStartTime < 0.0);
}

void FLandscapeNotification::SetNotificationText()
{
	UpdateTextCallback(NotificationText);
}

// ----------------------------------------------------------------------------------

FLandscapeNotificationManager::~FLandscapeNotificationManager()
{
	if (NotificationItem.IsValid())
	{
		HideNotificationItem();
	}
}

void FLandscapeNotificationManager::Tick()
{
	// Cleanup notifications for stale landscapes:
	LandscapeNotifications.RemoveAll([](const TWeakPtr<FLandscapeNotification>& Notification) 
		{ return !Notification.IsValid() || !Notification.Pin()->GetLandscape().IsValid(); });

	// Don't keep notifications that should not yet be displayed: 
	TArray<TWeakPtr<FLandscapeNotification>> ValidNotifications = LandscapeNotifications.FilterByPredicate([=](const TWeakPtr<FLandscapeNotification>& Notification) 
		{ return Notification.Pin()->ShouldShowNotification(); });
	
	if (ValidNotifications.IsEmpty())
	{
		HideNotificationItem();
	}
	else
	{
		// Sort by priority: 
		ValidNotifications.Sort([](const TWeakPtr<FLandscapeNotification>& LHS, const TWeakPtr<FLandscapeNotification>& RHS) { return (*LHS.Pin().Get() < *RHS.Pin().Get()); });

		// Only display the topmost priority:
		ActiveNotification = ValidNotifications[0];
		FLandscapeNotification* LocalActiveNotification = ActiveNotification.Pin().Get();
		check(LocalActiveNotification != nullptr);

		LocalActiveNotification->SetNotificationText();

		// There might be multiple notifications for this type:
		const size_t NumIdenticalNotifications = Algo::CountIf(ValidNotifications,
		                                                       [NotificationType = LocalActiveNotification->GetNotificationType()](
		                                                       const TWeakPtr<FLandscapeNotification>& Notification)
		                                                       {
			                                                       return (Notification.Pin().Get()->GetNotificationType() == NotificationType);
		                                                       });

		FText Text = FText::Format(LOCTEXT("NotificationFooter", "{0} for {1}"),
			LocalActiveNotification->NotificationText,
			(NumIdenticalNotifications > 1) ? LOCTEXT("MultipleLandscapes", "multiple landscapes") : FText::Format(LOCTEXT("NotificationLandscapeName", "landscape named: {0}"), FText::FromString(LocalActiveNotification->GetLandscape()->GetActorLabel())));

		ShowNotificationItem(Text);
	}
}

void FLandscapeNotificationManager::RegisterNotification(const TWeakPtr<FLandscapeNotification>& InNotification)
{
	LandscapeNotifications.AddUnique(InNotification);
}

void FLandscapeNotificationManager::UnregisterNotification(const TWeakPtr<FLandscapeNotification>& InNotification)
{
	LandscapeNotifications.Remove(InNotification);
}

void FLandscapeNotificationManager::ShowNotificationItem(const FText& InText)
{
	TSharedPtr<SNotificationItem> PinnedItem = NotificationItem.Pin();
	if (!PinnedItem.IsValid())
	{
		FNotificationInfo Info(InText);
		Info.bUseThrobber = true;
		Info.bFireAndForget = false;
		PinnedItem = FSlateNotificationManager::Get().AddNotification(Info);
		PinnedItem->SetFadeInDuration(0.0f);
		NotificationItem = PinnedItem;
	}
	else
	{
		PinnedItem->SetText(InText);
	}
	PinnedItem->SetCompletionState(SNotificationItem::ECompletionState::CS_Pending);
}

void FLandscapeNotificationManager::HideNotificationItem()
{
	TSharedPtr<SNotificationItem> PinnedItem = NotificationItem.Pin();
	if (PinnedItem.IsValid() && (PinnedItem->GetCompletionState() != SNotificationItem::ECompletionState::CS_Success))
	{
		PinnedItem->SetCompletionState(SNotificationItem::ECompletionState::CS_Success);
		PinnedItem->SetExpireDuration(1.0f);
		PinnedItem->ExpireAndFadeout();
	}
}

#undef LOCTEXT_NAMESPACE

#endif // WITH_EDITOR