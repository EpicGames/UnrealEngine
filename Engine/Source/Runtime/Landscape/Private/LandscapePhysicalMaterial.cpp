// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapePhysicalMaterial.h"

#if WITH_EDITOR

#include "EngineModule.h"
#include "LandscapeComponent.h"
#include "LandscapePrivate.h"
#include "LandscapeRender.h"
#include "MeshMaterialShader.h"
#include "MeshPassProcessor.h"
#include "MeshPassProcessor.inl"
#include "RenderCommandFence.h"
#include "RenderTargetPool.h"
#include "RHIResources.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "RHIStaticStates.h"
#include "SceneRenderTargetParameters.h"
#include "SimpleMeshDrawCommandPass.h"
#include "RHIGPUReadback.h"
#include "RenderGraphUtils.h"
#include "RenderCaptureInterface.h"
#include "SceneRendererInterface.h"

DECLARE_GPU_STAT_NAMED(LandscapePhysicalMaterial_Draw, TEXT("LandscapePhysicalMaterial"));

int32 RenderCaptureLayersNextPhysicalMaterialDraws = 0;
static FAutoConsoleVariableRef CVarRenderCaptureLayersNextPhysicalMaterialDraws(
	TEXT("landscape.RenderCaptureLayersNextPhysicalMaterialDraws"),
	RenderCaptureLayersNextPhysicalMaterialDraws,
	TEXT("Trigger N render captures during the next landscape physical material draw calls."),
	EConsoleVariableFlags::ECVF_RenderThreadSafe);

/** Material shader for rendering physical material IDs. */
class FLandscapePhysicalMaterial : public FMeshMaterialShader
{
public:
	FLandscapePhysicalMaterial()
	{}

	FLandscapePhysicalMaterial(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer)
		: FMeshMaterialShader(Initializer)
	{
		PassUniformBuffer.Bind(Initializer.ParameterMap, FSceneTextureUniformParameters::FTypeInfo::GetStructMetadata()->GetShaderVariableName());
	}

	static bool ShouldCompilePermutation(const FMeshMaterialShaderPermutationParameters& Parameters)
	{
		return (Parameters.MaterialParameters.bIsUsedWithLandscape || Parameters.MaterialParameters.bIsSpecialEngineMaterial) &&
			IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5) &&
			Parameters.VertexFactoryType == FindVertexFactoryType(FName(TEXT("FLandscapeFixedGridVertexFactory"), FNAME_Find)) &&
			!IsConsolePlatform(Parameters.Platform);
	}
};

class FLandscapePhysicalMaterialVS : public FLandscapePhysicalMaterial
{
	DECLARE_SHADER_TYPE(FLandscapePhysicalMaterialVS, MeshMaterial);

public:
	FLandscapePhysicalMaterialVS()
	{}

	FLandscapePhysicalMaterialVS(const FMeshMaterialShaderType::CompiledShaderInitializerType& Initializer)
		: FLandscapePhysicalMaterial(Initializer)
	{}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FLandscapePhysicalMaterialVS, TEXT("/Engine/Private/LandscapePhysicalMaterial.usf"), TEXT("VSMain"), SF_Vertex);

class FLandscapePhysicalMaterialPS : public FLandscapePhysicalMaterial
{
	DECLARE_SHADER_TYPE(FLandscapePhysicalMaterialPS, MeshMaterial);

public:
	FLandscapePhysicalMaterialPS()
	{}

	FLandscapePhysicalMaterialPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FLandscapePhysicalMaterial(Initializer)
	{}
};

IMPLEMENT_MATERIAL_SHADER_TYPE(, FLandscapePhysicalMaterialPS, TEXT("/Engine/Private/LandscapePhysicalMaterial.usf"), TEXT("PSMain"), SF_Pixel);

// Tries to retrieve the VS/PS for rendering landscape physical materials. If the shaders are not available yet (e.g. not compiled), it will return false and will request them to become available . 
//  This can be called repeatedly until the shaders become available :
bool TryGetLandscapePhysicalMaterialShaders(const FMaterial& InMaterialResource, FMaterialShaders& OutShaders)
{
	FMaterialShaderTypes ShaderTypes;
	ShaderTypes.AddShaderType<FLandscapePhysicalMaterialVS>();
	ShaderTypes.AddShaderType<FLandscapePhysicalMaterialPS>();
	
	FVertexFactoryType* VertexFactoryType = FindVertexFactoryType(FName(TEXT("FLandscapeFixedGridVertexFactory"), FNAME_Find));
	check(VertexFactoryType != nullptr);
	return InMaterialResource.TryGetShaders(ShaderTypes, VertexFactoryType, OutShaders);
}


/** Simple mesh processor implementation to draw using the FLandscapePhysicalMaterial mesh material shader. */
class FLandscapePhysicalMaterialMeshProcessor : public FMeshPassProcessor
{
public:
	FLandscapePhysicalMaterialMeshProcessor(
		const FScene* Scene, 
		const FSceneView* InViewIfDynamicMeshCommand, 
		FMeshPassDrawListContext* InDrawListContext);

	virtual void AddMeshBatch(
		const FMeshBatch& RESTRICT MeshBatch,
		uint64 BatchElementMask,
		const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
		int32 StaticMeshId = -1) override final;

private:
	FMeshPassProcessorRenderState PassDrawRenderState;
};

FLandscapePhysicalMaterialMeshProcessor::FLandscapePhysicalMaterialMeshProcessor(
	const FScene* Scene, 
	const FSceneView* InViewIfDynamicMeshCommand, 
	FMeshPassDrawListContext* InDrawListContext)
	: FMeshPassProcessor(EMeshPass::Num, Scene, InViewIfDynamicMeshCommand->GetFeatureLevel(), InViewIfDynamicMeshCommand, InDrawListContext)
{
	PassDrawRenderState.SetBlendState(TStaticBlendState<>::GetRHI());
	PassDrawRenderState.SetDepthStencilState(TStaticDepthStencilState<false, CF_Always>::GetRHI());
}

void FLandscapePhysicalMaterialMeshProcessor::AddMeshBatch(
	const FMeshBatch& RESTRICT MeshBatch,
	uint64 BatchElementMask,
	const FPrimitiveSceneProxy* RESTRICT PrimitiveSceneProxy,
	int32 StaticMeshId)
{
	const FMaterialRenderProxy* MaterialRenderProxy = MeshBatch.MaterialRenderProxy;
		const FMaterial* Material = MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
	checkf(Material != nullptr, TEXT("InitTaskRenderResources should ensure that by the time we render the physical materials, the shaders are valid and ready."));

	FMaterialShaders Shaders;
	verify(TryGetLandscapePhysicalMaterialShaders(*Material, Shaders));

	TMeshProcessorShaders<
		FLandscapePhysicalMaterialVS,
		FLandscapePhysicalMaterialPS> PassShaders;
	Shaders.TryGetVertexShader(PassShaders.VertexShader);
	Shaders.TryGetPixelShader(PassShaders.PixelShader);

	const FMeshDrawingPolicyOverrideSettings OverrideSettings = ComputeMeshOverrideSettings(MeshBatch);
	const ERasterizerFillMode MeshFillMode = ComputeMeshFillMode(*Material, OverrideSettings);
	const ERasterizerCullMode MeshCullMode = CM_None;

	FMeshMaterialShaderElementData ShaderElementData;
	ShaderElementData.InitializeMeshMaterialData(ViewIfDynamicMeshCommand, PrimitiveSceneProxy, MeshBatch, -1, true);

	const FMeshDrawCommandSortKey SortKey = CalculateMeshStaticSortKey(PassShaders.VertexShader, PassShaders.PixelShader);

	BuildMeshDrawCommands(
		MeshBatch,
		BatchElementMask,
		PrimitiveSceneProxy,
		*MaterialRenderProxy,
		*Material,
		PassDrawRenderState,
		PassShaders,
		MeshFillMode,
		MeshCullMode,
		SortKey,
		EMeshPassFeatures::Default,
		ShaderElementData);
}

namespace
{
	/** A description of a mesh to render. */
	struct FMeshInfo
	{
		FPrimitiveSceneProxy const* Proxy;
		FMeshBatch const* MeshBatch;
		uint64 MeshBatchElementMask;
	};
	typedef TArray< FMeshInfo, TInlineAllocator<1> > FMeshInfoArray;

	/**
	 * Collect the meshes to render for a landscape component.
	 * Initially we only collect the base landscape mesh, but potentially we could gather other objects like roads?
	 * WARNING: This gets the SceneProxy pointer from the UComponent on the render thread. This doesn't feel safe but it's what the grass renderer does...
	 */
	void FillMeshInfos_RenderThread(const FLandscapeComponentSceneProxy* InSceneProxy, FMeshInfoArray& OutMeshInfos)
	{
		const int32 MeshInfoIndex = OutMeshInfos.AddUninitialized();
		OutMeshInfos[MeshInfoIndex].Proxy = InSceneProxy;
		OutMeshInfos[MeshInfoIndex].MeshBatch = &(InSceneProxy->GetGrassMeshBatch());
		OutMeshInfos[MeshInfoIndex].MeshBatchElementMask = 1 << 0; // LOD 0 only
	}
	
	BEGIN_SHADER_PARAMETER_STRUCT(FLandscapePhysicalMaterialPassParameters, )
		SHADER_PARAMETER_STRUCT_REF(FViewUniformShaderParameters, View)
		SHADER_PARAMETER_RDG_UNIFORM_BUFFER(FSceneUniformParameters, Scene)
		SHADER_PARAMETER_STRUCT_INCLUDE(FInstanceCullingDrawParams, InstanceCullingDrawParams)
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()

	/** Render the landscape physical material IDs and copy to the read back texture. */
	void Render_RenderThread(
		FRHICommandListImmediate& RHICmdList,
		FSceneInterface* SceneInterface,
		FMeshInfoArray const& MeshInfos,
		FIntPoint TargetSize,
		FVector ViewOrigin,
		FMatrix ViewRotationMatrix,
		FMatrix ProjectionMatrix,
		FRHIGPUTextureReadback* Readback)
	{
		FRDGBuilder GraphBuilder(RHICmdList);

		{
			RenderCaptureInterface::FScopedCapture RenderCapture((RenderCaptureLayersNextPhysicalMaterialDraws != 0), GraphBuilder, TEXT("LandscapePhysicalMaterialCapture"));
			RenderCaptureLayersNextPhysicalMaterialDraws = FMath::Max(0, RenderCaptureLayersNextPhysicalMaterialDraws - 1);

		// Create the view
		FSceneViewFamily::ConstructionValues ViewFamilyInit(nullptr, SceneInterface, FEngineShowFlags(ESFIM_Game));
		ViewFamilyInit.SetTime(FGameTime());
		FSceneViewFamilyContext ViewFamily(ViewFamilyInit);
		ViewFamily.LandscapeLODOverride = 0; // Force LOD 0 render

		FScenePrimitiveRenderingContextScopeHelper ScenePrimitiveRenderingContextScopeHelper(GetRendererModule().BeginScenePrimitiveRendering(GraphBuilder, &ViewFamily));

		FSceneViewInitOptions ViewInitOptions;
		ViewInitOptions.SetViewRectangle(FIntRect(0, 0, TargetSize.X, TargetSize.Y));
		ViewInitOptions.ViewOrigin = ViewOrigin;
		ViewInitOptions.ViewRotationMatrix = ViewRotationMatrix;
		ViewInitOptions.ProjectionMatrix = ProjectionMatrix;
		ViewInitOptions.ViewFamily = &ViewFamily;

		GetRendererModule().CreateAndInitSingleView(RHICmdList, &ViewFamily, &ViewInitOptions);
		const FSceneView* View = ViewFamily.Views[0];

		FRDGTextureRef OutputTexture = GraphBuilder.CreateTexture(
			FRDGTextureDesc::Create2D(TargetSize, PF_G8, FClearValueBinding::Black, TexCreate_RenderTargetable | TexCreate_ShaderResource),
			TEXT("LandscapePhysicalMaterialTarget"));

		auto* PassParameters = GraphBuilder.AllocParameters<FLandscapePhysicalMaterialPassParameters>();
		PassParameters->View = View->ViewUniformBuffer;
		PassParameters->Scene = GetSceneUniformBufferRef(GraphBuilder, *View);
		PassParameters->RenderTargets[0] = FRenderTargetBinding(OutputTexture, ERenderTargetLoadAction::EClear);

		AddSimpleMeshPass(GraphBuilder, PassParameters, SceneInterface->GetRenderScene(), *View, nullptr, RDG_EVENT_NAME("LandscapePhysicalMaterial"), View->UnscaledViewRect,
			[View, &MeshInfos](FDynamicPassMeshDrawListContext* DynamicMeshPassContext)
			{
				FLandscapePhysicalMaterialMeshProcessor PassMeshProcessor(
					nullptr,
					View,
					DynamicMeshPassContext);
				for (auto& MeshInfo : MeshInfos)
				{
					const FMeshBatch& Mesh = *MeshInfo.MeshBatch;
					if (Mesh.MaterialRenderProxy != nullptr)
					{
						Mesh.MaterialRenderProxy->UpdateUniformExpressionCacheIfNeeded(View->GetFeatureLevel());
						PassMeshProcessor.AddMeshBatch(Mesh, MeshInfo.MeshBatchElementMask, MeshInfo.Proxy);
					}
				}
			});

		AddEnqueueCopyPass(GraphBuilder, Readback, OutputTexture);
		}

		GraphBuilder.Execute();
	}

	/** Fetch the landscape physical material IDs from a read back texture. */
	void FetchResults_RenderThread(
		FRHICommandListImmediate& RHICmdList,
		FIntPoint TargetSize,
		FRHIGPUTextureReadback* Readback,
		TArray<uint8>& OutPhysicalMaterialIds)
	{
		int32 Pitch = 0;
		void* Data = Readback->Lock(Pitch);
		check(Data && TargetSize.X <= Pitch);

		OutPhysicalMaterialIds.Empty(TargetSize.X * TargetSize.Y);
		OutPhysicalMaterialIds.AddUninitialized(TargetSize.X * TargetSize.Y);

		uint8 const* ReadPtr = (uint8*)Data;
		uint8* WritePtr = OutPhysicalMaterialIds.GetData();
		for (int32 Y = 0; Y < TargetSize.Y; ++Y)
		{
			FMemory::Memcpy(WritePtr, ReadPtr, TargetSize.X);
			ReadPtr += Pitch;
			WritePtr += TargetSize.X;
		}

		Readback->Unlock();
	}
}


namespace
{
	/** Completion state for a physical material render task. */
	enum class ECompletionState : uint64
	{
		None = 0,					// Task initialized
		WaitForRenderResources = 1,	// Waiting for shaders to compile
		Pending = 2,				// Draw submitted, waiting for GPU
		Complete = 3				// Result copied back from GPU
	};

	/** Data for a physical material render task. */
	struct FLandscapePhysicalMaterialRenderTaskImpl
	{
		// Create on game thread
		ULandscapeComponent const* LandscapeComponent = nullptr; // !Do not access on render thread!
		uint32 InitFrameId = 0;
		FIntPoint TargetSize = FIntPoint(ForceInitToZero);
		FVector ViewOrigin = FVector(ForceInitToZero);
		FMatrix ViewRotationMatrix = FMatrix(ForceInitToZero);
		FMatrix ProjectionMatrix = FMatrix(ForceInitToZero);

		// Result written on game thread and read on game thread
		TArray<UPhysicalMaterial*> ResultMaterials;

		// Create on render thread
		TUniquePtr<FRHIGPUTextureReadback> Readback;

		// Result written on render thread and read on game thread
		ECompletionState CompletionState = ECompletionState::None;
		TArray<uint8> ResultIds;
	};

	/** Initialize the physical material render task data. Return false if there are no physical materials needing to be rendered by the graphical material. */
	bool InitTask(FLandscapePhysicalMaterialRenderTaskImpl& Task, ULandscapeComponent const* InLandscapeComponent, uint32 InFrameId)
	{
		if (InLandscapeComponent != nullptr)
		{
			if (InLandscapeComponent->GetRenderPhysicalMaterials(Task.ResultMaterials))
			{
				Task.LandscapeComponent = InLandscapeComponent;
				Task.InitFrameId = InFrameId;
				Task.CompletionState = ECompletionState::None;

				const FTransform& ComponentTransform = InLandscapeComponent->GetComponentTransform();

				// request physical material render at the desired collision resolution
				const int32 CollisionSubsectionVerts = ((InLandscapeComponent->SubsectionSizeQuads + 1) >> InLandscapeComponent->CollisionMipLevel);
				const int32 CollisionSubsectionQuads = CollisionSubsectionVerts - 1;
				const int32 CollisionComponentVerts = CollisionSubsectionQuads * InLandscapeComponent->NumSubsections + 1;

				const FIntPoint TargetSize(CollisionComponentVerts, CollisionComponentVerts);
				const FIntPoint TargetSizeMinusOne(TargetSize - FIntPoint(1, 1));
				const FVector TargetCenter = ComponentTransform.TransformPosition(FVector(TargetSizeMinusOne, 0.f) * 0.5f);
				const FVector TargetExtent = FVector(TargetSize, 0.0f) * ComponentTransform.GetScale3D() * 0.5f;
				const FMatrix ViewRotationMatrix = FInverseRotationMatrix(ComponentTransform.Rotator()) * FMatrix(FPlane(1, 0, 0, 0), FPlane(0, -1, 0, 0), FPlane(0, 0, -1, 0), FPlane(0, 0, 0, 1));
				const FMatrix::FReal ZOffset = UE_OLD_WORLD_MAX;
				const FMatrix ProjectionMatrix = FReversedZOrthoMatrix(TargetExtent.X, TargetExtent.Y, 0.5f / ZOffset, ZOffset);

				if (Task.TargetSize != TargetSize)
				{
					Task.Readback = nullptr;
				}

				Task.TargetSize = TargetSize;
				Task.ViewOrigin = TargetCenter;
				Task.ViewRotationMatrix = ViewRotationMatrix;
				Task.ProjectionMatrix = ProjectionMatrix;

				return true;
			}
		}

		return false;
	}

	bool InitTaskRenderResources(FLandscapePhysicalMaterialRenderTaskImpl& InTask, const FLandscapeComponentSceneProxy* InSceneProxy)
	{
	/** Initialize the physical material render task read back resources. */
		//todo: Consider pooling these and throttling to the pool size?
		if (!InTask.Readback.IsValid())
		{
			InTask.Readback = MakeUnique<FRHIGPUTextureReadback>(TEXT("LandscapePhysicalMaterialReadback"));
		}

		// Make sure the landscape physical material shader is compiled and if not, initiate compilation : 
		const FMeshBatch& GrassMeshBatch = InSceneProxy->GetGrassMeshBatch();
		check(GrassMeshBatch.MaterialRenderProxy != nullptr);

		ERHIFeatureLevel::Type FeatureLevel = InSceneProxy->GetScene().GetFeatureLevel();
		const FMaterial* Material = GrassMeshBatch.MaterialRenderProxy->GetMaterialNoFallback(FeatureLevel);
		if (Material == nullptr)
		{
			return false;
		}

		// Only return true when the shaders needed for rendering landscape physical materials are ready : 
		FMaterialShaders Shaders;
		return TryGetLandscapePhysicalMaterialShaders(*Material, Shaders);
	}

	/** Update the physical material render task on the render thread. */
	void UpdateTask_RenderThread(FRHICommandListImmediate& RHICmdList, FLandscapePhysicalMaterialRenderTaskImpl& Task, const FLandscapeComponentSceneProxy* InSceneProxy, bool bFlush, uint32 FrameCount)
	{
		if (Task.CompletionState <= ECompletionState::WaitForRenderResources && InSceneProxy)
		{
			// Make sure everything is ready to render, if not (e.g. shader not yet compiled), wait until it is:
			if (!InitTaskRenderResources(Task, InSceneProxy))
			{
			    // resources not available (and may not be for many frames)
				// reset the init frame, to delay garbage collection
				Task.InitFrameId = FrameCount;
				FPlatformMisc::MemoryBarrier();
				Task.CompletionState = ECompletionState::WaitForRenderResources;
			}
			else
			{
				// Render the pending item.
				FMeshInfoArray MeshInfos;
				FillMeshInfos_RenderThread(InSceneProxy, MeshInfos);

				Render_RenderThread(
					RHICmdList,
					&InSceneProxy->GetScene(),
					MeshInfos,
					Task.TargetSize,
					Task.ViewOrigin,
					Task.ViewRotationMatrix,
					Task.ProjectionMatrix,
					Task.Readback.Get());

				FPlatformMisc::MemoryBarrier();
				Task.CompletionState = ECompletionState::Pending;
			}
		}
		else if (Task.CompletionState == ECompletionState::Pending)
		{
			if (bFlush || Task.Readback->IsReady())
			{
				if (!Task.Readback->IsReady())
				{
					RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
				}

				FetchResults_RenderThread(RHICmdList, Task.TargetSize, Task.Readback.Get(), Task.ResultIds);

				FPlatformMisc::MemoryBarrier();
				Task.CompletionState = ECompletionState::Complete;
			}
		}
	}
}


/**
 * Pool for storing physical material render task data.
 */
class FLandscapePhysicalMaterialRenderTaskPool : public FRenderResource
{
public:
	/** Pool uses chunked array to avoid task data being moved by a realloc. */
	TChunkedArray< FLandscapePhysicalMaterialRenderTaskImpl > Pool;

	/** Frame count used to validate and garbage collect. */
	uint32 FrameCount = 0;

	/** Allocate task data from the pool. Returns false if no physical material needs to be rendered. */
	bool Allocate(FLandscapePhysicalMaterialRenderTask& InTask, ULandscapeComponent const* InLandscapeComponent)
	{
		check(InTask.PoolHandle == -1);

		// search the pool to find an existing free entry
		int32 Index = 0;
		auto ItEnd = Pool.end();
		for (auto It = Pool.begin(); It != ItEnd; ++It, ++Index)
		{
			if ((*It).LandscapeComponent == nullptr)
			{
				break;
			}
		}

		// if none found, create a new entry in the pool
		if (Index == Pool.Num())
		{
			Pool.Add();
		}

		const bool bSuccess = InitTask(Pool[Index], InLandscapeComponent, FrameCount);
		InTask.PoolHandle = bSuccess ? Index : -1;

		return bSuccess;
	}

	/** Return task data to the pool. */
	void Free(FLandscapePhysicalMaterialRenderTask& InTask)
	{
		check(InTask.PoolHandle != -1);

		FLandscapePhysicalMaterialRenderTaskImpl* Task = &Pool[InTask.PoolHandle];

		// Invalidate the task object.
		InTask.PoolHandle = -1;

		// Submit render thread command to mark pooled task as free.
		ENQUEUE_RENDER_COMMAND(FLandscapePhysicalMaterialClear)(
			[Task](FRHICommandListImmediate& RHICmdList)
			{
				Task->LandscapeComponent = nullptr;
			});
	}

	/** Free render resources that have been unused for long enough. */
	void GarbageCollect()
	{
		const uint32 PoolSize = Pool.Num();
		if (PoolSize > 0)
		{
			// Garbage collect a maximum of one item per call to reduce overhead if pool has grown large.
			FLandscapePhysicalMaterialRenderTaskImpl* Task = &Pool[FrameCount % PoolSize];

			// Only garbage collect tasks that are at least 100 frames old (allows resources to be re-used for some time)
			if (Task->InitFrameId + 100 < FrameCount)
			{
				if (Task->LandscapeComponent != nullptr)
				{
					// Task not completed after 100 updates. We are probably leaking tasks!
					UE_LOGF(LogLandscape, Warning, "Leaking landscape physical material tasks.")
				}
				else
				{
					// Free the array allocations
					Task->ResultMaterials.Empty();
					Task->ResultIds.Empty();

					// Free the render resources (which may already be free)
					ENQUEUE_RENDER_COMMAND(FLandscapePhysicalMaterialFree)(
						[Task](FRHICommandListImmediate& RHICmdList)
						{
							Task->Readback = nullptr;
						});
				}
			}
		}

		FrameCount++;
	}

	void ReleaseRHI() override
	{
		Pool.Empty();
	}
};

/** Static global pool object. */
static TGlobalResource< FLandscapePhysicalMaterialRenderTaskPool > GTaskPool;


bool FLandscapePhysicalMaterialRenderTask::Init(ULandscapeComponent const* LandscapeComponent, uint32 InHash)
{
	check(IsInGameThread());
	if (IsValid())
	{
		GTaskPool.Free(*this);
	}
	Hash = InHash;
	return GTaskPool.Allocate(*this, LandscapeComponent);
}

void FLandscapePhysicalMaterialRenderTask::Release()
{
	check(IsInGameThread());
	if (IsValid())
	{
		GTaskPool.Free(*this);
		Hash = 0;
	}
}

bool FLandscapePhysicalMaterialRenderTask::IsValid() const
{
	check(IsInParallelGameThread() || IsInGameThread());
	return PoolHandle != -1;
}

bool FLandscapePhysicalMaterialRenderTask::IsComplete() const
{
	check(IsInGameThread());
	check(IsValid());
	return GTaskPool.Pool[PoolHandle].CompletionState == ECompletionState::Complete;
}

bool FLandscapePhysicalMaterialRenderTask::IsInProgress() const
{
	return IsValid() && !IsComplete();
}

void FLandscapePhysicalMaterialRenderTask::Tick()
{
	UpdateInternal(/*bInFlush = */false);
}

// Note: We could add a global function that calls Flush() on multiple tasks. 
// That could enqueue a single render thread function and use a single rendering command flush.
// It could be useful if we see performance issue with any path that calls Flush individually for each landscape component.
void FLandscapePhysicalMaterialRenderTask::Flush()
{
	UpdateInternal(/*bInFlush = */true);
}

void FLandscapePhysicalMaterialRenderTask::UpdateInternal(bool bInFlush)
{
	check(IsInGameThread());
	if (IsValid() && !IsComplete())
	{
		uint32 FrameCount = GTaskPool.FrameCount;

		FLandscapePhysicalMaterialRenderTaskImpl* Task = &GTaskPool.Pool[PoolHandle];
		check(Task->LandscapeComponent != nullptr);

		UE::RenderCommandPipe::FSyncScope SyncScope;

		// If the material gets recompiled, for example, the component's scene proxy could be changed before the RT update runs, so we need to make sure we're getting the scene proxy on the game thread :
		const FLandscapeComponentSceneProxy* SceneProxy = static_cast<const FLandscapeComponentSceneProxy*>(Task->LandscapeComponent->SceneProxy);
		ENQUEUE_RENDER_COMMAND(FLandscapePhysicalMaterialFlush)(
			[Task, SceneProxy, bInFlush, FrameCount](FRHICommandListImmediate& RHICmdList)
			{
				UpdateTask_RenderThread(RHICmdList, *Task, SceneProxy, bInFlush, FrameCount);
			});

		if (bInFlush)
		{
			FlushRenderingCommands();
		}
	}
}

TArray<uint8> const& FLandscapePhysicalMaterialRenderTask::GetResultIds() const
{
	check(IsInGameThread());
	check(IsValid() && IsComplete());
	return GTaskPool.Pool[PoolHandle].ResultIds;
}

TArray<UPhysicalMaterial*> const& FLandscapePhysicalMaterialRenderTask::GetResultMaterials() const
{
	check(IsInGameThread());
	check(IsValid() && IsComplete());
	return GTaskPool.Pool[PoolHandle].ResultMaterials;
}


namespace LandscapePhysicalMaterial
{
	void GarbageCollectTasks()
	{
		check(IsInGameThread());
		GTaskPool.GarbageCollect();
	}
}

#endif // WITH_EDITOR
