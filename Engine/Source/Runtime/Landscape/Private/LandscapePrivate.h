// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"


LANDSCAPE_API DECLARE_LOG_CATEGORY_EXTERN(LogLandscape, Log, All);
LANDSCAPE_API DECLARE_LOG_CATEGORY_EXTERN(LogLandscapeBP, Display, All);
LANDSCAPE_API DECLARE_LOG_CATEGORY_EXTERN(LogGrass, Log, All);

#define GRASS_DEBUG_LOG(...) UE_LOG(LogGrass, Verbose, __VA_ARGS__)

/**
 * Landscape stats
 */
DECLARE_CYCLE_STAT_EXTERN(TEXT("Dynamic Draw Time"), STAT_LandscapeDynamicDrawTime, STATGROUP_Landscape, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Render SetMesh Draw Time VS"), STAT_LandscapeVFDrawTimeVS, STATGROUP_Landscape, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("Render SetMesh Draw Time PS"), STAT_LandscapeVFDrawTimePS, STATGROUP_Landscape, );

DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Regenerate Layers DrawCalls"), STAT_LandscapeLayersRegenerateDrawCalls, STATGROUP_Landscape, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Processed Triangles"), STAT_LandscapeTriangles, STATGROUP_Landscape, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Render Passes"), STAT_LandscapeComponentRenderPasses, STATGROUP_Landscape, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("DrawCalls"), STAT_LandscapeDrawCalls, STATGROUP_Landscape, );
