// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeProxy.h"
#include "LandscapeInfo.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialExpressionLandscapeVisibilityMask.h"
#include "Materials/MaterialExpressionLandscapeLayerWeight.h"
#include "Materials/MaterialExpressionLandscapeLayerSample.h"
#include "Materials/MaterialExpressionLandscapeLayerBlend.h"
#include "Materials/MaterialExpressionLandscapeLayerSwitch.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapeProxy)

#if WITH_EDITOR


// ----------------------------------------------------------------------------------

FLandscapeImportLayerInfo::FLandscapeImportLayerInfo(const FLandscapeInfoLayerSettings& InLayerSettings)
	: LayerName(InLayerSettings.GetLayerName())
	, LayerInfo(InLayerSettings.LayerInfoObj)
	, SourceFilePath(InLayerSettings.GetTargetLayerSettings().ReimportLayerFilePath)
{
}

#endif // WITH_EDITOR

void ALandscapeProxy::SetPerLODOverrideMaterials(const TArray<FLandscapePerLODMaterialOverride>& InValue)
{
	PerLODOverrideMaterials = InValue;
}

uint32 ALandscapeProxy::ComputeLandscapeKey() const
{
	return ComputeLandscapeKey(GetWorld(), LODGroupKey, LandscapeGuid);
}

uint32 ALandscapeProxy::ComputeLandscapeKey(const UWorld* InWorld, uint32 InLODGroupKey, FGuid InLandscapeGuid)
{
	// use LODGroupKey instead of LandscapeGUID when LODGroupKey is non-zero
	return HashCombine(GetTypeHash(InWorld), (InLODGroupKey != 0) ? InLODGroupKey : GetTypeHash(InLandscapeGuid));
}
