// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
LandscapeRender.cpp: New terrain rendering
=============================================================================*/

#include "LandscapeRender.h"
#include "LightMap.h"
#include "ShadowMap.h"
#include "LandscapeLayerInfoObject.h"
#include "LandscapePrivate.h"
#include "LandscapeMeshProxyComponent.h"
#include "LandscapeNaniteComponent.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpressionTextureCoordinate.h"
#include "Materials/MaterialExpressionLandscapeLayerCoords.h"
#include "Materials/MaterialExpressionLandscapeVisibilityMask.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInstanceConstant.h"
#include "MeshDrawShaderBindings.h"
#include "ShaderParameterUtils.h"
#include "LandscapeEdit.h"
#include "Engine/Level.h"
#include "Materials/MaterialExpressionTextureSample.h"
#include "LandscapeMaterialInstanceConstant.h"
#include "Engine/ShadowMapTexture2D.h"
#include "EngineGlobals.h"
#include "EngineModule.h"
#include "UnrealEngine.h"
#include "LandscapeLight.h"
#include "Algo/Find.h"
#include "Engine/StaticMesh.h"
#include "LandscapeInfo.h"
#include "LandscapeDataAccess.h"
#include "DrawDebugHelpers.h"
#include "RHIStaticStates.h"
#include "PrimitiveSceneInfo.h"
#include "SceneView.h"
#include "LandscapeProxy.h"
#include "HAL/LowLevelMemTracker.h"
#include "MeshMaterialShader.h"
#include "VT/RuntimeVirtualTexture.h"
#include "RayTracingInstance.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "StaticMeshResources.h"
#include "TextureResource.h"
#include "NaniteSceneProxy.h"
#include "Rendering/Texture2DResource.h"
#include "RenderCore.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Algo/Transform.h"
#include "LandscapeCulling.h"
#include "RenderGraphBuilder.h"
#include "Scalability.h"
#include "Rendering/CustomRenderPass.h"
#include "LandscapeUtils.h"
#include "LandscapeUtilsPrivate.h"
#include "SceneRendererInterface.h"
#include "PSOPrecacheSettings.h"

#if WITH_EDITOR
#include "LandscapeEditResourcesSubsystem.h"
#endif

using namespace UE::Landscape;

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FLandscapeUniformShaderParameters, "LandscapeParameters");
IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FLandscapeFixedGridUniformShaderParameters, "LandscapeFixedGrid");
IMPLEMENT_TYPE_LAYOUT(FLandscapeVertexFactoryVertexShaderParameters);
IMPLEMENT_TYPE_LAYOUT(FLandscapeVertexFactoryPixelShaderParameters);

static void OnCVarNeedingRenderStateInvalidationChanged(IConsoleVariable* CVar)
{
	for (ULandscapeComponent* LandscapeComponent : TObjectRange<ULandscapeComponent>(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage))
	{
		LandscapeComponent->MarkRenderStateDirty();
	}
}

static void InvalidateRenderStateForLandscapeNaniteComponents(IConsoleVariable* CVar)
{
	for (ULandscapeNaniteComponent* LandscapeComponent : TObjectRange<ULandscapeNaniteComponent>(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage))
	{
		LandscapeComponent->MarkRenderStateDirty();
	}
}

#if !UE_BUILD_SHIPPING
float GLandscapeLOD0ScreenSizeOverride = -1.f;
FAutoConsoleVariableRef CVarLandscapeLOD0ScreenSizeOverride(
	TEXT("landscape.OverrideLOD0ScreenSize"),
	GLandscapeLOD0ScreenSizeOverride,
	TEXT("When > 0, force override the landscape LOD0ScreenSize property on all landscapes"),
	FConsoleVariableDelegate::CreateStatic(&OnCVarNeedingRenderStateInvalidationChanged),
	ECVF_Cheat
);

float GLandscapeLOD0DistributionOverride = -1.f;
FAutoConsoleVariableRef CVarLandscapeLOD0DistributionOverride(
	TEXT("landscape.OverrideLOD0Distribution"),
	GLandscapeLOD0DistributionOverride,
	TEXT("When > 0, force override the LOD0DistributionSetting property on all landscapes, and ignore r.LandscapeLOD0DistributionScale"),
	FConsoleVariableDelegate::CreateStatic(&OnCVarNeedingRenderStateInvalidationChanged),
	ECVF_Cheat
);

float GLandscapeLODDistributionOverride = -1.f;
FAutoConsoleVariableRef CVarLandscapeLODDistributionOverride(
	TEXT("landscape.OverrideLODDistribution"),
	GLandscapeLODDistributionOverride,
	TEXT("When > 0, force override the landscape LODDistributionSetting property on all landscapes, and ignore r.LandscapeLODDistributionScale"),
	FConsoleVariableDelegate::CreateStatic(&OnCVarNeedingRenderStateInvalidationChanged),
	ECVF_Cheat
);

float GLandscapeLODBlendRangeOverride = -1.f;
FAutoConsoleVariableRef CVarLandscapeLODBlendRangeOverride(
	TEXT("landscape.OverrideLODBlendRange"),
	GLandscapeLODBlendRangeOverride,
	TEXT("When > 0, force the LODBlendRange property on all landscapes"),
	FConsoleVariableDelegate::CreateStatic(&OnCVarNeedingRenderStateInvalidationChanged),
	ECVF_Cheat
);

float GLandscapeNonNaniteVirtualShadowMapConstantDepthBiasOverride = -1.f;
FAutoConsoleVariableRef CVarLandscapeNonNaniteVirtualShadowMapConstantDepthBiasOverrideOverride(
	TEXT("landscape.OverrideNonNaniteVirtualShadowMapConstantDepthBiasOverride"),
	GLandscapeNonNaniteVirtualShadowMapConstantDepthBiasOverride,
	TEXT("When > 0, force override the landscape NonNaniteVirtualShadowMapConstantDepthBias property on all landscapes"),
	FConsoleVariableDelegate::CreateStatic(&OnCVarNeedingRenderStateInvalidationChanged),
	ECVF_Cheat
);

float GLandscapeNonNaniteVirtualShadowMapInvalidationHeightErrorThresholdOverride = -1.f;
FAutoConsoleVariableRef CVarLandscapeNonNaniteVirtualShadowMapInvalidationHeightErrorThresholdOverride(
	TEXT("landscape.OverrideNonNaniteVirtualShadowMapInvalidationHeightErrorThreshold"),
	GLandscapeNonNaniteVirtualShadowMapInvalidationHeightErrorThresholdOverride,
	TEXT("When > 0, force override the landscape NonNaniteVirtualShadowMapInvalidationHeightErrorThreshold property on all landscapes"),
	FConsoleVariableDelegate::CreateStatic(&OnCVarNeedingRenderStateInvalidationChanged),
	ECVF_Cheat
);

float GLandscapeNonNaniteVirtualShadowMapInvalidationScreenSizeLimitOverride = -1.f;
FAutoConsoleVariableRef CVarLandscapeNonNaniteVirtualShadowMapInvalidationScreenSizeLimitOverride(
	TEXT("landscape.OverrideNonNaniteVirtualShadowMapInvalidationScreenSizeLimit"),
	GLandscapeNonNaniteVirtualShadowMapInvalidationScreenSizeLimitOverride,
	TEXT("When > 0, force override the landscape NonNaniteVirtualShadowMapInvalidationScreenSizeLimit property on all landscapes"),
	FConsoleVariableDelegate::CreateStatic(&OnCVarNeedingRenderStateInvalidationChanged),
	ECVF_Cheat
);
#else
constexpr float GLandscapeLODBlendRangeOverride = -1.f;
#endif // !UE_BUILD_SHIPPING

float GLandscapeLOD0DistributionScale = 1.f;
FAutoConsoleVariableRef CVarLandscapeLOD0DistributionScale(
	TEXT("r.LandscapeLOD0DistributionScale"),
	GLandscapeLOD0DistributionScale,
	TEXT("Multiplier for the landscape LOD0DistributionSetting property"),
	FConsoleVariableDelegate::CreateStatic(&OnCVarNeedingRenderStateInvalidationChanged),
	ECVF_Scalability
);

float GLandscapeLODDistributionScale = 1.f;
FAutoConsoleVariableRef CVarLandscapeLODDistributionScale(
	TEXT("r.LandscapeLODDistributionScale"),
	GLandscapeLODDistributionScale,
	TEXT("Multiplier for the landscape LODDistributionSetting property"),
	FConsoleVariableDelegate::CreateStatic(&OnCVarNeedingRenderStateInvalidationChanged),
	ECVF_Scalability
);

int32 GAllowLandscapeShadows = 1;
static FAutoConsoleVariableRef CVarAllowLandscapeShadows(
	TEXT("r.AllowLandscapeShadows"),
	GAllowLandscapeShadows,
	TEXT("Allow Landscape Shadows"),
	ECVF_RenderThreadSafe | ECVF_Scalability
);

int32 GLandscapeUseAsyncTasksForLODComputation = 1;
FAutoConsoleVariableRef CVarLandscapeUseAsyncTasksForLODComputation(
	TEXT("r.LandscapeUseAsyncTasksForLODComputation"),
	GLandscapeUseAsyncTasksForLODComputation,
	TEXT("Use async tasks for computing per-landscape component LOD biases."), 
	ECVF_RenderThreadSafe
);

int32 GDisableLandscapeNaniteGI = 1;
static FAutoConsoleVariableRef CVarDisableLandscapeNaniteGI(
	TEXT("r.DisableLandscapeNaniteGI"),
	GDisableLandscapeNaniteGI,
	TEXT("Disable Landscape Nanite GI"),
	FConsoleVariableDelegate::CreateStatic(&OnCVarNeedingRenderStateInvalidationChanged), 
	ECVF_Scalability
);

static TAutoConsoleVariable<int32> CVarLumenSceneSurfaceCacheCaptureNaniteLandscape(
	TEXT("r.LumenScene.SurfaceCache.CaptureNaniteLandscapes"),
	0,
	TEXT("Whether to use Nanite landscape meshes when capturing landscape Lumen cards."),
	FConsoleVariableDelegate::CreateStatic(&InvalidateRenderStateForLandscapeNaniteComponents),
	ECVF_RenderThreadSafe | ECVF_Scalability
);

bool GLandscapeAllowNonNaniteVirtualShadowMapInvalidation = true;
FAutoConsoleVariableRef CVarLandscapeAllowNonNaniteVirtualShadowMapInvalidation(
	TEXT("landscape.AllowNonNaniteVirtualShadowMapInvalidation"),
	GLandscapeAllowNonNaniteVirtualShadowMapInvalidation,
	TEXT("For non-Nanite landscape, cached virtual shadow map pages need to be invalidated when the vertex morphing introduces a height difference that is too large. This enables or disables this behavior entirely"),
	ECVF_RenderThreadSafe | ECVF_Scalability
);

float GLandscapeNonNaniteVirtualShadowMapInvalidationLODAttenuationExponent = 2.0f;
FAutoConsoleVariableRef CVarLandscapeNonNaniteVirtualShadowMapInvalidationLODAttenuationExponent(
	TEXT("landscape.NonNaniteVirtualShadowMapInvalidationLODAttenuationExponent"),
	GLandscapeNonNaniteVirtualShadowMapInvalidationLODAttenuationExponent,
	TEXT("For non-Nanite landscape, controls the shape of the curve of the attenuation of the virtual shadow map pages' invalidation rate (1 - X^N), where X is the relative LOD value (LODValue/NumMips in the [0,1] range) and N, the CVar"),
	ECVF_RenderThreadSafe | ECVF_Scalability
);

int32 GLandscapeAllowNanitePerClusterDisplacementDisable = 1;
static FAutoConsoleVariableRef CLandscapeAllowNanitePerClusterDisplacementDisable(
	TEXT("r.Landscape.AllowNanitePerClusterDisplacementDisable"),
	GLandscapeAllowNanitePerClusterDisplacementDisable,
	TEXT("Allow Nanite landscape to disable displcement on individual clusters in the distance."),
	FConsoleVariableDelegate::CreateStatic(&OnCVarNeedingRenderStateInvalidationChanged),
	ECVF_RenderThreadSafe
);

int32 GLandscapeRuntimeVirtualTextureRenderWithQuad = 0;
FAutoConsoleVariableRef CVarLandscapeRuntimeVirtualTextureRenderWithQuad(
	TEXT("landscape.RuntimeVirtualTextureRenderWithQuad"),
	GLandscapeRuntimeVirtualTextureRenderWithQuad,
	TEXT("Force whether to use of a single quad when rendering landscape to runtime virtual texture pages\n")
	TEXT("0 takes setting from landscape, 1 forces on, 2 forces off."),
	FConsoleVariableDelegate::CreateStatic(&OnCVarNeedingRenderStateInvalidationChanged),
	ECVF_RenderThreadSafe | ECVF_Scalability
);

static int32 GLandscapePSOPrecacheProxyCreationStrategy = -1;
static FAutoConsoleVariableRef CVarLandscapePSOPrecacheProxyCreationStrategy(
	TEXT("r.PSOPrecache.LandscapeProxyCreationStrategy"), 
	GLandscapePSOPrecacheProxyCreationStrategy, 
	TEXT("-1 : Use global PSOPrecache settings (default)\n")
	TEXT(" 0 : Always create the proxy even when PSOs are still compiling\n")
	TEXT(" 1 : Delay proxy creation when PSOs are still compiling\n")
	TEXT(" 2 : Use fallback material when PSOs are still compiling"),
	ECVF_ReadOnly);

static int32 GLandscapePSOPrecacheMeshProxyCreationStrategy = 1;
static FAutoConsoleVariableRef CVarLandscapePSOPrecacheMeshProxyCreationStrategy(
	TEXT("r.PSOPrecache.LandscapeMeshProxyCreationStrategy"), 
	GLandscapePSOPrecacheMeshProxyCreationStrategy, 
	TEXT("-1 : Use global PSOPrecache settings\n")
	TEXT(" 0 : Always create the proxy even when PSOs are still compiling\n")
	TEXT(" 1 : Delay proxy creation when PSOs are still compiling (default)\n")
	TEXT(" 2 : Use fallback material when PSOs are still compiling"),
	ECVF_ReadOnly);

#if WITH_EDITOR
extern TAutoConsoleVariable<int32> CVarLandscapeShowDirty;
#endif

extern RENDERER_API TAutoConsoleVariable<float> CVarStaticMeshLODDistanceScale;

#if !UE_BUILD_SHIPPING
uint32 GVarDumpLandscapeLODsCurrentFrame = 0;
bool GVarDumpLandscapeLODsDetails = false;

static void OnDumpLandscapeLODs(const TArray< FString >& Args)
{
	GVarDumpLandscapeLODsDetails = false;
	for (const FString& Arg : Args)
	{
		if (FParse::Param(*Arg, TEXT("detailed")))
		{
			GVarDumpLandscapeLODsDetails = true;
		}
	}

	if (GVarDumpLandscapeLODsCurrentFrame == 0)
	{
		// Add some buffer to be able to correctly catch the frame during the rendering
		GVarDumpLandscapeLODsCurrentFrame = GFrameNumberRenderThread + 3;
	}
}

static FAutoConsoleCommand CVarDumpLandscapeLODs(
	TEXT("landscape.DumpLODs"),
	TEXT("Will dump the current status of LOD value and current texture streaming status. Use the -detailed option to include all LOD-related infos"),
	FConsoleCommandWithArgsDelegate::CreateStatic(&OnDumpLandscapeLODs)
);
#endif

#if WITH_EDITOR
LANDSCAPE_API int32 GLandscapeViewMode = ELandscapeViewMode::Normal;
FAutoConsoleVariableRef CVarLandscapeDebugViewMode(
	TEXT("Landscape.DebugViewMode"),
	GLandscapeViewMode,
	TEXT("Change the view mode of the landscape rendering. Valid Input: 0 = Normal, 2 = DebugLayer, 3 = LayerDensity, 4 = LayerUsage, 5 = LOD Distribution, 6 = WireframeOnTop, 7 = LayerContribution"),
	ECVF_Cheat
);
#endif

#if RHI_RAYTRACING
static TAutoConsoleVariable<int32> CVarRayTracingLandscape(
	TEXT("r.RayTracing.Geometry.Landscape"),
	1,
	TEXT("Include landscapes in ray tracing effects (default = 1 (landscape enabled in ray tracing))"));

int32 GLandscapeRayTracingGeometryLODsThatUpdateEveryFrame = 0;
static FAutoConsoleVariableRef CVarLandscapeRayTracingGeometryLODsThatUpdateEveryFrame(
	TEXT("r.RayTracing.Geometry.Landscape.LODsUpdateEveryFrame"),
	GLandscapeRayTracingGeometryLODsThatUpdateEveryFrame,
	TEXT("If on, LODs that are lower than the specified level will be updated every frame, which can be used to workaround some artifacts caused by texture streaming if you're using WorldPositionOffset on the landscape")
);

int32 GLandscapeRayTracingGeometryDetectTextureStreaming = 0;
static FAutoConsoleVariableRef CVarLandscapeRayTracingGeometryDetectTextureStreaming(
	TEXT("r.RayTracing.Geometry.Landscape.DetectTextureStreaming"),
	GLandscapeRayTracingGeometryDetectTextureStreaming,
	TEXT("If on, update ray tracing geometry when texture streaming state changes. Useful when WorldPositionOffset is used in the landscape material")
);

float GLandscapeRayTracingGeometryFractionalLODUpdateThreshold = 0.0f;
static FAutoConsoleVariableRef CVarLandscapeRayTracingGeometryFractionalLODUpdateThreshold(
	TEXT("r.RayTracing.Geometry.Landscape.FractionalLODUpdateThreshold"),
	GLandscapeRayTracingGeometryFractionalLODUpdateThreshold,
	TEXT("Minimal difference in fractional LOD between latest built/cached ray tracing geometry and latest value used for rendering (default 0)\n")
	TEXT("0.1 implies a 10% change in the fraction LOD, for instance a change from LOD level 1.1 to 1.2\n")
	TEXT("Larger values will reduce the number of landscape tile updates, but introduce more error between the ray tracing and raster representations")
);

#endif

int32 GLandscapeRayTracingSections = 1;
static FAutoConsoleVariableRef CVarLandscapeNumRayTracingSectionsPerSubsection(
	TEXT("r.RayTracing.Geometry.Landscape.SectionsPerSubsection"),
	GLandscapeRayTracingSections,
	TEXT("Number of raytracing BLAS sections per landscape sub section - ray tracing sections will be evenly split (needs to be power of 2)"),
	FConsoleVariableDelegate::CreateStatic(&OnCVarNeedingRenderStateInvalidationChanged)
);

int32 GLandscapeRayTracingLODBias = 0;
static FAutoConsoleVariableRef CVarLandscapeRayTracingVertexLODBias(
	TEXT("r.RayTracing.Geometry.Landscape.LODBias"),
	GLandscapeRayTracingLODBias,
	TEXT("LOD Bias to use when building ray tracing dynamic geometry elements"),
	FConsoleVariableDelegate::CreateStatic(&OnCVarNeedingRenderStateInvalidationChanged),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarRayTracingLandscapeVertexMask(
	TEXT("r.RayTracing.Geometry.Landscape.VertexMask"),
	0,
	TEXT("If landscape should use vertex mask when building ray tracing dynamic geometry (works only with inline raytracing)"),
	FConsoleVariableDelegate::CreateStatic(&OnCVarNeedingRenderStateInvalidationChanged),
	ECVF_RenderThreadSafe
);

// ----------------------------------------------------------------------------------

namespace UE::Landscape
{
	bool NeedsFixedGridVertexFactory(EShaderPlatform InShaderPlatform)
	{
		bool bNeedsFixedGridVertexFactory = false;
		// We need the fixed grid vertex factory for virtual texturing : 
		bNeedsFixedGridVertexFactory |= UseVirtualTexturing(InShaderPlatform);

		// We need the fixed grid vertex factory for rendering Landscape into Lumen Surface Cache: 
		bNeedsFixedGridVertexFactory |= DoesPlatformSupportLumenGI(InShaderPlatform);

		// We need the fixed grid vertex factory for rendering the water info texture : 
		// This cvar is defined in the water plugin and searching for it should return nullptr if the plugin is not loaded
		static const bool bWaterPluginLoaded = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Water.WaterInfo.RenderMethod")) != nullptr;
		bNeedsFixedGridVertexFactory |= bWaterPluginLoaded;

		return bNeedsFixedGridVertexFactory;
	}

	bool ShouldBuildGrassMapRenderingResources()
	{
		#if WITH_EDITOR
			return true;
		#else
			return SupportsRuntimeGrassMapGeneration();
		#endif // WITH_EDITOR
	}

	bool UseLandscapeVertexMask()
	{
		return CVarRayTracingLandscapeVertexMask.GetValueOnAnyThread() != 0;
	}

	EPSOPrecacheProxyCreationStrategy GetLandscapePSOPrecacheProxyCreationStrategy()
	{
		return GetPSOPrecacheProxyCreationStrategy(GLandscapePSOPrecacheProxyCreationStrategy);
	}
} // namespace UE::Landscape

//
// FLandscapeDebugOptions
//
FLandscapeDebugOptions::FLandscapeDebugOptions()
	: bShowPatches(false)
	, bDisableStatic(false)
	, PatchesConsoleCommand(
		TEXT("Landscape.Patches"),
		TEXT("Show/hide Landscape patches"),
		FConsoleCommandDelegate::CreateRaw(this, &FLandscapeDebugOptions::Patches))
	, StaticConsoleCommand(
		TEXT("Landscape.Static"),
		TEXT("Enable/disable Landscape static drawlists"),
		FConsoleCommandDelegate::CreateRaw(this, &FLandscapeDebugOptions::Static))
{
}

void FLandscapeDebugOptions::Patches()
{
	bShowPatches = !bShowPatches;
	UE_LOGF(LogLandscape, Display, "Landscape.Patches: %ls", bShowPatches ? TEXT("Show") : TEXT("Hide"));
}

void FLandscapeDebugOptions::Static()
{
	bDisableStatic = !bDisableStatic;
	UE_LOGF(LogLandscape, Display, "Landscape.Static: %ls", bDisableStatic ? TEXT("Disabled") : TEXT("Enabled"));
}

FLandscapeDebugOptions GLandscapeDebugOptions;


#if WITH_EDITOR
LANDSCAPE_API bool GLandscapeEditModeActive = false;
LANDSCAPE_API int32 GLandscapeEditRenderMode = ELandscapeEditRenderMode::None;
#endif

void ULandscapeComponent::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const
{
	// TODO - investigate whether this is correct

	ALandscapeProxy* Actor = GetLandscapeProxy();

	if (GMaxRHIFeatureLevel > ERHIFeatureLevel::ES3_1)
	{
		if (Actor != nullptr && Actor->bUseDynamicMaterialInstance)
		{
			OutMaterials.Append(MaterialInstancesDynamic.FilterByPredicate([](UMaterialInstanceDynamic* MaterialInstance) { return MaterialInstance != nullptr; }));
		}
		else
		{
			OutMaterials.Append(MaterialInstances.FilterByPredicate([](UMaterialInstanceConstant* MaterialInstance) { return MaterialInstance != nullptr; }));
		}
	}

	if (MobileMaterialInterfaces.Num())
	{
		OutMaterials.Append(MobileMaterialInterfaces.FilterByPredicate([](UMaterialInterface* MaterialInstance) { return MaterialInstance != nullptr; }));
	}

#if	WITH_EDITORONLY_DATA	
	if (MobileCombinationMaterialInstances.Num())
	{
		OutMaterials.Append(MobileCombinationMaterialInstances.FilterByPredicate([](UMaterialInstanceConstant* MaterialInstance) { return MaterialInstance != nullptr; }));
	}
#endif
	
	if (OverrideMaterial)
	{
		OutMaterials.Add(OverrideMaterial);
	}

	if (OverrideHoleMaterial)
	{
		OutMaterials.Add(OverrideHoleMaterial);
	}

#if WITH_EDITORONLY_DATA
	if (EditToolRenderData.ToolMaterial)
	{
		OutMaterials.Add(EditToolRenderData.ToolMaterial);
	}

	if (EditToolRenderData.GizmoMaterial)
	{
		OutMaterials.Add(EditToolRenderData.GizmoMaterial);
	}
#endif

#if WITH_EDITOR
	//if (bGetDebugMaterials) // TODO: This should be tested and enabled
	{
		ULandscapeEditResourcesSubsystem* LandscapeEditResourcesSubsystem = GEngine->GetEngineSubsystem<ULandscapeEditResourcesSubsystem>();
		check(LandscapeEditResourcesSubsystem != nullptr);
		OutMaterials.Add(LandscapeEditResourcesSubsystem->GetLayerDebugColorMaterial());
		OutMaterials.Add(LandscapeEditResourcesSubsystem->GetSelectionColorMaterial());
		OutMaterials.Add(LandscapeEditResourcesSubsystem->GetSelectionRegionMaterial());
		OutMaterials.Add(LandscapeEditResourcesSubsystem->GetMaskRegionMaterial());
		OutMaterials.Add(LandscapeEditResourcesSubsystem->GetColorMaskRegionMaterial());
		OutMaterials.Add(LandscapeEditResourcesSubsystem->GetLandscapeLayerUsageMaterial());
		OutMaterials.Add(LandscapeEditResourcesSubsystem->GetLandscapeDirtyMaterial());
	}
#endif
}

/**
 * Return any global Lod override for landscape.
 * A return value less than 0 means no override.
 * Any positive value must still be clamped into the valid Lod range for the landscape.
 */
static int32 GetViewLodOverride(FSceneView const& View, uint32 LandscapeKey)
{
	// Apply r.ForceLOD override
	int32 LodOverride = GetCVarForceLOD_AnyThread();
	// Apply editor landscape lod override
	LodOverride = View.Family->LandscapeLODOverride >= 0 ? View.Family->LandscapeLODOverride : LodOverride;
	// Use lod 0 if lodding is disabled
	LodOverride = View.Family->EngineShowFlags.LOD == 0 ? 0 : LodOverride;
	
	if (View.CustomRenderPass)
	{
		if (FLandscapeLODOverridesCustomRenderPassUserData* LandscapeLODOverridesUserData = View.CustomRenderPass->GetUserDataTyped<FLandscapeLODOverridesCustomRenderPassUserData>())
		{
			const int32* LandscapeLODOverride = LandscapeLODOverridesUserData->GetLandscapeLODOverrides().Find(LandscapeKey);
			LodOverride = LandscapeLODOverride ? *LandscapeLODOverride : LodOverride;
		}
	}

	return LodOverride;
}

static int32 GetDrawCollisionLodOverride(bool bShowCollisionPawn, bool bShowCollisionVisibility, int32 DrawCollisionPawnLOD, int32 DrawCollisionVisibilityLOD)
{
#if WITH_EDITOR || !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	return bShowCollisionPawn ? FMath::Max(DrawCollisionPawnLOD, DrawCollisionVisibilityLOD) : bShowCollisionVisibility ? DrawCollisionVisibilityLOD : -1;
#else
	return -1;
#endif
}

static int32 GetDrawCollisionLodOverride(FSceneView const& View, FCollisionResponseContainer const& CollisionResponse, int32 CollisionLod, int32 SimpleCollisionLod)
{
#if WITH_EDITOR || !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	bool bShowCollisionPawn = View.Family->EngineShowFlags.CollisionPawn;
	bool bShowCollisionVisibility = View.Family->EngineShowFlags.CollisionVisibility;
	int32 DrawCollisionPawnLOD = CollisionResponse.GetResponse(ECC_Pawn) == ECR_Ignore ? -1 : SimpleCollisionLod;
	int32 DrawCollisionVisibilityLOD = CollisionResponse.GetResponse(ECC_Visibility) == ECR_Ignore ? -1 : CollisionLod;
	return GetDrawCollisionLodOverride(bShowCollisionPawn, bShowCollisionVisibility, DrawCollisionPawnLOD, DrawCollisionVisibilityLOD);
#else
	return -1;
#endif
}


static uint32 GetRayTracingSectionVertCountForLOD(uint32 SubsectionSizeVerts, int32 RayTracingNumSections, uint32 LODIndex)
{
	return ((SubsectionSizeVerts / RayTracingNumSections) >> LODIndex) + 1;
}

//
// FLandscapeComponentSceneProxy
//
TMap<uint32, FLandscapeSharedBuffers*> FLandscapeComponentSceneProxy::SharedBuffersMap;

const static FName NAME_LandscapeResourceNameForDebugging("Landscape");

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FLandscapeSectionLODUniformParameters, "LandscapeContinuousLODParameters");

// TODO [jonathan.bard] Move this to a ISceneExtension, where it belongs : the FLandscapeRenderSystem are associated with a scene and should have the same lifetime
TMap<uint32, FLandscapeRenderSystem*> LandscapeRenderSystems;

float FLandscapeRenderSystem::ComputeLODFromScreenSize(const LODSettingsComponent& InLODSettings, float InScreenSizeSquared)
{
	if (InScreenSizeSquared <= InLODSettings.LastLODScreenSizeSquared)
	{
		return InLODSettings.LastLODIndex;
	}
	else if (InScreenSizeSquared > InLODSettings.LOD1ScreenSizeSquared)
	{
		return (InLODSettings.LOD0ScreenSizeSquared - FMath::Min(InScreenSizeSquared, InLODSettings.LOD0ScreenSizeSquared)) / (InLODSettings.LOD0ScreenSizeSquared - InLODSettings.LOD1ScreenSizeSquared);
	}
	else
	{
		// No longer linear fraction, but worth the cache misses
		return 1.0f + FMath::LogX(InLODSettings.LODOnePlusDistributionScalarSquared, InLODSettings.LOD1ScreenSizeSquared / InScreenSizeSquared);
	}
}

TBitArray<> FLandscapeRenderSystem::LandscapeIndexAllocator;

#if RHI_RAYTRACING

struct FLandscapeSectionRayTracingState
{
	FRayTracingGeometry Geometry;
	FRWBuffer RayTracingDynamicVertexBuffer;
	FLandscapeVertexFactoryMVFUniformBufferRef UniformBuffer;
};

// Where we are rendering multiple views, we need to branch the landscape ray tracing state (BLAS data) per view, for performance reasons.
// Without this, the BLAS data ends up getting rebuilt from scratch every frame due to LOD thrashing, costing 30-50 ms per view, or 60-100 ms
// for a scene with two views.  This structure represents the state for a single view.
struct FLandscapeRayTracingState
{
	FLandscapeRayTracingState() = default;
	~FLandscapeRayTracingState();

	// View state key from FSceneViewState.  Zero is used if FSceneViewState is null (view state keys start at 1, so 0 is invalid for an actual view).
	uint32 ViewKey = -1;

	// Rendering data
	int8 CurrentLOD = -1;
	float FractionalLOD = -1000.0f;
	float HeightmapLODBias = -1000.0f;
	uint32 ReferencedTextureRHIHash = 0;	
	TArray<FLandscapeSectionRayTracingState> Sections;
};

// This wrapper holds the ray tracing state for a single scene proxy for all views
struct FLandscapeRayTracingImpl
{
	TArray<FLandscapeRayTracingState> PerViewRayTracingState;

	// ViewStateInterface pointer can be NULL
	FLandscapeRayTracingState* FindOrCreateRayTracingState(
		FRHICommandListBase& RHICmdList, 
		FSceneViewStateInterface* ViewStateInterface, 
		int32 NumSubsections,
		int32 NumRayTracingSections,
	    int32 RayTracingSectionVertCount);
};

// When views get deleted, we need to clean up the per view ray tracing state, which is handled by this class.  On deletion of the view,
// iterate over active landscape systems and their sections (each is a render proxy), and notify them that the view was deleted.  Note that
// originally this class included an actual list of the render proxies, but it was removed in favor of iterating over the global list, and
// now just includes the view's key.
class FLandscapeRayTracingStateList
{
public:
	uint32 ViewKey;
	FLandscapeRayTracingStateList(uint32 InViewKey) : ViewKey(InViewKey) {}

	~FLandscapeRayTracingStateList()
	{
		for (auto System : LandscapeRenderSystems)
		{
			for (FLandscapeSectionInfo* SectionInfo : System.Value->SectionInfos)
			{
				if (SectionInfo)
				{
					SectionInfo->RayTracingNotifyViewDeleted(ViewKey);
				}
			}
		}
	}
};
#endif	// RHI_RAYTRACING


//
// FLandscapeRenderSystem
//
FLandscapeRenderSystem::FLandscapeRenderSystem(uint32 InLandscapeKey, FSceneInterface* InScene)
	: Min(MAX_int32, MAX_int32)
	, Size(EForceInit::ForceInitToZero)
	, ReferenceCount(0)
	, RegisteredCount(0)
	, ForcedLODOverride(-1)
	, SectionsRemovedSinceLastCompact(0)
	, LandscapeKey(InLandscapeKey)
	, Scene(InScene)
{
	SectionLODBiases.SetAllowCPUAccess(true);

	LandscapeIndex = LandscapeIndexAllocator.FindAndSetFirstZeroBit();
	if (LandscapeIndex == INDEX_NONE)
	{
		LandscapeIndex = LandscapeIndexAllocator.Add(true);
	}

	FLandscapeSectionLODUniformParameters Parameters;
	Parameters.LandscapeIndex = LandscapeIndex;
	Parameters.Size = FIntPoint(1, 1);
	Parameters.SectionLODBias = GWhiteVertexBufferWithSRV->ShaderResourceViewRHI;
	SectionLODUniformBuffer = TUniformBufferRef<FLandscapeSectionLODUniformParameters>::CreateUniformBufferImmediate(Parameters, UniformBuffer_MultiFrame);
}

FLandscapeRenderSystem::~FLandscapeRenderSystem()
{
	check(LandscapeIndexAllocator[LandscapeIndex]);
	LandscapeIndexAllocator[LandscapeIndex] = false;

	// Clear slack in the array
	int32 LastSetIndex = LandscapeIndexAllocator.FindLast(true);
	LandscapeIndexAllocator.SetNumUninitialized(LastSetIndex + 1);
}

void FLandscapeRenderSystem::CreateResources(FRHICommandListBase& RHICmdList, FLandscapeSectionInfo* SectionInfo)
{
	FLandscapeRenderSystem*& LandscapeRenderSystem = LandscapeRenderSystems.FindOrAdd(SectionInfo->LandscapeKey);
	if (!LandscapeRenderSystem)
	{
		LandscapeRenderSystem = new FLandscapeRenderSystem(SectionInfo->LandscapeKey, SectionInfo->Scene);
	}

	LandscapeRenderSystem->CreateResources_Internal(RHICmdList, SectionInfo);
}

void FLandscapeRenderSystem::DestroyResources(FLandscapeSectionInfo* SectionInfo)
{
	FLandscapeRenderSystem* LandscapeRenderSystem = LandscapeRenderSystems.FindChecked(SectionInfo->LandscapeKey);
	LandscapeRenderSystem->DestroyResources_Internal(SectionInfo);

	if (LandscapeRenderSystem->ReferenceCount == 0)
	{
		check(LandscapeRenderSystem->RegisteredCount == 0);
		delete LandscapeRenderSystem;
		LandscapeRenderSystems.Remove(SectionInfo->LandscapeKey);
	}
}

void FLandscapeRenderSystem::CreateResources_Internal(FRHICommandListBase& RHICmdList, FLandscapeSectionInfo* SectionInfo)
{
	check(SectionInfo != nullptr);
	check(!SectionInfo->bRegistered);
	check(!SectionInfo->bResourcesCreated);

	SectionInfo->RenderCoord = SectionInfo->ComponentBase;
	SectionInfo->bResourcesCreated = true;

	if (SectionInfo->LODGroupKey != 0)
	{
		// Record and check settings (resolution, scale and orientation) that need to match across landscapes in an LOD Group
		FVector SectionCenterWorldSpace, SectionXVector, SectionYVector;
		SectionInfo->GetSectionCenterAndVectors(SectionCenterWorldSpace, SectionXVector, SectionYVector);
		int32 SectionComponentResolution = SectionInfo->GetComponentResolution();

		if (ComponentResolution < 0)
		{
			// the first time we register a section, set up our RenderCoord grid so the component is located at the origin
			ComponentResolution = SectionComponentResolution;
			ComponentOrigin = SectionCenterWorldSpace;
			ComponentXVector = SectionXVector;
			ComponentYVector = SectionYVector;
		}
		else
		{
			// validate each section has a matching resolution, scale and orientation
			bool bResolutionMatches = (ComponentResolution == SectionComponentResolution);
			// TODO [alan.roach]:  The scale values are showing up slightly off from expected values. 32767.999268 vs 32768.0  Temporarily suppress the warnings with a larger tolerance.
			bool bXVectorMatches = (SectionXVector - ComponentXVector).IsNearlyZero(UE_KINDA_SMALL_NUMBER * 100.0);
			bool bYVectorMatches = (SectionYVector - ComponentYVector).IsNearlyZero(UE_KINDA_SMALL_NUMBER * 100.0);
			if (!(bResolutionMatches && bXVectorMatches && bYVectorMatches))
			{
				UE_LOGF(LogLandscape, Warning, "Landscapes in LOD Group %u do not have matching resolution (%d == %d), scale (%f == %f, %f == %f) and/or rotation; geometry seam artifacts may appear. If using HLOD, it may need to be updated to reflect changed resolutions or transforms.",
					SectionInfo->LODGroupKey,
					ComponentResolution, SectionComponentResolution,
					ComponentXVector.Length(), SectionXVector.Length(),
					ComponentYVector.Length(), SectionYVector.Length());
			}
		}
			
		// project onto the Component X/Y plane to calculate the render coordinates
		FVector Delta = SectionCenterWorldSpace - ComponentOrigin;
		SectionInfo->RenderCoord.X = FMath::RoundToInt32(Delta.Dot(ComponentXVector) / ComponentXVector.SquaredLength());
		SectionInfo->RenderCoord.Y = FMath::RoundToInt32(Delta.Dot(ComponentYVector) / ComponentYVector.SquaredLength());
	}

	// we changed the RenderCoord, need to update the uniform buffer
	SectionInfo->OnRenderCoordsChanged(RHICmdList);

	check(SectionInfo->RenderCoord.X > INT32_MIN);
	ResizeToInclude(SectionInfo->RenderCoord);

	ReferenceCount++;
}

void FLandscapeRenderSystem::DestroyResources_Internal(FLandscapeSectionInfo* SectionInfo)
{
	check(SectionInfo != nullptr);
	check(!SectionInfo->bRegistered);
	check(SectionInfo->bResourcesCreated);

	ReferenceCount--;

	SectionInfo->bResourcesCreated = false;

	// try to compact the map every once in a while
	SectionsRemovedSinceLastCompact++;

	// When ReferenceCount == RegisteredCount then we know all resource-created sections are registered.
	// So any empty sections of the map can be compacted without issues.  When they are not equal
	// then compacting may erroneously remove areas that are still allocated to resource-created sections,
	// but are not yet registered.
	if ((SectionsRemovedSinceLastCompact >= 128) && ReferenceCount == RegisteredCount)
	{
		// if there are at least 128 free entries, run a compact step
		if (Size.X * Size.Y - ReferenceCount >= 128)
		{
			CompactMap();
		}
		SectionsRemovedSinceLastCompact = 0;
	}
}

void FLandscapeRenderSystem::RegisterSection(FLandscapeSectionInfo* SectionInfo)
{
	check(SectionInfo != nullptr);
	check(SectionInfo->bResourcesCreated);
	check(!SectionInfo->bRegistered);

	// With HLODs, it's possible to have multiple loaded sections representing the same
	// landscape patch. For example, raytracing may keep the HLOD proxy around (far field),
	// even if the actual landscape is loaded & visible.
	// We keep a linked list of the section infos, sorted by priority, so that unregistration can
	// properly restore a previously registered section info.

	FLandscapeRenderSystem*& LandscapeRenderSystem = LandscapeRenderSystems.FindChecked(SectionInfo->LandscapeKey);

	check(SectionInfo->RenderCoord.X > INT32_MIN);
	FLandscapeSectionInfo* ExistingSection = LandscapeRenderSystem->GetSectionInfo(SectionInfo->RenderCoord);
	if (ExistingSection == nullptr)
	{
		LandscapeRenderSystem->SetSectionInfo(SectionInfo->RenderCoord, SectionInfo);
	}
	else
	{
		FLandscapeSectionInfo* CurrentSection = nullptr;
		FLandscapeSectionInfo::TIterator SectionIt(ExistingSection);
		for (; SectionIt; ++SectionIt)
		{
			CurrentSection = &*SectionIt;

			// Sort on insertion
			if (SectionInfo->GetSectionPriority() < CurrentSection->GetSectionPriority())
			{
				SectionInfo->LinkBefore(CurrentSection);
				break;
			}
		}

		if (!SectionIt)
		{
			// Set as tail
			SectionInfo->LinkAfter(CurrentSection);
		}
		else if (CurrentSection == ExistingSection)
		{
			// Set as head (SectionInfo was inserted before the previous head in the loop above)
			LandscapeRenderSystem->SetSectionInfo(SectionInfo->RenderCoord, SectionInfo);
		}
	}

	SectionInfo->bRegistered = true;
	LandscapeRenderSystem->RegisteredCount++;
}

void FLandscapeRenderSystem::UnregisterSection(FLandscapeSectionInfo* SectionInfo)
{
	check(SectionInfo != nullptr);
	check(SectionInfo->bResourcesCreated);

	// Sections may be unregistered multiple times
	if (SectionInfo->bRegistered)
	{
		FLandscapeRenderSystem* LandscapeRenderSystem = LandscapeRenderSystems.FindChecked(SectionInfo->LandscapeKey);

		FLandscapeSectionInfo* ExistingSection = LandscapeRenderSystem->GetSectionInfo(SectionInfo->RenderCoord);
		if (ExistingSection == SectionInfo)
		{
			LandscapeRenderSystem->SetSectionInfo(SectionInfo->RenderCoord, SectionInfo->GetNextLink());
		}

		SectionInfo->Unlink();
		SectionInfo->bRegistered = false;
		LandscapeRenderSystem->RegisteredCount--;
	}
}

void FLandscapeRenderSystem::ResizeAndMoveTo(FIntPoint NewMin, FIntPoint NewMax)
{
	// chunk size reduces the number of times we need to reallocate the section map when resizing it. should be a power of two
	static constexpr int32 SectionMapChunkSize = 4;

	// round down to multiple of SectionMapChunkSize
	NewMin.X = (NewMin.X) & ~(SectionMapChunkSize - 1);
	NewMin.Y = (NewMin.Y) & ~(SectionMapChunkSize - 1);

	// round up to multiple of SectionMapChunkSize (note that before this operation NewMax is inclusive, but after it is exclusive)
	NewMax.X = (NewMax.X + SectionMapChunkSize) & ~(SectionMapChunkSize - 1);
	NewMax.Y = (NewMax.Y + SectionMapChunkSize) & ~(SectionMapChunkSize - 1);

	FIntPoint NewSize = (NewMax - NewMin);
	if (NewMin != Min || Size != NewSize)
	{
		check((Size.X & (SectionMapChunkSize-1)) == 0);
		check((Size.Y & (SectionMapChunkSize-1)) == 0);

		SectionLODBiasBuffer.SafeRelease();

		TResourceArray<float> NewSectionLODBiases;
		TArray<FLandscapeSectionInfo*> NewSectionInfos;

		NewSectionLODBiases.AddZeroed(NewSize.X * NewSize.Y);
		NewSectionInfos.AddZeroed(NewSize.X * NewSize.Y);

		for (int32 OldY = 0; OldY < Size.Y; OldY++)
		{
			int32 Y = OldY + Min.Y;
			int32 NewY = Y - NewMin.Y;
			if ((NewY >= 0) && (NewY < NewSize.Y))
			{
				int32 OldYBase = OldY * Size.X;
				int32 NewYBase = NewY * NewSize.X;
				for (int32 OldX = 0; OldX < Size.X; OldX++)
				{
					int32 X = OldX + Min.X;
					int32 NewX = X - NewMin.X;
					if ((NewX >= 0) && (NewX < NewSize.X))
					{
						int32 OldLinearIndex = OldYBase + OldX;
						int32 NewLinearIndex = NewYBase + NewX;
						NewSectionLODBiases[NewLinearIndex] = SectionLODBiases[OldLinearIndex];
						NewSectionInfos[NewLinearIndex] = SectionInfos[OldLinearIndex];

						// we null this out in order to check below that we have moved everything.
						SectionInfos[OldLinearIndex] = nullptr;
					}
				}
			}
		}

		// check that we have moved everything out of the old section infos, to ensure that this resize doesn't lose data
		check(!AnySectionsInRangeInclusive(Min, Min + Size - 1));

		Min = NewMin;
		Size = NewSize;
		SectionLODBiases = MoveTemp(NewSectionLODBiases);
		SectionInfos = MoveTemp(NewSectionInfos);

		SectionLODBiases.SetAllowCPUAccess(true);
	}
}

void FLandscapeRenderSystem::ResizeToInclude(const FIntPoint& NewCoord)
{
	if (!SectionInfos.IsEmpty())
	{
		// Calculate new bounding rect of landscape components
		FIntPoint OriginalMin = Min;
		FIntPoint OriginalMax = Min + Size - FIntPoint(1, 1);

		FIntPoint NewMin(FMath::Min(OriginalMin.X, NewCoord.X), FMath::Min(OriginalMin.Y, NewCoord.Y));
		FIntPoint NewMax(FMath::Max(OriginalMax.X, NewCoord.X), FMath::Max(OriginalMax.Y, NewCoord.Y));

		ResizeAndMoveTo(NewMin, NewMax);
	}
	else
	{
		ResizeAndMoveTo(NewCoord, NewCoord);
	}
}

void FLandscapeRenderSystem::CompactMap()
{
	FIntPoint NewMin = Min;
	FIntPoint NewMax = Min + Size - FIntPoint(1, 1);

	// Shrink Min X 
	while ((NewMin.X < NewMax.X) &&
		!AnySectionsInRangeInclusive(FIntPoint(NewMin.X, NewMin.Y), FIntPoint(NewMin.X, NewMax.Y)))
	{
		NewMin.X++;
	}

	// Shrink Max X 
	while ((NewMin.X < NewMax.X) &&
		!AnySectionsInRangeInclusive(FIntPoint(NewMax.X, NewMin.Y), FIntPoint(NewMax.X, NewMax.Y)))
	{
		NewMax.X--;
	}

	// Shrink Min Y
	while ((NewMin.Y < NewMax.Y) &&
		!AnySectionsInRangeInclusive(FIntPoint(NewMin.X, NewMin.Y), FIntPoint(NewMax.X, NewMin.Y)))
	{
		NewMin.Y++;
	}

	// Shrink Max Y
	while ((NewMin.Y < NewMax.Y) &&
		!AnySectionsInRangeInclusive(FIntPoint(NewMin.X, NewMax.Y), FIntPoint(NewMax.X, NewMax.Y)))
	{
		NewMax.Y--;
	}

	ResizeAndMoveTo(NewMin, NewMax);
}

bool FLandscapeRenderSystem::AnySectionsInRangeInclusive(FIntPoint RangeMin, FIntPoint RangeMax)
{
	// convert to range relative to Min
	RangeMin -= Min;
	RangeMax -= Min;
	for (int32 Y = RangeMin.Y; Y <= RangeMax.Y; Y++)
	{
		for (int32 X = RangeMin.X; X <= RangeMax.X; X++)
		{
			int32 LinearIndex = Y * Size.X + X;
			if (SectionInfos[LinearIndex] != nullptr)
			{
				return true;
			}
		}
	}
	return false;
}

float FLandscapeRenderSystem::GetSectionLODValue(const FSceneView& InView, FIntPoint InRenderCoord) const
{
	return GetCachedSectionLODValues(InView)[GetSectionLinearIndex(InRenderCoord)];
}

const TResourceArray<float>& FLandscapeRenderSystem::GetCachedSectionLODValues(const FSceneView& InView) const
{
	const uint32 ViewStateKey = InView.GetViewKey();
	const TResourceArray<float>* CachedSectionLODValues = (ViewStateKey != 0) ? PerViewStateCachedSectionLODValues.Find(ViewStateKey) : PerViewCachedSectionLODValues.Find(&InView);
	const FSceneView* SnapshotOriginView = InView.GetSnapshotOriginView();
	// If we couldn't find this view or view state in our maps, it's possible the view originates from another one, which we must have recorded in our maps : 
	if ((CachedSectionLODValues == nullptr) && (SnapshotOriginView != nullptr))
	{
		checkf(SnapshotOriginView != &InView, TEXT("Infinite loop will happen if the view originates from itself!"));
		CachedSectionLODValues = &GetCachedSectionLODValues(*SnapshotOriginView);
	}
	checkf(CachedSectionLODValues != nullptr, TEXT("No section LOD value cached for this view. Make sure FLandscapeRenderSystem::ComputeSectionsLODForView (FLandscapeSceneViewExtension::PreRenderView_RenderThread) was called"));
	return *CachedSectionLODValues;
}

float FLandscapeRenderSystem::GetSectionLODBias(FIntPoint InRenderCoord) const
{
	return SectionLODBiases[GetSectionLinearIndex(InRenderCoord)];
}

const TResourceArray<float>& FLandscapeRenderSystem::ComputeSectionsLODForView(const FSceneView& InView, UE::Renderer::Private::IShadowInvalidatingInstances* InShadowInvalidatingInstances)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeRenderSystem::ComputeSectionsLODForView);

	// Find where the CachedSectionLODValues lie (either in the persistent map, if we have a persistent view state or in the transient one) : 
	TResourceArray<float>* CachedSectionLODValues = nullptr;
	SectionKeyToLODValueMap* LastShadowInvalidationSectionLODValues = nullptr;
	TArray<const FPrimitiveSceneInfo*>* PrimitivesToInvalidateForShadows = nullptr;
	const uint32 ViewStateKey = InView.GetViewKey();
	if (ViewStateKey != 0)
	{
		CachedSectionLODValues = &PerViewStateCachedSectionLODValues.FindOrAdd(ViewStateKey);
		if (InShadowInvalidatingInstances != nullptr)
		{
			LastShadowInvalidationSectionLODValues = &PerViewStateLastShadowInvalidationSectionLODValues.FindOrAdd(ViewStateKey);
			PrimitivesToInvalidateForShadows = &ShadowInvalidationRequests.FindOrAdd(InShadowInvalidatingInstances);
		}
	}
	else
	{
		CachedSectionLODValues = &PerViewCachedSectionLODValues.FindOrAdd(&InView);
	}

	CachedSectionLODValues->Reset(SectionInfos.Num());
	for (FLandscapeSectionInfo* SectionInfo : SectionInfos)
	{
		constexpr float DefaultLODValue = 0.0f;
		float& LODSectionValue = CachedSectionLODValues->Add_GetRef(DefaultLODValue);
		if (SectionInfo != nullptr)
		{
			LODSectionValue = SectionInfo->ComputeLODForView(InView);

			if ((InShadowInvalidatingInstances != nullptr) && (PrimitivesToInvalidateForShadows != nullptr))
			{
				// The shadow invalidation system requires the actual LOD that will be used in the shader, so we need to "correct" the desired LODSectionValue with the value that we *can* 
				//  actually render the landscape with (i.e. LODSectionBias) : 
				const float LODSectionBias = GetSectionLODBias(SectionInfo->RenderCoord);
				const float FinalLODSectionValue = FMath::Max(LODSectionBias, LODSectionValue);

				float& LastShadowInvalidationLODValue = LastShadowInvalidationSectionLODValues->FindOrAdd(SectionInfo->RenderCoord, FinalLODSectionValue);
				if (SectionInfo->ShouldInvalidateShadows(InView, FinalLODSectionValue, LastShadowInvalidationLODValue))
				{
					LastShadowInvalidationLODValue = FinalLODSectionValue;
					PrimitivesToInvalidateForShadows->Add(SectionInfo->GetPrimitiveSceneInfo());
				}
			}
		}
	}

	return *CachedSectionLODValues;
}

void FLandscapeRenderSystem::PerformShadowInvalidations(UE::Renderer::Private::IShadowInvalidatingInstances& InShadowInvalidatingInstances)
{
	checkf(IsInRenderingThread(), TEXT("Using IShadowInvalidatingInstances is only allowed from the rendering thread!"));
	if (TArray<const FPrimitiveSceneInfo*>* PrimitivesToInvalidateForShadows = ShadowInvalidationRequests.Find(&InShadowInvalidatingInstances))
	{
		for (const FPrimitiveSceneInfo* Primitive : *PrimitivesToInvalidateForShadows)
		{
			InShadowInvalidatingInstances.AddPrimitive(Primitive);
		}
		ShadowInvalidationRequests.Remove(&InShadowInvalidatingInstances);
	}
}

void FLandscapeRenderSystem::FetchHeightmapLODBiases()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeRenderSystem::FetchHeightmapLODBiases);

	for (int32 SectionIndex = 0; SectionIndex < SectionInfos.Num(); SectionIndex++)
	{
		const float DefaultLODBias = 0.0f;

		FLandscapeSectionInfo* SectionInfo = SectionInfos[SectionIndex];
		SectionLODBiases[SectionIndex] = SectionInfo ? SectionInfo->ComputeLODBias() : DefaultLODBias;
	}
}

void FLandscapeRenderSystem::UpdateBuffers(FRHICommandListBase& RHICmdList)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeRenderSystem::UpdateBuffers);

	bool bUpdateUB = false;

	if (Size != FIntPoint::ZeroValue)
	{
		if (!SectionLODBiasBuffer.IsValid())
		{
			const static FLazyName ClassName(TEXT("FLandscapeRenderSystem"));

			const FRHIBufferCreateDesc CreateDesc =
				FRHIBufferCreateDesc::CreateVertex(TEXT("SectionLODBiasBuffer"), SectionLODBiases.GetResourceDataSize())
				.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::Dynamic)
				.SetClassName(ClassName)
				.SetInitActionResourceArray(&SectionLODBiases)
				.DetermineInitialState();

			SectionLODBiasBuffer = RHICmdList.CreateBuffer(CreateDesc);
			SectionLODBiasSRV = RHICmdList.CreateShaderResourceView(
				SectionLODBiasBuffer,
				FRHIViewDesc::CreateBufferSRV()
					.SetType(FRHIViewDesc::EBufferType::Typed)
					.SetFormat(PF_R32_FLOAT));
			bUpdateUB = true;
		}
		else
		{
			float* Data = (float*)RHICmdList.LockBuffer(SectionLODBiasBuffer, 0, SectionLODBiases.GetResourceDataSize(), RLM_WriteOnly);
			FMemory::Memcpy(Data, SectionLODBiases.GetData(), SectionLODBiases.GetResourceDataSize());
			RHICmdList.UnlockBuffer(SectionLODBiasBuffer);
		}

		if (bUpdateUB)
		{
			FLandscapeSectionLODUniformParameters Parameters;
			Parameters.LandscapeIndex = LandscapeIndex;
			Parameters.Min = Min;
			Parameters.Size = Size;
			Parameters.SectionLODBias = SectionLODBiasSRV;

			RHICmdList.UpdateUniformBuffer(SectionLODUniformBuffer, &Parameters);
		}
	}
}


//
// FLandscapeSceneViewExtension::FLandscapeViewData
//
FLandscapeSceneViewExtension::FLandscapeViewData::FLandscapeViewData(FSceneView& InView)
	: View(&InView)
{
	if (ISceneRenderer* ScenerRenderer = InView.Family->GetSceneRenderer())
	{
		ShadowInvalidatingInstances = ScenerRenderer->GetShadowInvalidatingInstancesInterface(&InView);
	}
}


//
// FLandscapeSceneViewExtension
//
FLandscapeSceneViewExtension::FLandscapeSceneViewExtension(const FAutoRegister& AutoReg) : FSceneViewExtensionBase(AutoReg)
{
	FCoreDelegates::OnEndFrame.AddRaw(this, &FLandscapeSceneViewExtension::EndFrame_GameThread);
	FCoreDelegates::OnEndFrameRT.AddRaw(this, &FLandscapeSceneViewExtension::EndFrame_RenderThread);
}

FLandscapeSceneViewExtension::~FLandscapeSceneViewExtension()
{
	FCoreDelegates::OnEndFrameRT.RemoveAll(this);
	FCoreDelegates::OnEndFrame.RemoveAll(this);
}

void FLandscapeSceneViewExtension::BeginRenderViewFamily(FSceneViewFamily& InViewFamily) 
{
	if (InViewFamily.EngineShowFlags.Collision)
	{
		NumViewsWithShowCollisionAcc++;
	}
}

void FLandscapeSceneViewExtension::PreRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily)
{
	Culling::PreRenderViewFamily(InViewFamily);
}

void FLandscapeSceneViewExtension::PreRenderView_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView)
{
	LandscapeViews.Emplace(InView);

#if RHI_RAYTRACING
	if (InView.State)
	{
		// Create the ray tracing state list class if necessary
		FSceneViewStateInterface* ViewState = InView.State;
		if (ViewState->GetLandscapeRayTracingStates() == nullptr)
		{
			ViewState->SetLandscapeRayTracingStates( MakePimpl<FLandscapeRayTracingStateList>(ViewState->GetViewKey()) );
		}
	}
#endif	// RHI_RAYTRACING

	TArray<FLandscapeRenderSystem*> SceneLandscapeRenderSystems = GetLandscapeRenderSystems(InView.Family->Scene);

	// Kick the job once all views have been collected.
	if (!SceneLandscapeRenderSystems.IsEmpty() && LandscapeViews.Num() == InView.Family->AllViews.Num())
	{
		LandscapeSetupTask = GraphBuilder.AddCommandListSetupTask([SceneRenderSystems = MoveTemp(SceneLandscapeRenderSystems), LocalLandscapeViewsPtr = &LandscapeViews](FRHICommandListBase& RHICmdList)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeRenderSystem::ComputeLODs);
			FTaskTagScope Scope(ETaskTag::EParallelRenderingThread);
			check(!SceneRenderSystems.IsEmpty())

			for (FLandscapeRenderSystem* RenderSystem : SceneRenderSystems)
			{
				RenderSystem->FetchHeightmapLODBiases();
			}

			for (FLandscapeViewData& LandscapeView : *LocalLandscapeViewsPtr)
			{
				const uint32 ViewStateKey = LandscapeView.View->GetViewKey() ;

				LandscapeView.LandscapeIndirection.SetNum(FLandscapeRenderSystem::LandscapeIndexAllocator.Num());

				for (FLandscapeRenderSystem* RenderSystem : SceneRenderSystems)
				{
					// Store index where the LOD data for this landscape starts
					LandscapeView.LandscapeIndirection[RenderSystem->LandscapeIndex] = LandscapeView.LandscapeLODData.Num();

					// Compute sections lod values for this view & append to the global landscape LOD data
					const TResourceArray<float>& CachedSectionLODValues = RenderSystem->ComputeSectionsLODForView(*LandscapeView.View, LandscapeView.ShadowInvalidatingInstances);
					LandscapeView.LandscapeLODData.Append(CachedSectionLODValues);
				}
			}

			for (FLandscapeRenderSystem* RenderSystem : SceneRenderSystems)
			{
				RenderSystem->UpdateBuffers(RHICmdList);
			}

			for (FLandscapeViewData& LandscapeView : *LocalLandscapeViewsPtr)
			{
				{
					const FRHIBufferCreateDesc CreateDesc =
						FRHIBufferCreateDesc::CreateVertex(TEXT("LandscapeLODDataBuffer"), LandscapeView.LandscapeLODData.GetResourceDataSize())
						.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::Volatile)
						.SetInitActionResourceArray(&LandscapeView.LandscapeLODData)
						.DetermineInitialState();

					FBufferRHIRef LandscapeLODDataBuffer = RHICmdList.CreateBuffer(CreateDesc);
					LandscapeView.View->LandscapePerComponentDataBuffer = RHICmdList.CreateShaderResourceView(
						LandscapeLODDataBuffer,
						FRHIViewDesc::CreateBufferSRV()
							.SetType(FRHIViewDesc::EBufferType::Typed)
							.SetFormat(PF_R32_FLOAT));
				}

				{
					const FRHIBufferCreateDesc CreateDesc =
						FRHIBufferCreateDesc::CreateVertex(TEXT("LandscapeIndirectionBuffer"), LandscapeView.LandscapeIndirection.GetResourceDataSize())
						.AddUsage(EBufferUsageFlags::ShaderResource | EBufferUsageFlags::Volatile)
						.SetInitActionResourceArray(&LandscapeView.LandscapeIndirection)
						.DetermineInitialState();

					FBufferRHIRef LandscapeIndirectionBuffer = RHICmdList.CreateBuffer(CreateDesc);
					LandscapeView.View->LandscapeIndirectionBuffer = RHICmdList.CreateShaderResourceView(
						LandscapeIndirectionBuffer,
						FRHIViewDesc::CreateBufferSRV()
							.SetType(FRHIViewDesc::EBufferType::Typed)
							.SetFormat(PF_R32_UINT));
				}
			}

		}, GIsThreadedRendering && GLandscapeUseAsyncTasksForLODComputation);
	}
}

void FLandscapeSceneViewExtension::PreInitViews_RenderThread(FRDGBuilder& GraphBuilder)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeSceneViewExtension::PreInitViews_RenderThread);
	LandscapeSetupTask.Wait();

	TArray<const FSceneView*, TInlineAllocator<2>> SceneViews;
	for (FLandscapeViewData& LandscapeView : LandscapeViews)
	{
		SceneViews.Add(LandscapeView.View);

		// Perform the accumulated shadow invalidations for this view: 
		if (LandscapeView.ShadowInvalidatingInstances != nullptr)
		{
			for (auto& Pair : LandscapeRenderSystems)
			{
				FLandscapeRenderSystem* RenderSystem = Pair.Value;
				RenderSystem->PerformShadowInvalidations(*LandscapeView.ShadowInvalidatingInstances);
			}
		}
	}

	if (Culling::UseCulling(SceneViews[0]->GetShaderPlatform()))
	{
		Culling::InitMainViews(GraphBuilder, SceneViews);
	}

	LandscapeViews.Reset();
}

void FLandscapeSceneViewExtension::EndFrame_GameThread()
{
	NumViewsWithShowCollision = NumViewsWithShowCollisionAcc;
	NumViewsWithShowCollisionAcc = 0;
}

void FLandscapeSceneViewExtension::EndFrame_RenderThread()
{
	for (auto& Pair : LandscapeRenderSystems)
	{
		FLandscapeRenderSystem* RenderSystem = Pair.Value;
		// Cleanup the transient list of cached section LOD values : they are only valid for the frame since they are associated with views that have no persistent view state :
		RenderSystem->PerViewCachedSectionLODValues.Reset();
		// COMMENT [jonathan.bard] : we cannot do the same for the LOD values associated with persistent view states since it's possible to run certain frames without rendering any view 
		//  at all, which would lead us to needlessly lose the history (PreviousPerViewStateCachedSectionLODValues). We need a proper point to attach per-view state data like this, 
		//  with the same lifetime as the view state itself, but we don't have anything for this ATM. This is good enough for now, as we don't have too many view states, so this data shouldn't
		//  grow too much unless creating/destroying several view states, which should usually never really occur
		checkf(RenderSystem->ShadowInvalidationRequests.IsEmpty(), TEXT("All shadow invalidation requests should have been processed by now"));
	}
}

const TMap<uint32, FLandscapeRenderSystem*>& FLandscapeSceneViewExtension::GetLandscapeRenderSystems()
{
	checkf(IsInParallelRenderingThread(), TEXT("Accessing the Landscape render systems is only valid from the rendering thread!"));
	return LandscapeRenderSystems;
}

TArray<FLandscapeRenderSystem*> FLandscapeSceneViewExtension::GetLandscapeRenderSystems(const FSceneInterface* InScene)
{
	checkf(IsInParallelRenderingThread(), TEXT("Accessing the Landscape render systems is only valid from the rendering thread!"));
	check(InScene != nullptr);

	TArray<FLandscapeRenderSystem*> Result;
	Result.Reserve(LandscapeRenderSystems.Num());
	for (auto& Pair : LandscapeRenderSystems)
	{
		FLandscapeRenderSystem* RenderSystem = Pair.Value;
		if (RenderSystem->Scene == InScene)
		{
			Result.Add(RenderSystem);
		}
	}
	return Result;
}

FLandscapeRenderSystem* FLandscapeSceneViewExtension::GetLandscapeRenderSystem(const class FSceneInterface* InScene, uint32 InLandscapeKey)
{
	TArray<FLandscapeRenderSystem*> ViewLandscapeRenderSystems = GetLandscapeRenderSystems(InScene);
	FLandscapeRenderSystem** Found = ViewLandscapeRenderSystems.FindByPredicate([InLandscapeKey](FLandscapeRenderSystem* InRenderSystem) { return InRenderSystem->LandscapeKey == InLandscapeKey; });
	return (Found != nullptr) ? *Found : nullptr;
}

#if WITH_EDITOR
ULandscapeEditResourcesSubsystem* FLandscapeComponentSceneProxy::LandscapeEditResourcesSubsystem = nullptr;
#endif // WITH_EDITOR

FLandscapeComponentSceneProxy::FLandscapeComponentSceneProxy(ULandscapeComponent* InComponent)
	: FPrimitiveSceneProxy(InComponent, NAME_LandscapeResourceNameForDebugging)
	, FLandscapeSectionInfo(InComponent->GetWorld(), InComponent->GetLandscapeProxy()->GetLandscapeGuid(), InComponent->GetSectionBase() / InComponent->ComponentSizeQuads, InComponent->GetLandscapeProxy()->LODGroupKey, InComponent->GetLandscapeProxy()->ComputeLandscapeKey())
	, MaxLOD(static_cast<int8>(FMath::CeilLogTwo(InComponent->SubsectionSizeQuads + 1) - 1))
	, NumWeightmapLayerAllocations(static_cast<int8>(InComponent->GetWeightmapLayerAllocations().Num()))
	, StaticLightingLOD(static_cast<uint8>(InComponent->GetLandscapeProxy()->StaticLightingLOD))
	, WeightmapSubsectionOffset(InComponent->WeightmapSubsectionOffset)
	, FirstLOD(0)
	, LastLOD(MaxLOD)
	, ComponentMaxExtend(0.0f)
	, InvLODBlendRange(1.0f / FMath::Max(0.01f, GLandscapeLODBlendRangeOverride > 0.0f ? GLandscapeLODBlendRangeOverride : InComponent->GetLandscapeProxy()->LODBlendRange))
	, NumSubsections(InComponent->NumSubsections)
	, SubsectionSizeQuads(InComponent->SubsectionSizeQuads)
	, SubsectionSizeVerts(InComponent->SubsectionSizeQuads + 1)
	, ComponentSizeQuads(InComponent->ComponentSizeQuads)
	, ComponentSizeVerts(InComponent->ComponentSizeQuads + 1)
	, NumRayTracingSections(GLandscapeRayTracingSections)
	, SectionBase(InComponent->GetSectionBase())
	, bUsesLandscapeCulling(false)
	, WeightmapScaleBias(InComponent->WeightmapScaleBias)
	, VisibilityWeightmapTexture(nullptr)
	, VisibilityWeightmapChannel(-1)
	, HeightmapTexture(InComponent->GetHeightmap())
	, HeightmapScaleBias(InComponent->HeightmapScaleBias)
	, SharedBuffersKey(0)
	, SharedBuffers(nullptr)
	, VertexFactory(nullptr)
	, FixedGridVertexFactory(nullptr)
	, ComponentLightInfo(nullptr)
	, NumRelevantMips(InComponent->GetNumRelevantMips())
#if WITH_EDITORONLY_DATA
	, EditToolRenderData(InComponent->EditToolRenderData)
#endif
#if WITH_EDITOR || !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	, CollisionMipLevel(InComponent->CollisionMipLevel)
	, SimpleCollisionMipLevel(InComponent->SimpleCollisionMipLevel)
	, CollisionResponse(InComponent->GetLandscapeProxy()->BodyInstance.GetResponseToChannels())
#endif
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	, LightMapResolution(InComponent->GetStaticLightMapResolution())
#endif
{
	bIsLandscapeProxy = true;
	
	// Landscape meshes do not deform internally (save by material effects such as WPO and PDO, which is allowed).
	// They do however have continuous LOD which is problematic, considered static as the LODs (are intended to) represent the same static surface.
	bHasDeformableMesh = false;

	// Enabled just so the GVarDumpLandscapeLODsCurrentFrame debug feature only runs once per render proxy in GetDynamicMeshElements
	bSinglePassGDME = true;

	bNaniteActive = InComponent->IsNaniteActive();
	bUsesLandscapeCulling = (!bNaniteActive && Culling::UseCulling(GetScene().GetShaderPlatform()));

	// Landscape meshes can write to runtime virtual texture if they are set to do so.
	bSupportsRuntimeVirtualTexture = true;

	EnableGPUSceneSupportFlags();

	const ERHIFeatureLevel::Type FeatureLevel = GetScene().GetFeatureLevel();

	auto GetRenderProxy = 
		[](const TObjectPtr<UMaterialInterface>& Material)
		{
			return Material ? Material->GetRenderProxy() : nullptr;
		};
	TArray<UMaterialInterface*> AvailableMaterialInterfaces;
	if (FeatureLevel == ERHIFeatureLevel::ES3_1)
	{
		WeightmapTextures = InComponent->MobileWeightmapTextures;
		Algo::Transform(InComponent->MobileMaterialInterfaces, AvailableMaterials, GetRenderProxy);
		AvailableMaterialInterfaces.Append(InComponent->MobileMaterialInterfaces);
		//TODO: Add support for bUseDynamicMaterialInstance ?
	}
	else
	{
		WeightmapTextures = InComponent->GetWeightmapTextures();
		if (InComponent->GetLandscapeProxy()->bUseDynamicMaterialInstance)
		{
			Algo::Transform(InComponent->MaterialInstancesDynamic, AvailableMaterials, GetRenderProxy);
			AvailableMaterialInterfaces.Append(InComponent->MaterialInstancesDynamic);
		}
		else
		{
			Algo::Transform(InComponent->MaterialInstances, AvailableMaterials, GetRenderProxy);
			AvailableMaterialInterfaces.Append(InComponent->MaterialInstances);
		}
	}

	LODIndexToMaterialIndex = InComponent->LODIndexToMaterialIndex;
	check(LODIndexToMaterialIndex.Num() == MaxLOD + 1);

	HeightmapSubsectionOffsetU = 0;
	HeightmapSubsectionOffsetV = 0;
	if (HeightmapTexture)
	{
		HeightmapSubsectionOffsetU = ((float)(InComponent->SubsectionSizeQuads + 1) / (float)FMath::Max<int32>(1, HeightmapTexture->GetSizeX()));
		HeightmapSubsectionOffsetV = ((float)(InComponent->SubsectionSizeQuads + 1) / (float)FMath::Max<int32>(1, HeightmapTexture->GetSizeY()));
	}


	const ALandscapeProxy* Proxy = InComponent->GetLandscapeProxy();

	float LOD0ScreenSize;
	float LOD0Distribution;
	float LODDistribution;
	VirtualShadowMapConstantDepthBias = Proxy->NonNaniteVirtualShadowMapConstantDepthBias;
	VirtualShadowMapInvalidationHeightErrorThreshold = Proxy->NonNaniteVirtualShadowMapInvalidationHeightErrorThreshold;
	float NonNaniteVirtualShadowMapInvalidationScreenSizeLimit = Proxy->NonNaniteVirtualShadowMapInvalidationScreenSizeLimit;
	
	if (Proxy->bUseScalableLODSettings)
	{
		const int32 LandscapeQuality = Scalability::GetQualityLevels().LandscapeQuality;
		
		LOD0ScreenSize = Proxy->ScalableLOD0ScreenSize.GetValue(LandscapeQuality);
		LOD0Distribution = Proxy->ScalableLOD0DistributionSetting.GetValue(LandscapeQuality);
		LODDistribution = Proxy->ScalableLODDistributionSetting.GetValue(LandscapeQuality);		
	}
	else
	{
		LOD0ScreenSize = Proxy->LOD0ScreenSize;
		LOD0Distribution = Proxy->LOD0DistributionSetting * GLandscapeLOD0DistributionScale;
		LODDistribution = Proxy->LODDistributionSetting * GLandscapeLODDistributionScale;
	}

#if !UE_BUILD_SHIPPING
	if (GLandscapeLOD0ScreenSizeOverride > 0.0)
	{
		LOD0ScreenSize = GLandscapeLOD0ScreenSizeOverride;
	}
	if (GLandscapeLOD0DistributionOverride > 0.0)
	{
		LOD0Distribution = GLandscapeLOD0DistributionOverride;
	}
	if (GLandscapeLODDistributionOverride > 0.0)
	{
		LODDistribution = GLandscapeLODDistributionOverride;
	}
	if (GLandscapeNonNaniteVirtualShadowMapConstantDepthBiasOverride > 0.0)
	{
		VirtualShadowMapConstantDepthBias = GLandscapeNonNaniteVirtualShadowMapConstantDepthBiasOverride;
	}
	if (GLandscapeNonNaniteVirtualShadowMapInvalidationHeightErrorThresholdOverride > 0.0)
	{
		VirtualShadowMapInvalidationHeightErrorThreshold = GLandscapeNonNaniteVirtualShadowMapInvalidationHeightErrorThresholdOverride;
	}
	if (GLandscapeNonNaniteVirtualShadowMapInvalidationScreenSizeLimitOverride > 0.0)
	{
		NonNaniteVirtualShadowMapInvalidationScreenSizeLimit = GLandscapeNonNaniteVirtualShadowMapInvalidationScreenSizeLimitOverride;
	}

	// For the display name, some timing issues can lead to a temporarily invalid parent actor, so fallback to the parent actor's name if that's the case, no big deal : 
	const FString& LandscapeName = InComponent->GetLandscapeActor() ? InComponent->GetLandscapeActor()->GetName() : Proxy->GetName();
	const FString& ComponentName = InComponent->GetName();
	DebugName = FName(FString::Printf(TEXT(" Landscape: %s, Component: %s [%s]"), *ComponentName, *LandscapeName, *SectionBase.ToString()));
#endif // !UE_BUILD_SHIPPING

	// Precompute screen ratios for each LOD level : 
	{
		float ScreenSizeRatioDivider = FMath::Max(LOD0Distribution, 1.01f);
		// Cancel out so that landscape is not affected by r.StaticMeshLODDistanceScale
		float CurrentScreenSizeRatio = LOD0ScreenSize / CVarStaticMeshLODDistanceScale.GetValueOnAnyThread();

		LODScreenRatioSquared.AddUninitialized(MaxLOD + 1);

		// LOD 0 handling
		LODScreenRatioSquared[0] = FMath::Square(CurrentScreenSizeRatio);
		LODSettings.LOD0ScreenSizeSquared = FMath::Square(CurrentScreenSizeRatio);
		CurrentScreenSizeRatio /= ScreenSizeRatioDivider;
		LODSettings.LOD1ScreenSizeSquared = FMath::Square(CurrentScreenSizeRatio);
		ScreenSizeRatioDivider = FMath::Max(LODDistribution, 1.01f);
		LODSettings.LODOnePlusDistributionScalarSquared = FMath::Square(ScreenSizeRatioDivider);

		// Other LODs
		for (int32 LODIndex = 1; LODIndex <= MaxLOD; ++LODIndex) // This should ALWAYS be calculated from the component size, not user MaxLOD override
		{
			LODScreenRatioSquared[LODIndex] = FMath::Square(CurrentScreenSizeRatio);
			CurrentScreenSizeRatio /= ScreenSizeRatioDivider;
		}
	}

	FirstLOD = 0;
	LastLOD = MaxLOD;	// we always need to go to MaxLOD regardless of LODBias as we could need the lowest LODs due to streaming.

	// Make sure out LastLOD is > of MinStreamedLOD otherwise we would not be using the right LOD->MIP, the only drawback is a possible minor memory usage for overallocating static mesh element batch
	const int32 MinStreamedLOD = HeightmapTexture ? FMath::Min<int32>(HeightmapTexture->GetNumMips() - HeightmapTexture->GetNumResidentMips(), FMath::CeilLogTwo(SubsectionSizeVerts) - 1) : 0;
	LastLOD = FMath::Max(MinStreamedLOD, LastLOD);

	// Clamp to MaxLODLevel
	const int32 MaxLODLevel = InComponent->GetLandscapeProxy()->MaxLODLevel;
	if (MaxLODLevel >= 0)
	{
		MaxLOD = FMath::Min<int8>(static_cast<int8>(MaxLODLevel), MaxLOD);
		LastLOD = FMath::Min<int32>(MaxLODLevel, LastLOD);
	}

	// Clamp ForcedLOD to the valid range and then apply
	int8 ForcedLOD = static_cast<int8>(InComponent->ForcedLOD);
	ForcedLOD = static_cast<int8>(ForcedLOD >= 0 ? FMath::Clamp<int32>(ForcedLOD, FirstLOD, LastLOD) : ForcedLOD);
	FirstLOD = ForcedLOD >= 0 ? ForcedLOD : FirstLOD;
	LastLOD = ForcedLOD >= 0 ? ForcedLOD : LastLOD;

	LODSettings.LastLODIndex = static_cast<int8>(LastLOD);
	LODSettings.LastLODScreenSizeSquared = LODScreenRatioSquared[LastLOD];
	LODSettings.ForcedLOD = ForcedLOD;

	bool bVirtualTextureRenderWithQuad = InComponent->GetLandscapeProxy()->bVirtualTextureRenderWithQuad;
	bVirtualTextureRenderWithQuad |= GLandscapeRuntimeVirtualTextureRenderWithQuad == 1;
	bVirtualTextureRenderWithQuad &= GLandscapeRuntimeVirtualTextureRenderWithQuad != 2;
	const bool bVirtualTextureRenderWithQuadHQ = InComponent->GetLandscapeProxy()->bVirtualTextureRenderWithQuadHQ;
	VirtualTexturePerPixelHeight = bVirtualTextureRenderWithQuad ? bVirtualTextureRenderWithQuadHQ ? 2 : 1 : 0;

	LastVirtualTextureLOD = MaxLOD;
	FirstVirtualTextureLOD = bVirtualTextureRenderWithQuad ? MaxLOD : FMath::Max(MaxLOD - InComponent->GetLandscapeProxy()->VirtualTextureNumLods, 0);
	VirtualTextureLodBias = static_cast<int8>(bVirtualTextureRenderWithQuad ? 0 : InComponent->GetLandscapeProxy()->VirtualTextureLodBias);

#if WITH_EDITOR || !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	LODSettings.DrawCollisionPawnLOD = static_cast<int8>(CollisionResponse.GetResponse(ECC_Pawn) == ECR_Ignore ? -1 : SimpleCollisionMipLevel);
	LODSettings.DrawCollisionVisibilityLOD = static_cast<int8>(CollisionResponse.GetResponse(ECC_Visibility) == ECR_Ignore ? -1 : CollisionMipLevel);
#else
	LODSettings.DrawCollisionPawnLOD = LODSettings.DrawCollisionVisibilityLOD = -1;
#endif

	// Pre-compute the last LOD value at which VSM invalidation will stop occurring : 
	if (NonNaniteVirtualShadowMapInvalidationScreenSizeLimit <= 0.0f)
	{
		// Special case to disable attenuation of invalidation altogether : 
		LODSettings.VirtualShadowMapInvalidationLimitLOD = -1.0f;
	}
	else
	{
		// Screen size under which VSM invalidation stops occurring :
		float VirtualShadowMapInvalidationScreenSizeLimitSquared = FMath::Square(NonNaniteVirtualShadowMapInvalidationScreenSizeLimit);
		LODSettings.VirtualShadowMapInvalidationLimitLOD = 0.0f;
		if (VirtualShadowMapInvalidationScreenSizeLimitSquared <= LODScreenRatioSquared[MaxLOD])
		{
			LODSettings.VirtualShadowMapInvalidationLimitLOD = MaxLOD;
		}
		else
		{
			// Since the screen size distribution between LOD levels is not linear, we have to iterate through the LOD levels to find 
			//  where NonNaniteVirtualShadowMapInvalidationScreenSizeLimit lays and convert back to a LOD value : 
			float CurrentScreenSizeRatioSquared = LODScreenRatioSquared[0];
			for (int32 LODIndex = 1; LODIndex < MaxLOD; ++LODIndex) 
			{
				float NextScreenSizeRatioSquared = LODScreenRatioSquared[LODIndex];
				if ((VirtualShadowMapInvalidationScreenSizeLimitSquared > NextScreenSizeRatioSquared) && (VirtualShadowMapInvalidationScreenSizeLimitSquared <= CurrentScreenSizeRatioSquared))
				{
					LODSettings.VirtualShadowMapInvalidationLimitLOD = LODIndex + (CurrentScreenSizeRatioSquared - VirtualShadowMapInvalidationScreenSizeLimitSquared) / (CurrentScreenSizeRatioSquared - NextScreenSizeRatioSquared);
					break;
				}
				CurrentScreenSizeRatioSquared = NextScreenSizeRatioSquared;
			}
		}
	}

	const FVector ComponentScale = InComponent->GetComponentTransform().GetScale3D();
	ComponentMaxExtend = static_cast<float>(SubsectionSizeQuads * FMath::Max(ComponentScale.X, ComponentScale.Y));

	if (InComponent->StaticLightingResolution > 0.f)
	{
		StaticLightingResolution = InComponent->StaticLightingResolution;
	}
	else
	{
		StaticLightingResolution = InComponent->GetLandscapeProxy()->StaticLightingResolution;
	}

	// Landscape GPU culling uses VF that requires primitive UB
	// Mobile does not use GPUScene and requires primitive UB for landscape, see FLandscapeVertexFactory::ModifyCompilationEnvironment
	bVFRequiresPrimitiveUniformBuffer |= (bUsesLandscapeCulling || FeatureLevel == ERHIFeatureLevel::ES3_1);

	ComponentLightInfo = MakeUnique<FLandscapeLCI>(InComponent, FeatureLevel, bVFRequiresPrimitiveUniformBuffer != 0);
	check(ComponentLightInfo);

	const bool bHasStaticLighting = ComponentLightInfo->GetLightMap() || ComponentLightInfo->GetShadowMap();
	
	bool bUsePSOPrecacheFallbackMaterial = false;
#if UE_WITH_PSO_PRECACHING
	bUsePSOPrecacheFallbackMaterial = InComponent->GetPSOPrecacheComponentData().IsPSOPrecaching() && UE::Landscape::GetLandscapePSOPrecacheProxyCreationStrategy() == EPSOPrecacheProxyCreationStrategy::UseFallbackMaterialUntilPSOPrecached;
#endif // UE_WITH_PSO_PRECACHING

	check(AvailableMaterialInterfaces.Num() == AvailableMaterials.Num());
	// Check material usage and validity. Replace invalid entries by default material so that indexing AvailableMaterials with LODIndexToMaterialIndex still works :
	if (ensure(AvailableMaterialInterfaces.Num() > 0))
	{
		for(int Index = 0; Index < AvailableMaterialInterfaces.Num(); ++Index)
		{
			bool bIsValidMaterial = false;
			UMaterialInterface* MaterialInterface = AvailableMaterialInterfaces[Index];
			if (MaterialInterface != nullptr)
			{
				bIsValidMaterial = true;

				const UMaterial* LandscapeMaterial = MaterialInterface->GetMaterial_Concurrent();

				// In some case it's possible that the Material Instance we have and the Material are not related, for example, in case where content was force deleted, we can have a MIC with no parent, so GetMaterial will fallback to the default material.
				// and since the MIC is not really valid, fallback to 
				UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(MaterialInterface);
				bIsValidMaterial &= (MaterialInstance == nullptr) || MaterialInstance->IsChildOf(LandscapeMaterial);

				// Check usage flags : 
				bIsValidMaterial &= !bHasStaticLighting || MaterialInterface->CheckMaterialUsage_Concurrent(MATUSAGE_StaticLighting);
			}

			if (!bIsValidMaterial || bUsePSOPrecacheFallbackMaterial)
			{
				// Replace the landscape material by the default material or fallback PSO precaching material 
				MaterialInterface = (bUsePSOPrecacheFallbackMaterial && UPSOPrecacheSettingsManager::GetFallbackMaterial()) ? UPSOPrecacheSettingsManager::GetFallbackMaterial() : UMaterial::GetDefaultMaterial(MD_Surface);
				AvailableMaterialInterfaces[Index] = MaterialInterface;
				AvailableMaterials[Index] = MaterialInterface->GetRenderProxy();
			}
		}
	}
	else
	{
		AvailableMaterialInterfaces.Add(UMaterial::GetDefaultMaterial(MD_Surface));
		AvailableMaterials.Add(AvailableMaterialInterfaces.Last()->GetRenderProxy());
	}

	EShaderPlatform ShaderPlatform = GetScene().GetShaderPlatform();

	Algo::Transform(AvailableMaterialInterfaces, MaterialRelevances, [ShaderPlatform](UMaterialInterface* InMaterialInterface) { check(InMaterialInterface != nullptr); return InMaterialInterface->GetRelevance_Concurrent(ShaderPlatform); });

	const int8 SubsectionSizeLog2 = static_cast<int8>(FMath::CeilLogTwo(InComponent->SubsectionSizeQuads + 1));
	SharedBuffersKey = (SubsectionSizeLog2 & 0xf) | ((NumSubsections & 0xf) << 4) | ((NumRayTracingSections & 0xf) << 8);

	bSupportsHeightfieldRepresentation = true;

	// Find where the visibility weightmap lies, if available
	// TODO: Mobile has its own MobileWeightmapLayerAllocations, and visibility layer could be in a different channel potentially?
	if (FeatureLevel > ERHIFeatureLevel::ES3_1)
	{
		for (int32 Idx = 0; Idx < InComponent->WeightmapLayerAllocations.Num(); Idx++)
		{
			const FWeightmapLayerAllocationInfo& Allocation = InComponent->WeightmapLayerAllocations[Idx];
			if (Allocation.GetLayerName() == UMaterialExpressionLandscapeVisibilityMask::ParameterName && Allocation.IsAllocated())
			{
				VisibilityWeightmapTexture = WeightmapTextures[Allocation.WeightmapTextureIndex];
				VisibilityWeightmapChannel = Allocation.WeightmapTextureChannel;
				break;
			}
		}
	}

#if WITH_EDITOR
	LandscapeEditResourcesSubsystem = GEngine->GetEngineSubsystem<ULandscapeEditResourcesSubsystem>();

	const TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = InComponent->GetWeightmapLayerAllocations();
	for (const FWeightmapLayerAllocationInfo& Allocation : ComponentWeightmapLayerAllocations)
	{
		if (Allocation.LayerInfo != nullptr)
		{
			LayerColors.Add(Allocation.LayerInfo->GetLayerUsageDebugColor());
		}
	}
#endif

	bOpaqueOrMasked = true; // Landscape is always opaque
	UpdateVisibleInLumenScene();

	// Read and transform MipToMipMaxDeltas to world space to avoid doing this at runtime: 
	WorldSpaceMipToMipMaxDeltas.Reserve(InComponent->MipToMipMaxDeltas.Num());
	Algo::Transform(InComponent->MipToMipMaxDeltas, WorldSpaceMipToMipMaxDeltas, [ZScale = ComponentScale.Z](double InMaxDelta) { return ZScale * InMaxDelta; });
}

void FLandscapeComponentSceneProxy::CreateRenderThreadResources(FRHICommandListBase& RHICmdList)
{
	LLM_SCOPE(ELLMTag::Landscape);

	check(HeightmapTexture != nullptr);

	FLandscapeRenderSystem::CreateResources(RHICmdList, this);

	if (!IsForceHidden())
	{
		FLandscapeRenderSystem::RegisterSection(this);
	}

	auto FeatureLevel = GetScene().GetFeatureLevel();

	SharedBuffers = FLandscapeComponentSceneProxy::SharedBuffersMap.FindRef(SharedBuffersKey);
	if (SharedBuffers == nullptr)
	{
		FName BufferOwnerName;
#if !UE_BUILD_SHIPPING
		BufferOwnerName = DebugName;
#endif // !UE_BUILD_SHIPPING

		SharedBuffers = new FLandscapeSharedBuffers(
			RHICmdList, SharedBuffersKey, SubsectionSizeQuads, NumSubsections, NumRayTracingSections, FeatureLevel, BufferOwnerName);

		FLandscapeComponentSceneProxy::SharedBuffersMap.Add(SharedBuffersKey, SharedBuffers);

		FLandscapeVertexFactory* LandscapeVertexFactory = new FLandscapeVertexFactory(FeatureLevel);
		LandscapeVertexFactory->Data.PositionComponent = FVertexStreamComponent(SharedBuffers->VertexBuffer, 0, sizeof(FLandscapeVertex), VET_UByte4);
		LandscapeVertexFactory->InitResource(RHICmdList);
		SharedBuffers->VertexFactory = LandscapeVertexFactory;

		bool bNeedsFixedGridVertexFactory = NeedsFixedGridVertexFactory(GetScene().GetShaderPlatform());

		// We also need the fixed vertex factor for grass/physical materials : 
		bNeedsFixedGridVertexFactory |= (SharedBuffers->GrassIndexBuffer != nullptr);

		if (bNeedsFixedGridVertexFactory)
		{
			FLandscapeFixedGridVertexFactory* FixedLandscapeVertexFactory = new FLandscapeFixedGridVertexFactory(FeatureLevel);
			FixedLandscapeVertexFactory->Data.PositionComponent = FVertexStreamComponent(SharedBuffers->VertexBuffer, 0, sizeof(FLandscapeVertex), VET_UByte4);
			FixedLandscapeVertexFactory->InitResource(RHICmdList);
			SharedBuffers->FixedGridVertexFactory = FixedLandscapeVertexFactory;
		}
	}

	SharedBuffers->AddRef();

	// Assign vertex factory
	VertexFactory = SharedBuffers->VertexFactory;
	FixedGridVertexFactory = SharedBuffers->FixedGridVertexFactory;

	if (bUsesLandscapeCulling)
	{
		Culling::RegisterLandscape(RHICmdList, *SharedBuffers, FeatureLevel, LandscapeKey, SubsectionSizeVerts, NumSubsections);
	}
	
	// Assign LandscapeUniformShaderParameters
	LandscapeUniformShaderParameters.InitResource(RHICmdList);

	const int32 NumMips = FMath::CeilLogTwo(SubsectionSizeVerts);
	// Create MeshBatch for grass rendering
	if (SharedBuffers->GrassIndexBuffer)
	{
		check(FixedGridVertexFactory != nullptr);

		GrassMeshBatch.Elements.Empty(NumMips);
		GrassMeshBatch.Elements.AddDefaulted(NumMips);
		GrassBatchParams.Empty(NumMips);
		GrassBatchParams.AddDefaulted(NumMips);

		// Grass is being generated using LOD0 material only and uses the fixed grid vertex factory
		FMaterialRenderProxy* RenderProxy = AvailableMaterials[LODIndexToMaterialIndex[0]];
		GrassMeshBatch.VertexFactory = FixedGridVertexFactory;
		GrassMeshBatch.MaterialRenderProxy = RenderProxy;
		GrassMeshBatch.LCI = nullptr;
		GrassMeshBatch.ReverseCulling = false;
		GrassMeshBatch.CastShadow = false;
		GrassMeshBatch.Type = PT_PointList;
		GrassMeshBatch.DepthPriorityGroup = SDPG_World;

		// Combined grass rendering batch element
		FMeshBatchElement* GrassBatchElement = &GrassMeshBatch.Elements[0];
		FLandscapeBatchElementParams* BatchElementParams = &GrassBatchParams[0];
		BatchElementParams->LandscapeUniformShaderParametersResource = &LandscapeUniformShaderParameters;
		BatchElementParams->FixedGridUniformShaderParameters = &SharedBuffers->LandscapeFixedGridUniformShaderParameters;
		BatchElementParams->LandscapeSectionLODUniformParameters = nullptr; // Not needed for grass rendering
		BatchElementParams->SceneProxy = this;
		BatchElementParams->CurrentLOD = 0;
		GrassBatchElement->UserData = BatchElementParams;
		GrassBatchElement->PrimitiveUniformBuffer = GetUniformBuffer();
		GrassBatchElement->IndexBuffer = SharedBuffers->GrassIndexBuffer;
		GrassBatchElement->NumPrimitives = FMath::Square(NumSubsections) * FMath::Square(SubsectionSizeVerts);
		GrassBatchElement->FirstIndex = 0;
		GrassBatchElement->MinVertexIndex = 0;
		GrassBatchElement->MaxVertexIndex = SharedBuffers->NumVertices - 1;

		// Grass system is also used to bake out heights which are source for collision data when bBakeMaterialPositionOffsetIntoCollision is enabled
		for (int32 Mip = 1; Mip < NumMips; ++Mip)
		{
			const int32 MipSubsectionSizeVerts = SubsectionSizeVerts >> Mip;

			FMeshBatchElement* CollisionBatchElement = &GrassMeshBatch.Elements[Mip];
			*CollisionBatchElement = *GrassBatchElement;
			FLandscapeBatchElementParams* CollisionBatchElementParams = &GrassBatchParams[Mip];
			*CollisionBatchElementParams = *BatchElementParams;
			CollisionBatchElementParams->CurrentLOD = Mip;
			CollisionBatchElement->UserData = CollisionBatchElementParams;
			CollisionBatchElement->NumPrimitives = FMath::Square(NumSubsections) * FMath::Square(MipSubsectionSizeVerts);
			CollisionBatchElement->FirstIndex = SharedBuffers->GrassIndexMipOffsets[Mip];
		}
	}
}

#if RHI_RAYTRACING

FLandscapeRayTracingState* FLandscapeRayTracingImpl::FindOrCreateRayTracingState(
	FRHICommandListBase& RHICmdList, 
	FSceneViewStateInterface* ViewStateInterface, 
	int32 NumSubsections,
	int32 NumRayTracingSections,
	int32 RayTracingSectionVertCount)
{
	// Default view key of zero if there's no ViewStateInterface provided.  View keys start at 1, so 0 wouldn't be a valid key on an actual view.
	uint32 ViewKey = 0;
	FSceneViewStateInterface* ViewState = nullptr;
	if (ViewStateInterface)
	{
		ViewState = ViewStateInterface;
		ViewKey = ViewState->GetViewKey();
	}

	// Check for existing state for this view.  We're just doing a linear search of the array, because practical applications won't have
	// more than two or three views running ray tracing, for overall frame performance reasons.  If this assumption changes, we could
	// implement a more efficient lookup in the future.
	for (FLandscapeRayTracingState& PerView : PerViewRayTracingState)
	{
		if (PerView.ViewKey == ViewKey)
		{
			return &PerView;
		}
	}

	// Need to create a new one
	FLandscapeRayTracingState* RayTracingState = &PerViewRayTracingState.AddDefaulted_GetRef();

	RayTracingState->ViewKey = ViewKey;

	// Initialize rendering data
	uint32 TotalSections = FMath::Square(NumRayTracingSections * NumSubsections);
	RayTracingState->Sections.SetNum(TotalSections);

	for (int32 SubY = 0; SubY < NumSubsections; SubY++)
	{
		for (int32 SubX = 0; SubX < NumSubsections; SubX++)
		{
			for (int32 RTSubY = 0; RTSubY < NumRayTracingSections; RTSubY++)
			{
				for (int32 RTSubX = 0; RTSubX < NumRayTracingSections; RTSubX++)
				{
					const int32 SubSectionIdx = SubX + SubY * NumSubsections;
					const int32 InstanceIdx = SubSectionIdx * FMath::Square(NumRayTracingSections) + RTSubX + RTSubY * NumRayTracingSections;

					FLandscapeSectionRayTracingState& SectionRayTracingState = RayTracingState->Sections[InstanceIdx];

					FRayTracingGeometryInitializer Initializer;
					static const FName DebugName("FLandscapeComponentSceneProxy");
					static int32 DebugNumber = 0;
					Initializer.DebugName = FDebugName(DebugName, DebugNumber++);
					Initializer.IndexBuffer = nullptr;
					Initializer.GeometryType = RTGT_Triangles;
					Initializer.bFastBuild = true;
					Initializer.bAllowUpdate = true;

					FRayTracingGeometrySegment Segment;
					Segment.VertexBuffer = nullptr;
					Segment.VertexBufferStride = sizeof(FVector3f);
					Segment.VertexBufferElementType = VET_Float3;
					Segment.MaxVertices = FMath::Square(RayTracingSectionVertCount);
					Initializer.Segments.Add(Segment);

					SectionRayTracingState.Geometry.SetInitializer(Initializer);
					SectionRayTracingState.Geometry.InitResource(RHICmdList);

					FLandscapeVertexFactoryMVFParameters UniformBufferParams;
					UniformBufferParams.SubXY = FIntPoint(SubX, SubY);
					UniformBufferParams.RayTracingSectionXY = FIntPoint(RTSubX, RTSubY);
					SectionRayTracingState.UniformBuffer = FLandscapeVertexFactoryMVFUniformBufferRef::CreateUniformBufferImmediate(UniformBufferParams, UniformBuffer_MultiFrame);
				}
			}
		}
	}

	return RayTracingState;
}

FLandscapeRayTracingState::~FLandscapeRayTracingState()
{
	// Clean up the contents
	for (FLandscapeSectionRayTracingState& SectionRayTracingState : Sections)
	{
		SectionRayTracingState.Geometry.ReleaseResource();
		SectionRayTracingState.RayTracingDynamicVertexBuffer.Release();
	}
}
#endif	// RHI_RAYTRACING

void FLandscapeComponentSceneProxy::DestroyRenderThreadResources()
{
	FPrimitiveSceneProxy::DestroyRenderThreadResources();
	FLandscapeRenderSystem::UnregisterSection(this);
	FLandscapeRenderSystem::DestroyResources(this);
	Culling::UnregisterLandscape(LandscapeKey);
}

void FLandscapeComponentSceneProxy::OnForceHiddenChanged()
{
	if (IsForceHidden())
	{
		FLandscapeRenderSystem::UnregisterSection(this);
	}
	else
	{
		FLandscapeRenderSystem::RegisterSection(this);
	}
}

FLandscapeComponentSceneProxy::~FLandscapeComponentSceneProxy()
{
	// Free the subsection uniform buffer
	LandscapeUniformShaderParameters.ReleaseResource();

	if (SharedBuffers)
	{
		check(SharedBuffers == FLandscapeComponentSceneProxy::SharedBuffersMap.FindRef(SharedBuffersKey));
		SharedBuffers->Release();
		SharedBuffers = nullptr;
	}
}

bool FLandscapeComponentSceneProxy::CanBeOccluded() const
{
	if (IsRuntimeVirtualTextureOnly())
	{
		return false;
	}

	for (const FMaterialRelevance& Relevance : MaterialRelevances)
	{
		if (!Relevance.bDisableDepthTest)
		{
			return true;
		}
	}

	return false;
}

FPrimitiveViewRelevance FLandscapeComponentSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	FPrimitiveViewRelevance Result;
	const bool bCollisionView = (View->Family->EngineShowFlags.CollisionVisibility || View->Family->EngineShowFlags.CollisionPawn);
	Result.bDrawRelevance = (IsShown(View) || bCollisionView) && View->Family->EngineShowFlags.Landscape;
	Result.bRenderInMainPass = ShouldRenderInMainPass();
	Result.bRenderCustomDepth = ShouldRenderCustomDepth();
	Result.bUsesLightingChannels = GetLightingChannelMask() != GetDefaultLightingChannelMask();
	Result.bTranslucentSelfShadow = bCastVolumetricTranslucentShadow;

	auto FeatureLevel = View->GetFeatureLevel();
	EShaderPlatform ShaderPlatform = View->GetShaderPlatform();

#if WITH_EDITOR
	if (!GLandscapeEditModeActive)
	{
		// No tools to render, just use the cached material relevance.
#endif
		for (const FMaterialRelevance& MaterialRelevance : MaterialRelevances)
		{
			MaterialRelevance.SetPrimitiveViewRelevance(Result);
		}

#if WITH_EDITOR
	}
	else
	{
		for (const FMaterialRelevance& MaterialRelevance : MaterialRelevances)
		{
			// Also add the tool material(s)'s relevance to the MaterialRelevance
			FMaterialRelevance ToolRelevance = MaterialRelevance;

			// Tool brushes and Gizmo
			if (EditToolRenderData.ToolMaterial)
			{
				Result.bDynamicRelevance = true;
				ToolRelevance |= EditToolRenderData.ToolMaterial->GetRelevance_Concurrent(ShaderPlatform);
			}

			if (EditToolRenderData.GizmoMaterial)
			{
				Result.bDynamicRelevance = true;
				ToolRelevance |= EditToolRenderData.GizmoMaterial->GetRelevance_Concurrent(ShaderPlatform);
			}

			// Region selection
			if (EditToolRenderData.SelectedType)
			{
				TObjectPtr<UMaterialInterface> SelectionRegionMaterial = LandscapeEditResourcesSubsystem->GetSelectionRegionMaterial();
				TObjectPtr<UMaterialInterface> SelectionColorMaterial = LandscapeEditResourcesSubsystem->GetSelectionColorMaterial();

				if ((GLandscapeEditRenderMode & ELandscapeEditRenderMode::SelectRegion) && (EditToolRenderData.SelectedType & FLandscapeEditToolRenderData::ST_REGION)
					&& !(GLandscapeEditRenderMode & ELandscapeEditRenderMode::Mask) && SelectionRegionMaterial)
				{
					Result.bDynamicRelevance = true;
					ToolRelevance |= SelectionRegionMaterial->GetRelevance_Concurrent(ShaderPlatform);
				}
				if ((GLandscapeEditRenderMode & ELandscapeEditRenderMode::SelectComponent) && (EditToolRenderData.SelectedType & FLandscapeEditToolRenderData::ST_COMPONENT) && SelectionColorMaterial)
				{
					Result.bDynamicRelevance = true;
					ToolRelevance |= SelectionColorMaterial->GetRelevance_Concurrent(ShaderPlatform);
				}
			}

			TObjectPtr<UMaterialInterface> MaskRegionMaterial = LandscapeEditResourcesSubsystem->GetMaskRegionMaterial();
			TObjectPtr<UMaterialInterface> ColorMaskRegionMaterial = LandscapeEditResourcesSubsystem->GetColorMaskRegionMaterial();
			TObjectPtr<UMaterialInterface> LandscapeDirtyMaterial = LandscapeEditResourcesSubsystem->GetLandscapeDirtyMaterial();

			// Mask
			if ((GLandscapeEditRenderMode & ELandscapeEditRenderMode::Mask) && MaskRegionMaterial != nullptr &&
				(((EditToolRenderData.SelectedType & FLandscapeEditToolRenderData::ST_REGION)) || (!(GLandscapeEditRenderMode & ELandscapeEditRenderMode::InvertedMask))))
			{
				Result.bDynamicRelevance = true;
				ToolRelevance |= MaskRegionMaterial->GetRelevance_Concurrent(ShaderPlatform);
			}

			if (GLandscapeViewMode == ELandscapeViewMode::LayerContribution)
			{
				Result.bDynamicRelevance = true;
				ToolRelevance |= ColorMaskRegionMaterial->GetRelevance_Concurrent(ShaderPlatform);
			}

			if (CVarLandscapeShowDirty.GetValueOnRenderThread() && LandscapeDirtyMaterial)
			{
				Result.bDynamicRelevance = true;
				ToolRelevance |= LandscapeDirtyMaterial->GetRelevance_Concurrent(ShaderPlatform);
			}

			ToolRelevance.SetPrimitiveViewRelevance(Result);
		}
	}

	// Various visualizations need to render using dynamic relevance
	if ((View->Family->EngineShowFlags.Bounds && IsSelected()) ||
		GLandscapeDebugOptions.bShowPatches)
	{
		Result.bDynamicRelevance = true;
	}
#endif

#if WITH_EDITOR || !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	const bool bInCollisionView = View->Family->EngineShowFlags.CollisionVisibility || View->Family->EngineShowFlags.CollisionPawn;
#endif

	// Use the dynamic path for rendering landscape components pass only for Rich Views or if the static path is disabled for debug.
	if (IsRichView(*View->Family) ||
#if WITH_EDITOR || !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		bInCollisionView ||
#endif
		GLandscapeDebugOptions.bDisableStatic ||
		View->Family->EngineShowFlags.Wireframe ||
#if WITH_EDITOR
		(IsSelected() && !GLandscapeEditModeActive) ||
		(GLandscapeViewMode != ELandscapeViewMode::Normal) ||
		(CVarLandscapeShowDirty.GetValueOnAnyThread() && LandscapeEditResourcesSubsystem->GetLandscapeDirtyMaterial()) ||
		(GetViewLodOverride(*View, LandscapeKey) >= 0)
#else
		IsSelected() ||
		(View->CustomRenderPass && GetViewLodOverride(*View, LandscapeKey) >= 0)
#endif
		)
	{
		Result.bDynamicRelevance = true;
	}
	else
	{
		Result.bStaticRelevance = true;
#if WITH_EDITOR
		Result.bEditorVisualizeLevelInstanceRelevance = IsEditingLevelInstanceChild();
#endif
	}

	Result.bVelocityRelevance = (VelocityIncludeStationaryPrimitives(View->GetShaderPlatform()) || DrawsVelocity()) && Result.bOpaque && Result.bRenderInMainPass;
	Result.bShadowRelevance = (GAllowLandscapeShadows > 0) && IsShadowCast(View) && View->Family->EngineShowFlags.Landscape;

#if !UE_BUILD_SHIPPING
	if (GVarDumpLandscapeLODsCurrentFrame != 0)
	{
		if (GVarDumpLandscapeLODsCurrentFrame == GFrameNumberRenderThread)
		{
			// Enable dynamic relevance to let the dump code run on GetDynamicMeshElements : 
			Result.bDynamicRelevance = true;
		}
		// Assume we've dumped the info already, reset the counter : 
		else if (GVarDumpLandscapeLODsCurrentFrame < GFrameNumberRenderThread)
		{
			GVarDumpLandscapeLODsCurrentFrame = 0;
		}
	}
#endif // !UE_BUILD_SHIPPING

	if (bNaniteActive && View->Family->EngineShowFlags.NaniteMeshes && !View->CustomRenderPass)
	{
		Result.bShadowRelevance = false;
		Result.bVelocityRelevance = false;
		Result.bRenderCustomDepth = false;
		Result.bTranslucentSelfShadow = false;
	#if WITH_EDITOR
		if (GLandscapeEditModeActive && Result.bDynamicRelevance)
		{
			if (!View->bIsVirtualTexture || View->bIsSceneCapture)
			{
				Result.bStaticRelevance = false;
			}
		}
		else
	#endif
		{
			Result.bRenderInMainPass = false;
		}
	}

	return Result;
}

/**
*	Determines the relevance of this primitive's elements to the given light.
*	@param	LightSceneProxy			The light to determine relevance for
*	@param	bDynamic (output)		The light is dynamic for this primitive
*	@param	bRelevant (output)		The light is relevant for this primitive
*	@param	bLightMapped (output)	The light is light mapped for this primitive
*/
void FLandscapeComponentSceneProxy::GetLightRelevance(const FLightSceneProxy* LightSceneProxy, bool& bDynamic, bool& bRelevant, bool& bLightMapped, bool& bShadowMapped) const
{
	// Attach the light to the primitive's static meshes.
	bDynamic = true;
	bRelevant = false;
	bLightMapped = true;
	bShadowMapped = true;

	if (ComponentLightInfo)
	{
		ELightInteractionType InteractionType = ComponentLightInfo->GetInteraction(LightSceneProxy).GetType();

		if (InteractionType != LIT_CachedIrrelevant)
		{
			bRelevant = true;
		}

		if (InteractionType != LIT_CachedLightMap && InteractionType != LIT_CachedIrrelevant)
		{
			bLightMapped = false;
		}

		if (InteractionType != LIT_Dynamic)
		{
			bDynamic = false;
		}

		if (InteractionType != LIT_CachedSignedDistanceFieldShadowMap2D)
		{
			bShadowMapped = false;
		}
	}
	else
	{
		bRelevant = true;
		bLightMapped = false;
	}
}

SIZE_T FLandscapeComponentSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

FLightInteraction FLandscapeComponentSceneProxy::FLandscapeLCI::GetInteraction(const class FLightSceneProxy* LightSceneProxy) const
{
	// ask base class
	ELightInteractionType LightInteraction = GetStaticInteraction(LightSceneProxy, IrrelevantLights);

	if (LightInteraction != LIT_MAX)
	{
		return FLightInteraction(LightInteraction);
	}

	// Use dynamic lighting if the light doesn't have static lighting.
	return FLightInteraction::Dynamic();
}

#if WITH_EDITOR
namespace DebugColorMask
{
	const FLinearColor Masks[5] =
	{
		FLinearColor(1.f, 0.f, 0.f, 0.f),
		FLinearColor(0.f, 1.f, 0.f, 0.f),
		FLinearColor(0.f, 0.f, 1.f, 0.f),
		FLinearColor(0.f, 0.f, 0.f, 1.f),
		FLinearColor(0.f, 0.f, 0.f, 0.f)
	};
};
#endif

void FLandscapeComponentSceneProxy::OnTransformChanged(FRHICommandListBase& RHICmdList)
{
	// resource creation will call OnTransformChanged(), so don't bother updating everything here if resources haven't been created yet
	if (!bResourcesCreated)
	{
		return;
	}
	
	check(RenderCoord.X > INT32_MIN);

	// Set Lightmap ScaleBias
	int32 PatchExpandCountX = 0;
	int32 PatchExpandCountY = 0;
	int32 DesiredSize = 1; // output by GetTerrainExpandPatchCount but not used below
	const float LightMapRatio = ::GetTerrainExpandPatchCount(StaticLightingResolution, PatchExpandCountX, PatchExpandCountY, ComponentSizeQuads, (NumSubsections * (SubsectionSizeQuads + 1)), DesiredSize, StaticLightingLOD);
	const float LightmapLODScaleX = LightMapRatio / ((ComponentSizeVerts >> StaticLightingLOD) + 2 * PatchExpandCountX);
	const float LightmapLODScaleY = LightMapRatio / ((ComponentSizeVerts >> StaticLightingLOD) + 2 * PatchExpandCountY);
	const float LightmapBiasX = PatchExpandCountX * LightmapLODScaleX;
	const float LightmapBiasY = PatchExpandCountY * LightmapLODScaleY;
	const float LightmapScaleX = LightmapLODScaleX * (float)((ComponentSizeVerts >> StaticLightingLOD) - 1) / ComponentSizeQuads;
	const float LightmapScaleY = LightmapLODScaleY * (float)((ComponentSizeVerts >> StaticLightingLOD) - 1) / ComponentSizeQuads;
	const float LightmapExtendFactorX = (float)SubsectionSizeQuads * LightmapScaleX;
	const float LightmapExtendFactorY = (float)SubsectionSizeQuads * LightmapScaleY;

	// cache component's WorldToLocal
	FMatrix LtoW = GetLocalToWorld();
	WorldToLocal = LtoW.Inverse();

	// cache component's LocalToWorldNoScaling
	LocalToWorldNoScaling = LtoW;
	LocalToWorldNoScaling.RemoveScaling();

	// Set FLandscapeUniformShaderParameters for this subsection
	FLandscapeUniformShaderParameters LandscapeParams;
	LandscapeParams.ComponentBaseX = RenderCoord.X;
	LandscapeParams.ComponentBaseY = RenderCoord.Y;
	LandscapeParams.SubsectionSizeVerts = SubsectionSizeVerts;
	LandscapeParams.NumSubsections = NumSubsections;
	LandscapeParams.RayTracingSectionSizeVerts = SubsectionSizeVerts / NumRayTracingSections;
	LandscapeParams.NumRayTracingSections = NumRayTracingSections;
	LandscapeParams.RayTracingLODBias = GLandscapeRayTracingLODBias;
	LandscapeParams.LastLOD = LastLOD;
	LandscapeParams.VirtualTexturePerPixelHeight = VirtualTexturePerPixelHeight;
	LandscapeParams.HeightmapUVScaleBias = HeightmapScaleBias;
	LandscapeParams.WeightmapUVScaleBias = WeightmapScaleBias;
	LandscapeParams.LocalToWorldNoScaling = FMatrix44f(LocalToWorldNoScaling);			// LWC_TODO: Precision loss
	LandscapeParams.InvLODBlendRange = InvLODBlendRange;
	LandscapeParams.NonNaniteVirtualShadowMapConstantDepthBias = VirtualShadowMapConstantDepthBias;

	LandscapeParams.LandscapeLightmapScaleBias = FVector4f(
		LightmapScaleX,
		LightmapScaleY,
		LightmapBiasY,
		LightmapBiasX);
	LandscapeParams.SubsectionSizeVertsLayerUVPan = FVector4f(
		static_cast<float>(SubsectionSizeVerts),
		1.f / (float)SubsectionSizeQuads,
		static_cast<float>(SectionBase.X),
		static_cast<float>(SectionBase.Y)
	);
	LandscapeParams.SubsectionOffsetParams = FVector4f(
		HeightmapSubsectionOffsetU,
		HeightmapSubsectionOffsetV,
		WeightmapSubsectionOffset,
		static_cast<float>(SubsectionSizeQuads)
	);
	LandscapeParams.LightmapSubsectionOffsetParams = FVector4f(
		LightmapExtendFactorX,
		LightmapExtendFactorY,
		0,
		0
	);

	FTextureResource* HeightmapResource = HeightmapTexture ? HeightmapTexture->GetResource() : nullptr;
	if (HeightmapResource)
	{
		const float SizeX = static_cast<float>(FMath::Max(HeightmapResource->GetSizeX(), 1u));
		const float SizeY = static_cast<float>(FMath::Max(HeightmapResource->GetSizeY(), 1u));
		LandscapeParams.HeightmapTextureSize = FVector4f(SizeX, SizeY, 1.f / SizeX, 1.f / SizeY);
		LandscapeParams.HeightmapTexture = HeightmapTexture->TextureReference.TextureReferenceRHI;
		LandscapeParams.HeightmapTextureSampler = TStaticSamplerState<SF_Point>::GetRHI();
		LandscapeParams.NormalmapTexture = HeightmapTexture->TextureReference.TextureReferenceRHI;
		LandscapeParams.NormalmapTextureSampler = TStaticSamplerState<SF_Bilinear>::GetRHI();
	}
	else
	{
		LandscapeParams.HeightmapTextureSize = FVector4f(1, 1, 1, 1);
		LandscapeParams.HeightmapTexture = GBlackTexture->TextureRHI;
		LandscapeParams.HeightmapTextureSampler = GBlackTexture->SamplerStateRHI;
		LandscapeParams.NormalmapTexture = GBlackTexture->TextureRHI;
		LandscapeParams.NormalmapTextureSampler = GBlackTexture->SamplerStateRHI;
	}

	LandscapeUniformShaderParameters.SetContents(RHICmdList, LandscapeParams);

	// Recache mesh draw commands for changed uniform buffers
	GetScene().UpdateCachedRenderStates(this);
}

/** Creates a mesh batch for virtual texture or water info texture rendering. The caller is responsible for setting the required flags (bRenderToVirtualTexture, bUseForWaterInfoTextureDepth). Will render a simple fixed grid with combined subsections. */
bool FLandscapeComponentSceneProxy::GetMeshElementForFixedGrid(int32 InLodIndex, FMaterialRenderProxy* InMaterialInterface, FMeshBatch& OutMeshBatch, TArray<FLandscapeBatchElementParams>& OutStaticBatchParamArray) const
{
	check(FixedGridVertexFactory != nullptr);

	if (InMaterialInterface == nullptr)
	{
		return false;
	}

	OutMeshBatch.VertexFactory = FixedGridVertexFactory;
	OutMeshBatch.MaterialRenderProxy = InMaterialInterface;
	OutMeshBatch.ReverseCulling = IsLocalToWorldDeterminantNegative();
	OutMeshBatch.CastShadow = false;
	OutMeshBatch.bUseForDepthPass = false;
	OutMeshBatch.bUseAsOccluder = false;
	OutMeshBatch.bUseForMaterial = false;
	OutMeshBatch.Type = PT_TriangleList;
	OutMeshBatch.DepthPriorityGroup = SDPG_World;
	OutMeshBatch.LODIndex = static_cast<int8>(InLodIndex);
	OutMeshBatch.bDitheredLODTransition = false;

	OutMeshBatch.Elements.Empty(1);

	const FLandscapeRenderSystem& RenderSystem = *LandscapeRenderSystems.FindChecked(LandscapeKey);

	FLandscapeBatchElementParams* BatchElementParams = new(OutStaticBatchParamArray) FLandscapeBatchElementParams;
	BatchElementParams->SceneProxy = this;
	BatchElementParams->LandscapeUniformShaderParametersResource = &LandscapeUniformShaderParameters;
	BatchElementParams->FixedGridUniformShaderParameters = &SharedBuffers->LandscapeFixedGridUniformShaderParameters;
	BatchElementParams->LandscapeSectionLODUniformParameters = RenderSystem.SectionLODUniformBuffer;
	BatchElementParams->CurrentLOD = InLodIndex;

	int32 LodSubsectionSizeVerts = SubsectionSizeVerts >> InLodIndex;

	FMeshBatchElement BatchElement;
	BatchElement.UserData = BatchElementParams;
	BatchElement.PrimitiveUniformBuffer = GetUniformBuffer();
	BatchElement.IndexBuffer = SharedBuffers->IndexBuffers[InLodIndex];
	BatchElement.NumPrimitives = FMath::Square((LodSubsectionSizeVerts - 1)) * FMath::Square(NumSubsections) * 2;
	BatchElement.FirstIndex = 0;
	BatchElement.MinVertexIndex = SharedBuffers->IndexRanges[InLodIndex].MinIndexFull;
	BatchElement.MaxVertexIndex = SharedBuffers->IndexRanges[InLodIndex].MaxIndexFull;

	OutMeshBatch.Elements.Add(BatchElement);

	return true;
}

template<class ArrayType>
bool FLandscapeComponentSceneProxy::GetStaticMeshElement(int32 LODIndex, bool bForToolMesh, FMeshBatch& MeshBatch, ArrayType& OutStaticBatchParamArray) const
{
	FMaterialRenderProxy* Material = nullptr;

	{
		int32 MaterialIndex = LODIndexToMaterialIndex[LODIndex];

		// Defaults to the material interface w/ potential tessellation
		Material = AvailableMaterials[MaterialIndex];

		if (!Material)
		{
			return false;
		}
	}

	{
		MeshBatch.VertexFactory = VertexFactory;
		MeshBatch.MaterialRenderProxy = Material;

		MeshBatch.LCI = ComponentLightInfo.Get();
		MeshBatch.ReverseCulling = IsLocalToWorldDeterminantNegative();
		MeshBatch.CastShadow = bForToolMesh ? false : true;
		MeshBatch.bUseForDepthPass = true;
		MeshBatch.bUseAsOccluder = ShouldUseAsOccluder() && GetScene().GetShadingPath() == EShadingPath::Deferred && !IsMovable();
		MeshBatch.bUseForMaterial = true;
		MeshBatch.Type = PT_TriangleList;
		MeshBatch.DepthPriorityGroup = SDPG_World;
		MeshBatch.LODIndex = static_cast<int8>(LODIndex);
		MeshBatch.bDitheredLODTransition = false;

		const FLandscapeRenderSystem& RenderSystem = *LandscapeRenderSystems.FindChecked(LandscapeKey);

		FLandscapeBatchElementParams* BatchElementParams = new(OutStaticBatchParamArray) FLandscapeBatchElementParams;
		BatchElementParams->LandscapeUniformShaderParametersResource = &LandscapeUniformShaderParameters;
		BatchElementParams->FixedGridUniformShaderParameters = &SharedBuffers->LandscapeFixedGridUniformShaderParameters;
		BatchElementParams->LandscapeSectionLODUniformParameters = RenderSystem.SectionLODUniformBuffer;
		BatchElementParams->SceneProxy = this;
		BatchElementParams->CurrentLOD = LODIndex;

		// Combined batch element
		FMeshBatchElement& BatchElement = MeshBatch.Elements[0];
		BatchElement.UserData = BatchElementParams;
		BatchElement.PrimitiveUniformBuffer = GetUniformBuffer();
		BatchElement.IndexBuffer = SharedBuffers->IndexBuffers[LODIndex];
		BatchElement.NumPrimitives = FMath::Square((SubsectionSizeVerts >> LODIndex) - 1) * FMath::Square(NumSubsections) * 2;
		BatchElement.FirstIndex = 0;
		BatchElement.MinVertexIndex = SharedBuffers->IndexRanges[LODIndex].MinIndexFull;
		BatchElement.MaxVertexIndex = SharedBuffers->IndexRanges[LODIndex].MaxIndexFull;
				
		if (!bForToolMesh && bUsesLandscapeCulling)
		{
			Culling::SetupMeshBatch(*SharedBuffers, MeshBatch);
		}
	}

	return true;
}

void FLandscapeComponentSceneProxy::DrawStaticElements(FStaticPrimitiveDrawInterface* PDI)
{
	if (AvailableMaterials.Num() == 0)
	{
		return;
	}

	int32 TotalBatchCount = 1 + LastLOD - FirstLOD;

	if (FixedGridVertexFactory != nullptr)
	{
		TotalBatchCount += (1 + LastVirtualTextureLOD - FirstVirtualTextureLOD) * RuntimeVirtualTextureMaterialTypes.Num();
		TotalBatchCount += 1; // TODO: Currently we always add a single LOD0 fixed grid landscape mesh batch for rendering the water info texture. Higher LODs might be better and we might not always need to do this.
		TotalBatchCount += 1; // LOD0 for lumen surface cache capture
	}

	StaticBatchParamArray.Empty(TotalBatchCount);
	PDI->ReserveMemoryForMeshes(TotalBatchCount);

	// Create batches for fixed grid : 
	if (FixedGridVertexFactory != nullptr)
	{
		// Add fixed grid mesh batches for runtime virtual texture usage
		for (ERuntimeVirtualTextureMaterialType MaterialType : RuntimeVirtualTextureMaterialTypes)
		{
			const int32 MaterialIndex = LODIndexToMaterialIndex[FirstLOD];
	
			for (int32 LODIndex = FirstVirtualTextureLOD; LODIndex <= LastVirtualTextureLOD; ++LODIndex)
			{
				FMeshBatch RuntimeVirtualTextureMeshBatch;
				if (GetMeshElementForFixedGrid(LODIndex, AvailableMaterials[MaterialIndex], RuntimeVirtualTextureMeshBatch, StaticBatchParamArray))
				{
					RuntimeVirtualTextureMeshBatch.bRenderToVirtualTexture = true;
					RuntimeVirtualTextureMeshBatch.RuntimeVirtualTextureMaterialType = (uint32)MaterialType;
					PDI->DrawMesh(RuntimeVirtualTextureMeshBatch, FLT_MAX);
				}
			}
		}
	
		// Add fixed grid mesh batch for rendering the water info texture
		{
			int32 LODIndex = 0;
			int32 MaterialIndex = LODIndexToMaterialIndex[LODIndex];
	
			FMeshBatch MeshBatch;
			if (GetMeshElementForFixedGrid(LODIndex, AvailableMaterials[MaterialIndex], MeshBatch, StaticBatchParamArray))
			{
				MeshBatch.bUseForWaterInfoTextureDepth = true;
				PDI->DrawMesh(MeshBatch, FLT_MAX);
			}
		}
	
		// add fixed grid for lumen card captures
		{
			FMeshBatch MeshBatch;
	
			if (GetStaticMeshElement(0, false, MeshBatch, StaticBatchParamArray))
			{
				MeshBatch.VertexFactory = FixedGridVertexFactory;
				MeshBatch.CastShadow = false;
				MeshBatch.bUseForDepthPass = false;
				MeshBatch.bUseAsOccluder = false;
				MeshBatch.bUseForMaterial = false;
				MeshBatch.bDitheredLODTransition = false;
				MeshBatch.bRenderToVirtualTexture = false;
				MeshBatch.bUseForLumenSurfaceCacheCapture = true;
	
				PDI->DrawMesh(MeshBatch, FLT_MAX);
			}
		}
	}

	if (SharedBuffers->GrassIndexBuffer && DoesVFRequirePrimitiveUniformBuffer())
	{
		// Assign grass primitive UB here, GrassBatch is initialized too early before UB is even created
		const int32 NumMips = FMath::CeilLogTwo(SubsectionSizeVerts);
		for (int32 Mip = 0; Mip < NumMips; ++Mip)
		{
			FMeshBatchElement* GrassBatchElement = &GrassMeshBatch.Elements[Mip];
			GrassBatchElement->PrimitiveUniformBuffer = GetUniformBuffer();
		}
	}

	for (int32 LODIndex = FirstLOD; LODIndex <= LastLOD; LODIndex++)
	{
		FMeshBatch MeshBatch;

		if (GetStaticMeshElement(LODIndex, false, MeshBatch, StaticBatchParamArray))
		{
			PDI->DrawMesh(MeshBatch, LODIndex == FirstLOD ? FLT_MAX : (FMath::Sqrt(LODScreenRatioSquared[LODIndex]) * 2.0f));
		}
	}

	check(StaticBatchParamArray.Num() <= TotalBatchCount);
}

namespace
{
	FLinearColor GetColorForLod(int32 CurrentLOD, int32 ForcedLOD, bool DisplayCombinedBatch)
	{
		int32 ColorIndex = INDEX_NONE;
		if (GEngine->LODColorationColors.Num() > 0)
		{
			ColorIndex = CurrentLOD;
			ColorIndex = FMath::Clamp(ColorIndex, 0, GEngine->LODColorationColors.Num() - 1);
		}
		const FLinearColor& LODColor = ColorIndex != INDEX_NONE ? GEngine->LODColorationColors[ColorIndex] : FLinearColor::Gray;

		if (ForcedLOD >= 0)
		{
			return LODColor;
		}

		if (DisplayCombinedBatch)
		{
			return LODColor * 0.2f;
		}

		return LODColor * 0.1f;
	}
}

void FLandscapeComponentSceneProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FLandscapeComponentSceneProxy_GetMeshElements);
	SCOPE_CYCLE_COUNTER(STAT_LandscapeDynamicDrawTime);

	if (!bRegistered)
	{
		return;
	}

	int32 NumPasses = 0;
	int32 NumTriangles = 0;
	int32 NumDrawCalls = 0;

	const FLandscapeRenderSystem& RenderSystem = *LandscapeRenderSystems.FindChecked(LandscapeKey);

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if (VisibilityMap & (1 << ViewIndex))
		{
			FLandscapeElementParamArray& ParameterArray = Collector.AllocateOneFrameResource<FLandscapeElementParamArray>();
			ParameterArray.ElementParams.AddDefaulted(1);

			const FSceneView* View = Views[ViewIndex];

			// With bSinglePassGDME == true, there can be a different view family per active view, so grab a reference to the specific view family
			const FSceneViewFamily& ViewSpecificFamily = *View->Family;

			const bool bIsWireframe = ViewSpecificFamily.EngineShowFlags.Wireframe;

			int32 LODToRender = static_cast<int32>(RenderSystem.GetSectionLODValue(*View, RenderCoord));

			FMeshBatch& Mesh = Collector.AllocateMesh();
			GetStaticMeshElement(LODToRender, false, Mesh, ParameterArray.ElementParams);

#if WITH_EDITOR
			FMeshBatch& MeshTools = Collector.AllocateMesh();
			// No Tessellation on tool material
			GetStaticMeshElement(LODToRender, true, MeshTools, ParameterArray.ElementParams);
#endif

			// Render the landscape component
#if WITH_EDITOR
			TObjectPtr<UTexture2D> LandscapeBlackTexture = LandscapeEditResourcesSubsystem->GetLandscapeBlackTexture();

			switch (GLandscapeViewMode)
			{
			case ELandscapeViewMode::DebugLayer:
			{
				if (TObjectPtr<UMaterialInterface> LayerDebugColorMaterial = LandscapeEditResourcesSubsystem->GetLayerDebugColorMaterial())
				{
					auto DebugColorMaterialInstance = new FLandscapeDebugMaterialRenderProxy(LayerDebugColorMaterial->GetRenderProxy(),
						(EditToolRenderData.DebugChannelR >= 0 ? WeightmapTextures[EditToolRenderData.DebugChannelR / 4] : nullptr),
						(EditToolRenderData.DebugChannelG >= 0 ? WeightmapTextures[EditToolRenderData.DebugChannelG / 4] : nullptr),
						(EditToolRenderData.DebugChannelB >= 0 ? WeightmapTextures[EditToolRenderData.DebugChannelB / 4] : nullptr),
						(EditToolRenderData.DebugChannelR >= 0 ? DebugColorMask::Masks[EditToolRenderData.DebugChannelR % 4] : DebugColorMask::Masks[4]),
						(EditToolRenderData.DebugChannelG >= 0 ? DebugColorMask::Masks[EditToolRenderData.DebugChannelG % 4] : DebugColorMask::Masks[4]),
						(EditToolRenderData.DebugChannelB >= 0 ? DebugColorMask::Masks[EditToolRenderData.DebugChannelB % 4] : DebugColorMask::Masks[4])
					);

					MeshTools.MaterialRenderProxy = DebugColorMaterialInstance;
					Collector.RegisterOneFrameMaterialProxy(DebugColorMaterialInstance);

					MeshTools.bCanApplyViewModeOverrides = true;
					MeshTools.bUseWireframeSelectionColoring = IsSelected();

					Collector.AddMesh(ViewIndex, MeshTools);

					NumPasses++;
					NumTriangles += MeshTools.GetNumPrimitives();
					NumDrawCalls += MeshTools.Elements.Num();
				}
			}
			break;

			case ELandscapeViewMode::LayerDensity:
			{
				int32 ColorIndex = FMath::Min<int32>(NumWeightmapLayerAllocations, GEngine->ShaderComplexityColors.Num());
				auto LayerDensityMaterialInstance = new FColoredMaterialRenderProxy(GEngine->LevelColorationUnlitMaterial->GetRenderProxy(), ColorIndex ? GEngine->ShaderComplexityColors[ColorIndex - 1] : FLinearColor::Black);

				MeshTools.MaterialRenderProxy = LayerDensityMaterialInstance;
				Collector.RegisterOneFrameMaterialProxy(LayerDensityMaterialInstance);

				MeshTools.bCanApplyViewModeOverrides = true;
				MeshTools.bUseWireframeSelectionColoring = IsSelected();

				Collector.AddMesh(ViewIndex, MeshTools);

				NumPasses++;
				NumTriangles += MeshTools.GetNumPrimitives();
				NumDrawCalls += MeshTools.Elements.Num();
			}
			break;

			case ELandscapeViewMode::LayerUsage:
			{
				if (TObjectPtr<UMaterialInterface> LandscapeLayerUsageMaterial = LandscapeEditResourcesSubsystem->GetLandscapeLayerUsageMaterial())
				{
					float Rotation = ((SectionBase.X / ComponentSizeQuads) ^ (SectionBase.Y / ComponentSizeQuads)) & 1 ? 0 : 2.f * PI;
					auto LayerUsageMaterialInstance = new FLandscapeLayerUsageRenderProxy(LandscapeLayerUsageMaterial->GetRenderProxy(), ComponentSizeVerts, LayerColors, Rotation);
					MeshTools.MaterialRenderProxy = LayerUsageMaterialInstance;
					Collector.RegisterOneFrameMaterialProxy(LayerUsageMaterialInstance);
					MeshTools.bCanApplyViewModeOverrides = true;
					MeshTools.bUseWireframeSelectionColoring = IsSelected();
					Collector.AddMesh(ViewIndex, MeshTools);
					NumPasses++;
					NumTriangles += MeshTools.GetNumPrimitives();
					NumDrawCalls += MeshTools.Elements.Num();
				}
			}
			break;

			case ELandscapeViewMode::LOD:
			{

				const bool bMaterialModifiesMeshPosition = Mesh.MaterialRenderProxy->GetIncompleteMaterialWithFallback(View->GetFeatureLevel()).MaterialModifiesMeshPosition_RenderThread();

				auto& TemplateMesh = bIsWireframe ? Mesh : MeshTools;
				for (int32 i = 0; i < TemplateMesh.Elements.Num(); i++)
				{
					FMeshBatch& LODMesh = Collector.AllocateMesh();
					LODMesh = TemplateMesh;
					LODMesh.Elements.Empty(1);
					LODMesh.Elements.Add(TemplateMesh.Elements[i]);
					int32 CurrentLOD = ((FLandscapeBatchElementParams*)TemplateMesh.Elements[i].UserData)->CurrentLOD;
					LODMesh.VisualizeLODIndex = static_cast<int8>(CurrentLOD);
					FLinearColor Color = GetColorForLod(CurrentLOD, LODSettings.ForcedLOD, true);
					FMaterialRenderProxy* LODMaterialProxy = (FMaterialRenderProxy*)new FColoredMaterialRenderProxy(GEngine->LevelColorationUnlitMaterial->GetRenderProxy(), Color);
					Collector.RegisterOneFrameMaterialProxy(LODMaterialProxy);
					LODMesh.MaterialRenderProxy = LODMaterialProxy;
					LODMesh.bCanApplyViewModeOverrides = !bIsWireframe;
					LODMesh.bWireframe = bIsWireframe;
					LODMesh.bUseWireframeSelectionColoring = IsSelected();
					Collector.AddMesh(ViewIndex, LODMesh);

					NumTriangles += TemplateMesh.Elements[i].NumPrimitives;
					NumDrawCalls++;
				}
				NumPasses++;

			}
			break;

			case ELandscapeViewMode::WireframeOnTop:
			{
				Mesh.bCanApplyViewModeOverrides = false;
				Collector.AddMesh(ViewIndex, Mesh);
				NumPasses++;
				NumTriangles += Mesh.GetNumPrimitives();
				NumDrawCalls += Mesh.Elements.Num();

				// wireframe on top
				FMeshBatch& WireMesh = Collector.AllocateMesh();
				WireMesh = MeshTools;
				auto WireMaterialInstance = new FColoredMaterialRenderProxy(GEngine->LevelColorationUnlitMaterial->GetRenderProxy(), FLinearColor(0, 0, 1));
				WireMesh.MaterialRenderProxy = WireMaterialInstance;
				Collector.RegisterOneFrameMaterialProxy(WireMaterialInstance);
				WireMesh.bCanApplyViewModeOverrides = false;
				WireMesh.bWireframe = true;
				Collector.AddMesh(ViewIndex, WireMesh);
				NumPasses++;
				NumTriangles += WireMesh.GetNumPrimitives();
				NumDrawCalls++;
			}
			break;

			case ELandscapeViewMode::LayerContribution:
			{
				Mesh.bCanApplyViewModeOverrides = false;
				Collector.AddMesh(ViewIndex, Mesh);
				NumPasses++;
				NumTriangles += Mesh.GetNumPrimitives();
				NumDrawCalls += Mesh.Elements.Num();

				FMeshBatch& MaskMesh = Collector.AllocateMesh();
				MaskMesh = MeshTools;
				TObjectPtr<UMaterialInterface> ColorMaskRegionMaterial = LandscapeEditResourcesSubsystem->GetColorMaskRegionMaterial();
				auto ColorMaskMaterialInstance = new FLandscapeMaskMaterialRenderProxy(ColorMaskRegionMaterial->GetRenderProxy(), EditToolRenderData.LayerContributionTexture ? ToRawPtr(EditToolRenderData.LayerContributionTexture) : ToRawPtr(LandscapeBlackTexture), true);
				MaskMesh.MaterialRenderProxy = ColorMaskMaterialInstance;
				Collector.RegisterOneFrameMaterialProxy(ColorMaskMaterialInstance);
				Collector.AddMesh(ViewIndex, MaskMesh);
				NumPasses++;
				NumTriangles += MaskMesh.GetNumPrimitives();
				NumDrawCalls += MaskMesh.Elements.Num();
			}
			break;

			default:
				UMaterialInterface* LandscapeDirtyMaterial = LandscapeEditResourcesSubsystem->GetLandscapeDirtyMaterial();
#endif // WITH_EDITOR

#if WITH_EDITOR || !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				const bool bInCollisionView = View->Family->EngineShowFlags.CollisionVisibility || View->Family->EngineShowFlags.CollisionPawn;

				if (AllowDebugViewmodes() && bInCollisionView)
				{
					const bool bDrawSimpleCollision = View->Family->EngineShowFlags.CollisionPawn && CollisionResponse.GetResponse(ECC_Pawn) != ECR_Ignore;
					const bool bDrawComplexCollision = View->Family->EngineShowFlags.CollisionVisibility && CollisionResponse.GetResponse(ECC_Visibility) != ECR_Ignore;
					if (bDrawSimpleCollision || bDrawComplexCollision)
					{
						// Override the mesh's material with our material that draws the collision color
						auto CollisionMaterialInstance = new FColoredMaterialRenderProxy(
							GEngine->ShadedLevelColorationUnlitMaterial->GetRenderProxy(),
							GetWireframeColor()
						);
						Collector.RegisterOneFrameMaterialProxy(CollisionMaterialInstance);

						Mesh.MaterialRenderProxy = CollisionMaterialInstance;
						Mesh.bCanApplyViewModeOverrides = true;
						Mesh.bUseWireframeSelectionColoring = IsSelected();

						Collector.AddMesh(ViewIndex, Mesh);

						NumPasses++;
						NumTriangles += Mesh.GetNumPrimitives();
						NumDrawCalls += Mesh.Elements.Num();
					}
				}
#if WITH_EDITOR
				else if (CVarLandscapeShowDirty.GetValueOnRenderThread() && LandscapeDirtyMaterial)
				{
					Mesh.bCanApplyViewModeOverrides = false;
					Collector.AddMesh(ViewIndex, Mesh);
					NumPasses++;
					NumTriangles += Mesh.GetNumPrimitives();
					NumDrawCalls += Mesh.Elements.Num();

					FMeshBatch& MaskMesh = Collector.AllocateMesh();
					MaskMesh = MeshTools;

					auto DirtyMaterialInstance = new FLandscapeMaskMaterialRenderProxy(LandscapeDirtyMaterial->GetRenderProxy(), EditToolRenderData.DirtyTexture ? ToRawPtr(EditToolRenderData.DirtyTexture) : ToRawPtr(LandscapeBlackTexture), true);
					MaskMesh.MaterialRenderProxy = DirtyMaterialInstance;
					Collector.RegisterOneFrameMaterialProxy(DirtyMaterialInstance);
					Collector.AddMesh(ViewIndex, MaskMesh);
					NumPasses++;
					NumTriangles += MaskMesh.GetNumPrimitives();
					NumDrawCalls += MaskMesh.Elements.Num();
				}
#endif
				else
#endif
					// Regular Landscape rendering. Only use the dynamic path if we're rendering a rich view or we've disabled the static path for debugging.
					if (IsRichView(ViewSpecificFamily) ||
						GLandscapeDebugOptions.bDisableStatic ||
						bIsWireframe ||
#if WITH_EDITOR
						(IsSelected() && !GLandscapeEditModeActive) ||
						(GetViewLodOverride(*View, LandscapeKey) >= 0)
#else
						IsSelected() ||
						(View->CustomRenderPass && GetViewLodOverride(*View, LandscapeKey) >= 0)
#endif
						)
					{
						Mesh.bCanApplyViewModeOverrides = true;
						Mesh.bUseWireframeSelectionColoring = IsSelected();

						Collector.AddMesh(ViewIndex, Mesh);

						NumPasses++;
						NumTriangles += Mesh.GetNumPrimitives();
						NumDrawCalls += Mesh.Elements.Num();
					}

#if WITH_EDITOR
			} // switch
#endif

#if WITH_EDITOR
			  // Extra render passes for landscape tools
			if (GLandscapeEditModeActive && !View->bIsSceneCapture)
			{
				// Region selection
				if (EditToolRenderData.SelectedType)
				{
					if ((GLandscapeEditRenderMode & ELandscapeEditRenderMode::SelectRegion) && (EditToolRenderData.SelectedType & FLandscapeEditToolRenderData::ST_REGION)
						&& !(GLandscapeEditRenderMode & ELandscapeEditRenderMode::Mask))
					{
						TObjectPtr<UMaterialInterface> SelectionRegionMaterial = LandscapeEditResourcesSubsystem->GetSelectionRegionMaterial();
						FMeshBatch& SelectMesh = Collector.AllocateMesh();
						SelectMesh = MeshTools;
						auto SelectMaterialInstance = new FLandscapeSelectMaterialRenderProxy(SelectionRegionMaterial->GetRenderProxy(), EditToolRenderData.DataTexture ? ToRawPtr(EditToolRenderData.DataTexture) : ToRawPtr(LandscapeBlackTexture));
						SelectMesh.MaterialRenderProxy = SelectMaterialInstance;
						Collector.RegisterOneFrameMaterialProxy(SelectMaterialInstance);
						Collector.AddMesh(ViewIndex, SelectMesh);
						NumPasses++;
						NumTriangles += SelectMesh.GetNumPrimitives();
						NumDrawCalls += SelectMesh.Elements.Num();
					}

					if ((GLandscapeEditRenderMode & ELandscapeEditRenderMode::SelectComponent) && (EditToolRenderData.SelectedType & FLandscapeEditToolRenderData::ST_COMPONENT))
					{
						TObjectPtr<UMaterialInterface> SelectionColorMaterial = LandscapeEditResourcesSubsystem->GetSelectionColorMaterial();
						FMeshBatch& SelectMesh = Collector.AllocateMesh();
						SelectMesh = MeshTools;
						SelectMesh.MaterialRenderProxy = SelectionColorMaterial->GetRenderProxy();
						Collector.AddMesh(ViewIndex, SelectMesh);
						NumPasses++;
						NumTriangles += SelectMesh.GetNumPrimitives();
						NumDrawCalls += SelectMesh.Elements.Num();
					}
				}

				// Mask
				if ((GLandscapeEditRenderMode & ELandscapeEditRenderMode::SelectRegion) && (GLandscapeEditRenderMode & ELandscapeEditRenderMode::Mask))
				{
					TObjectPtr<UMaterialInterface> MaskRegionMaterial = LandscapeEditResourcesSubsystem->GetMaskRegionMaterial();
					
					if (EditToolRenderData.SelectedType & FLandscapeEditToolRenderData::ST_REGION)
					{
						FMeshBatch& MaskMesh = Collector.AllocateMesh();
						MaskMesh = MeshTools;
						auto MaskMaterialInstance = new FLandscapeMaskMaterialRenderProxy(MaskRegionMaterial->GetRenderProxy(), EditToolRenderData.DataTexture ? ToRawPtr(EditToolRenderData.DataTexture) : ToRawPtr(LandscapeBlackTexture), !!(GLandscapeEditRenderMode & ELandscapeEditRenderMode::InvertedMask));
						MaskMesh.MaterialRenderProxy = MaskMaterialInstance;
						Collector.RegisterOneFrameMaterialProxy(MaskMaterialInstance);
						Collector.AddMesh(ViewIndex, MaskMesh);
						NumPasses++;
						NumTriangles += MaskMesh.GetNumPrimitives();
						NumDrawCalls += MaskMesh.Elements.Num();
					}
					else if (!(GLandscapeEditRenderMode & ELandscapeEditRenderMode::InvertedMask))
					{
						FMeshBatch& MaskMesh = Collector.AllocateMesh();
						MaskMesh = MeshTools;
						auto MaskMaterialInstance = new FLandscapeMaskMaterialRenderProxy(MaskRegionMaterial->GetRenderProxy(), LandscapeBlackTexture, false);
						MaskMesh.MaterialRenderProxy = MaskMaterialInstance;
						Collector.RegisterOneFrameMaterialProxy(MaskMaterialInstance);
						Collector.AddMesh(ViewIndex, MaskMesh);
						NumPasses++;
						NumTriangles += MaskMesh.GetNumPrimitives();
						NumDrawCalls += MaskMesh.Elements.Num();
					}
				}

				// Edit mode tools
				if (EditToolRenderData.ToolMaterial)
				{
					FMeshBatch& EditMesh = Collector.AllocateMesh();
					EditMesh = MeshTools;
					EditMesh.MaterialRenderProxy = EditToolRenderData.ToolMaterial->GetRenderProxy();
					Collector.AddMesh(ViewIndex, EditMesh);
					NumPasses++;
					NumTriangles += EditMesh.GetNumPrimitives();
					NumDrawCalls += EditMesh.Elements.Num();
				}

				if (EditToolRenderData.GizmoMaterial && GLandscapeEditRenderMode & ELandscapeEditRenderMode::Gizmo)
				{
					FMeshBatch& EditMesh = Collector.AllocateMesh();
					EditMesh = MeshTools;
					EditMesh.MaterialRenderProxy = EditToolRenderData.GizmoMaterial->GetRenderProxy();
					Collector.AddMesh(ViewIndex, EditMesh);
					NumPasses++;
					NumTriangles += EditMesh.GetNumPrimitives();
					NumDrawCalls += EditMesh.Elements.Num();
				}
			}
#endif // WITH_EDITOR

			if (GLandscapeDebugOptions.bShowPatches)
			{
				DrawWireBox(Collector.GetPDI(ViewIndex), GetBounds().GetBox(), FColor(255, 255, 0), SDPG_World);
			}

			if (ViewSpecificFamily.EngineShowFlags.Bounds)
			{
				RenderBounds(Collector.GetPDI(ViewIndex), ViewSpecificFamily.EngineShowFlags, GetBounds(), IsSelected());
			}
		}
	}

	INC_DWORD_STAT_BY(STAT_LandscapeComponentRenderPasses, NumPasses);
	INC_DWORD_STAT_BY(STAT_LandscapeDrawCalls, NumDrawCalls);
	INC_DWORD_STAT_BY(STAT_LandscapeTriangles, NumTriangles * NumPasses);

#if !UE_BUILD_SHIPPING
	if (GVarDumpLandscapeLODsCurrentFrame == GFrameNumberRenderThread)
	{
		for (const FSceneView* View : Views)
		{
			const float LODValue = ComputeLODForView(*View);
			const int32 LOD = FMath::FloorToInt(LODValue);
			const int32 Resolution = (ComponentSizeQuads + 1) >> LOD;

			const int32 LoadedHeightmapResolution = [this]
			{
				if (!(HeightmapTexture && HeightmapTexture->GetResource()))
				{
					return 0;
				}
				const int32 MipCount = HeightmapTexture->GetResource()->GetCurrentMipCount();
				return MipCount > 0 ? 1 << (MipCount - 1) : 0;
			}();

			const int32 LoadedWeightmapResolution = [this]
			{
				int32 MaxMipCount = 0;
				for (const UTexture2D* WeightmapTexture : WeightmapTextures)
				{
					if (!(WeightmapTexture && WeightmapTexture->GetResource()))
					{
						continue;
					}
					MaxMipCount = FMath::Max(MaxMipCount, WeightmapTexture->GetResource()->GetCurrentMipCount());
				}
				return MaxMipCount > 0 ? 1 << (MaxMipCount - 1) : 0;
			}();

			UE_LOGF(LogLandscape, Display, "\nView: %d, %ls, LODValue: %f (LOD: %d), Resolution: %d, LoadedHeightmapResolution: %d, LoadedWeightmapResolution: %d",
			       View->GetViewKey(), *DebugName.ToString(), LODValue, LOD, Resolution, LoadedHeightmapResolution, LoadedWeightmapResolution);
		}

		if (GVarDumpLandscapeLODsDetails)
		{
			// Dump the mip-to-mip info for this component to evaluate how different mips are from one another: 
			FString MipToMipInfoString;
			for (int32 SourceMipIndex = 0; SourceMipIndex < NumRelevantMips - 1; ++SourceMipIndex)
			{
				for (int32 DestinationMipIndex = SourceMipIndex + 1; DestinationMipIndex < NumRelevantMips; ++DestinationMipIndex)
				{
					int32 MipToMipDeltaIndex = UE::Landscape::Private::ComputeMipToMipMaxDeltasIndex(SourceMipIndex, DestinationMipIndex, NumRelevantMips);
					MipToMipInfoString += FString::Printf(TEXT("- %i->%i: %f\n"), SourceMipIndex, DestinationMipIndex, WorldSpaceMipToMipMaxDeltas[MipToMipDeltaIndex]);
				}
			}
			UE_LOGF(LogLandscape, Display, "%ls, WorldSpaceMipToMipMaxDeltas:\n%ls", *DebugName.ToString(), *MipToMipInfoString);
		}
	}
#endif // !UE_BUILD_SHIPPING
}

void FLandscapeComponentSceneProxy::ApplyViewDependentMeshArguments(const FSceneView& View, FMeshBatch& ViewDependentMeshBatch) const
{
	Culling::FArguments CullingArgs{};
	check(RenderCoord.X > INT32_MIN);
	if (Culling::GetViewArguments(View, LandscapeKey, RenderCoord, ViewDependentMeshBatch.LODIndex, CullingArgs))
	{
		ViewDependentMeshBatch.Elements[0].NumPrimitives = 0;
		ViewDependentMeshBatch.Elements[0].NumInstances = 0;
		ViewDependentMeshBatch.Elements[0].IndirectArgsBuffer = CullingArgs.IndirectArgsBuffer;
		ViewDependentMeshBatch.Elements[0].IndirectArgsOffset = CullingArgs.IndirectArgsOffset;
	}
}

#if RHI_RAYTRACING
void FLandscapeComponentSceneProxy::GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector)
{
	if (!bRegistered || !CVarRayTracingLandscape.GetValueOnRenderThread())
	{
		return;
	}

	if (AvailableMaterials.Num() == 0)
	{
		return;
	}

	TConstArrayView<const FSceneView*> Views = Collector.GetViews();
	const uint32 VisibilityMap = Collector.GetVisibilityMap();

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if ((VisibilityMap & (1 << ViewIndex)) == 0)
		{
			continue;
		}

		const FSceneView& SceneView = *Views[ViewIndex];
		const FLandscapeRenderSystem& RenderSystem = *LandscapeRenderSystems.FindChecked(LandscapeKey);

		int32 RayTracingLODBias = GLandscapeRayTracingLODBias;
		int32 LODToRender = static_cast<int32>(RenderSystem.GetSectionLODValue(SceneView, RenderCoord)) + RayTracingLODBias;
		LODToRender = FMath::Min(LODToRender, LastLOD);

		const int8 CurrentLODIndex = static_cast<int8>(LODToRender);
		int8 MaterialIndex = LODIndexToMaterialIndex.IsValidIndex(CurrentLODIndex) ? LODIndexToMaterialIndex[CurrentLODIndex] : INDEX_NONE;
		FMaterialRenderProxy* SelectedMaterial = MaterialIndex != INDEX_NONE ? AvailableMaterials[MaterialIndex] : nullptr;

		// this is really not normal that we have no material at this point, so do not continue
		if (SelectedMaterial == nullptr)
		{
			continue;
		}

		int32 RayTracingSectionLODVertCount = GetRayTracingSectionVertCountForLOD(SubsectionSizeVerts, NumRayTracingSections, CurrentLODIndex);
		if (RayTracingSectionLODVertCount <= 0)
		{
			continue;
		}

		if (!RayTracingImpl.IsValid())
		{
			RayTracingImpl = MakePimpl<FLandscapeRayTracingImpl>();
		}

		int32 RayTracingSectionVertCount = GetRayTracingSectionVertCountForLOD(SubsectionSizeVerts, NumRayTracingSections, 0);
		FLandscapeRayTracingState* RayTracingState = RayTracingImpl.Get()->FindOrCreateRayTracingState(Collector.GetRHICommandList(), SceneView.State, NumSubsections, NumRayTracingSections, RayTracingSectionVertCount);

		FLandscapeElementParamArray& ParameterArray = Collector.AllocateOneFrameResource<FLandscapeElementParamArray>();

		uint32 TotalInstances = FMath::Square(NumRayTracingSections * NumSubsections);
		ParameterArray.ElementParams.AddDefaulted(TotalInstances);

		FMeshBatch BaseMeshBatch;
		BaseMeshBatch.VertexFactory = VertexFactory;
		BaseMeshBatch.MaterialRenderProxy = SelectedMaterial;
		BaseMeshBatch.LCI = ComponentLightInfo.Get();
		BaseMeshBatch.CastShadow = true;
		BaseMeshBatch.CastRayTracedShadow = true;
		BaseMeshBatch.bUseForMaterial = true;
		BaseMeshBatch.SegmentIndex = 0;

		BaseMeshBatch.Elements.Empty();

		uint32 NumPrimitives = FMath::Square(RayTracingSectionLODVertCount - 1) * 2;

		// Detect force update CVar
		bool bNeedsRayTracingGeometryUpdate = (GLandscapeRayTracingGeometryLODsThatUpdateEveryFrame >= 0 && CurrentLODIndex <= (GLandscapeRayTracingGeometryLODsThatUpdateEveryFrame + RayTracingLODBias)) ? true : false;
		bool bLODChanged = false;

		// Detect continuous LOD parameter changes. This is for far-away high LODs - they change rarely yet the BLAS refit time is not ideal, even if they contains tiny amount of triangles
		{
			if (RayTracingState->CurrentLOD != CurrentLODIndex)
			{
				bNeedsRayTracingGeometryUpdate = true;
				RayTracingState->CurrentLOD = CurrentLODIndex;
				bLODChanged = true;
			}
			if (RayTracingState->HeightmapLODBias != RenderSystem.GetSectionLODBias(RenderCoord))
			{
				bNeedsRayTracingGeometryUpdate = true;
				RayTracingState->HeightmapLODBias = RenderSystem.GetSectionLODBias(RenderCoord);
			}

			const float PendingFractionalLOD = RenderSystem.GetSectionLODValue(SceneView, RenderCoord);
			const float FractionLODAbsoluteDifference = FMath::Abs(RayTracingState->FractionalLOD - PendingFractionalLOD);
			if (FractionLODAbsoluteDifference > GLandscapeRayTracingGeometryFractionalLODUpdateThreshold)
			{
				bNeedsRayTracingGeometryUpdate = true;
				RayTracingState->FractionalLOD = PendingFractionalLOD;
			}
		}

		const FMaterialRenderProxy* FallbackMaterialRenderProxyPtr = nullptr;
		const FMaterial& Material = SelectedMaterial->GetMaterialWithFallback(GetScene().GetFeatureLevel(), FallbackMaterialRenderProxyPtr);

		if (GLandscapeRayTracingGeometryDetectTextureStreaming > 0)
		{
			if (Material.GetRenderingThreadShaderMap()->UsesWorldPositionOffset())
			{
				const FMaterialRenderProxy* MaterialRenderProxy = FallbackMaterialRenderProxyPtr ? FallbackMaterialRenderProxyPtr : SelectedMaterial;

				FMaterialRenderContext MaterialRenderContext(MaterialRenderProxy, Material, &SceneView);

				const FUniformExpressionSet& UniformExpressionSet = Material.GetRenderingThreadShaderMap()->GetUniformExpressionSet();
				const uint32 Hash = UniformExpressionSet.GetReferencedTexture2DRHIHash(MaterialRenderContext);

				if (RayTracingState->ReferencedTextureRHIHash != Hash)
				{
					bNeedsRayTracingGeometryUpdate = true;
					RayTracingState->ReferencedTextureRHIHash = Hash;
				}
			}
		}

		for (int32 SubY = 0; SubY < NumSubsections; SubY++)
		{
			for (int32 SubX = 0; SubX < NumSubsections; SubX++)
			{
				for (int32 RTSubY = 0; RTSubY < NumRayTracingSections; RTSubY++)
				{
					for (int32 RTSubX = 0; RTSubX < NumRayTracingSections; RTSubX++)
					{
						const int32 SubSectionIdx = SubX + SubY * NumSubsections;
						const int32 InstanceIdx = SubSectionIdx * FMath::Square(NumRayTracingSections) + RTSubX + RTSubY * NumRayTracingSections;

						FLandscapeSectionRayTracingState& SectionRayTracingState = RayTracingState->Sections[InstanceIdx];

						FLandscapeBatchElementParams& BatchElementParams = ParameterArray.ElementParams[InstanceIdx];
						BatchElementParams.LandscapeUniformShaderParametersResource = &LandscapeUniformShaderParameters;
						BatchElementParams.FixedGridUniformShaderParameters = &SharedBuffers->LandscapeFixedGridUniformShaderParameters;
						BatchElementParams.LandscapeSectionLODUniformParameters = RenderSystem.SectionLODUniformBuffer;
						BatchElementParams.LandscapeVertexFactoryMVFUniformBuffer = SectionRayTracingState.UniformBuffer;
						BatchElementParams.SceneProxy = this;
						BatchElementParams.CurrentLOD = CurrentLODIndex;

						FMeshBatchElement BatchElement;
						BatchElement.UserData = &BatchElementParams;
						BatchElement.PrimitiveUniformBuffer = GetUniformBuffer();
						BatchElement.IndexBuffer = SharedBuffers->ZeroOffsetIndexBuffers[CurrentLODIndex];
						BatchElement.FirstIndex = 0;
						BatchElement.NumPrimitives = NumPrimitives;
						BatchElement.MinVertexIndex = 0;
						BatchElement.MaxVertexIndex = 0;

						FMeshBatch MeshBatch = BaseMeshBatch;
						MeshBatch.Elements.Add(BatchElement);

						check(SectionRayTracingState.Geometry.IsValid() || bNeedsRayTracingGeometryUpdate);

						TArray<FMeshBatch>& CollectorMeshBatches = Collector.AllocateMeshBatchArray();
						CollectorMeshBatches.Add(MeshBatch);

						FRayTracingInstance RayTracingInstance;
						RayTracingInstance.Geometry = &SectionRayTracingState.Geometry;
						RayTracingInstance.InstanceTransforms.Add(GetLocalToWorld());
						RayTracingInstance.MaterialsView = CollectorMeshBatches;

						bool bIsEvicted = SectionRayTracingState.Geometry.IsEvicted();
						if ((bNeedsRayTracingGeometryUpdate || bIsEvicted) && VertexFactory->GetType()->SupportsRayTracingDynamicGeometry())
						{
							// Use the internal managed vertex buffer because landscape dynamic RT geometries are not updated every frame
							// which is a requirement for the shared vertex buffer usage

							// Index buffer could have been changed due to LOD change
							PRAGMA_DISABLE_DEPRECATION_WARNINGS
							// TODO: Shouldn't directly modify FRayTracingGeometry initializer
							SectionRayTracingState.Geometry.Initializer.IndexBuffer = BatchElement.IndexBuffer->IndexBufferRHI;
							PRAGMA_ENABLE_DEPRECATION_WARNINGS

							// Recreate the dynamic vertex buffer if the LOD changed - different primitive count
							if (bLODChanged)
							{
								SectionRayTracingState.RayTracingDynamicVertexBuffer.Release();
							}

							const bool bAlphaMasked = Material.IsMasked() && (CVarRayTracingLandscapeVertexMask.GetValueOnRenderThread() != 0);

							FRayTracingDynamicGeometryUpdateParams UpdateParams;
							UpdateParams.MeshBatchesView = CollectorMeshBatches;
							UpdateParams.bUsingIndirectDraw = false;
							UpdateParams.NumVertices = (uint32)FMath::Square(RayTracingSectionLODVertCount);
							UpdateParams.VertexBufferSize = FMath::Square(RayTracingSectionLODVertCount) * (uint32)sizeof(FVector3f);
							UpdateParams.NumTriangles = (uint32)FMath::Square(RayTracingSectionLODVertCount - 1) * 2;
							UpdateParams.Geometry = &SectionRayTracingState.Geometry;
							UpdateParams.Buffer = &SectionRayTracingState.RayTracingDynamicVertexBuffer;
							UpdateParams.bApplyWorldPositionOffset = true;
							UpdateParams.InstanceId = 0;
							UpdateParams.WorldToInstance = FMatrix44f::Identity;
							UpdateParams.bAlphaMasked = bAlphaMasked;
							UpdateParams.IndexBuffer = SharedBuffers->ZeroOffsetIndexBuffers[CurrentLODIndex]->IndexBufferRHI;

							Collector.AddRayTracingGeometryUpdate(ViewIndex, MoveTemp(UpdateParams));
						}

						Collector.AddRayTracingInstance(ViewIndex, MoveTemp(RayTracingInstance));
					}
				}
			}
		}
	}
}
#endif

//
// FLandscapeVertexBuffer
//

/**
* Initialize the RHI for this rendering resource
*/
void FLandscapeVertexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	SCOPED_LOADTIMER(FLandscapeVertexBuffer_InitRHI);

	const static FLazyName ClassName(TEXT("FLandscapeVertexBuffer"));

	// create a static vertex buffer
	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateVertex<FLandscapeVertex>(TEXT("FLandscapeVertexBuffer"), NumVertices)
		.AddUsage(EBufferUsageFlags::Static)
		.SetInitialState(ERHIAccess::VertexOrIndexBuffer)
		.SetClassName(ClassName)
		.SetOwnerName(GetOwnerName())
		.SetInitActionInitializer();

	TRHIBufferInitializer<FLandscapeVertex> Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);

	int32 VertexIndex = 0;
	for (int32 SubY = 0; SubY < NumSubsections; SubY++)
	{
		for (int32 SubX = 0; SubX < NumSubsections; SubX++)
		{
			for (int32 y = 0; y < SubsectionSizeVerts; y++)
			{
				for (int32 x = 0; x < SubsectionSizeVerts; x++)
				{
					FLandscapeVertex Vertex{};
					Vertex.VertexX = static_cast<uint8>(x);
					Vertex.VertexY = static_cast<uint8>(y);
					Vertex.SubX = static_cast<uint8>(SubX);
					Vertex.SubY = static_cast<uint8>(SubY);

					Initializer.WriteValueAtIndex(VertexIndex, Vertex);
					VertexIndex++;
				}
			}
		}
	}
	check(NumVertices == VertexIndex);

	VertexBufferRHI = Initializer.Finalize();
}

//
// FLandscapeSharedBuffers
//

template <typename INDEX_TYPE>
void FLandscapeSharedBuffers::CreateIndexBuffers(FRHICommandListBase& RHICmdList, const FName& InOwnerName)
{
	TArray<INDEX_TYPE> VertexToIndexMap;
	VertexToIndexMap.AddUninitialized(FMath::Square(SubsectionSizeVerts * NumSubsections));
	FMemory::Memset(VertexToIndexMap.GetData(), 0xff, NumVertices * sizeof(INDEX_TYPE));

	INDEX_TYPE VertexCount = 0;
	int32 SubsectionSizeQuads = SubsectionSizeVerts - 1;

	// Layout index buffer to determine best vertex order
	int32 MaxLOD = NumIndexBuffers - 1;
	for (int32 Mip = MaxLOD; Mip >= 0; Mip--)
	{
		int32 LodSubsectionSizeQuads = (SubsectionSizeVerts >> Mip) - 1;

		TArray<INDEX_TYPE> NewIndices;
		int32 ExpectedNumIndices = FMath::Square(NumSubsections) * FMath::Square(LodSubsectionSizeQuads) * 6;
		NewIndices.Empty(ExpectedNumIndices);

		int32& MaxIndexFull = IndexRanges[Mip].MaxIndexFull;
		int32& MinIndexFull = IndexRanges[Mip].MinIndexFull;
		MaxIndexFull = 0;
		MinIndexFull = MAX_int32;
		{
			int32 SubOffset = 0;
			for (int32 SubY = 0; SubY < NumSubsections; SubY++)
			{
				for (int32 SubX = 0; SubX < NumSubsections; SubX++)
				{
					int32& MaxIndex = IndexRanges[Mip].MaxIndex[SubX][SubY];
					int32& MinIndex = IndexRanges[Mip].MinIndex[SubX][SubY];
					MaxIndex = 0;
					MinIndex = MAX_int32;

					for (int32 y = 0; y < LodSubsectionSizeQuads; y++)
					{
						for (int32 x = 0; x < LodSubsectionSizeQuads; x++)
						{
							INDEX_TYPE i00 = static_cast<INDEX_TYPE>((x + 0) + (y + 0) * SubsectionSizeVerts + SubOffset);
							INDEX_TYPE i10 = static_cast<INDEX_TYPE>((x + 1) + (y + 0) * SubsectionSizeVerts + SubOffset);
							INDEX_TYPE i11 = static_cast<INDEX_TYPE>((x + 1) + (y + 1) * SubsectionSizeVerts + SubOffset);
							INDEX_TYPE i01 = static_cast<INDEX_TYPE>((x + 0) + (y + 1) * SubsectionSizeVerts + SubOffset);

							NewIndices.Add(i00);
							NewIndices.Add(i11);
							NewIndices.Add(i10);

							NewIndices.Add(i00);
							NewIndices.Add(i01);
							NewIndices.Add(i11);

							// Update the min/max index ranges
							MaxIndex = FMath::Max<int32>(MaxIndex, i00);
							MinIndex = FMath::Min<int32>(MinIndex, i00);
							MaxIndex = FMath::Max<int32>(MaxIndex, i10);
							MinIndex = FMath::Min<int32>(MinIndex, i10);
							MaxIndex = FMath::Max<int32>(MaxIndex, i11);
							MinIndex = FMath::Min<int32>(MinIndex, i11);
							MaxIndex = FMath::Max<int32>(MaxIndex, i01);
							MinIndex = FMath::Min<int32>(MinIndex, i01);
						}
					}

					// update min/max for full subsection
					MaxIndexFull = FMath::Max<int32>(MaxIndexFull, MaxIndex);
					MinIndexFull = FMath::Min<int32>(MinIndexFull, MinIndex);

					SubOffset += FMath::Square(SubsectionSizeVerts);
				}
			}

			check(MinIndexFull <= (uint32)((INDEX_TYPE)(~(INDEX_TYPE)0)));
			check(NewIndices.Num() == ExpectedNumIndices);
		}

		// Create and init new index buffer with index data
		FRawStaticIndexBuffer16or32<INDEX_TYPE>* IndexBuffer = (FRawStaticIndexBuffer16or32<INDEX_TYPE>*)IndexBuffers[Mip];
		if (!IndexBuffer)
		{
			IndexBuffer = new FRawStaticIndexBuffer16or32<INDEX_TYPE>(false);
		}
		IndexBuffer->AssignNewBuffer(NewIndices);
		IndexBuffer->SetOwnerName(InOwnerName);
		IndexBuffer->InitResource(RHICmdList);

		IndexBuffers[Mip] = IndexBuffer;

#if RHI_RAYTRACING
		if (IsRayTracingEnabled())
		{
			int32 LodRayTracingSectionSizeVerts = GetRayTracingSectionVertCountForLOD(SubsectionSizeVerts, NumRayTracingSections, Mip);
			int32 LodRayTracingSectionSizeQuads = LodRayTracingSectionSizeVerts - 1;
			
			TArray<INDEX_TYPE> ZeroOffsetIndices;

			for (int32 y = 0; y < LodRayTracingSectionSizeQuads; y++)
			{
				for (int32 x = 0; x < LodRayTracingSectionSizeQuads; x++)
				{
					INDEX_TYPE i00 = static_cast<INDEX_TYPE>((x + 0) + (y + 0) * LodRayTracingSectionSizeVerts);
					INDEX_TYPE i10 = static_cast<INDEX_TYPE>((x + 1) + (y + 0) * LodRayTracingSectionSizeVerts);
					INDEX_TYPE i11 = static_cast<INDEX_TYPE>((x + 1) + (y + 1) * LodRayTracingSectionSizeVerts);
					INDEX_TYPE i01 = static_cast<INDEX_TYPE>((x + 0) + (y + 1) * LodRayTracingSectionSizeVerts);

					ZeroOffsetIndices.Add(i00);
					ZeroOffsetIndices.Add(i11);
					ZeroOffsetIndices.Add(i10);

					ZeroOffsetIndices.Add(i00);
					ZeroOffsetIndices.Add(i01);
					ZeroOffsetIndices.Add(i11);
				}
			}

			FRawStaticIndexBuffer16or32<INDEX_TYPE>* ZeroOffsetIndexBuffer = new FRawStaticIndexBuffer16or32<INDEX_TYPE>(false);
			ZeroOffsetIndexBuffer->AssignNewBuffer(ZeroOffsetIndices);
			ZeroOffsetIndexBuffer->InitResource(RHICmdList);
			ZeroOffsetIndexBuffers[Mip] = ZeroOffsetIndexBuffer;
		}
#endif
	}
}

template <typename INDEX_TYPE>
void FLandscapeSharedBuffers::CreateGrassIndexBuffer(FRHICommandListBase& RHICmdList, const FName& InOwnerName)
{
	TArray<INDEX_TYPE> NewIndices;

	int32 ExpectedNumIndices = FMath::Square(NumSubsections) * (FMath::Square(SubsectionSizeVerts) * 4 / 3 - 1); // *4/3 is for mips, -1 because we only go down to 2x2 not 1x1
	NewIndices.Empty(ExpectedNumIndices);

	int32 NumMips = FMath::CeilLogTwo(SubsectionSizeVerts);

	for (int32 Mip = 0; Mip < NumMips; ++Mip)
	{
		// Store offset to the start of this mip in the index buffer
		GrassIndexMipOffsets.Add(NewIndices.Num());

		int32 MipSubsectionSizeVerts = SubsectionSizeVerts >> Mip;
		int32 SubOffset = 0;
		for (int32 SubY = 0; SubY < NumSubsections; SubY++)
		{
			for (int32 SubX = 0; SubX < NumSubsections; SubX++)
			{
				for (int32 y = 0; y < MipSubsectionSizeVerts; y++)
				{
					for (int32 x = 0; x < MipSubsectionSizeVerts; x++)
					{
						// intentionally using SubsectionSizeVerts not MipSubsectionSizeVerts, this is a vert buffer index not a mip vert index
						NewIndices.Add(static_cast<INDEX_TYPE>(x + y * SubsectionSizeVerts + SubOffset));
					}
				}

				// intentionally using SubsectionSizeVerts not MipSubsectionSizeVerts (as above)
				SubOffset += FMath::Square(SubsectionSizeVerts);
			}
		}
	}

	check(NewIndices.Num() == ExpectedNumIndices);

	// Create and init new index buffer with index data
	FRawStaticIndexBuffer16or32<INDEX_TYPE>* IndexBuffer = new FRawStaticIndexBuffer16or32<INDEX_TYPE>(false);
	IndexBuffer->AssignNewBuffer(NewIndices);
	IndexBuffer->SetOwnerName(InOwnerName);
	IndexBuffer->InitResource(RHICmdList);
	GrassIndexBuffer = IndexBuffer;
}

FLandscapeSharedBuffers::FLandscapeSharedBuffers(FRHICommandListBase& RHICmdList, const int32 InSharedBuffersKey, const int32 InSubsectionSizeQuads, const int32 InNumSubsections, const int32 InNumRayTracingSections, const ERHIFeatureLevel::Type InFeatureLevel, const FName& InOwnerName)
	: SharedBuffersKey(InSharedBuffersKey)
	, NumIndexBuffers(FMath::CeilLogTwo(InSubsectionSizeQuads + 1))
	, SubsectionSizeVerts(InSubsectionSizeQuads + 1)
	, NumSubsections(InNumSubsections)
	, NumRayTracingSections(InNumRayTracingSections)
	, VertexFactory(nullptr)
	, FixedGridVertexFactory(nullptr)
	, VertexBuffer(nullptr)
	, TileMesh(nullptr)
	, TileVertexFactory(nullptr)
	, TileDataBuffer(nullptr)
	, bUse32BitIndices(false)
	, GrassIndexBuffer(nullptr)
{
	NumVertices = FMath::Square(SubsectionSizeVerts) * FMath::Square(NumSubsections);
	
	VertexBuffer = new FLandscapeVertexBuffer(RHICmdList, InFeatureLevel, NumVertices, SubsectionSizeVerts, NumSubsections, InOwnerName);

	IndexBuffers = new FIndexBuffer * [NumIndexBuffers];
	FMemory::Memzero(IndexBuffers, sizeof(FIndexBuffer*) * NumIndexBuffers);
	IndexRanges = new FLandscapeIndexRanges[NumIndexBuffers]();

#if RHI_RAYTRACING
	if (IsRayTracingEnabled())
	{
		ZeroOffsetIndexBuffers.AddZeroed(NumIndexBuffers);
	}
#endif

	// See if we need to use 16 or 32-bit index buffers
	if (NumVertices > 65535)
	{
		bUse32BitIndices = true;
		CreateIndexBuffers<uint32>(RHICmdList, InOwnerName);
		if (UE::Landscape::ShouldBuildGrassMapRenderingResources())
		{
			CreateGrassIndexBuffer<uint32>(RHICmdList, InOwnerName);
		}
	}
	else
	{
		CreateIndexBuffers<uint16>(RHICmdList, InOwnerName);
		if (UE::Landscape::ShouldBuildGrassMapRenderingResources())
		{
			CreateGrassIndexBuffer<uint16>(RHICmdList, InOwnerName);
		}
	}

	// create as many as there are potential mips (even if MaxLOD can be inferior than that), because the grass could need that much :
	const int32 NumMips = FMath::CeilLogTwo(SubsectionSizeVerts);
	LandscapeFixedGridUniformShaderParameters.AddDefaulted(NumMips);
	for (int32 LodIndex = 0; LodIndex < NumMips; ++LodIndex)
	{
		FLandscapeFixedGridUniformShaderParameters Parameters;
		Parameters.LodValues = FVector4f(
			static_cast<float>(LodIndex),
			0.f,
			(float)((SubsectionSizeVerts >> LodIndex) - 1),
			1.f / (float)((SubsectionSizeVerts >> LodIndex) - 1));
		LandscapeFixedGridUniformShaderParameters[LodIndex].SetContents(RHICmdList, Parameters);
		LandscapeFixedGridUniformShaderParameters[LodIndex].InitResource(RHICmdList);
	}
}

FLandscapeSharedBuffers::~FLandscapeSharedBuffers()
{
	// If we run the destructor we are the last reference so we need to remove
	// ourselves from the shared pool of buffers.
    FLandscapeComponentSceneProxy::SharedBuffersMap.Remove(SharedBuffersKey);

	delete VertexBuffer;

	for (int32 i = 0; i < NumIndexBuffers; i++)
	{
		IndexBuffers[i]->ReleaseResource();
		delete IndexBuffers[i];
	}
	delete[] IndexBuffers;
	delete[] IndexRanges;

#if RHI_RAYTRACING
	if (IsRayTracingEnabled())
	{
		while (ZeroOffsetIndexBuffers.Num() > 0)
		{
			FIndexBuffer* Buffer = ZeroOffsetIndexBuffers.Pop();
			Buffer->ReleaseResource();
			delete Buffer;
		}
	}
#endif

	// Free the lod uniform buffers
	for (int32 i = 0; i < LandscapeFixedGridUniformShaderParameters.Num(); ++i)
	{
		LandscapeFixedGridUniformShaderParameters[i].ReleaseResource();
	}

	if (GrassIndexBuffer)
	{
		GrassIndexBuffer->ReleaseResource();
		delete GrassIndexBuffer;
	}
	
	delete FixedGridVertexFactory;
	delete VertexFactory;

	delete TileMesh;
	delete TileDataBuffer;
	delete TileVertexFactory;
}

//
// FLandscapeVertexFactoryVertexShaderParameters
//
void FLandscapeVertexFactoryVertexShaderParameters::GetElementShaderBindings(
	const class FSceneInterface* Scene,
	const FSceneView* InView,
	const class FMeshMaterialShader* Shader,
	const EVertexInputStreamType InputStreamType,
	ERHIFeatureLevel::Type FeatureLevel,
	const FVertexFactory* VertexFactory,
	const FMeshBatchElement& BatchElement,
	class FMeshDrawSingleShaderBindings& ShaderBindings,
	FVertexInputStreamArray& VertexStreams
) const
{
	SCOPE_CYCLE_COUNTER(STAT_LandscapeVFDrawTimeVS);

	const FLandscapeBatchElementParams* BatchElementParams = (const FLandscapeBatchElementParams*)BatchElement.UserData;
	check(BatchElementParams);

	const FLandscapeComponentSceneProxy* SceneProxy = BatchElementParams->SceneProxy;

	ShaderBindings.Add(Shader->GetUniformBufferParameter<FLandscapeUniformShaderParameters>(), *BatchElementParams->LandscapeUniformShaderParametersResource);
	ShaderBindings.Add(Shader->GetUniformBufferParameter<FLandscapeSectionLODUniformParameters>(), BatchElementParams->LandscapeSectionLODUniformParameters);

	if (InView && VertexFactory->GetType() == Culling::GetTileVertexFactoryType())
	{
		// TileVF is used only for LOD0
		int32 LODIndex = 0; 
		Culling::FArguments CullingArgs{};

		if (Culling::GetViewArguments(*InView, SceneProxy->LandscapeKey, SceneProxy->RenderCoord, LODIndex, CullingArgs))
		{
			check(VertexStreams.IsValidIndex(1));
			VertexStreams[1].Offset = CullingArgs.TileDataOffset;
			VertexStreams[1].VertexBuffer = CullingArgs.TileDataVertexBuffer;
		}
	}

#if RHI_RAYTRACING
	if (IsRayTracingEnabled())
	{
		ShaderBindings.Add(Shader->GetUniformBufferParameter<FLandscapeVertexFactoryMVFParameters>(), BatchElementParams->LandscapeVertexFactoryMVFUniformBuffer);
	}
#endif
}

/**
  * Shader parameters for use with FLandscapeFixedGridVertexFactory
  * Simple grid rendering (without dynamic lod blend) needs a simpler fixed setup.
  */
class FLandscapeFixedGridVertexFactoryVertexShaderParameters : public FLandscapeVertexFactoryVertexShaderParameters
{
	DECLARE_TYPE_LAYOUT(FLandscapeFixedGridVertexFactoryVertexShaderParameters, NonVirtual);
public:
	void GetElementShaderBindings(
		const class FSceneInterface* Scene,
		const FSceneView* InView,
		const class FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FVertexFactory* VertexFactory,
		const FMeshBatchElement& BatchElement,
		class FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams
	) const
	{
		SCOPE_CYCLE_COUNTER(STAT_LandscapeVFDrawTimeVS);

		const FLandscapeBatchElementParams* BatchElementParams = (const FLandscapeBatchElementParams*)BatchElement.UserData;
		check(BatchElementParams);

		ShaderBindings.Add(Shader->GetUniformBufferParameter<FLandscapeUniformShaderParameters>(), *BatchElementParams->LandscapeUniformShaderParametersResource);
		ShaderBindings.Add(Shader->GetUniformBufferParameter<FLandscapeFixedGridUniformShaderParameters>(), (*BatchElementParams->FixedGridUniformShaderParameters)[BatchElementParams->CurrentLOD]);

#if RHI_RAYTRACING
		if (IsRayTracingEnabled())
		{
			ShaderBindings.Add(Shader->GetUniformBufferParameter<FLandscapeVertexFactoryMVFParameters>(), BatchElementParams->LandscapeVertexFactoryMVFUniformBuffer);
		}
#endif
	}
};

IMPLEMENT_TYPE_LAYOUT(FLandscapeFixedGridVertexFactoryVertexShaderParameters);

//
// FLandscapeVertexFactoryPixelShaderParameters
//

void FLandscapeVertexFactoryPixelShaderParameters::GetElementShaderBindings(
	const class FSceneInterface* Scene,
	const FSceneView* InView,
	const class FMeshMaterialShader* Shader,
	const EVertexInputStreamType InputStreamType,
	ERHIFeatureLevel::Type FeatureLevel,
	const FVertexFactory* VertexFactory,
	const FMeshBatchElement& BatchElement,
	class FMeshDrawSingleShaderBindings& ShaderBindings,
	FVertexInputStreamArray& VertexStreams
) const
{
	SCOPE_CYCLE_COUNTER(STAT_LandscapeVFDrawTimePS);

	const FLandscapeBatchElementParams* BatchElementParams = (const FLandscapeBatchElementParams*)BatchElement.UserData;

	ShaderBindings.Add(Shader->GetUniformBufferParameter<FLandscapeUniformShaderParameters>(), *BatchElementParams->LandscapeUniformShaderParametersResource);
}

//
// FLandscapeVertexFactory
//

IMPLEMENT_GLOBAL_SHADER_PARAMETER_STRUCT(FLandscapeVertexFactoryMVFParameters, "LandscapeMVF");

void FLandscapeVertexFactory::InitRHI(FRHICommandListBase& RHICmdList)
{
	// list of declaration items
	FVertexDeclarationElementList Elements;

	// position decls
	Elements.Add(AccessStreamComponent(Data.PositionComponent, 0));

	// see FLandscapeVertexFactory::ModifyCompilationEnvironment for mobile GPUScene exception
	AddPrimitiveIdStreamElement(EVertexInputStreamType::Default, Elements, /* AttributeIndex = */ 1, /* AttributeIndex_Mobile = */0xFF);
	// create the actual device decls
	InitDeclaration(Elements);
}

FLandscapeVertexFactory::FLandscapeVertexFactory(ERHIFeatureLevel::Type InFeatureLevel)
	: FVertexFactory(InFeatureLevel)
{
}

bool FLandscapeVertexFactory::ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters)
{
	// only compile landscape materials for landscape vertex factory
	// The special engine materials must be compiled for the landscape vertex factory because they are used with it for wireframe, etc.
	return (Parameters.MaterialParameters.bIsUsedWithLandscape || Parameters.MaterialParameters.bIsSpecialEngineMaterial);
}

void FLandscapeVertexFactory::ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FVertexFactory::ModifyCompilationEnvironment(Parameters, OutEnvironment);

	const FStaticFeatureLevel MaxSupportedFeatureLevel = GetMaxSupportedFeatureLevel(Parameters.Platform);
	// TODO: support GPUScene on mobile. We need to pass a correct LODLightmapDataIndex to a batching CS, which we do only for StaticMesh atm only
	const bool bUseGPUScene = UseGPUScene(Parameters.Platform) && (MaxSupportedFeatureLevel > ERHIFeatureLevel::ES3_1);

	OutEnvironment.SetDefine(TEXT("VF_SUPPORTS_PRIMITIVE_SCENE_DATA"), Parameters.VertexFactoryType->SupportsPrimitiveIdStream() && bUseGPUScene);

	// Make sure landscape vertices go back to local space so that we have consistency between the transform on normals and geometry
	OutEnvironment.SetDefine(TEXT("RAY_TRACING_DYNAMIC_MESH_IN_LOCAL_SPACE"), TEXT("1"));
}

void FLandscapeVertexFactory::GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements)
{
	Elements.Add(FVertexElement(0, 0, VET_UByte4, 0, sizeof(FLandscapeVertex), false));
	
	if (UseGPUScene(GMaxRHIShaderPlatform)
		&& GMaxRHIFeatureLevel > ERHIFeatureLevel::ES3_1) // see FLandscapeVertexFactory::ModifyCompilationEnvironment
	{
		Elements.Add(FVertexElement(1, 0, VET_UInt, 1, sizeof(uint32), true));
	}
}

IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FLandscapeVertexFactory, SF_Vertex, FLandscapeVertexFactoryVertexShaderParameters);
#if RHI_RAYTRACING
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FLandscapeVertexFactory, SF_Compute, FLandscapeVertexFactoryVertexShaderParameters);
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FLandscapeVertexFactory, SF_RayHitGroup, FLandscapeVertexFactoryVertexShaderParameters);
#endif // RHI_RAYTRACING
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FLandscapeVertexFactory, SF_Pixel, FLandscapeVertexFactoryPixelShaderParameters);

IMPLEMENT_VERTEX_FACTORY_TYPE(FLandscapeVertexFactory, "/Engine/Private/LandscapeVertexFactory.ush",
	  EVertexFactoryFlags::UsedWithMaterials
	| EVertexFactoryFlags::SupportsStaticLighting
	| EVertexFactoryFlags::SupportsDynamicLighting
	| EVertexFactoryFlags::SupportsCachingMeshDrawCommands
	| EVertexFactoryFlags::SupportsRayTracing
	| EVertexFactoryFlags::SupportsRayTracingDynamicGeometry
	| EVertexFactoryFlags::SupportsLightmapBaking
	| EVertexFactoryFlags::SupportsPrimitiveIdStream
	| EVertexFactoryFlags::SupportsPSOPrecaching
	| EVertexFactoryFlags::SupportsLandscape
);

/**
* Copy the data from another vertex factory
* @param Other - factory to copy from
*/
void FLandscapeVertexFactory::Copy(const FLandscapeVertexFactory& Other)
{
	//SetSceneProxy(Other.Proxy());
	FLandscapeVertexFactory* VertexFactory = this;
	const FDataType* DataCopy = &Other.Data;
	ENQUEUE_RENDER_COMMAND(FLandscapeVertexFactoryCopyData)(
		[VertexFactory, DataCopy](FRHICommandListImmediate& RHICmdList)
	{
		VertexFactory->Data = *DataCopy;
	});
	BeginUpdateResourceRHI(this);
}

//
// FLandscapeFixedGridVertexFactory
//

void FLandscapeFixedGridVertexFactory::ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	FLandscapeVertexFactory::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	OutEnvironment.SetDefine(TEXT("FIXED_GRID"), TEXT("1"));
}

IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FLandscapeFixedGridVertexFactory, SF_Vertex, FLandscapeFixedGridVertexFactoryVertexShaderParameters);
#if RHI_RAYTRACING
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FLandscapeFixedGridVertexFactory, SF_Compute, FLandscapeFixedGridVertexFactoryVertexShaderParameters);
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FLandscapeFixedGridVertexFactory, SF_RayHitGroup, FLandscapeFixedGridVertexFactoryVertexShaderParameters);
#endif
IMPLEMENT_VERTEX_FACTORY_PARAMETER_TYPE(FLandscapeFixedGridVertexFactory, SF_Pixel, FLandscapeVertexFactoryPixelShaderParameters);

IMPLEMENT_VERTEX_FACTORY_TYPE(FLandscapeFixedGridVertexFactory, "/Engine/Private/LandscapeVertexFactory.ush",
	  EVertexFactoryFlags::UsedWithMaterials
	| EVertexFactoryFlags::SupportsStaticLighting
	| EVertexFactoryFlags::SupportsDynamicLighting
	| EVertexFactoryFlags::SupportsCachingMeshDrawCommands
	| EVertexFactoryFlags::SupportsRayTracing
	| EVertexFactoryFlags::SupportsRayTracingDynamicGeometry
	| EVertexFactoryFlags::SupportsLightmapBaking
	| EVertexFactoryFlags::SupportsPrimitiveIdStream
	| EVertexFactoryFlags::SupportsPSOPrecaching
	| EVertexFactoryFlags::SupportsLumenMeshCards
	| EVertexFactoryFlags::SupportsLandscape
);

/** ULandscapeMaterialInstanceConstant */
ULandscapeMaterialInstanceConstant::ULandscapeMaterialInstanceConstant(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsLayerThumbnail = false;
}

void ULandscapeMaterialInstanceConstant::PostLoad()
{
	Super::PostLoad();
#if WITH_EDITOR
	UpdateCachedTextureStreaming();
#endif // WITH_EDITOR
}

float ULandscapeMaterialInstanceConstant::GetLandscapeTexelFactor(const FName& TextureName) const
{
	for (const FLandscapeMaterialTextureStreamingInfo& Info : TextureStreamingInfo)
	{
		if (Info.TextureName == TextureName)
		{
			return Info.TexelFactor;
		}
	}
	return 1.0f;
}

bool ULandscapeMaterialInstanceConstant::WritesToRuntimeVirtualTexture() const
{
	// Don't invalidate the RVTs for the thumbnail material updates, that would be a waste : 
	return !bIsLayerThumbnail && Super::WritesToRuntimeVirtualTexture();
}

#if WITH_EDITOR

void ULandscapeMaterialInstanceConstant::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	UpdateCachedTextureStreaming();
}

FLandscapeMaterialTextureStreamingInfo& ULandscapeMaterialInstanceConstant::AcquireTextureStreamingInfo(const FName& TextureName)
{
	for (FLandscapeMaterialTextureStreamingInfo& Info : TextureStreamingInfo)
	{
		if (Info.TextureName == TextureName)
		{
			return Info;
		}
	}
	FLandscapeMaterialTextureStreamingInfo& Info = TextureStreamingInfo.AddDefaulted_GetRef();
	Info.TextureName = TextureName;
	Info.TexelFactor = 1.0f;
	return Info;
}

void ULandscapeMaterialInstanceConstant::UpdateCachedTextureStreaming()
{
	// Remove outdated elements that no longer match the material's expressions.
	TextureStreamingInfo.Empty();

	const UMaterial* Material = GetMaterial();
	if (Material)
	{
		for (UMaterialExpression* Expression : Material->GetExpressions())
		{
			UMaterialExpressionTextureSample* TextureSample = Cast<UMaterialExpressionTextureSample>(Expression);

			// TODO: This is only works for direct Coordinate Texture Sample cases
			if (TextureSample && TextureSample->Texture && TextureSample->Coordinates.IsConnected())
			{
				if (UMaterialExpressionTextureCoordinate* TextureCoordinate = Cast<UMaterialExpressionTextureCoordinate>(TextureSample->Coordinates.Expression))
				{
					FLandscapeMaterialTextureStreamingInfo& Info = AcquireTextureStreamingInfo(TextureSample->Texture->GetFName());
					Info.TexelFactor *= FPlatformMath::Max(TextureCoordinate->UTiling, TextureCoordinate->VTiling);
				}
				else if (UMaterialExpressionLandscapeLayerCoords* TerrainTextureCoordinate = Cast<UMaterialExpressionLandscapeLayerCoords>(TextureSample->Coordinates.Expression))
				{
					FLandscapeMaterialTextureStreamingInfo& Info = AcquireTextureStreamingInfo(TextureSample->Texture->GetFName());
					Info.TexelFactor *= TerrainTextureCoordinate->MappingScale;
				}
			}
		}
	}
}

#endif // WITH_EDITOR

class FLandscapeMaterialResource : public FMaterialResource
{
	const bool bIsLayerThumbnail;
	const bool bMobile;
	const bool bEditorToolUsage;

public:
	FLandscapeMaterialResource(ULandscapeMaterialInstanceConstant* Parent)
		: bIsLayerThumbnail(Parent->bIsLayerThumbnail)
		, bMobile(Parent->bMobile)
		, bEditorToolUsage(Parent->bEditorToolUsage)
	{
	}

	bool IsUsedWithLandscape() const override
	{
		return !bIsLayerThumbnail;
	}

	bool IsUsedWithStaticLighting() const override
	{
		if (bIsLayerThumbnail)
		{
			return false;
		}
		return FMaterialResource::IsUsedWithStaticLighting();
	}

	bool IsUsedWithNanite() const override 
	{ 
		if (bIsLayerThumbnail)
		{
			return false;
		}
		return FMaterialResource::IsUsedWithNanite();
	}

	bool IsUsedWithWater() const override { return false; }
	bool IsUsedWithHairStrands() const override { return false; }
	bool IsUsedWithLidarPointCloud() const override { return false; }
	bool IsUsedWithSkeletalMesh() const override { return false; }
	bool IsUsedWithParticleSystem() const override { return false; }
	bool IsUsedWithParticleSprites() const override { return false; }
	bool IsUsedWithBeamTrails() const override { return false; }
	bool IsUsedWithMeshParticles() const override { return false; }
	bool IsUsedWithNiagaraSprites() const override { return false; }
	bool IsUsedWithNiagaraRibbons() const override { return false; }
	bool IsUsedWithNiagaraMeshParticles() const override { return false; }
	bool IsUsedWithMorphTargets() const override { return false; }
	bool IsUsedWithSplineMeshes() const override { return false; }
	bool IsUsedWithInstancedStaticMeshes() const override { return false; }
	bool IsUsedWithAPEXCloth() const override { return false; }
	bool IsUsedWithGeometryCollections() const override { return false; }
	bool IsUsedWithGeometryCache() const override { return false; }

	bool ShouldCache(const FShaderType* ShaderType, const FVertexFactoryType* VertexFactoryType) const override
	{
		if (bIsLayerThumbnail)
		{
			// Always check against FLocalVertexFactory in editor builds as it is required to render thumbnails
			// Thumbnail MICs are only rendered in the preview scene using a simple LocalVertexFactory
			static const FName LocalVertexFactory = FName("FLocalVertexFactory");
			if (!IsMobilePlatform(GetShaderPlatform()) && (VertexFactoryType != nullptr) && (VertexFactoryType->GetFName() == LocalVertexFactory))
			{
				if (Algo::Find(GetAllowedShaderTypesInThumbnailRender(), ShaderType->GetFName()))
				{
					return FMaterialResource::ShouldCache(ShaderType, VertexFactoryType);
				}
				else
				{
					// No ray tracing on thumbnails : we don't need any variation of ray hit group shaders : 
					const bool bIsRayHitGroupShader = (ShaderType->GetFrequency() == SF_RayHitGroup);
					if (bIsRayHitGroupShader
						|| Algo::Find(GetExcludedShaderTypesInThumbnailRender(), ShaderType->GetFName()))
					{
						UE_LOGF(LogLandscape, VeryVerbose, "Excluding shader %ls from landscape thumbnail material", ShaderType->GetName());
						return false;
					}
					else
					{
						UE_LOGF(LogLandscape, Warning, "Shader %ls unknown by landscape thumbnail material, please add to either AllowedShaderTypes or ExcludedShaderTypes", ShaderType->GetName());
						return FMaterialResource::ShouldCache(ShaderType, VertexFactoryType);
					}
				}
			}
		}
		else if (FMaterialResource::ShouldCache(ShaderType, VertexFactoryType))
		{
			// Landscape MICs are only for use with the Landscape vertex factories
			if (VertexFactoryType && VertexFactoryType->SupportsLandscape())
			{
				// For now only compile FLandscapeFixedGridVertexFactory for grass and runtime virtual texture page rendering (can change if we need for other cases)
				const bool bIsGrassShaderType = Algo::Find(GetGrassShaderTypes(), ShaderType->GetFName()) != nullptr;
				const bool bIsGPULightmassShaderType = Algo::Find(GetGPULightmassShaderTypes(), ShaderType->GetFName()) != nullptr;
				const bool bIsRuntimeVirtualTextureShaderType = Algo::Find(GetRuntimeVirtualTextureShaderTypes(), ShaderType->GetFName()) != nullptr;
				const bool bIsLumen = Algo::Find(GetLumenCardShaderTypes(), ShaderType->GetFName()) != nullptr;
				const bool bIsShaderTypeUsingFixedGrid = bIsGrassShaderType || bIsRuntimeVirtualTextureShaderType || bIsGPULightmassShaderType || bIsLumen;
				const bool bIsFixedGridVertexFactory = (&FLandscapeFixedGridVertexFactory::StaticType == VertexFactoryType);
				
				static const FName RayTracingDynamicGeometryConverterCS = FName("FRayTracingDynamicGeometryConverterCS");
				const bool bIsRayTracingShaderType = ShaderType->GetFName() == RayTracingDynamicGeometryConverterCS;

				return (bIsRayTracingShaderType 
					|| (bIsFixedGridVertexFactory == bIsShaderTypeUsingFixedGrid));
			}
			else
			{
				// Allow the landscape MICs on rasterizer shader types as well, 
				const bool bIsRasterizerShaderType = Algo::Find(GetRasterizeShaderTypes(), ShaderType->GetFName()) != nullptr;
				return bIsRasterizerShaderType;
			}
		}

		return false;
	}

	static const TArray<FName>& GetAllowedShaderTypesInThumbnailRender()
	{
		// reduce the number of shaders compiled for the thumbnail materials by only compiling with shader types known to be used by the preview scene
		static const TArray<FName> AllowedShaderTypes =
		{
			FName("TBasePassVSFNoLightMapPolicy"),
			FName("TBasePassPSFNoLightMapPolicy"),
			FName("TBasePassVSFCachedPointIndirectLightingPolicy"),
			FName("TBasePassPSFCachedPointIndirectLightingPolicy"),
			FName("FAnisotropyVS"),
			FName("FAnisotropyPS"),
			FName("TDepthOnlyVS<false>"),
			FName("TDepthOnlyVS<true>"),
			FName("FDepthOnlyPS"),
			// UE-44519, masked material with landscape layers requires FHitProxy shaders.
			FName("FHitProxyVS"),
			FName("FHitProxyPS"),
			FName("FVelocityVS"),
			FName("FVelocityPS"),

			FName("TBasePassVSFNoLightMapPolicySkyAtmosphereAP"),
			FName("TLightMapDensityVSFNoLightMapPolicy"),
			FName("TLightMapDensityPSFNoLightMapPolicy"),

			// Mobile
			FName("TMobileBasePassPSFMobileDirectionalLightAndSHIndirectPolicy"),
			FName("TMobileBasePassVSFMobileDirectionalLightAndSHIndirectPolicy"),
			FName("TMobileBasePassPSFNoLightMapPolicy"),
			FName("TMobileBasePassVSFNoLightMapPolicy"),

			// Runtime virtual texture
			FName("TVirtualTextureVSBaseColor"),
			FName("TVirtualTextureVSBaseColorNormal"),
			FName("TVirtualTextureVSBaseColorNormalSpecular"),
			FName("TVirtualTextureVSBaseColorNormalRoughness"),
			FName("TVirtualTextureVSMask4"),
			FName("TVirtualTextureVSWorldHeight"),
			FName("TVirtualTextureVSDisplacement"),
			FName("TVirtualTexturePSBaseColor"),
			FName("TVirtualTexturePSBaseColorNormal"),
			FName("TVirtualTexturePSBaseColorNormalSpecular"),
			FName("TVirtualTexturePSBaseColorNormalRoughness"),
			FName("TVirtualTexturePSMask4"),
			FName("TVirtualTexturePSWorldHeight"),
			FName("TVirtualTexturePSDisplacement"),
		};
		return AllowedShaderTypes;
	}

	static const TArray<FName>& GetExcludedShaderTypesInThumbnailRender()
	{
		// shader types known *not* to be used by the preview scene
		static const TArray<FName> ExcludedShaderTypes =
		{
			// This is not an exhaustive list
			FName("FDebugViewModeVS"),

			// No lightmap on thumbnails
			FName("TLightMapDensityVSFDummyLightMapPolicy"),
			FName("TLightMapDensityPSFDummyLightMapPolicy"),
			FName("TLightMapDensityPSTLightMapPolicyHQ"),
			FName("TLightMapDensityVSTLightMapPolicyHQ"),
			FName("TLightMapDensityPSTLightMapPolicyLQ"),
			FName("TLightMapDensityVSTLightMapPolicyLQ"),
			FName("TBasePassPSTDistanceFieldShadowsAndLightMapPolicyHQ"),
			FName("TBasePassVSTDistanceFieldShadowsAndLightMapPolicyHQ"),
			FName("TBasePassPSTLightMapPolicyHQ"),
			FName("TBasePassVSTLightMapPolicyHQ"),
			FName("TBasePassPSTLightMapPolicyLQ"),
			FName("TBasePassVSTLightMapPolicyLQ"),

			// Debug materials : 
			FName("FDebugViewModePS"),

			// Mobile
			FName("TMobileBasePassPSFMobileMovableDirectionalLightCSMWithLightmapPolicy"),
			FName("TMobileBasePassVSFMobileMovableDirectionalLightCSMWithLightmapPolicy"),
			FName("TMobileBasePassPSFMobileMovableDirectionalLightWithLightmapPolicy"),
			FName("TMobileBasePassVSFMobileMovableDirectionalLightWithLightmapPolicy"),
			FName("TMobileBasePassPSFMobileDistanceFieldShadowsLightMapAndCSMLightingPolicy"),
			FName("TMobileBasePassVSFMobileDistanceFieldShadowsLightMapAndCSMLightingPolicy"),
			FName("TMobileBasePassPSFMobileDistanceFieldShadowsAndLQLightMapPolicy"),
			FName("TMobileBasePassVSFMobileDistanceFieldShadowsAndLQLightMapPolicy"),
			FName("TMobileBasePassPSTLightMapPolicyLQ"),
			FName("TMobileBasePassVSTLightMapPolicyLQ"),
			FName("TMobileBasePassPSFMobileDirectionalLightAndCSMPolicy"),
			FName("TMobileBasePassVSFMobileDirectionalLightAndCSMPolicy"),

			FName("TBasePassVSFCachedVolumeIndirectLightingPolicy"),
			FName("TBasePassPSFCachedVolumeIndirectLightingPolicy"),
			FName("TBasePassVSFPrecomputedVolumetricLightmapLightingPolicy"),
			FName("TBasePassPSFPrecomputedVolumetricLightmapLightingPolicy"),

			FName("TBasePassVSFCachedPointIndirectLightingPolicy"),
			FName("TBasePassVSFSelfShadowedCachedPointIndirectLightingPolicy"),
			FName("TBasePassPSFSelfShadowedCachedPointIndirectLightingPolicy"),
			FName("TBasePassVSFSelfShadowedTranslucencyPolicy"),
			FName("TBasePassPSFSelfShadowedTranslucencyPolicy"),

			FName("TShadowDepthVSVertexShadowDepth_PerspectiveCorrect"),
			FName("TShadowDepthVSVertexShadowDepth_OutputDepth"),
			FName("TShadowDepthVSVertexShadowDepth_OnePassPointLight"),
			FName("TShadowDepthVSVertexShadowDepth_VirtualShadowMap"),

			FName("TShadowDepthVSVertexShadowDepth_PerspectiveCorrectPositionOnly"),
			FName("TShadowDepthVSVertexShadowDepth_OutputDepthPositionOnly"),
			FName("TShadowDepthVSVertexShadowDepth_OnePassPointLightPositionOnly"),
			FName("TShadowDepthVSVertexShadowDepth_VirtualShadowMapPositionOnly"),

			FName("TShadowDepthPSPixelShadowDepth_VirtualShadowMap"),
			FName("TShadowDepthPSPixelShadowDepth_PerspectiveCorrect"),
			FName("TShadowDepthPSPixelShadowDepth_OnePassPointLight"),
			FName("TShadowDepthPSPixelShadowDepth_NonPerspectiveCorrect"),

			FName("TShadowDepthVSForGSVertexShadowDepth_OnePassPointLight"),
			FName("TShadowDepthVSForGSVertexShadowDepth_OnePassPointLightPositionOnly"),
			FName("TShadowDepthVSVertexShadowDepth_VSLayer"),
			FName("TShadowDepthVSVertexShadowDepth_VSLayerPositionOnly"),
			FName("TShadowDepthVSVertexShadowDepth_VSLayerGS"),
			FName("TShadowDepthVSVertexShadowDepth_VSLayerGSPositionOnly"),

			FName("FOnePassPointShadowDepthGS"),

			FName("TTranslucencyShadowDepthVS<TranslucencyShadowDepth_Standard>"),
			FName("TTranslucencyShadowDepthPS<TranslucencyShadowDepth_Standard>"),
			FName("TTranslucencyShadowDepthVS<TranslucencyShadowDepth_PerspectiveCorrect>"),
			FName("TTranslucencyShadowDepthPS<TranslucencyShadowDepth_PerspectiveCorrect>"),

			FName("TBasePassVSTDistanceFieldShadowsAndLightMapPolicyHQ"),
			FName("TBasePassVSTLightMapPolicyHQ"),
			FName("TBasePassVSTLightMapPolicyLQ"),
			FName("TBasePassPSFSelfShadowedVolumetricLightmapPolicy"),
			FName("TBasePassVSFSelfShadowedVolumetricLightmapPolicy"),

			// No need for velocity in thumbnails
			FName("TVelocityVS<EVelocityPassMode::Velocity_Standard>"),
			FName("TVelocityPS<EVelocityPassMode::Velocity_Standard>"),
			FName("TVelocityVS<EVelocityPassMode::Velocity_ClippedDepth>"),
			FName("TVelocityPS<EVelocityPassMode::Velocity_ClippedDepth>"),
			FName("TVelocityVS<EVelocityPassMode::Velocity_StereoMotionVectors>"),
			FName("TVelocityPS<EVelocityPassMode::Velocity_StereoMotionVectors>"),

#if RHI_RAYTRACING
			// No ray tracing on thumbnails
			FName("FRayTracingDynamicGeometryConverterCS"),
			FName("FTrivialMaterialCHS"),
#endif // RHI_RAYTRACING

			// No Lumen on thumbnails
			FName("FLumenCardCS"),
			FName("FLumenCardVS"),
			FName("FLumenCardPS"),

			// No material cache on thumbnails
			FName("FMaterialCacheUnwrapVS0"),
			FName("FMaterialCacheUnwrapVS1"),
			FName("FMaterialCacheUnwrapPS"),
			FName("FMaterialCacheNaniteShadeCS"),
			FName("FMaterialCacheShadeCS"),
		};
		return ExcludedShaderTypes;
	}

	static const TArray<FName>& GetGPULightmassShaderTypes()
	{
		static const TArray<FName> ShaderTypes =
		{
			FName("TLightmapMaterialCHS<true>"),
			FName("TLightmapMaterialCHS<false>"),
			FName("FVLMVoxelizationVS"),
			FName("FVLMVoxelizationGS"),
			FName("FVLMVoxelizationPS"),
			FName("FLightmapGBufferVS"),
			FName("FLightmapGBufferPS"),
			FName("FGPULightmassCHS"),
			FName("FGPULightmassCHS_AHS"),
		};
		return ShaderTypes;
	}

	static const TArray<FName>& GetGrassShaderTypes()
	{
		static const TArray<FName> ShaderTypes =
		{
			FName("FLandscapeGrassWeightVS"),
			FName("FLandscapeGrassWeightPS"),
			FName("FLandscapePhysicalMaterialVS"),
			FName("FLandscapePhysicalMaterialPS"),
		};
		return ShaderTypes;
	}

	static const TArray<FName>& GetRuntimeVirtualTextureShaderTypes()
	{
		static const TArray<FName> ShaderTypes =
		{
			FName("TVirtualTextureVSBaseColor"),
			FName("TVirtualTextureVSBaseColorNormal"),
			FName("TVirtualTextureVSBaseColorNormalSpecular"),
			FName("TVirtualTextureVSBaseColorNormalRoughness"),
			FName("TVirtualTextureVSMask4"),
			FName("TVirtualTextureVSWorldHeight"),
			FName("TVirtualTextureVSDisplacement"),
			FName("TVirtualTexturePSBaseColor"),
			FName("TVirtualTexturePSBaseColorNormal"),
			FName("TVirtualTexturePSBaseColorNormalSpecular"),
			FName("TVirtualTexturePSBaseColorNormalRoughness"),
			FName("TVirtualTexturePSMask4"),
			FName("TVirtualTexturePSWorldHeight"),
			FName("TVirtualTexturePSDisplacement"),
		};
		return ShaderTypes;
	}

	static const TArray<FName>& GetLumenCardShaderTypes()
	{
		static const TArray<FName> ShaderTypes =
		{
			FName("FLumenCardVS"),
			FName("FLumenCardPS"),
		};
		return ShaderTypes;
	}

	static const TArray<FName>& GetRasterizeShaderTypes()
	{
		static const TArray<FName> ShaderTypes =
		{
			FName("FHWRasterizeVS"),
			FName("FHWRasterizeMS"),
			FName("FHWRasterizePS"),
			FName("FMicropolyRasterizeCS"),
		};
		return ShaderTypes;
	}

};

FMaterialResource* ULandscapeMaterialInstanceConstant::AllocatePermutationResource()
{
	return new FLandscapeMaterialResource(this);
}

bool ULandscapeMaterialInstanceConstant::HasOverridenBaseProperties() const
{
	if (Parent)
	{
		// force a static permutation for ULandscapeMaterialInstanceConstants
		if (!Parent->IsA<ULandscapeMaterialInstanceConstant>())
		{
			return true;
		}
	}

	return Super::HasOverridenBaseProperties();
}

//////////////////////////////////////////////////////////////////////////

void ULandscapeComponent::GetStreamingRenderAssetInfo(FStreamingTextureLevelContext& LevelContext, TArray<FStreamingRenderAssetPrimitiveInfo>& OutStreamingRenderAssets) const
{
	ALandscapeProxy* Proxy = Cast<ALandscapeProxy>(GetOuter());
	FSphere BoundingSphere = Bounds.GetSphere();
	float LocalStreamingDistanceMultiplier = 1.f;
	float TexelFactor = 0.0f;
	if (Proxy)
	{
		double ScaleFactor = 1.0;
		if (USceneComponent* ProxyRootComponent = Proxy->GetRootComponent())
		{
			ScaleFactor = FMath::Abs(ProxyRootComponent->GetRelativeScale3D().X);
		}
		LocalStreamingDistanceMultiplier = FMath::Max(0.0f, Proxy->StreamingDistanceMultiplier);
		TexelFactor = static_cast<float>(0.75f * LocalStreamingDistanceMultiplier * ComponentSizeQuads * ScaleFactor);
	}

	ERHIFeatureLevel::Type FeatureLevel = LevelContext.GetFeatureLevel();
	int32 MaterialInstanceCount = GetCurrentRuntimeMaterialInstanceCount();

	for (int32 MaterialIndex = 0; MaterialIndex < MaterialInstanceCount; ++MaterialIndex)
	{
		const UMaterialInterface* MaterialInterface = GetCurrentRuntimeMaterialInterface(MaterialIndex);

		// Normal usage...
		// Enumerate the textures used by the material.
		if (MaterialInterface)
		{
			TArray<UTexture*> Textures;
			MaterialInterface->GetUsedTextures(Textures, GetCurrentMaterialQualityLevelChecked(), GetFeatureLevelShaderPlatform_Checked(FeatureLevel));

			const ULandscapeMaterialInstanceConstant* LandscapeMaterial = Cast<ULandscapeMaterialInstanceConstant>(MaterialInterface);

			// Add each texture to the output with the appropriate parameters.
			// TODO: Take into account which UVIndex is being used.
			for (int32 TextureIndex = 0; TextureIndex < Textures.Num(); TextureIndex++)
			{
				UTexture2D* Texture2D = Cast<UTexture2D>(Textures[TextureIndex]);
				if (!Texture2D) continue;

				FStreamingRenderAssetPrimitiveInfo& StreamingTexture = *new(OutStreamingRenderAssets)FStreamingRenderAssetPrimitiveInfo;
				StreamingTexture.Bounds = BoundingSphere;
				StreamingTexture.TexelFactor = TexelFactor;
				StreamingTexture.RenderAsset = Texture2D;

				if (LandscapeMaterial)
				{
					const float MaterialTexelFactor = LandscapeMaterial->GetLandscapeTexelFactor(Texture2D->GetFName());
					StreamingTexture.TexelFactor *= MaterialTexelFactor;
				}
			}

			// Lightmap
			const FMeshMapBuildData* MapBuildData = GetMeshMapBuildData();

			FLightMap2D* Lightmap = MapBuildData && MapBuildData->LightMap ? MapBuildData->LightMap->GetLightMap2D() : nullptr;
			uint32 LightmapIndex = AllowHighQualityLightmaps(FeatureLevel) ? 0 : 1;
			if (Lightmap && Lightmap->IsValid(LightmapIndex))
			{
				const FVector2D& Scale = Lightmap->GetCoordinateScale();
				if (Scale.X > SMALL_NUMBER && Scale.Y > SMALL_NUMBER)
				{
					const float LightmapTexelFactor = static_cast<float>(TexelFactor / FMath::Min(Scale.X, Scale.Y));
					new (OutStreamingRenderAssets) FStreamingRenderAssetPrimitiveInfo(Lightmap->GetTexture(LightmapIndex), Bounds, LightmapTexelFactor);
					new (OutStreamingRenderAssets) FStreamingRenderAssetPrimitiveInfo(Lightmap->GetAOMaterialMaskTexture(), Bounds, LightmapTexelFactor);
					new (OutStreamingRenderAssets) FStreamingRenderAssetPrimitiveInfo(Lightmap->GetSkyOcclusionTexture(), Bounds, LightmapTexelFactor);
				}
			}

			// Shadowmap
			FShadowMap2D* Shadowmap = MapBuildData && MapBuildData->ShadowMap ? MapBuildData->ShadowMap->GetShadowMap2D() : nullptr;
			if (Shadowmap && Shadowmap->IsValid())
			{
				const FVector2D& Scale = Shadowmap->GetCoordinateScale();
				if (Scale.X > SMALL_NUMBER && Scale.Y > SMALL_NUMBER)
				{
					const float ShadowmapTexelFactor = static_cast<float>(TexelFactor / FMath::Min(Scale.X, Scale.Y));
					new (OutStreamingRenderAssets) FStreamingRenderAssetPrimitiveInfo(Shadowmap->GetTexture(), Bounds, ShadowmapTexelFactor);
				}
			}
		}
	}

	// Heightmap has usually not been accounted for by GetUsedTextures on the material (unless referenced directly by name, but that's borderline a hack) :
	if (HeightmapTexture)
	{
		FStreamingRenderAssetPrimitiveInfo& StreamingHeightmap = *new(OutStreamingRenderAssets)FStreamingRenderAssetPrimitiveInfo;
		StreamingHeightmap.Bounds = BoundingSphere;

		float HeightmapTexelFactor = TexelFactor * (static_cast<float>(HeightmapTexture->GetSizeY()) / (ComponentSizeQuads + 1));
		StreamingHeightmap.TexelFactor = ForcedLOD >= 0 ? -(1 << (13 - ForcedLOD)) : HeightmapTexelFactor; // Minus Value indicate forced resolution (Mip 13 for 8k texture)
		StreamingHeightmap.RenderAsset = HeightmapTexture;
	}

#if WITH_EDITOR
	if (GIsEditor)
	{
		if (EditToolRenderData.DataTexture)
		{
			FStreamingRenderAssetPrimitiveInfo& StreamingDatamap = *new(OutStreamingRenderAssets)FStreamingRenderAssetPrimitiveInfo;
			StreamingDatamap.Bounds = BoundingSphere;
			StreamingDatamap.TexelFactor = TexelFactor;
			StreamingDatamap.RenderAsset = EditToolRenderData.DataTexture;
		}

		if (EditToolRenderData.LayerContributionTexture)
		{
			FStreamingRenderAssetPrimitiveInfo& StreamingDatamap = *new(OutStreamingRenderAssets)FStreamingRenderAssetPrimitiveInfo;
			StreamingDatamap.Bounds = BoundingSphere;
			StreamingDatamap.TexelFactor = TexelFactor;
			StreamingDatamap.RenderAsset = EditToolRenderData.LayerContributionTexture;
		}

		if (EditToolRenderData.DirtyTexture)
		{
			FStreamingRenderAssetPrimitiveInfo& StreamingDatamap = *new(OutStreamingRenderAssets)FStreamingRenderAssetPrimitiveInfo;
			StreamingDatamap.Bounds = BoundingSphere;
			StreamingDatamap.TexelFactor = TexelFactor;
			StreamingDatamap.RenderAsset = EditToolRenderData.DirtyTexture;
		}
	}
#endif
}

void ALandscapeProxy::ChangeComponentScreenSizeToUseSubSections(float InComponentScreenSizeToUseSubSections)
{
	// Deprecated
}

void ALandscapeProxy::ChangeLODDistanceFactor(float InLODDistanceFactor)
{
	// Deprecated
}

bool FLandscapeComponentSceneProxy::HeightfieldHasPendingStreaming() const
{
	bool bHeightmapTextureStreaming = false;
	if (HeightmapTexture)
	{
		// this is technically a game thread value and not render-thread safe, but it shouldn't ever crash, may just be out of date.
		// there doesn't appear to be any render thread equivalent, the render thread is ignorant of streaming state.
		// in general, HeightfieldHasPendingStreaming() should only be used if the code is ok with a slightly out of date value being returned.
		bHeightmapTextureStreaming |= HeightmapTexture->bHasStreamingUpdatePending;
#if WITH_EDITOR
		if (const FTexture2DResource* HeightmapTextureResource = (const FTexture2DResource*)HeightmapTexture->GetResource())
		{
			bHeightmapTextureStreaming |= HeightmapTextureResource->IsProxy();
		}
#endif
	}

	bool bVisibilityTextureStreaming = false;
	if (VisibilityWeightmapTexture)
	{
		// again, not render thread safe (see above)
		bVisibilityTextureStreaming |= VisibilityWeightmapTexture->bHasStreamingUpdatePending;
#if WITH_EDITOR
		if (const FTexture2DResource* VisibilityTextureResource = (const FTexture2DResource*)VisibilityWeightmapTexture->GetResource())
		{
			bVisibilityTextureStreaming |= VisibilityTextureResource->IsProxy();
		}
#endif
	}

	return bHeightmapTextureStreaming || bVisibilityTextureStreaming;
}

void FLandscapeComponentSceneProxy::GetHeightfieldRepresentation(UTexture2D*& OutHeightmapTexture, UTexture2D*& OutVisibilityTexture, FHeightfieldComponentDescription& OutDescription) const
{
	OutHeightmapTexture = HeightmapTexture;
	OutVisibilityTexture = VisibilityWeightmapTexture;

	OutDescription.HeightfieldScaleBias = HeightmapScaleBias;

	OutDescription.MinMaxUV = FVector4f(
		HeightmapScaleBias.Z,
		HeightmapScaleBias.W,
		HeightmapScaleBias.Z + SubsectionSizeVerts * NumSubsections * HeightmapScaleBias.X - HeightmapScaleBias.X,
		HeightmapScaleBias.W + SubsectionSizeVerts * NumSubsections * HeightmapScaleBias.Y - HeightmapScaleBias.Y);

	OutDescription.HeightfieldRect = FIntRect(SectionBase.X, SectionBase.Y, SectionBase.X + NumSubsections * SubsectionSizeQuads, SectionBase.Y + NumSubsections * SubsectionSizeQuads);

	OutDescription.NumSubsections = NumSubsections;

	OutDescription.SubsectionScaleAndBias = FVector4(SubsectionSizeQuads, SubsectionSizeQuads, HeightmapSubsectionOffsetU, HeightmapSubsectionOffsetV);

	OutDescription.VisibilityChannel = VisibilityWeightmapChannel;
}

void FLandscapeComponentSceneProxy::GetLCIs(FLCIArray& LCIs)
{
	FLightCacheInterface* LCI = ComponentLightInfo.Get();
	if (LCI)
	{
		LCIs.Push(LCI);
	}
}

float FLandscapeComponentSceneProxy::ComputeLODForView(const FSceneView& InView) const
{
	int32 ViewLODOverride = GetViewLodOverride(InView, LandscapeKey);
	float ViewLODDistanceFactor = InView.LODDistanceFactor;
	bool ViewEngineShowFlagCollisionPawn = InView.Family->EngineShowFlags.CollisionPawn;
	bool ViewEngineShowFlagCollisionVisibility = InView.Family->EngineShowFlags.CollisionVisibility;
	const FVector& ViewOrigin = GetLODView(InView).ViewMatrices.GetViewOrigin();
	const FMatrix& ViewProjectionMatrix = GetLODView(InView).ViewMatrices.GetViewToClip();

	float LODScale = ViewLODDistanceFactor * CVarStaticMeshLODDistanceScale.GetValueOnRenderThread();

	FLandscapeRenderSystem* LandscapeRenderSystem = LandscapeRenderSystems.FindChecked(LandscapeKey);

	// Prefer the RenderSystem's ForcedLODOverride if set over any per-component LOD override
	int32 ForcedLODLevel = LandscapeRenderSystem->ForcedLODOverride >= 0 ? LandscapeRenderSystem->ForcedLODOverride : LODSettings.ForcedLOD;
	ForcedLODLevel = ViewLODOverride >= 0 ? ViewLODOverride : ForcedLODLevel;
	const int32 DrawCollisionLODOverride = GetDrawCollisionLodOverride(ViewEngineShowFlagCollisionPawn, ViewEngineShowFlagCollisionVisibility, LODSettings.DrawCollisionPawnLOD, LODSettings.DrawCollisionVisibilityLOD);
	ForcedLODLevel = DrawCollisionLODOverride >= 0 ? DrawCollisionLODOverride : ForcedLODLevel;
	ForcedLODLevel = FMath::Min<int32>(ForcedLODLevel, LODSettings.LastLODIndex);

	float LODLevel = static_cast<float>(ForcedLODLevel);
	if (ForcedLODLevel < 0)
	{
		float SectionScreenSizeSquared = ComputeBoundsScreenRadiusSquared(GetBounds().Origin, static_cast<float>(GetBounds().SphereRadius), ViewOrigin, ViewProjectionMatrix);
		SectionScreenSizeSquared /= FMath::Max(LODScale * LODScale, UE_SMALL_NUMBER);
		LODLevel = FLandscapeRenderSystem::ComputeLODFromScreenSize(LODSettings, SectionScreenSizeSquared);
	}

	return FMath::Max(LODLevel, 0.f);
}

float FLandscapeComponentSceneProxy::ComputeLODBias() const
{
	float ComputedLODBias = 0;

	if (HeightmapTexture)
	{
		if (const FTexture2DResource* TextureResource = (const FTexture2DResource*)HeightmapTexture->GetResource())
		{
			ComputedLODBias = static_cast<float>(TextureResource->GetCurrentFirstMip());
		}
	}

	return ComputedLODBias;
}

const FPrimitiveSceneInfo* FLandscapeComponentSceneProxy::GetPrimitiveSceneInfo() const
{
	return FPrimitiveSceneProxy::GetPrimitiveSceneInfo();
}

void FLandscapeComponentSceneProxy::OnRenderCoordsChanged(FRHICommandListBase& RHICmdList)
{
	// need to rebuild the uniforms that contain render coords
	OnTransformChanged(RHICmdList);
}

double FLandscapeComponentSceneProxy::ComputeSectionResolution() const
{
	// ComponentMaxExtend is the max(length,width) of the component, in world units
	const double ComponentFullExtent = ComponentMaxExtend;
	const double ComponentQuads = ComponentSizeVerts - 1.0;		// verts = quads + 1
	return ComponentFullExtent / ComponentQuads;
}

int32 FLandscapeComponentSceneProxy::GetComponentResolution() const
{
	return ComponentSizeVerts;
}

void FLandscapeComponentSceneProxy::GetSectionBoundsAndLocalToWorld(FBoxSphereBounds& OutLocalBounds, FMatrix& OutLocalToWorld) const
{
	OutLocalBounds = GetLocalBounds();
	OutLocalToWorld = GetLocalToWorld();
}

void FLandscapeComponentSceneProxy::GetSectionCenterAndVectors(FVector& OutSectionCenterWorldSpace, FVector& OutSectionXVectorWorldSpace, FVector& OutSectionYVectorWorldSpace) const
{
	FBoxSphereBounds ComponentLocalBounds = GetLocalBounds();
	FMatrix ComponentLocalToWorld = GetLocalToWorld();
	int32 ComponentResolution = GetComponentResolution();

	OutSectionCenterWorldSpace = ComponentLocalToWorld.TransformPosition(ComponentLocalBounds.Origin);
	OutSectionXVectorWorldSpace = ComponentLocalToWorld.TransformVector(FVector::XAxisVector) * ComponentResolution;
	OutSectionYVectorWorldSpace = ComponentLocalToWorld.TransformVector(FVector::YAxisVector) * ComponentResolution;
}

bool FLandscapeComponentSceneProxy::ShouldInvalidateShadows(const FSceneView& InView, float InLODValue, float InLastShadowInvalidationLODValue) const
{
	if (WorldSpaceMipToMipMaxDeltas.IsEmpty() // Only apply if we have computed the error estimates
		|| !GLandscapeAllowNonNaniteVirtualShadowMapInvalidation // Global switch
		|| (bNaniteActive && InView.Family->EngineShowFlags.NaniteMeshes) // Only applies if Nanite is not active
		|| (VirtualShadowMapInvalidationHeightErrorThreshold <= 0.0f)) // Only applies if the threshold is valid
	{
		return false;
	}

	// We want to estimate the height error between the height at the current LOD transition and the height at which the last shadow invalidation took place
	// Start by clamping the values so that we're always in the [0, last relevant mip[ range (we only have mip-to-mip data up until that mip) 
	const float LODValueClamped = FMath::Min(InLODValue, static_cast<float>(NumRelevantMips - 1) - UE_KINDA_SMALL_NUMBER);
	const float LastShadowInvalidationLODValueClamped = FMath::Min(InLastShadowInvalidationLODValue, static_cast<float>(NumRelevantMips - 1) - UE_KINDA_SMALL_NUMBER);

	// We have at our disposal the max error between a mip and any of its higher mips (N->N+1, N->N+2, etc.) so let's evaluate the error of both LODValue and LastShadowInvalidationLODValue with the 
	//  lowest of the 2 mips as the common basis :
	//  e.g.:
	//  |          Mip N           |         Mip N + 1        |         Mip N + 2        |
	//  -------------------------------------------------------------------------------------...
	//  ^       ^                  ^                          ^  ^                       ^
	//  |       |                  |                          |  |                       |
	//  |       SourceLODValue     |                          |  DestinationLODValue     |    
	//  SourceMipIndex             SourceMipIndex + 1         DestinationMipIndex        DestinationMipIndex + 1

	// It doesn't matter the direction of the change, the height difference is computed in absolute values, so invert the 2 if necessary :
	float SourceLODValue = (LODValueClamped < LastShadowInvalidationLODValueClamped) ? LODValueClamped : LastShadowInvalidationLODValueClamped;
	float DestinationLODValue = (LODValueClamped < LastShadowInvalidationLODValueClamped) ? LastShadowInvalidationLODValueClamped : LODValueClamped;

	const int32 SourceMipIndex = FMath::FloorToInt32(SourceLODValue);
	check(SourceMipIndex + 1 < NumRelevantMips);
	const int32 DestinationMipIndex = FMath::FloorToInt32(DestinationLODValue);
	check(DestinationMipIndex + 1 < NumRelevantMips);

	// Evaluate the max delta for both SourceLODValue and DestinationLODValue against SourceMipIndex :
	const int32 SourceMipToMipMaxDeltaIndex = UE::Landscape::Private::ComputeMipToMipMaxDeltasIndex(SourceMipIndex, SourceMipIndex + 1, NumRelevantMips);
	const double SourceMipToMipMaxDelta = WorldSpaceMipToMipMaxDeltas[SourceMipToMipMaxDeltaIndex];
	// MipToMipMaxDelta represents the maximum delta if we were to transition from SourceMipIndex to SourceMipIndex + 1 but we want to compute the error at SourceLODValue
	//  so re-scale the delta within that range to evaluate the actual error : 
	const double SourceMaxDelta = SourceMipToMipMaxDelta * (SourceLODValue - SourceMipIndex);

	const int32 DestinationMipToMipMaxDeltaIndex = UE::Landscape::Private::ComputeMipToMipMaxDeltasIndex(SourceMipIndex, DestinationMipIndex + 1, NumRelevantMips);
	const double DestinationMipToMipMaxDelta = WorldSpaceMipToMipMaxDeltas[DestinationMipToMipMaxDeltaIndex];
	// MipToMipMaxDelta represents the maximum delta if we were to transition from SourceMipIndex to DestinationMipIndex + 1 but we want to compute the error at DestinationLODValue
	//  so re-scale the delta within that range to evaluate the actual error : 
	check(SourceMipIndex < DestinationMipIndex + 1);
	const double DestinationMaxDelta = DestinationMipToMipMaxDelta * (DestinationLODValue - SourceMipIndex) / (DestinationMipIndex + 1 - SourceMipIndex);

	// Now we estimate that the MaxDelta between those 2 is the difference : 
	const double MaxDelta = FMath::Abs(DestinationMaxDelta - SourceMaxDelta);

	// Perform screen size-based attenuation in order to decrease the invalidation rate as the screen size decreases (as error tends to grow on higher mips, the invalidation rate increases, which 
	//  is not desirable, since it means the screen size of the landscape section actually decreases, and so the shadow artifacts actually become less noticeable)
	double MaxDeltaLODAttenuationFactor = 1.0;
	if (LODSettings.VirtualShadowMapInvalidationLimitLOD > 0.0f)
	{
		// The closer we get to the limit LOD, the more we attenuate : 
		MaxDeltaLODAttenuationFactor = FMath::Clamp(1.0f - (SourceLODValue / LODSettings.VirtualShadowMapInvalidationLimitLOD), 0.0f, 1.0f);
		// Now that we're in the [0,1] range, apply an exponent to have a non-linear attenuation : 
		MaxDeltaLODAttenuationFactor = FMath::Pow(MaxDeltaLODAttenuationFactor, GLandscapeNonNaniteVirtualShadowMapInvalidationLODAttenuationExponent);
	}

	const double AttenuatedMaxDelta = MaxDelta * MaxDeltaLODAttenuationFactor;

	bool bShouldInvalidateShadow = (AttenuatedMaxDelta > VirtualShadowMapInvalidationHeightErrorThreshold);

#if !UE_BUILD_SHIPPING
	if (bShouldInvalidateShadow)
	{
		UE_LOGF(LogLandscape, Verbose, "Shadow invalidation occured: View: %d, %ls, LODValue: %f, LastShadowInvalidationLODValue: %f, \n"
			"MipToMipMaxDelta(%d<->%d): %f (unscaled: %f), OtherMipToMipMaxDelta(%d<->%d): %f (unscaled: %f)\n"
			"AttenuatedMaxDelta: %f, (MaxDelta: %f, MaxDeltaLODAttenuationFactor: %f, InvalidationLimitLOD: %f)",
			InView.GetViewKey(), *DebugName.ToString(), InLODValue, InLastShadowInvalidationLODValue,
			SourceMipIndex, SourceMipIndex + 1, SourceMaxDelta, SourceMipToMipMaxDelta, SourceMipIndex, DestinationMipIndex + 1, DestinationMaxDelta, DestinationMipToMipMaxDelta,
			AttenuatedMaxDelta, MaxDelta, MaxDeltaLODAttenuationFactor, LODSettings.VirtualShadowMapInvalidationLimitLOD);
	}
#endif // !UE_BUILD_SHIPPING

	return bShouldInvalidateShadow;
}

#if RHI_RAYTRACING
void FLandscapeComponentSceneProxy::RayTracingNotifyViewDeleted(uint32 ViewKey)
{
	if (RayTracingImpl)
	{
		for (int32 ViewStateIndex = 0; ViewStateIndex < RayTracingImpl->PerViewRayTracingState.Num(); ViewStateIndex++)
		{
			if (RayTracingImpl->PerViewRayTracingState[ViewStateIndex].ViewKey == ViewKey)
			{
				RayTracingImpl->PerViewRayTracingState.RemoveAtSwap(ViewStateIndex);
				break;
			}
		}
	}
}
#endif


//
// FLandscapeSectionInfo
//
FLandscapeSectionInfo::FLandscapeSectionInfo(const UWorld* InWorld, const FGuid& InLandscapeGuid, const FIntPoint& InComponentBase, uint32 LODGroupKey, uint32 InLandscapeKey)
	: LandscapeKey(InLandscapeKey)
	, LODGroupKey(LODGroupKey)
	, ComponentBase(InComponentBase)
	, Scene(InWorld->Scene)
	, bResourcesCreated(false)
	, bRegistered(false)
{
}

//
// FLandscapeProxySectionInfo
//
class FLandscapeProxySectionInfo : public FLandscapeSectionInfo
{
public:
	FLandscapeProxySectionInfo(const UWorld* InWorld, const FGuid& InLandscapeGuid, const FIntPoint& InComponentBase, const FVector& InComponentCenterLocalSpace, const FVector& InComponentXVectorLocalSpace, const FVector& InComponentYVectorLocalSpace, const FTransform& LocalToWorld, int32 SectionComponentResolution, int8 InProxyLOD, uint32 LODGroupKey, uint32 InLandscapeKey)
		: FLandscapeSectionInfo(InWorld, InLandscapeGuid, InComponentBase, LODGroupKey, InLandscapeKey)
		, ProxyLOD(InProxyLOD)
		, ComponentResolution(SectionComponentResolution)
		, CenterWorldSpace(LocalToWorld.TransformPosition(InComponentCenterLocalSpace))
		, XVectorWorldSpace(LocalToWorld.TransformVector(InComponentXVectorLocalSpace))
		, YVectorWorldSpace(LocalToWorld.TransformVector(InComponentYVectorLocalSpace))
	{
	}

	virtual float ComputeLODForView(const FSceneView& InView) const override
	{
		return ProxyLOD;
	}

	virtual float ComputeLODBias() const override
	{
		return 0.0f;
	}

	virtual int32 GetSectionPriority() const override
	{
		return ProxyLOD;
	}
	
	virtual void GetSectionBoundsAndLocalToWorld(FBoxSphereBounds& LocalBounds, FMatrix& LocalToWorld) const override
	{
		LocalBounds = FBoxSphereBounds(ForceInit);
		LocalToWorld = FMatrix::Identity;
	}

	virtual void GetSectionCenterAndVectors(FVector& OutSectionCenterWorldSpace, FVector& OutSectionXVectorWorldSpace, FVector& OutSectionYVectorWorldSpace) const override
	{
		OutSectionCenterWorldSpace = CenterWorldSpace;
		OutSectionXVectorWorldSpace = XVectorWorldSpace;
		OutSectionYVectorWorldSpace = YVectorWorldSpace;
	}

	virtual int32 GetComponentResolution() const override
	{
		return ComponentResolution;
	}

	virtual void OnRenderCoordsChanged(FRHICommandListBase& RHICmdList)
	{
	}

	virtual const FPrimitiveSceneInfo* GetPrimitiveSceneInfo() const override
	{
		return nullptr;
	}

#if RHI_RAYTRACING
	virtual void RayTracingNotifyViewDeleted(uint32 ViewKey) override
	{
	}
#endif

private:
	int8 ProxyLOD;
	int32 ComponentResolution;
	FVector CenterWorldSpace;
	FVector XVectorWorldSpace;
	FVector YVectorWorldSpace;
};

//
// FLandscapeMeshProxySceneProxy
//
FLandscapeMeshProxySceneProxy::FLandscapeMeshProxySceneProxy(UStaticMeshComponent* InComponent, const FGuid& InLandscapeGuid, const TArray<FIntPoint>& InProxySectionsBases, const TArray<FVector>& InProxySectionsCentersLocalSpace, const FVector& InComponentXVector, const FVector& InComponentYVector, const FTransform& LocalToWorld, int32 ComponentResolution, int8 InProxyLOD, uint32 InLODGroupKey, uint32 InLandscapeKey)
	: FStaticMeshSceneProxy(InComponent, false)
{
	check(InLODGroupKey == 0 || ComponentResolution != 0);	// if using LODGroupKey, we require HLODs to be rebuilt to capture the component resolution and section transform data (position / orientation)

	ProxySectionsInfos.Empty(InProxySectionsBases.Num());
	
	check(InProxySectionsBases.Num() == InProxySectionsCentersLocalSpace.Num() ||
		  InProxySectionsCentersLocalSpace.Num() == 0);
	for (int32 Index = 0; Index < InProxySectionsBases.Num(); Index++)
	{
		FIntPoint SectionBase = InProxySectionsBases[Index];
		FVector ProxySectionCenterLocalSpace = InProxySectionsCentersLocalSpace.IsValidIndex(Index) ? InProxySectionsCentersLocalSpace[Index] : FVector::Zero();
		ProxySectionsInfos.Emplace(MakeUnique<FLandscapeProxySectionInfo>(InComponent->GetWorld(), InLandscapeGuid, SectionBase, ProxySectionCenterLocalSpace, InComponentXVector, InComponentYVector, LocalToWorld, ComponentResolution, InProxyLOD, InLODGroupKey, InLandscapeKey));
	}
}

void FLandscapeMeshProxySceneProxy::RegisterSections()
{
	for (auto& Info : ProxySectionsInfos)
	{
		FLandscapeRenderSystem::RegisterSection(Info.Get());
	}
}

void FLandscapeMeshProxySceneProxy::UnregisterSections()
{
	for (auto& Info : ProxySectionsInfos)
	{
		FLandscapeRenderSystem::UnregisterSection(Info.Get());
	}
}

SIZE_T FLandscapeMeshProxySceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}


void FLandscapeMeshProxySceneProxy::CreateRenderThreadResources(FRHICommandListBase& RHICmdList)
{
	FStaticMeshSceneProxy::CreateRenderThreadResources(RHICmdList);

	for (auto& Info : ProxySectionsInfos)
	{
		FLandscapeRenderSystem::CreateResources(RHICmdList, Info.Get());
	}

	if (!IsForceHidden())
	{
		RegisterSections();
	}
}

void FLandscapeMeshProxySceneProxy::OnForceHiddenChanged()
{
	if (IsForceHidden())
	{
		UnregisterSections();
	}
	else
	{
		RegisterSections();
	}
}

void FLandscapeMeshProxySceneProxy::DestroyRenderThreadResources()
{
	FStaticMeshSceneProxy::DestroyRenderThreadResources();
	UnregisterSections();

	for (auto& Info : ProxySectionsInfos)
	{
		FLandscapeRenderSystem::DestroyResources(Info.Get());
	}
}

FPrimitiveSceneProxy* ULandscapeMeshProxyComponent::CreateSceneProxy()
{
	if (GetStaticMesh() == nullptr
		|| GetStaticMesh()->IsCompiling()
		|| GetStaticMesh()->GetRenderData() == nullptr
		|| GetStaticMesh()->GetRenderData()->LODResources.Num() == 0
		|| GetStaticMesh()->GetRenderData()->LODResources[0].VertexBuffers.StaticMeshVertexBuffer.GetNumVertices() == 0)
	{
		return nullptr;
	}

	if ((LODGroupKey != 0) && (ComponentResolution == 0))
	{
		// this is an OLD HLOD not built with the new resolution/x/y/center information, which we need if LODGroups are used.
		UE_LOGF(LogLandscape, Warning, "HLOD for Landscape needs to be rebuilt!  Landscape Mesh Proxy Component '%ls' is using LODGroups but does not have the position and orientation information necessary to register it with the Landscape renderer.  This HLOD will not render until it is fixed.",
			*GetName());
		return nullptr;
	}

	// Check mesh overwrite version of proxy creation strategy to allow different strategy for HLODs
	if (CheckPSOPrecachingAndBoostPriority(EPSOPrecachePriority::Highest) && GetPSOPrecacheProxyCreationStrategy(GLandscapePSOPrecacheMeshProxyCreationStrategy) == EPSOPrecacheProxyCreationStrategy::DelayUntilPSOPrecached)
	{
		UE_LOGF(LogLandscape, Verbose, "Skipping CreateSceneProxy for ULandscapeMeshProxyComponent %ls (Its PSOs are still compiling)", *GetFullName());
		return nullptr;
	}

	return new FLandscapeMeshProxySceneProxy(this, LandscapeGuid, ProxyComponentBases, ProxyComponentCentersObjectSpace, ComponentXVectorObjectSpace, ComponentYVectorObjectSpace, GetComponentTransform(), ComponentResolution, ProxyLOD, LODGroupKey, ALandscapeProxy::ComputeLandscapeKey(GetWorld(), LODGroupKey, LandscapeGuid));
}

class FLandscapeNaniteSceneProxy : public ::Nanite::FSceneProxy
{
public:
	using Super = ::Nanite::FSceneProxy;

	FLandscapeNaniteSceneProxy(const ::Nanite::FMaterialAudit& MaterialAudit, ULandscapeNaniteComponent* Component) : Super(MaterialAudit, Component)
	{
		bIsLandscapeProxy = true;
		bIsLandscapeNaniteProxy = true;

		// Disable Nanite landscape representation for Lumen, distance fields, and ray tracing
		if (GDisableLandscapeNaniteGI != 0)
		{
			bVisibleInLumenScene = false;
			bSupportsDistanceFieldRepresentation = false;
			bAffectDynamicIndirectLighting = false;
			bAffectDistanceFieldLighting = false;

			// Remember the IDs of source components so they can find this Nanite proxy during Lumen card capture
			if (CVarLumenSceneSurfaceCacheCaptureNaniteLandscape.GetValueOnAnyThread() != 0)
			{
				const TArray<TObjectPtr<ULandscapeComponent>>& SourceComponents = Component->GetSourceLandscapeComponents();
				SourceComponentIds.Reserve(SourceComponents.Num());

				for (TObjectPtr<ULandscapeComponent> SourceComponent : SourceComponents)
				{
					if (SourceComponent)
					{
						SourceComponentIds.Add(SourceComponent->GetSceneData().PrimitiveSceneId);
					}
				}
			}
		}

		// Remove masking for any materials that are only pixel programmable because of masking
		bool bMaterialsNeedUpdate = false;
		for (::Nanite::FSceneProxyBase::FMaterialSection& MaterialSection : MaterialSections)
		{
			const bool bWasMasked = MaterialSection.MaterialRelevance.bMasked;
			MaterialSection.MaterialRelevance.bMasked = false;

			if (MaterialSection.IsPixelProgrammableRaster())
			{
				// Don't change bMasked if it is not the sole factor that makes the material section programmable
				MaterialSection.MaterialRelevance.bMasked = bWasMasked;
			}
			else if (bWasMasked)
			{
				bMaterialsNeedUpdate = true;
			}
		}

		if (bMaterialsNeedUpdate)
		{
			// Update our cumulative state based on new material settings and newly imposed material relevance
			// NOTE: This will reset any previously masked materials that are now non-programmable to fixed function
			OnMaterialsUpdated(/* bOverrideMaterialRelevance */ true);
		}

		// Check to disable per-cluster displacement fallback raster (must be done after updating materials)
		if (GLandscapeAllowNanitePerClusterDisplacementDisable != 0 &&
			MaterialDisplacementFadeOutSize > 0.0f)
		{
			bHasPerClusterDisplacementFallbackRaster = true;
		}

		// Overwrite filter flags to specify landscape instead of static mesh
		FilterFlags = ::Nanite::EFilterFlags::Landscape;
	}

	virtual TConstArrayView<FPrimitiveComponentId> GetSourceLandscapeComponentIds() const override
	{
		return SourceComponentIds;
	}

private:
	TArray<FPrimitiveComponentId, TInlineAllocator<1>> SourceComponentIds;
};

FPrimitiveSceneProxy* ULandscapeNaniteComponent::CreateSceneProxy()
{
	::Nanite::FMaterialAudit MaterialAudit{};

	// Is Nanite supported, and is there built Nanite data for this static mesh?
	if (IsEnabled() && ShouldCreateNaniteProxy(&MaterialAudit))
	{
		if (CheckPSOPrecachingAndBoostPriority(EPSOPrecachePriority::Highest) && GetPSOPrecacheProxyCreationStrategy(GLandscapePSOPrecacheProxyCreationStrategy) == EPSOPrecacheProxyCreationStrategy::DelayUntilPSOPrecached)
		{
			UE_LOGF(LogLandscape, Verbose, "Skipping CreateSceneProxy for ULandscapeNaniteComponent %ls (Its PSOs are still compiling)", *GetFullName());
			return nullptr;
		}

		return ::new FLandscapeNaniteSceneProxy(MaterialAudit, this);
	}

	// We *only* want a Nanite proxy for this component, otherwise return null to prevent fallback rendering.
	return nullptr;
}
