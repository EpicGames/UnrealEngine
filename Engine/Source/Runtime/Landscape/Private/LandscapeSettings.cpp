// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeSettings.h"
#include "LandscapeModule.h"
#include "LandscapeHeightfieldCollisionComponent.h"
#include "Modules/ModuleManager.h"
#include "LandscapeEditorServices.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapeSettings)

#if WITH_EDITOR

void ULandscapeSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	ILandscapeModule& LandscapeModule = FModuleManager::GetModuleChecked<ILandscapeModule>("Landscape");
	const FName PropertyName = PropertyChangedEvent.MemberProperty ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;
	if ((PropertyName == GET_MEMBER_NAME_CHECKED(ULandscapeSettings, BrushSizeUIMax))
		|| (PropertyName == GET_MEMBER_NAME_CHECKED(ULandscapeSettings, BrushSizeClampMax)))
	{
		// If landscape mode is active, refresh the detail panel to apply the changes immediately : 
		LandscapeModule.GetLandscapeEditorServices()->RefreshDetailPanel();
	}

	if (PropertyName == GET_MEMBER_NAME_CHECKED(ULandscapeSettings, bDisplayTargetLayerThumbnails))
	{
		LandscapeModule.GetLandscapeEditorServices()->RegenerateLayerThumbnails();
		LandscapeModule.GetLandscapeEditorServices()->RefreshDetailPanel();
	}

	// If setting the snapping mode to SnapEverywhere, execute it on all existing components right away :
	if ((PropertyName == GET_MEMBER_NAME_CHECKED(ULandscapeSettings, FoliageLandscapeSnappingMode)) && (FoliageLandscapeSnappingMode == EFoliageLandscapeSnappingMode::SnapEverywhere))
	{
		for (TObjectIterator<ULandscapeHeightfieldCollisionComponent> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
		{
			It->SnapFoliageInstances(ULandscapeHeightfieldCollisionComponent::ESnapFoliageInstancesFlags::ForceAll);
		}
	}
}

void ULandscapeSettings::PreEditUndo()
{
	Super::PreEditUndo();

	check(!DisplayTargetLayerThumbnailsBeforeUndo.IsSet());
	DisplayTargetLayerThumbnailsBeforeUndo = bDisplayTargetLayerThumbnails;
}

void ULandscapeSettings::PostEditUndo()
{
	Super::PostEditUndo();

	if (DisplayTargetLayerThumbnailsBeforeUndo.GetValue() != bDisplayTargetLayerThumbnails)
	{
		ILandscapeModule& LandscapeModule = FModuleManager::GetModuleChecked<ILandscapeModule>("Landscape");
		LandscapeModule.GetLandscapeEditorServices()->RegenerateLayerThumbnails();
		LandscapeModule.GetLandscapeEditorServices()->RefreshDetailPanel();
	}
	DisplayTargetLayerThumbnailsBeforeUndo.Reset();
}

#endif // WITH_EDITOR
