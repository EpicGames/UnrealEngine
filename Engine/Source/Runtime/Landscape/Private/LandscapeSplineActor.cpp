// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeSplineActor.h"
#include "Engine/World.h"
#include "LandscapeInfo.h"
#include "LandscapeInfoMap.h"
#include "Landscape.h"
#include "LandscapeSplinesComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapeSplineActor)

#if WITH_EDITOR
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "Components/StaticMeshComponent.h"
#include "Components/SplineMeshComponent.h"
#include "ControlPointMeshComponent.h"
#endif

ALandscapeSplineActor::ALandscapeSplineActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ULandscapeSplinesComponent* SplineComponent = CreateDefaultSubobject<ULandscapeSplinesComponent>(TEXT("RootComponent0"));
	
	RootComponent = SplineComponent;	
	RootComponent->Mobility = EComponentMobility::Static;
}

ULandscapeSplinesComponent* ALandscapeSplineActor::GetSplinesComponent() const
{
	return Cast<ULandscapeSplinesComponent>(RootComponent);
}

FTransform ALandscapeSplineActor::LandscapeActorToWorld() const
{
	return GetLandscapeInfo()->LandscapeActor->LandscapeActorToWorld();
}

ULandscapeInfo* ALandscapeSplineActor::GetLandscapeInfo() const
{
	return ULandscapeInfo::Find(GetWorld(), LandscapeGuid);
}

// this is called when Splines or LandscapeActors are registered with LandscapeInfo, to update the LandscapeActor pointer
void ALandscapeSplineActor::UpdateSharedProperties(ULandscapeInfo* InLandscapeInfo)
{
	check(InLandscapeInfo);
	check(InLandscapeInfo->LandscapeGuid == LandscapeGuid);
#if WITH_EDITOR
	if (ALandscape* LandscapeInfoActor = InLandscapeInfo->LandscapeActor.Get())
	{
		LandscapeActor = LandscapeInfoActor;
	}
#endif // WITH_EDITOR
}

// Is this in a valid state to be safe to call LandscapeActorToWorld?
bool ALandscapeSplineActor::IsSplineOwnerValid() const
{
	ULandscapeInfo* Info = GetLandscapeInfo();
	return Info != nullptr && Info->LandscapeActor != nullptr;
}

#if WITH_EDITOR
void ALandscapeSplineActor::GetActorDescProperties(FPropertyPairsMap& PropertyPairsMap) const
{
	Super::GetActorDescProperties(PropertyPairsMap);

	if (LandscapeGuid.IsValid())
	{
		PropertyPairsMap.AddProperty(ALandscape::AffectsLandscapeActorDescProperty, *LandscapeGuid.ToString());
	}
}

// this is called when a LandscapeSplineActor is created, to copy it's landscape target data from the landscape info
void ALandscapeSplineActor::GetSharedProperties(ULandscapeInfo* InLandscapeInfo)
{
	Modify();
	LandscapeGuid = InLandscapeInfo->LandscapeGuid;
	LandscapeActor = InLandscapeInfo->LandscapeActor.Get();
}

void ALandscapeSplineActor::SetLandscapeGuid(const FGuid& InGuid)
{ 
	LandscapeGuid = InGuid;
	check(!LandscapeActor || (LandscapeGuid == LandscapeActor->GetLandscapeGuid()));
}

// This function is only allowed to be called to fixup LandscapeActor pointer
void ALandscapeSplineActor::SetLandscapeActor(ALandscape* InLandscapeActor)
{
	if (LandscapeActor != InLandscapeActor)
	{
		Modify();
		check(LandscapeActor == nullptr);
		check(LandscapeGuid == InLandscapeActor->GetLandscapeGuid());
		LandscapeActor = InLandscapeActor;
	}
}

void ALandscapeSplineActor::Destroyed()
{
	Super::Destroyed();

	UWorld* World = GetWorld();

	if (GIsEditor && !World->IsGameWorld())
	{
		// Modify Splines Objects to support Undo/Redo
		GetSplinesComponent()->ModifySplines();
	}
}

void ALandscapeSplineActor::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();
	if (!IsPendingKillPending())
	{
#if WITH_EDITORONLY_DATA
		// note that when running PIE or when instancing a level, Landscape might change its GUID (see FixupLandscapeGuidsIfInstanced), so we need to copy it.
		// (otherwise we would register with the wrong landscape info and our LandscapeActor reference will be stomped)
		if (LandscapeActor)
		{
			LandscapeGuid = LandscapeActor->GetLandscapeGuid();
		}
#endif // WITH_EDITORONLY_DATA

		UWorld* World = GetWorld();
		if (LandscapeGuid.IsValid())
		{
			ULandscapeInfo* LandscapeInfo = ULandscapeInfo::FindOrCreate(World, LandscapeGuid);
			LandscapeInfo->RegisterSplineActor(this);
		}

		// If Landscape uses generated LandscapeSplineMeshesActors, ensure SplineMeshComponents & ControlPointMeshComponents are hidden in PIE
		if (World->IsGameWorld() && HasGeneratedLandscapeSplineMeshesActors())
		{
			ForEachComponent<UStaticMeshComponent>(true, [](UStaticMeshComponent* Component)
			{
				if (Component->IsA<USplineMeshComponent>() || Component->IsA<UControlPointMeshComponent>())
				{
					Component->SetHiddenInGame(true);
				}
			});
		}
	}
}

void ALandscapeSplineActor::UnregisterAllComponents(bool bForReregister)
{
	if (GetWorld() && IsValidChecked(GetWorld()) && !GetWorld()->IsUnreachable() && LandscapeGuid.IsValid())
	{
		if (ULandscapeInfo* LandscapeInfo = GetLandscapeInfo())
		{
			LandscapeInfo->UnregisterSplineActor(this);
		}
	}

	Super::UnregisterAllComponents(bForReregister);
}

void ALandscapeSplineActor::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	const FName PropertyName = PropertyChangedEvent.GetMemberPropertyName();
	if (PropertyName == FName("LandscapeActor"))
	{
		if (LandscapeActor && !LandscapeActor->HasAnyFlags(RF_BeginDestroyed))
		{
			LandscapeGuid = LandscapeActor->GetLandscapeGuid();
		}
		else
		{
			LandscapeActor = nullptr;
			LandscapeGuid.Invalidate();
		}

		ULandscapeInfo::UpdateRegistrationForSplineActor(GetWorld(), this);
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void ALandscapeSplineActor::PostEditMove(bool bFinished)
{
	Super::PostEditMove(bFinished);

	if (bFinished)
	{
		if (ULandscapeInfo* LandscapeInfo = GetLandscapeInfo())
		{
			LandscapeInfo->RequestSplineLayerUpdate();
		}
	}
}

AActor* ALandscapeSplineActor::GetSceneOutlinerParent() const
{
	if (ULandscapeInfo* LandscapeInfo = GetLandscapeInfo())
	{
		return LandscapeInfo->LandscapeActor.Get();
	}

	return nullptr;
}

bool ALandscapeSplineActor::ShouldPartitionSpline() const
{
	return !IsEditorOnly() && bIsSpatiallyLoaded;
}

bool ALandscapeSplineActor::HasGeneratedLandscapeSplineMeshesActors() const
{
	return ShouldPartitionSpline() && IsValid(LandscapeActor) && LandscapeActor->GetUseGeneratedLandscapeSplineMeshesActors();
}

#endif

