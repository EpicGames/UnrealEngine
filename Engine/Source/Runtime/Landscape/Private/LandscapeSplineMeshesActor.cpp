// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeSplineMeshesActor.h"
#include "Components/StaticMeshComponent.h"
#include "ControlPointMeshComponent.h"
#include "Components/SplineMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Misc/MapErrors.h"
#include "GameFramework/WorldSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapeSplineMeshesActor)

#define LOCTEXT_NAMESPACE "LandscapeSplineMeshesActor"

// ----------------------------------------------------------------
// ALandscapeSplineMeshesActor Implementation
// ----------------------------------------------------------------

ALandscapeSplineMeshesActor::ALandscapeSplineMeshesActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetCanBeDamaged(false);
}

const TArray<UStaticMeshComponent*>& ALandscapeSplineMeshesActor::GetStaticMeshComponents() const
{
	return StaticMeshComponents;
}

#if WITH_EDITOR
uint32 ALandscapeSplineMeshesActor::GetDefaultGridSize(UWorld* InWorld) const
{
	return InWorld->GetWorldSettings()->LandscapeSplineMeshesGridSize;
}

UStaticMeshComponent* ALandscapeSplineMeshesActor::CreateStaticMeshComponent(const TSubclassOf<UStaticMeshComponent>& InComponentClass)
{
	check(InComponentClass->IsChildOf<USplineMeshComponent>() || InComponentClass->IsChildOf<UControlPointMeshComponent>());
	Modify();

	UStaticMeshComponent* Component = NewObject<UStaticMeshComponent>(this, InComponentClass.Get(), NAME_None, RF_Transactional);
	AddInstanceComponent(Component);
	Component->SetupAttachment(RootComponent);
	if (GetRootComponent()->IsRegistered())
	{
		Component->RegisterComponent();
	}
	Component->SetWorldTransform(RootComponent->GetComponentTransform());
	Component->SetCollisionProfileName(UCollisionProfile::BlockAll_ProfileName);
	Component->Mobility = EComponentMobility::Static;
	Component->SetGenerateOverlapEvents(false);
	StaticMeshComponents.Add(Component);
	return Component;
}

void ALandscapeSplineMeshesActor::ClearStaticMeshComponents()
{
	Modify();

	for (UStaticMeshComponent* Component : StaticMeshComponents)
	{
		if (Component)
		{
			Component->Modify();
			Component->DestroyComponent();
		}
	}

	StaticMeshComponents.Empty();
}

void ALandscapeSplineMeshesActor::CheckForErrors()
{
	Super::CheckForErrors();

	FMessageLog MapCheck("MapCheck");

	for (UStaticMeshComponent* Component : StaticMeshComponents)
	{
		if (Component && !Component->GetStaticMesh())
		{
			MapCheck.Warning()
				->AddToken(FUObjectToken::Create(this))
				->AddToken(FTextToken::Create(LOCTEXT("MapCheck_Message_SplineMeshesActorNull", "LandscapeSplineMeshesActor has a component with NULL StaticMesh")))
				->AddToken(FMapErrorToken::Create(FMapErrors::StaticMeshNull));
		}
	}
}

bool ALandscapeSplineMeshesActor::GetReferencedContentObjects(TArray<UObject*>& Objects) const
{
	Super::GetReferencedContentObjects(Objects);

	for (UStaticMeshComponent* Component : StaticMeshComponents)
	{
		if (Component && Component->GetStaticMesh())
		{
			Objects.Add(Component->GetStaticMesh());
		}
	}
	return true;
}

#endif

FString ALandscapeSplineMeshesActor::GetDetailedInfoInternal() const
{
	TStringBuilder<1024> Builder;
	for (UStaticMeshComponent* Component : StaticMeshComponents)
	{
		if (Component && Component->GetStaticMesh())
		{
			Builder += Component->GetDetailedInfoInternal();
			Builder += TEXT("\n");
		}
	}
	return Builder.ToString();
}

#undef LOCTEXT_NAMESPACE
