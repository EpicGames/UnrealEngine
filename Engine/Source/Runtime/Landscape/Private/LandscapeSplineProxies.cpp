// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeSplineProxies.h"

#include "GenericPlatform/ICursor.h"
#include "LandscapeSplineControlPoint.h"
#include "LandscapeSplinesComponent.h"
#include "LandscapeSplineSegment.h"

HLandscapeSplineProxy::HLandscapeSplineProxy(ULandscapeSplinesComponent* SplineComponent, EHitProxyPriority InPriority)
	: HActor(SplineComponent->GetOwner(), SplineComponent, InPriority)
{
}

EMouseCursor::Type HLandscapeSplineProxy::GetMouseCursor()
{
	return EMouseCursor::Crosshairs;
}

HLandscapeSplineProxy_Segment::HLandscapeSplineProxy_Segment(class ULandscapeSplineSegment* InSplineSegment)
	: HLandscapeSplineProxy(InSplineSegment->GetOuterULandscapeSplinesComponent())
	, SplineSegment(InSplineSegment)
{
}

void HLandscapeSplineProxy_Segment::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(SplineSegment);
}

HLandscapeSplineProxy_ControlPoint::HLandscapeSplineProxy_ControlPoint(class ULandscapeSplineControlPoint* InControlPoint)
	: HLandscapeSplineProxy(InControlPoint->GetOuterULandscapeSplinesComponent(), HPP_Foreground)
	, ControlPoint(InControlPoint)
{
}

void HLandscapeSplineProxy_ControlPoint::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(ControlPoint);
}

HLandscapeSplineProxy_Tangent::HLandscapeSplineProxy_Tangent(class ULandscapeSplineSegment* InSplineSegment, bool InEnd)
	: HLandscapeSplineProxy(InSplineSegment->GetOuterULandscapeSplinesComponent(), HPP_UI)
	, SplineSegment(InSplineSegment)
	, End(InEnd)
{
}

LANDSCAPE_API void HLandscapeSplineProxy_Tangent::Serialize(FArchive& Ar)
{
	Ar << SplineSegment;
}

EMouseCursor::Type HLandscapeSplineProxy_Tangent::GetMouseCursor()
{
	return EMouseCursor::CardinalCross;
}

void HLandscapeSplineProxy_Tangent::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(SplineSegment);
}
