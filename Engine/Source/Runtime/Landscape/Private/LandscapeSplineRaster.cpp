// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LandscapeSplineRaster.cpp:
	Functions to rasterize a spline into landscape heights/weights
  =============================================================================*/

#include "LandscapeSplineRaster.h"
#include "LandscapeProxy.h"
#include "LandscapeInfo.h"
#include "AI/NavigationSystemBase.h"
#include "LandscapeComponent.h"
#include "LandscapeLayerInfoObject.h"
#include "LandscapeHeightfieldCollisionComponent.h"
#include "LandscapeDataAccess.h"
#include "LandscapeEdit.h"
#include "LandscapeEditLayer.h"
#include "LandscapeSplinesComponent.h"
#include "LandscapeSplineControlPoint.h"
#include "LandscapePrivate.h"
#if WITH_EDITOR
#include "ScopedTransaction.h"
#include "Raster.h"
#include "Landscape.h"
#include "ImageCore.h"
#endif

#define LOCTEXT_NAMESPACE "Landscape"

//////////////////////////////////////////////////////////////////////////
// Apply splines
//////////////////////////////////////////////////////////////////////////

#if WITH_EDITOR
TAutoConsoleVariable<int32> CVarLandscapeSplineFalloffModulation(
	TEXT("landscape.SplineFalloffModulation"),
	1,
	TEXT("Enable Texture Modulation fo Spline Layer Falloff."));

using FModulateAlphaFunc = TFunction<float(float InValue, int32 X, int32 Y)>;

class FLandscapeSplineHeightsRasterPolicy
{
public:
	// X = Side Alpha, Y = End Alpha, Z = Height
	typedef FVector InterpolantType;

	/** Initialization constructor. */
	FLandscapeSplineHeightsRasterPolicy(TArray<uint16>& InHeightData, int32 InMinX, int32 InMinY, int32 InMaxX, int32 InMaxY, bool InbRaiseTerrain, bool InbLowerTerrain, TArray<uint16>* InHeightAlphaBlendData = nullptr, TArray<uint8>* InHeightFlagsData = nullptr) :
		HeightData(InHeightData),
		HeightAlphaBlendData(InHeightAlphaBlendData),
		HeightFlagsData(InHeightFlagsData),
		MinX(InMinX),
		MinY(InMinY),
		MaxX(InMaxX),
		MaxY(InMaxY),
		bRaiseTerrain(InbRaiseTerrain),
		bLowerTerrain(InbLowerTerrain)
	{
	}

protected:

	// FTriangleRasterizer policy interface.
	int32 GetMinX() const { return MinX; }
	int32 GetMaxX() const { return MaxX; }
	int32 GetMinY() const { return MinY; }
	int32 GetMaxY() const { return MaxY; }

	inline void ProcessPixel(int32 X, int32 Y, const InterpolantType& Interpolant, bool BackFacing)
	{
		if (!bRaiseTerrain && !bLowerTerrain)
		{
			return;
		}

		const float CosInterpX = static_cast<float>(Interpolant.X >= 1 ? 1 : 0.5f - 0.5f * FMath::Cos(Interpolant.X * PI));
		const float CosInterpY = static_cast<float>(Interpolant.Y >= 1 ? 1 : 0.5f - 0.5f * FMath::Cos(Interpolant.Y * PI));
		const float Alpha = FMath::Clamp<float>(CosInterpX * CosInterpY, 0.f, 1.f);

		int32 DataIndex = (Y - MinY)*(1 + MaxX - MinX) + X - MinX;
		uint16& Dest = HeightData[DataIndex];

		if (HeightAlphaBlendData)
		{
			uint16 NewHeight = static_cast<uint16>(FMath::Clamp(static_cast<float>(Interpolant.Z), 0.0f, static_cast<float>(LandscapeDataAccess::MaxValue)));
			float InterpValue = (NewHeight * Alpha) + (Dest * (1.f - Alpha));
			Dest = (uint16)FMath::Clamp<float>(InterpValue, 0, (float)LandscapeDataAccess::MaxValue);

			uint16& DestAlphaValue = (*HeightAlphaBlendData)[DataIndex];
			float InterpAlphaValue = DestAlphaValue * (1.f - Alpha);
			DestAlphaValue = (uint16)FMath::Clamp<float>(InterpAlphaValue, 0.f, static_cast<float>(LandscapeDataAccess::MaxValue));

			if (HeightFlagsData)
			{
				uint8& DestHeightFlagsValue = (*HeightFlagsData)[DataIndex];
				uint8 Flags = DestHeightFlagsValue;
				if (bLowerTerrain)
					Flags |= 1;
				if (bRaiseTerrain)
					Flags |= 2;
				DestHeightFlagsValue = Flags;
			}
		}
		else
		{
			float Value = FMath::Lerp<float>(Dest, static_cast<float>(Interpolant.Z), Alpha);
			uint16 DValue = (uint16)FMath::Clamp<float>(Value, 0, (float)LandscapeDataAccess::MaxValue);
			if ((bRaiseTerrain && DValue > Dest) ||
				(bLowerTerrain && DValue < Dest))
			{
				Dest = DValue;
			}
		}
	}

private:
	TArray<uint16>& HeightData;
	TArray<uint16>* HeightAlphaBlendData;
	TArray<uint8>* HeightFlagsData;
	int32 MinX, MinY, MaxX, MaxY;
	uint32 bRaiseTerrain : 1, bLowerTerrain : 1;
};

extern const size_t ChannelOffsets[4];

class FModulateAlpha
{
public:
	static TSharedPtr<FModulateAlpha> CreateFromLayerInfo(ULandscapeLayerInfoObject* InLayerInfo, int32 InLandscapeMinX, int32 InLandscapeMinY)
	{
		if (CVarLandscapeSplineFalloffModulation.GetValueOnAnyThread() == 0
			|| (InLayerInfo == nullptr)
			|| (InLayerInfo->GetSplineFalloffModulationTexture() == nullptr))
		{
			return nullptr;
		}

		return MakeShareable(new FModulateAlpha(InLayerInfo, InLandscapeMinX, InLandscapeMinY));
	}

	float Modulate(float InValue, int32 InX, int32 InY) const
	{
		if (!bIsValid)
		{
			return InValue;
		}

		const float TextureWidth = Image.GetWidth();
		const float TextureHeight = Image.GetHeight();

		float X = FMath::Frac((float)(InX - LandscapeMinX) / (TextureWidth * Tiling)) * TextureWidth;
		float Y = FMath::Frac((float)(InY - LandscapeMinY) / (TextureHeight * Tiling)) * TextureHeight;

		int32 X0 = FMath::FloorToInt(X);
		int32 X1 = FMath::Min(FMath::CeilToInt(X), TextureWidth - 1);
		int32 Y0 = FMath::FloorToInt(Y);
		int32 Y1 = FMath::Min(FMath::CeilToInt(Y), TextureHeight - 1);

		float SampleX0Y0 = Image.GetOnePixelLinear(X0, Y0).Component(ChannelIndex);
		float SampleX1Y0 = Image.GetOnePixelLinear(X1, Y0).Component(ChannelIndex); 
		float SampleX0Y1 = Image.GetOnePixelLinear(X0, Y1).Component(ChannelIndex);
		float SampleX1Y1 = Image.GetOnePixelLinear(X1, Y1).Component(ChannelIndex);

		float LerpY0 = FMath::Lerp(SampleX0Y0, SampleX1Y0, X - X0);
		float LerpY1 = FMath::Lerp(SampleX0Y1, SampleX1Y1, X - X0);
		float FinalLerp = FMath::Lerp(LerpY0, LerpY1, Y - Y0);

		InValue = InValue + ((FinalLerp - Bias) * Scale);

		return FMath::Clamp(InValue, 0.0f, 1.0f);
	}

private:
	FModulateAlpha(ULandscapeLayerInfoObject* InLayerInfo, int32 InLandscapeMinX, int32 InLandscapeMinY)
	{
		check(InLayerInfo && InLayerInfo->GetSplineFalloffModulationTexture());

		bIsValid = Initialize(InLayerInfo, InLandscapeMinX, InLandscapeMinY);
	}

	bool Initialize(ULandscapeLayerInfoObject* InLayerInfo, int32 InLandscapeMinX, int32 InLandscapeMinY)
	{
		FTextureSource& TextureSource = InLayerInfo->GetSplineFalloffModulationTexture()->Source;
		if (!TextureSource.IsValid() || !TextureSource.GetMipImage(Image, /*MipIndex = */0))
		{
			UE_LOGF(LogLandscape, Error, "Invalid source data for spline falloff modulation texture (%ls) of layer %ls. Alpha modulation will be disabled.",
				*InLayerInfo->GetSplineFalloffModulationTexture()->GetPathName(), *InLayerInfo->GetLayerName().ToString());
			return false;
		}

		if ((TextureSource.GetSizeX() * TextureSource.GetSizeY()) <= 0)
		{
			UE_LOGF(LogLandscape, Error, "Invalid source data for spline falloff modulation texture (%ls) of layer %ls: incorrect texture size (%lli x %lli). Alpha modulation will be disabled.",
				*InLayerInfo->GetSplineFalloffModulationTexture()->GetPathName(), *InLayerInfo->GetLayerName().ToString(), TextureSource.GetSizeX(), TextureSource.GetSizeY());
			return false;
		}
		
		const ETextureSourceFormat Format = TextureSource.GetFormat();
		const FTextureSourceFormatInfo& FormatInfo = GTextureSourceFormats[Format];
		if ((int32)InLayerInfo->GetSplineFalloffModulationColorMask() >= FormatInfo.NumComponents)
		{
			UE_LOGF(LogLandscape, Error, "Invalid color mask (%ls) for spline falloff modulation texture (%ls) of layer %ls: the texture only has %i channel(s) available. Alpha modulation will be disabled.",
				*UEnum::GetValueAsString(InLayerInfo->GetSplineFalloffModulationColorMask()), *InLayerInfo->GetSplineFalloffModulationTexture()->GetPathName(), *InLayerInfo->GetLayerName().ToString(), FormatInfo.NumComponents);
			return false;
		}
		ChannelIndex = (int32)InLayerInfo->GetSplineFalloffModulationColorMask();

		if (FMath::Abs(InLayerInfo->GetSplineFalloffModulationTiling()) > UE_SMALL_NUMBER)
		{
			Tiling = 1.0f / InLayerInfo->GetSplineFalloffModulationTiling();
		}
		else
		{
			UE_LOGF(LogLandscape, Warning, "Invalid tiling for spline falloff modulation texture of layer %ls. Alpha modulation will be disabled.",
				*InLayerInfo->GetLayerName().ToString());
			return false;
		}

		Bias = InLayerInfo->GetSplineFalloffModulationBias();
		Scale = InLayerInfo->GetSplineFalloffModulationScale();

		LandscapeMinX = InLandscapeMinX;
		LandscapeMinY = InLandscapeMinY;

		return true;
	}

private:
	bool bIsValid = false;
	float Tiling = 1.0f;
	float Bias = 0.0f;
	float Scale = 1.0f;
	FImage Image;
	int32 LandscapeMinX = 0;
	int32 LandscapeMinY = 0;
	int32 ChannelIndex = 0;
};

class FLandscapeSplineBlendmaskRasterPolicy
{
public:
	// X = Side Alpha, Y = End Alpha, Z = Blend Value
	typedef FVector InterpolantType;

	/** Initialization constructor. */
	FLandscapeSplineBlendmaskRasterPolicy(TArray<uint8>& InData, int32 InMinX, int32 InMinY, int32 InMaxX, int32 InMaxY, const TSharedPtr<FModulateAlpha>& InModulateAlpha = nullptr) :
		Data(InData),
		MinX(InMinX),
		MinY(InMinY),
		MaxX(InMaxX),
		MaxY(InMaxY),
		ModulateAlpha(InModulateAlpha)
	{
	}
protected:

	// FTriangleRasterizer policy interface.

	int32 GetMinX() const { return MinX; }
	int32 GetMaxX() const { return MaxX; }
	int32 GetMinY() const { return MinY; }
	int32 GetMaxY() const { return MaxY; }

	inline void ProcessPixel(int32 X, int32 Y, const InterpolantType& Interpolant, bool BackFacing)
	{
		float Alpha = 0.0f;
		
		if (ModulateAlpha == nullptr)
		{
			const float CosInterpX = static_cast<float>(Interpolant.X >= 1 ? 1 : 0.5f - 0.5f * FMath::Cos(Interpolant.X * PI));
			const float CosInterpY = static_cast<float>(Interpolant.Y >= 1 ? 1 : 0.5f - 0.5f * FMath::Cos(Interpolant.Y * PI));
			Alpha = CosInterpX * CosInterpY;
		}
		else
		{
			const float InterpX = FMath::Clamp<float>(static_cast<float>(Interpolant.X), 0.0f, 1.0f);
			const float InterpY = FMath::Clamp<float>(static_cast<float>(Interpolant.Y), 0.0f, 1.0f);
			Alpha = ModulateAlpha->Modulate(InterpX * InterpY, X, Y);
		}

		uint8& Dest = Data[(Y - MinY)*(1 + MaxX - MinX) + X - MinX];
		float Value = FMath::Lerp<float>(Dest, static_cast<float>(Interpolant.Z), Alpha);
		Dest = FMath::Clamp<uint8>(static_cast<uint8>(Value), 0, 255);
	}

private:
	TArray<uint8>& Data;
	int32 MinX, MinY, MaxX, MaxY;
	const TSharedPtr<FModulateAlpha>& ModulateAlpha;
};

void RasterizeHeight(int32& MinX, int32& MinY, int32& MaxX, int32& MaxY, FLandscapeEditDataInterface& LandscapeEdit, bool bRaiseTerrain, bool bLowerTerrain, TSet<ULandscapeComponent*>& ModifiedComponents, TFunctionRef<void(FTriangleRasterizer<FLandscapeSplineHeightsRasterPolicy>&)> RasterizerFunction)
{
	if (!(bRaiseTerrain || bLowerTerrain))
	{
		return;
	}

	if (MinX > MaxX || MinY > MaxY)
	{
		return;
	}

	TArray<uint16> HeightData;
	HeightData.AddZeroed((1 + MaxY - MinY) * (1 + MaxX - MinX));

	int32 ValidMinX = MinX;
	int32 ValidMinY = MinY;
	int32 ValidMaxX = MaxX;
	int32 ValidMaxY = MaxY;
	LandscapeEdit.GetHeightData(ValidMinX, ValidMinY, ValidMaxX, ValidMaxY, HeightData.GetData(), 0);

	if (ValidMinX > ValidMaxX || ValidMinY > ValidMaxY)
	{
		// The bounds don't intersect any data, so skip it.  Return the invalid bounds.
		MinX = ValidMinX;
		MinY = ValidMinY;
		MaxX = ValidMaxX;
		MaxY = ValidMaxY;
			
		return;
	}

	// shrink our data to the valid range returned by GetHeightData
	FLandscapeEditDataInterface::ShrinkData(HeightData, MinX, MinY, MaxX, MaxY, ValidMinX, ValidMinY, ValidMaxX, ValidMaxY);

	const ALandscape* Landscape = LandscapeEdit.GetTargetLandscape();
	check(Landscape != nullptr);

	bool bIsEditingLayerReservedForSplines = false;
	if (!Landscape->IsTemplate())
	{
		const ULandscapeEditLayerBase* EditingLayer = Landscape->GetEditLayerConst(Landscape->GetEditingLayer());
		check(EditingLayer != nullptr);
		const ULandscapeEditLayerBase* SplinesLayer = Landscape->FindEditLayerOfTypeConst(ULandscapeEditLayerSplines::StaticClass());
		bIsEditingLayerReservedForSplines = (SplinesLayer == EditingLayer);
		check(!bIsEditingLayerReservedForSplines || (SplinesLayer->GetBlendMode() == LSBM_AlphaBlend));
	}

	TArray<uint16> HeightAlphaBlendData;
	TArray<uint8> HeightFlagsData;
	TArray<uint16>* HeightAlphaBlendDataPtr = nullptr;
	TArray<uint8>* HeightFlagsDataPtr = nullptr;

	bool bCalculateNormals = true;

	// when using spline edit layer there is additional alpha blending data
	if (bIsEditingLayerReservedForSplines)
	{
		bCalculateNormals = false;

		HeightAlphaBlendData.AddZeroed((1 + MaxY - MinY) * (1 + MaxX - MinX));
		int32 AlphaValidMinX = MinX;
		int32 AlphaValidMinY = MinY;
		int32 AlphaValidMaxX = MaxX;
		int32 AlphaValidMaxY = MaxY;
		LandscapeEdit.GetHeightAlphaBlendData(AlphaValidMinX, AlphaValidMinY, AlphaValidMaxX, AlphaValidMaxY, HeightAlphaBlendData.GetData(), 0);
		check(AlphaValidMinX == ValidMinX && AlphaValidMinY == ValidMinY && AlphaValidMaxX == ValidMaxX && AlphaValidMaxY == ValidMaxY);
		FLandscapeEditDataInterface::ShrinkData(HeightAlphaBlendData, MinX, MinY, MaxX, MaxY, AlphaValidMinX, AlphaValidMinY, AlphaValidMaxX, AlphaValidMaxY);
		HeightAlphaBlendDataPtr = &HeightAlphaBlendData;

		HeightFlagsData.AddZeroed((1 + MaxY - MinY) * (1 + MaxX - MinX));
		int32 FlagsValidMinX = MinX;
		int32 FlagsValidMinY = MinY;
		int32 FlagsValidMaxX = MaxX;
		int32 FlagsValidMaxY = MaxY;
		LandscapeEdit.GetHeightFlagsData(FlagsValidMinX, FlagsValidMinY, FlagsValidMaxX, FlagsValidMaxY, HeightFlagsData.GetData(), 0);
		check(FlagsValidMinX == ValidMinX && FlagsValidMinY == ValidMinY && FlagsValidMaxX == ValidMaxX && FlagsValidMaxY == ValidMaxY);
		FLandscapeEditDataInterface::ShrinkData(HeightFlagsData, MinX, MinY, MaxX, MaxY, FlagsValidMinX, FlagsValidMinY, FlagsValidMaxX, FlagsValidMaxY);
		HeightFlagsDataPtr = &HeightFlagsData;
	}

	MinX = ValidMinX;
	MinY = ValidMinY;
	MaxX = ValidMaxX;
	MaxY = ValidMaxY;

	FTriangleRasterizer<FLandscapeSplineHeightsRasterPolicy> Rasterizer(FLandscapeSplineHeightsRasterPolicy(HeightData, MinX, MinY, MaxX, MaxY, bRaiseTerrain, bLowerTerrain, HeightAlphaBlendDataPtr, HeightFlagsDataPtr));

	RasterizerFunction(Rasterizer);

	LandscapeEdit.SetHeightData(MinX, MinY, MaxX, MaxY, HeightData.GetData(), 0, bCalculateNormals, nullptr, HeightAlphaBlendDataPtr ? HeightAlphaBlendDataPtr->GetData() : nullptr, HeightFlagsDataPtr ? HeightFlagsDataPtr->GetData() : nullptr, false, nullptr, nullptr, true, !bIsEditingLayerReservedForSplines);
	LandscapeEdit.GetComponentsInRegion(MinX, MinY, MaxX, MaxY, &ModifiedComponents);
}

void RasterizeControlPointHeights(int32& MinX, int32& MinY, int32& MaxX, int32& MaxY, FLandscapeEditDataInterface& LandscapeEdit, FVector ControlPointLocation, const TArray<FLandscapeSplineInterpPoint>& Points, bool bRaiseTerrain, bool bLowerTerrain, TSet<ULandscapeComponent*>& ModifiedComponents)
{
	RasterizeHeight(MinX, MinY, MaxX, MaxY, LandscapeEdit, bRaiseTerrain, bLowerTerrain, ModifiedComponents, [&](FTriangleRasterizer<FLandscapeSplineHeightsRasterPolicy>& Rasterizer)
	{
		const FVector2D CenterPos = FVector2D(ControlPointLocation);
		const FVector Center = FVector(1.0f, Points[0].StartEndFalloff, ControlPointLocation.Z * LANDSCAPE_INV_ZSCALE + LandscapeDataAccess::MidValue);

		for (int32 i = Points.Num() - 1, j = 0; j < Points.Num(); i = j++)
		{
			// Solid center
			const FVector2D Right0Pos = FVector2D(Points[i].Right);
			const FVector2D Left1Pos = FVector2D(Points[j].Left);
			const FVector2D Right1Pos = FVector2D(Points[j].Right);
			const FVector Right0 = FVector(1.0f, Points[i].StartEndFalloff, Points[i].Right.Z);
			const FVector Left1 = FVector(1.0f, Points[j].StartEndFalloff, Points[j].Left.Z);
			const FVector Right1 = FVector(1.0f, Points[j].StartEndFalloff, Points[j].Right.Z);

			Rasterizer.DrawTriangle(Center, Right0, Left1, CenterPos, Right0Pos, Left1Pos, false);
			Rasterizer.DrawTriangle(Center, Left1, Right1, CenterPos, Left1Pos, Right1Pos, false);

			// Falloff
			FVector2D FalloffRight0Pos = FVector2D(Points[i].FalloffRight);
			FVector2D FalloffLeft1Pos = FVector2D(Points[j].FalloffLeft);
			FVector FalloffRight0 = FVector(0.0f, Points[i].StartEndFalloff, Points[i].FalloffRight.Z);
			FVector FalloffLeft1 = FVector(0.0f, Points[j].StartEndFalloff, Points[j].FalloffLeft.Z);
			Rasterizer.DrawTriangle(Right0, FalloffRight0, Left1, Right0Pos, FalloffRight0Pos, Left1Pos, false);
			Rasterizer.DrawTriangle(FalloffRight0, Left1, FalloffLeft1, FalloffRight0Pos, Left1Pos, FalloffLeft1Pos, false);
		}
	});
}

void RasterizeControlPointAlpha(int32& MinX, int32& MinY, int32& MaxX, int32& MaxY, FLandscapeEditDataInterface& LandscapeEdit, FVector ControlPointLocation, const TArray<FLandscapeSplineInterpPoint>& Points, ULandscapeLayerInfoObject* LayerInfo, TSet<ULandscapeComponent*>& ModifiedComponents, const TSharedPtr<FModulateAlpha>& ModulateAlpha = nullptr)
{
	if (LayerInfo == nullptr)
	{
		return;
	}

	if (MinX > MaxX || MinY > MaxY)
	{
		return;
	}

	TArray<uint8> Data;
	Data.AddZeroed((1 + MaxY - MinY) * (1 + MaxX - MinX));

	int32 ValidMinX = MinX;
	int32 ValidMinY = MinY;
	int32 ValidMaxX = MaxX;
	int32 ValidMaxY = MaxY;
	LandscapeEdit.GetWeightData(LayerInfo, ValidMinX, ValidMinY, ValidMaxX, ValidMaxY, Data.GetData(), 0);

	if (ValidMinX > ValidMaxX || ValidMinY > ValidMaxY)
	{
		// The control point's bounds don't intersect any data, so skip it.  Return the invalid bounds.
		MinX = ValidMinX;
		MinY = ValidMinY;
		MaxX = ValidMaxX;
		MaxY = ValidMaxY;

		return;
	}

	// shrink our data to the valid range returned by GetWeightData
	FLandscapeEditDataInterface::ShrinkData(Data, MinX, MinY, MaxX, MaxY, ValidMinX, ValidMinY, ValidMaxX, ValidMaxY);

	MinX = ValidMinX;
	MinY = ValidMinY;
	MaxX = ValidMaxX;
	MaxY = ValidMaxY;

	FTriangleRasterizer<FLandscapeSplineBlendmaskRasterPolicy> Rasterizer(
		FLandscapeSplineBlendmaskRasterPolicy(Data, MinX, MinY, MaxX, MaxY, ModulateAlpha));

	const float BlendValue = 255;

	const FVector2D CenterPos = FVector2D(ControlPointLocation);
	const FVector Center = FVector(1.0f, Points[0].StartEndFalloff, BlendValue);

	for (int32 i = Points.Num() - 1, j = 0; j < Points.Num(); i = j++)
	{
		// Solid center
		const FVector2D Right0Pos = FVector2D(Points[i].LayerRight);
		const FVector2D Left1Pos = FVector2D(Points[j].LayerLeft);
		const FVector2D Right1Pos = FVector2D(Points[j].LayerRight);
		const FVector Right0 = FVector(1.0f, Points[i].StartEndFalloff, BlendValue);
		const FVector Left1 = FVector(1.0f, Points[j].StartEndFalloff, BlendValue);
		const FVector Right1 = FVector(1.0f, Points[j].StartEndFalloff, BlendValue);

		Rasterizer.DrawTriangle(Center, Right0, Left1, CenterPos, Right0Pos, Left1Pos, false);
		Rasterizer.DrawTriangle(Center, Left1, Right1, CenterPos, Left1Pos, Right1Pos, false);

		// Falloff
		FVector2D FalloffRight0Pos = FVector2D(Points[i].LayerFalloffRight);
		FVector2D FalloffLeft1Pos = FVector2D(Points[j].LayerFalloffLeft);
		FVector FalloffRight0 = FVector(0.0f, Points[i].StartEndFalloff, BlendValue);
		FVector FalloffLeft1 = FVector(0.0f, Points[j].StartEndFalloff, BlendValue);
		Rasterizer.DrawTriangle(Right0, FalloffRight0, Left1, Right0Pos, FalloffRight0Pos, Left1Pos, false);
		Rasterizer.DrawTriangle(FalloffRight0, Left1, FalloffLeft1, FalloffRight0Pos, Left1Pos, FalloffLeft1Pos, false);
	}

	LandscapeEdit.SetAlphaData(LayerInfo, MinX, MinY, MaxX, MaxY, Data.GetData(), /*Stride = */0, ELandscapeLayerPaintingRestriction::None);

	LandscapeEdit.GetComponentsInRegion(MinX, MinY, MaxX, MaxY, &ModifiedComponents);
}

void RasterizeSegmentHeight(int32& MinX, int32& MinY, int32& MaxX, int32& MaxY, FLandscapeEditDataInterface& LandscapeEdit, const TArray<FLandscapeSplineInterpPoint>& Points, bool bRaiseTerrain, bool bLowerTerrain, TSet<ULandscapeComponent*>& ModifiedComponents)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeSpline_RasterizeSegmentHeight);

	RasterizeHeight(MinX, MinY, MaxX, MaxY, LandscapeEdit, bRaiseTerrain, bLowerTerrain, ModifiedComponents, [&](FTriangleRasterizer<FLandscapeSplineHeightsRasterPolicy>& Rasterizer)
	{
		for (int32 j = 1; j < Points.Num(); j++)
		{
			// Middle
			FVector2D Left0Pos = FVector2D(Points[j - 1].Left);
			FVector2D Right0Pos = FVector2D(Points[j - 1].Right);
			FVector2D Left1Pos = FVector2D(Points[j].Left);
			FVector2D Right1Pos = FVector2D(Points[j].Right);
			FVector Left0 = FVector(1.0f, Points[j - 1].StartEndFalloff, Points[j - 1].Left.Z);
			FVector Right0 = FVector(1.0f, Points[j - 1].StartEndFalloff, Points[j - 1].Right.Z);
			FVector Left1 = FVector(1.0f, Points[j].StartEndFalloff, Points[j].Left.Z);
			FVector Right1 = FVector(1.0f, Points[j].StartEndFalloff, Points[j].Right.Z);
			Rasterizer.DrawTriangle(Left0, Right0, Left1, Left0Pos, Right0Pos, Left1Pos, false);
			Rasterizer.DrawTriangle(Right0, Left1, Right1, Right0Pos, Left1Pos, Right1Pos, false);

			// Left Falloff
			FVector2D FalloffLeft0Pos = FVector2D(Points[j - 1].FalloffLeft);
			FVector2D FalloffLeft1Pos = FVector2D(Points[j].FalloffLeft);
			FVector FalloffLeft0 = FVector(0.0f, Points[j - 1].StartEndFalloff, Points[j - 1].FalloffLeft.Z);
			FVector FalloffLeft1 = FVector(0.0f, Points[j].StartEndFalloff, Points[j].FalloffLeft.Z);
			Rasterizer.DrawTriangle(FalloffLeft0, Left0, FalloffLeft1, FalloffLeft0Pos, Left0Pos, FalloffLeft1Pos, false);
			Rasterizer.DrawTriangle(Left0, FalloffLeft1, Left1, Left0Pos, FalloffLeft1Pos, Left1Pos, false);

			// Right Falloff
			FVector2D FalloffRight0Pos = FVector2D(Points[j - 1].FalloffRight);
			FVector2D FalloffRight1Pos = FVector2D(Points[j].FalloffRight);
			FVector FalloffRight0 = FVector(0.0f, Points[j - 1].StartEndFalloff, Points[j - 1].FalloffRight.Z);
			FVector FalloffRight1 = FVector(0.0f, Points[j].StartEndFalloff, Points[j].FalloffRight.Z);
			Rasterizer.DrawTriangle(Right0, FalloffRight0, Right1, Right0Pos, FalloffRight0Pos, Right1Pos, false);
			Rasterizer.DrawTriangle(FalloffRight0, Right1, FalloffRight1, FalloffRight0Pos, Right1Pos, FalloffRight1Pos, false);
		}
	});
}


void RasterizeSegmentAlpha(int32& MinX, int32& MinY, int32& MaxX, int32& MaxY, FLandscapeEditDataInterface& LandscapeEdit, const TArray<FLandscapeSplineInterpPoint>& Points, ULandscapeLayerInfoObject* LayerInfo, TSet<ULandscapeComponent*>& ModifiedComponents, const TSharedPtr<FModulateAlpha>& ModulateAlpha = nullptr)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeSpline_RasterizeSegmentAlpha);

	if (LayerInfo == nullptr)
	{
		return;
	}

	if (MinX > MaxX || MinY > MaxY)
	{
		return;
	}

	TArray<uint8> Data;
	Data.AddZeroed((1 + MaxY - MinY) * (1 + MaxX - MinX));

	int32 ValidMinX = MinX;
	int32 ValidMinY = MinY;
	int32 ValidMaxX = MaxX;
	int32 ValidMaxY = MaxY;
	LandscapeEdit.GetWeightData(LayerInfo, ValidMinX, ValidMinY, ValidMaxX, ValidMaxY, Data.GetData(), 0);

	if (ValidMinX > ValidMaxX || ValidMinY > ValidMaxY)
	{
		// The segment's bounds don't intersect any data, so skip it.  Return the invalid bounds.
		MinX = ValidMinX;
		MinY = ValidMinY;
		MaxX = ValidMaxX;
		MaxY = ValidMaxY;

		return;
	}

	// shrink our data to the valid range returned by GetWeightData
	FLandscapeEditDataInterface::ShrinkData(Data, MinX, MinY, MaxX, MaxY, ValidMinX, ValidMinY, ValidMaxX, ValidMaxY);

	MinX = ValidMinX;
	MinY = ValidMinY;
	MaxX = ValidMaxX;
	MaxY = ValidMaxY;

	FTriangleRasterizer<FLandscapeSplineBlendmaskRasterPolicy> Rasterizer(
		FLandscapeSplineBlendmaskRasterPolicy(Data, MinX, MinY, MaxX, MaxY, ModulateAlpha));

	const float BlendValue = 255;

	for (int32 j = 1; j < Points.Num(); j++)
	{
		// Middle
		FVector2D Left0Pos = FVector2D(Points[j - 1].LayerLeft);
		FVector2D Right0Pos = FVector2D(Points[j - 1].LayerRight);
		FVector2D Left1Pos = FVector2D(Points[j].LayerLeft);
		FVector2D Right1Pos = FVector2D(Points[j].LayerRight);
		FVector Left0 = FVector(1.0f, Points[j - 1].StartEndFalloff, BlendValue);
		FVector Right0 = FVector(1.0f, Points[j - 1].StartEndFalloff, BlendValue);
		FVector Left1 = FVector(1.0f, Points[j].StartEndFalloff, BlendValue);
		FVector Right1 = FVector(1.0f, Points[j].StartEndFalloff, BlendValue);
		Rasterizer.DrawTriangle(Left0, Right0, Left1, Left0Pos, Right0Pos, Left1Pos, false);
		Rasterizer.DrawTriangle(Right0, Left1, Right1, Right0Pos, Left1Pos, Right1Pos, false);

		// Left Falloff
		FVector2D FalloffLeft0Pos = FVector2D(Points[j - 1].LayerFalloffLeft);
		FVector2D FalloffLeft1Pos = FVector2D(Points[j].LayerFalloffLeft);
		FVector FalloffLeft0 = FVector(0.0f, Points[j - 1].StartEndFalloff, BlendValue);
		FVector FalloffLeft1 = FVector(0.0f, Points[j].StartEndFalloff, BlendValue);
		Rasterizer.DrawTriangle(FalloffLeft0, Left0, FalloffLeft1, FalloffLeft0Pos, Left0Pos, FalloffLeft1Pos, false);
		Rasterizer.DrawTriangle(Left0, FalloffLeft1, Left1, Left0Pos, FalloffLeft1Pos, Left1Pos, false);

		// Right Falloff
		FVector2D FalloffRight0Pos = FVector2D(Points[j - 1].LayerFalloffRight);
		FVector2D FalloffRight1Pos = FVector2D(Points[j].LayerFalloffRight);
		FVector FalloffRight0 = FVector(0.0f, Points[j - 1].StartEndFalloff, BlendValue);
		FVector FalloffRight1 = FVector(0.0f, Points[j].StartEndFalloff, BlendValue);
		Rasterizer.DrawTriangle(Right0, FalloffRight0, Right1, Right0Pos, FalloffRight0Pos, Right1Pos, false);
		Rasterizer.DrawTriangle(FalloffRight0, Right1, FalloffRight1, FalloffRight0Pos, Right1Pos, FalloffRight1Pos, false);
	}

	LandscapeEdit.SetAlphaData(LayerInfo, MinX, MinY, MaxX, MaxY, Data.GetData(), /*Stride = */0, ELandscapeLayerPaintingRestriction::None);

	LandscapeEdit.GetComponentsInRegion(MinX, MinY, MaxX, MaxY, &ModifiedComponents);
}

// Deprecated
bool ULandscapeInfo::ApplySplines(bool bOnlySelected, TSet<TObjectPtr<ULandscapeComponent>>* OutModifiedComponents, bool bMarkPackageDirty)
{
	return ApplySplines(OutModifiedComponents, bMarkPackageDirty);
}

bool ULandscapeInfo::ApplySplines(TSet<TObjectPtr<ULandscapeComponent>>* OutModifiedComponents, bool bMarkPackageDirty)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeInfo_ApplySplines);

	bool bResult = false;

	ALandscape* Landscape = LandscapeActor.Get();

	const ULandscapeEditLayerBase* Layer = Landscape ? Landscape->FindEditLayerOfTypeConst(ULandscapeEditLayerSplines::StaticClass()) : nullptr;
	FGuid SplinesTargetLayerGuid = Layer ? Layer->GetGuid() : Landscape ? Landscape->GetEditingLayer() : FGuid();
	FScopedSetLandscapeEditingLayer Scope(Landscape, SplinesTargetLayerGuid, [=] { Landscape->RequestLayersContentUpdate(ELandscapeLayerUpdateMode::Update_All); });

	TMap<ULandscapeLayerInfoObject*, TSharedPtr<FModulateAlpha>> ModulatePerLayerInfo;
	int32 LandscapeMinX, LandscapeMinY, LandscapeMaxX, LandscapeMaxY;
	if (!GetLandscapeExtent(LandscapeMinX, LandscapeMinY, LandscapeMaxX, LandscapeMaxY))
	{
		return false;
	}

	auto GetOrCreateModulate = [&](ULandscapeLayerInfoObject* LayerInfo) -> TSharedPtr<FModulateAlpha>
	{
		if (const TSharedPtr<FModulateAlpha>* SharedPtr = ModulatePerLayerInfo.Find(LayerInfo))
		{
			return *SharedPtr;
		}

		TSharedPtr<FModulateAlpha> SharedPtr = FModulateAlpha::CreateFromLayerInfo(LayerInfo, LandscapeMinX, LandscapeMinY);
		ModulatePerLayerInfo.Add(LayerInfo, SharedPtr);

		return SharedPtr;
	};

	ForAllSplineActors([&](TScriptInterface<ILandscapeSplineInterface> SplineOwner)
	{
		bResult |= ApplySplinesInternal(SplineOwner, OutModifiedComponents, bMarkPackageDirty, LandscapeMinX, LandscapeMinY, LandscapeMaxX, LandscapeMaxY, GetOrCreateModulate);
	});

	return bResult;
}

bool ULandscapeInfo::ApplySplinesInternal(TScriptInterface<ILandscapeSplineInterface> SplineOwner, TSet<TObjectPtr<ULandscapeComponent>>* OutModifiedComponents, bool bMarkPackageDirty, int32 LandscapeMinX, int32 LandscapeMinY, int32 LandscapeMaxX, int32 LandscapeMaxY, TFunctionRef<TSharedPtr<FModulateAlpha>(ULandscapeLayerInfoObject*)> GetOrCreateModulate)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeInfo_ApplySplinesInternal);

	if (!SplineOwner || !SplineOwner->IsSplineOwnerValid())
	{
		return false;
	}

	ULandscapeSplinesComponent* SplineComponent = SplineOwner->GetSplinesComponent();
	
	if (!SplineComponent || !SplineComponent->IsRegistered() || SplineComponent->ControlPoints.Num() == 0 || SplineComponent->Segments.Num() == 0)
	{
		return false;
	}


	const FTransform SplineToLandscape = SplineComponent->GetComponentTransform().GetRelativeTransform(SplineOwner->LandscapeActorToWorld());

	FLandscapeEditDataInterface LandscapeEdit(this);
	FLandscapeDoNotDirtyScope DoNotDirtyScope(LandscapeEdit, !bMarkPackageDirty);
	TSet<ULandscapeComponent*> ModifiedComponents;
	
	for (const ULandscapeSplineControlPoint* ControlPoint : SplineComponent->ControlPoints)
	{
		if (ControlPoint->GetPoints().Num() < 2)
		{
			continue;
		}

		FBox ControlPointBounds = ControlPoint->GetBounds();
		ControlPointBounds = ControlPointBounds.TransformBy(SplineToLandscape.ToMatrixWithScale());

		int32 MinX = FMath::FloorToInt32(ControlPointBounds.Min.X);
		int32 MinY = FMath::FloorToInt32(ControlPointBounds.Min.Y);
		int32 MaxX = FMath::CeilToInt32(ControlPointBounds.Max.X);
		int32 MaxY = FMath::CeilToInt32(ControlPointBounds.Max.Y);

		MinX = FMath::Max(MinX, LandscapeMinX);
		MinY = FMath::Max(MinY, LandscapeMinY);
		MaxX = FMath::Min(MaxX, LandscapeMaxX);
		MaxY = FMath::Min(MaxY, LandscapeMaxY);

		if (MinX > MaxX || MinY > MaxY)
		{
			// The control point's bounds don't intersect the landscape, so skip it entirely
			continue;
		}

		TArray<FLandscapeSplineInterpPoint> Points = ControlPoint->GetPoints();
		for (int32 j = 0; j < Points.Num(); j++)
		{
			Points[j].Center = SplineToLandscape.TransformPosition(Points[j].Center);
			Points[j].Left = SplineToLandscape.TransformPosition(Points[j].Left);
			Points[j].Right = SplineToLandscape.TransformPosition(Points[j].Right);
			Points[j].FalloffLeft = SplineToLandscape.TransformPosition(Points[j].FalloffLeft);
			Points[j].FalloffRight = SplineToLandscape.TransformPosition(Points[j].FalloffRight);

			Points[j].LayerLeft = SplineToLandscape.TransformPosition(Points[j].LayerLeft);
			Points[j].LayerRight = SplineToLandscape.TransformPosition(Points[j].LayerRight);
			Points[j].LayerFalloffLeft = SplineToLandscape.TransformPosition(Points[j].LayerFalloffLeft);
			Points[j].LayerFalloffRight = SplineToLandscape.TransformPosition(Points[j].LayerFalloffRight);

			// local-heights to texture value heights
			Points[j].Left.Z = Points[j].Left.Z * LANDSCAPE_INV_ZSCALE + LandscapeDataAccess::MidValue;
			Points[j].Right.Z = Points[j].Right.Z * LANDSCAPE_INV_ZSCALE + LandscapeDataAccess::MidValue;
			Points[j].FalloffLeft.Z = Points[j].FalloffLeft.Z * LANDSCAPE_INV_ZSCALE + LandscapeDataAccess::MidValue;
			Points[j].FalloffRight.Z = Points[j].FalloffRight.Z * LANDSCAPE_INV_ZSCALE + LandscapeDataAccess::MidValue;

			Points[j].LayerLeft.Z = Points[j].LayerLeft.Z * LANDSCAPE_INV_ZSCALE + LandscapeDataAccess::MidValue;
			Points[j].LayerRight.Z = Points[j].LayerRight.Z * LANDSCAPE_INV_ZSCALE + LandscapeDataAccess::MidValue;
			Points[j].LayerFalloffLeft.Z = Points[j].LayerFalloffLeft.Z * LANDSCAPE_INV_ZSCALE + LandscapeDataAccess::MidValue;
			Points[j].LayerFalloffRight.Z = Points[j].LayerFalloffRight.Z * LANDSCAPE_INV_ZSCALE + LandscapeDataAccess::MidValue;
		}

		// Heights raster
		if (ControlPoint->bRaiseTerrain || ControlPoint->bLowerTerrain)
		{
			const FVector Center3D = SplineToLandscape.TransformPosition(ControlPoint->Location);

			RasterizeControlPointHeights(MinX, MinY, MaxX, MaxY, LandscapeEdit, Center3D, Points, ControlPoint->bRaiseTerrain, ControlPoint->bLowerTerrain, ModifiedComponents);
		}

		// Blend layer raster
		ULandscapeLayerInfoObject* LayerInfo = GetLayerInfoByName(ControlPoint->LayerName);
		if (ControlPoint->LayerName != NAME_None && LayerInfo != NULL)
		{
			const FVector Center3D = SplineToLandscape.TransformPosition(ControlPoint->Location);

			TSharedPtr<FModulateAlpha> ModulateAlpha = GetOrCreateModulate(LayerInfo);
			RasterizeControlPointAlpha(MinX, MinY, MaxX, MaxY, LandscapeEdit, Center3D, Points, LayerInfo, ModifiedComponents, ModulateAlpha);
		}
	}

	for (const ULandscapeSplineSegment* Segment : SplineComponent->Segments)
	{
		FBox SegmentBounds = Segment->GetBounds();
		SegmentBounds = SegmentBounds.TransformBy(SplineToLandscape.ToMatrixWithScale());

		int32 MinX = FMath::FloorToInt32(SegmentBounds.Min.X);
		int32 MinY = FMath::FloorToInt32(SegmentBounds.Min.Y);
		int32 MaxX = FMath::CeilToInt32(SegmentBounds.Max.X);
		int32 MaxY = FMath::CeilToInt32(SegmentBounds.Max.Y);

		MinX = FMath::Max(MinX, LandscapeMinX);
		MinY = FMath::Max(MinY, LandscapeMinY);
		MaxX = FMath::Min(MaxX, LandscapeMaxX);
		MaxY = FMath::Min(MaxY, LandscapeMaxY);

		if (MinX > MaxX || MinY > MaxY)
		{
			// The segment's bounds don't intersect the landscape, so skip it entirely
			continue;
		}

		TArray<FLandscapeSplineInterpPoint> Points = Segment->GetPoints();
		for (int32 j = 0; j < Points.Num(); j++)
		{
			Points[j].Center = SplineToLandscape.TransformPosition(Points[j].Center);
			Points[j].Left = SplineToLandscape.TransformPosition(Points[j].Left);
			Points[j].Right = SplineToLandscape.TransformPosition(Points[j].Right);
			Points[j].FalloffLeft = SplineToLandscape.TransformPosition(Points[j].FalloffLeft);
			Points[j].FalloffRight = SplineToLandscape.TransformPosition(Points[j].FalloffRight);

			Points[j].LayerLeft = SplineToLandscape.TransformPosition(Points[j].LayerLeft);
			Points[j].LayerRight = SplineToLandscape.TransformPosition(Points[j].LayerRight);
			Points[j].LayerFalloffLeft = SplineToLandscape.TransformPosition(Points[j].LayerFalloffLeft);
			Points[j].LayerFalloffRight = SplineToLandscape.TransformPosition(Points[j].LayerFalloffRight);

			// local-heights to texture value heights
			Points[j].Left.Z = Points[j].Left.Z * LANDSCAPE_INV_ZSCALE + LandscapeDataAccess::MidValue;
			Points[j].Right.Z = Points[j].Right.Z * LANDSCAPE_INV_ZSCALE + LandscapeDataAccess::MidValue;
			Points[j].FalloffLeft.Z = Points[j].FalloffLeft.Z * LANDSCAPE_INV_ZSCALE + LandscapeDataAccess::MidValue;
			Points[j].FalloffRight.Z = Points[j].FalloffRight.Z * LANDSCAPE_INV_ZSCALE + LandscapeDataAccess::MidValue;

			Points[j].LayerLeft.Z = Points[j].LayerLeft.Z * LANDSCAPE_INV_ZSCALE + LandscapeDataAccess::MidValue;
			Points[j].LayerRight.Z = Points[j].LayerRight.Z * LANDSCAPE_INV_ZSCALE + LandscapeDataAccess::MidValue;
			Points[j].LayerFalloffLeft.Z = Points[j].LayerFalloffLeft.Z * LANDSCAPE_INV_ZSCALE + LandscapeDataAccess::MidValue;
			Points[j].LayerFalloffRight.Z = Points[j].LayerFalloffRight.Z * LANDSCAPE_INV_ZSCALE + LandscapeDataAccess::MidValue;
		}

		// Heights raster
		if (Segment->bRaiseTerrain || Segment->bLowerTerrain)
		{
			RasterizeSegmentHeight(MinX, MinY, MaxX, MaxY, LandscapeEdit, Points, Segment->bRaiseTerrain, Segment->bLowerTerrain, ModifiedComponents);

			if (MinX > MaxX || MinY > MaxY)
			{
				// The segment's bounds don't intersect any height data.
				// This means it wouldn't intersect any weightmap data either so we can skip the alpha rasterize below
				continue;
			}
		}

		// Blend layer raster
		ULandscapeLayerInfoObject* LayerInfo = GetLayerInfoByName(Segment->LayerName);
		if (Segment->LayerName != NAME_None && LayerInfo != NULL)
		{
			TSharedPtr<FModulateAlpha> ModulateAlpha = GetOrCreateModulate(LayerInfo);
			RasterizeSegmentAlpha(MinX, MinY, MaxX, MaxY, LandscapeEdit, Points, LayerInfo, ModifiedComponents, ModulateAlpha);
		}
	}

	LandscapeEdit.Flush();

	for (ULandscapeComponent* Component : ModifiedComponents)
	{	
		Component->RequestHeightmapUpdate();
		Component->RequestWeightmapUpdate();
		if (OutModifiedComponents)
		{
			OutModifiedComponents->Add(Component);
		}
	}
	

	return true;
}

namespace LandscapeSplineRaster
{
	void RasterizeSegmentPoints(ULandscapeInfo* LandscapeInfo, TArray<FLandscapeSplineInterpPoint> Points, const FTransform& SplineToWorld, bool bRaiseTerrain, bool bLowerTerrain, ULandscapeLayerInfoObject* LayerInfo)
	{
		ALandscapeProxy* LandscapeProxy = LandscapeInfo->GetLandscapeProxy();
		const FTransform SplineToLandscape = SplineToWorld.GetRelativeTransform(LandscapeProxy->LandscapeActorToWorld());

		FLandscapeEditDataInterface LandscapeEdit(LandscapeInfo);
		TSet<ULandscapeComponent*> ModifiedComponents;

		// I'd dearly love to use FIntRect in this code, but Landscape works with "Inclusive Max" and FIntRect is "Exclusive Max"
		int32 LandscapeMinX, LandscapeMinY, LandscapeMaxX, LandscapeMaxY;
		if (!LandscapeInfo->GetLandscapeExtent(LandscapeMinX, LandscapeMinY, LandscapeMaxX, LandscapeMaxY))
		{
			return;
		}

		FBox SegmentBounds = FBox(ForceInit);
		for (const FLandscapeSplineInterpPoint& Point : Points)
		{
			SegmentBounds += Point.FalloffLeft;
			SegmentBounds += Point.FalloffRight;
		}

		SegmentBounds = SegmentBounds.TransformBy(SplineToLandscape.ToMatrixWithScale());

		int32 MinX = FMath::FloorToInt32(SegmentBounds.Min.X);
		int32 MinY = FMath::FloorToInt32(SegmentBounds.Min.Y);
		int32 MaxX = FMath::CeilToInt32(SegmentBounds.Max.X);
		int32 MaxY = FMath::CeilToInt32(SegmentBounds.Max.Y);

		MinX = FMath::Max(MinX, LandscapeMinX);
		MinY = FMath::Max(MinY, LandscapeMinY);
		MaxX = FMath::Min(MaxX, LandscapeMaxX);
		MaxY = FMath::Min(MaxY, LandscapeMaxY);

		if (MinX > MaxX || MinY > MaxY)
		{
			// The segment's bounds don't intersect the landscape, so skip it entirely
			return;
		}

		for (int32 j = 0; j < Points.Num(); j++)
		{
			Points[j].Center = SplineToLandscape.TransformPosition(Points[j].Center);
			Points[j].Left = SplineToLandscape.TransformPosition(Points[j].Left);
			Points[j].Right = SplineToLandscape.TransformPosition(Points[j].Right);
			Points[j].FalloffLeft = SplineToLandscape.TransformPosition(Points[j].FalloffLeft);
			Points[j].FalloffRight = SplineToLandscape.TransformPosition(Points[j].FalloffRight);

			Points[j].LayerLeft = SplineToLandscape.TransformPosition(Points[j].LayerLeft);
			Points[j].LayerRight = SplineToLandscape.TransformPosition(Points[j].LayerRight);
			Points[j].LayerFalloffLeft = SplineToLandscape.TransformPosition(Points[j].LayerFalloffLeft);
			Points[j].LayerFalloffRight = SplineToLandscape.TransformPosition(Points[j].LayerFalloffRight);

			// local-heights to texture value heights
			Points[j].Left.Z = Points[j].Left.Z * LANDSCAPE_INV_ZSCALE + LandscapeDataAccess::MidValue;
			Points[j].Right.Z = Points[j].Right.Z * LANDSCAPE_INV_ZSCALE + LandscapeDataAccess::MidValue;
			Points[j].FalloffLeft.Z = Points[j].FalloffLeft.Z * LANDSCAPE_INV_ZSCALE + LandscapeDataAccess::MidValue;
			Points[j].FalloffRight.Z = Points[j].FalloffRight.Z * LANDSCAPE_INV_ZSCALE + LandscapeDataAccess::MidValue;

			Points[j].LayerLeft.Z = Points[j].LayerLeft.Z * LANDSCAPE_INV_ZSCALE + LandscapeDataAccess::MidValue;
			Points[j].LayerRight.Z = Points[j].LayerRight.Z * LANDSCAPE_INV_ZSCALE + LandscapeDataAccess::MidValue;
			Points[j].LayerFalloffLeft.Z = Points[j].LayerFalloffLeft.Z * LANDSCAPE_INV_ZSCALE + LandscapeDataAccess::MidValue;
			Points[j].LayerFalloffRight.Z = Points[j].LayerFalloffRight.Z * LANDSCAPE_INV_ZSCALE + LandscapeDataAccess::MidValue;
		}

		// Heights raster
		if (bRaiseTerrain || bLowerTerrain)
		{
			RasterizeSegmentHeight(MinX, MinY, MaxX, MaxY, LandscapeEdit, Points, bRaiseTerrain, bLowerTerrain, ModifiedComponents);

			if (MinX > MaxX || MinY > MaxY)
			{
				// The segment's bounds don't intersect any data, so we skip it entirely
				// it wouldn't intersect any weightmap data either so we don't even bother trying
			}
		}

		// Blend layer raster
		if (LayerInfo != NULL)
		{
			RasterizeSegmentAlpha(MinX, MinY, MaxX, MaxY, LandscapeEdit, Points, LayerInfo, ModifiedComponents);
		}

		LandscapeEdit.Flush();
	}

	static bool LineIntersect(const FVector2D& L1Start, const FVector2D& L1End, const FVector2D& L2Start, const FVector2D& L2End, FVector2D& Intersect, float Tolerance = KINDA_SMALL_NUMBER)
	{
		float tA = static_cast<float>((L2End - L2Start) ^ (L2Start - L1Start));
		float tB = static_cast<float>((L1End - L1Start) ^ (L2Start - L1Start));
		float Denom = static_cast<float>((L2End - L2Start) ^ (L1End - L1Start));

		if (FMath::IsNearlyZero(tA) && FMath::IsNearlyZero(tB))
		{
			// Lines are the same
			Intersect = (L2Start + L2End) / 2;
			return true;
		}

		if (FMath::IsNearlyZero(Denom))
		{
			// Lines are parallel
			Intersect = (L2Start + L2End) / 2;
			return false;
		}

		tA /= Denom;
		tB /= Denom;

		Intersect = L1Start + tA * (L1End - L1Start);

		if (tA >= -Tolerance && tA <= (1.0f + Tolerance) && tB >= -Tolerance && tB <= (1.0f + Tolerance))
		{
			return true;
		}

		return false;
	}

	bool FixSelfIntersection(TArray<FLandscapeSplineInterpPoint>& Points, FVector FLandscapeSplineInterpPoint::* Side)
	{
		int32 StartSide = INDEX_NONE;
		for (int32 i = 0; i < Points.Num(); i++)
		{
			bool bReversed = false;

			if (i < Points.Num() - 1)
			{
				const FLandscapeSplineInterpPoint& CurrentPoint = Points[i];
				const FLandscapeSplineInterpPoint& NextPoint = Points[i + 1];
				const FVector Direction = (NextPoint.Center - CurrentPoint.Center).GetSafeNormal();
				const FVector SideDirection = (NextPoint.*Side - CurrentPoint.*Side).GetSafeNormal();
				bReversed = (SideDirection | Direction) < 0;
			}

			if (bReversed)
			{
				if (StartSide == INDEX_NONE)
				{
					StartSide = i;
				}
			}
			else
			{
				if (StartSide != INDEX_NONE)
				{
					int32 EndSide = i;

					// step startSide back until before the endSide point
					while (StartSide > 0)
					{
						const float Projection = static_cast<float>((Points[StartSide].*Side - Points[StartSide - 1].*Side) | (Points[EndSide].*Side - Points[StartSide - 1].*Side));
						if (Projection >= 0)
						{
							break;
						}
						StartSide--;
					}
					// step endSide forwards until after the startSide point
					while (EndSide < Points.Num() - 1)
					{
						const float Projection = static_cast<float>((Points[EndSide].*Side - Points[EndSide + 1].*Side) | (Points[StartSide].*Side - Points[EndSide + 1].*Side));
						if (Projection >= 0)
						{
							break;
						}
						EndSide++;
					}

					// Can't do anything if the start and end intersect, as they're both unalterable
					if (StartSide == 0 && EndSide == Points.Num() - 1)
					{
						return false;
					}

					FVector2D Collapse;
					if (StartSide == 0)
					{
						Collapse = FVector2D(Points[StartSide].*Side);
						StartSide++;
					}
					else if (EndSide == Points.Num() - 1)
					{
						Collapse = FVector2D(Points[EndSide].*Side);
						EndSide--;
					}
					else
					{
						LineIntersect(FVector2D(Points[StartSide - 1].*Side), FVector2D(Points[StartSide].*Side),
							FVector2D(Points[EndSide + 1].*Side), FVector2D(Points[EndSide].*Side), Collapse);
					}

					for (int32 j = StartSide; j <= EndSide; j++)
					{
						(Points[j].*Side).X = Collapse.X;
						(Points[j].*Side).Y = Collapse.Y;
					}

					StartSide = INDEX_NONE;
					i = EndSide;
				}
			}
		}

		return true;
	}

	void Pointify(const FInterpCurveVector& SplineInfo, TArray<FLandscapeSplineInterpPoint>& Points, int32 NumSubdivisions,
		float StartFalloffFraction, float EndFalloffFraction,
		const float StartWidth, const float EndWidth,
		const float StartLayerWidth, const float EndLayerWidth,
		const FPointifyFalloffs& Falloffs,
		const float StartRollDegrees, const float EndRollDegrees)
	{
		// Stop the start and end fall-off overlapping
		const float TotalFalloff = StartFalloffFraction + EndFalloffFraction;
		if (TotalFalloff > 1.0f)
		{
			StartFalloffFraction /= TotalFalloff;
			EndFalloffFraction /= TotalFalloff;
		}

		const float StartRoll = FMath::DegreesToRadians(StartRollDegrees);
		const float EndRoll = FMath::DegreesToRadians(EndRollDegrees);

		float OldKeyTime = 0;
		for (int32 i = 0; i < SplineInfo.Points.Num(); i++)
		{
			const float NewKeyTime = SplineInfo.Points[i].InVal;
			const float NewKeyCosInterp = 0.5f - 0.5f * FMath::Cos(NewKeyTime * PI);
			const float NewKeyWidth = FMath::Lerp(StartWidth, EndWidth, NewKeyCosInterp);
			const float NewKeyLayerWidth = FMath::Lerp(StartLayerWidth, EndLayerWidth, NewKeyCosInterp);
			const float NewKeyLeftFalloff = FMath::Lerp(Falloffs.StartLeftSide, Falloffs.EndLeftSide, NewKeyCosInterp);
			const float NewKeyRightFalloff = FMath::Lerp(Falloffs.StartRightSide, Falloffs.EndRightSide, NewKeyCosInterp);
			const float NewKeyLeftLayerFalloff = FMath::Lerp(Falloffs.StartLeftSideLayer, Falloffs.EndLeftSideLayer, NewKeyCosInterp);
			const float NewKeyRightLayerFalloff = FMath::Lerp(Falloffs.StartRightSideLayer, Falloffs.EndRightSideLayer, NewKeyCosInterp);
			const float NewKeyRoll = FMath::Lerp(StartRoll, EndRoll, NewKeyCosInterp);
			const FVector NewKeyPos = SplineInfo.Eval(NewKeyTime, FVector::ZeroVector);
			const FVector NewKeyTangent = SplineInfo.EvalDerivative(NewKeyTime, FVector::ZeroVector).GetSafeNormal();
			const FVector NewKeyBiNormal = FQuat(NewKeyTangent, -NewKeyRoll).RotateVector((NewKeyTangent ^ FVector(0, 0, -1)).GetSafeNormal());
			const FVector NewKeyLeftPos = NewKeyPos - NewKeyBiNormal * NewKeyWidth;
			const FVector NewKeyRightPos = NewKeyPos + NewKeyBiNormal * NewKeyWidth;
			const FVector NewKeyFalloffLeftPos = NewKeyPos - NewKeyBiNormal * (NewKeyWidth + NewKeyLeftFalloff);
			const FVector NewKeyFalloffRightPos = NewKeyPos + NewKeyBiNormal * (NewKeyWidth + NewKeyRightFalloff);
			
			const FVector NewKeyLayerLeftPos = NewKeyPos - NewKeyBiNormal * NewKeyLayerWidth;
			const FVector NewKeyLayerRightPos = NewKeyPos + NewKeyBiNormal * NewKeyLayerWidth;
			const FVector NewKeyLayerFalloffLeftPos = NewKeyPos - NewKeyBiNormal * (NewKeyLayerWidth + NewKeyLeftLayerFalloff);
			const FVector NewKeyLayerFalloffRightPos = NewKeyPos + NewKeyBiNormal * (NewKeyLayerWidth + NewKeyRightLayerFalloff);
			const float NewKeyStartEndFalloff = FMath::Min((StartFalloffFraction > 0 ? NewKeyTime / StartFalloffFraction : 1.0f), (EndFalloffFraction > 0 ? (1 - NewKeyTime) / EndFalloffFraction : 1.0f));

			// If not the first keypoint, interp from the last keypoint.
			if (i > 0)
			{
				const int32 NumSteps = FMath::CeilToInt((NewKeyTime - OldKeyTime) * NumSubdivisions);
				const float DrawSubstep = (NewKeyTime - OldKeyTime) / NumSteps;

				// Add a point for each substep, except the ends because that's the point added outside the interp'ing.
				for (int32 j = 1; j < NumSteps; j++)
				{
					const float NewTime = OldKeyTime + j*DrawSubstep;
					const float NewCosInterp = 0.5f - 0.5f * FMath::Cos(NewTime * PI);
					const float NewWidth = FMath::Lerp(StartWidth, EndWidth, NewCosInterp);
					const float NewLayerWidth = FMath::Lerp(StartLayerWidth, EndLayerWidth, NewCosInterp);
					const float NewLeftFalloff = FMath::Lerp(Falloffs.StartLeftSide, Falloffs.EndLeftSide, NewCosInterp);
					const float NewRightFalloff = FMath::Lerp(Falloffs.StartRightSide, Falloffs.EndRightSide, NewCosInterp);
					const float NewLeftLayerFalloff = FMath::Lerp(Falloffs.StartLeftSideLayer, Falloffs.EndLeftSideLayer, NewCosInterp);
					const float NewRightLayerFalloff = FMath::Lerp(Falloffs.StartRightSideLayer, Falloffs.EndRightSideLayer, NewCosInterp);
					const float NewRoll = FMath::Lerp(StartRoll, EndRoll, NewCosInterp);
					const FVector NewPos = SplineInfo.Eval(NewTime, FVector::ZeroVector);
					const FVector NewTangent = SplineInfo.EvalDerivative(NewTime, FVector::ZeroVector).GetSafeNormal();
					const FVector NewBiNormal = FQuat(NewTangent, -NewRoll).RotateVector((NewTangent ^ FVector(0, 0, -1)).GetSafeNormal());
					const FVector NewLeftPos = NewPos - NewBiNormal * NewWidth;
					const FVector NewRightPos = NewPos + NewBiNormal * NewWidth;
					const FVector NewFalloffLeftPos = NewPos - NewBiNormal * (NewWidth + NewLeftFalloff);
					const FVector NewFalloffRightPos = NewPos + NewBiNormal * (NewWidth + NewRightFalloff);

					const FVector NewLayerLeftPos = NewPos - NewBiNormal * NewLayerWidth;
					const FVector NewLayerRightPos = NewPos + NewBiNormal * NewLayerWidth;
					const FVector NewLayerFalloffLeftPos = NewPos - NewBiNormal * (NewLayerWidth + NewLeftLayerFalloff);
					const FVector NewLayerFalloffRightPos = NewPos + NewBiNormal * (NewLayerWidth + NewRightLayerFalloff);
					const float NewStartEndFalloff = FMath::Min((StartFalloffFraction > 0 ? NewTime / StartFalloffFraction : 1.0f), (EndFalloffFraction > 0 ? (1 - NewTime) / EndFalloffFraction : 1.0f));

					Points.Emplace(NewPos, NewLeftPos, NewRightPos, NewFalloffLeftPos, NewFalloffRightPos, NewLayerLeftPos, NewLayerRightPos, NewLayerFalloffLeftPos, NewLayerFalloffRightPos, NewStartEndFalloff);
				}
			}

			Points.Emplace(NewKeyPos, NewKeyLeftPos, NewKeyRightPos, NewKeyFalloffLeftPos, NewKeyFalloffRightPos, NewKeyLayerLeftPos, NewKeyLayerRightPos, NewKeyLayerFalloffLeftPos, NewKeyLayerFalloffRightPos, NewKeyStartEndFalloff);

			OldKeyTime = NewKeyTime;
		}

		// Handle self-intersection errors due to tight turns
		FixSelfIntersection(Points, &FLandscapeSplineInterpPoint::Left);
		FixSelfIntersection(Points, &FLandscapeSplineInterpPoint::Right);
		FixSelfIntersection(Points, &FLandscapeSplineInterpPoint::FalloffLeft);
		FixSelfIntersection(Points, &FLandscapeSplineInterpPoint::FalloffRight);
		FixSelfIntersection(Points, &FLandscapeSplineInterpPoint::LayerLeft);
		FixSelfIntersection(Points, &FLandscapeSplineInterpPoint::LayerRight);
		FixSelfIntersection(Points, &FLandscapeSplineInterpPoint::LayerFalloffLeft);
		FixSelfIntersection(Points, &FLandscapeSplineInterpPoint::LayerFalloffRight);

	}
}
#endif

#undef LOCTEXT_NAMESPACE
