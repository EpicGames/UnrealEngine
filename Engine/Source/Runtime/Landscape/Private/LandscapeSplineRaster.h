// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LandscapeSplineSegment.h"

class ULandscapeInfo;
class ULandscapeLayerInfoObject;

namespace LandscapeSplineRaster
{
#if WITH_EDITOR
	bool FixSelfIntersection(TArray<FLandscapeSplineInterpPoint>& Points, FVector FLandscapeSplineInterpPoint::* Side);

	struct FPointifyFalloffs
	{
		FPointifyFalloffs(float StartFalloff, float EndFalloff)
			: StartLeftSide(StartFalloff)
			, EndLeftSide(EndFalloff)
			, StartRightSide(StartFalloff)
			, EndRightSide(EndFalloff)
			, StartLeftSideLayer(StartFalloff)
			, EndLeftSideLayer(EndFalloff)
			, StartRightSideLayer(StartFalloff)
			, EndRightSideLayer(EndFalloff)
		{
		}

		FPointifyFalloffs()
			: FPointifyFalloffs(0.0f, 0.0f)
		{

		}

		float StartLeftSide;
		float EndLeftSide;
		float StartRightSide;
		float EndRightSide;
		float StartLeftSideLayer;
		float EndLeftSideLayer;
		float StartRightSideLayer;
		float EndRightSideLayer;
	};

	void Pointify(const FInterpCurveVector& SplineInfo, TArray<FLandscapeSplineInterpPoint>& OutPoints, int32 NumSubdivisions,
		float StartFalloffFraction, float EndFalloffFraction,
		const float StartWidth, const float EndWidth,
		const float StartLayerWidth, const float EndLayerWidth,
		const FPointifyFalloffs& Falloffs,
		const float StartRollDegrees, const float EndRollDegrees);

	void RasterizeSegmentPoints(ULandscapeInfo* LandscapeInfo, TArray<FLandscapeSplineInterpPoint> Points, const FTransform& SplineToWorld, bool bRaiseTerrain, bool bLowerTerrain, ULandscapeLayerInfoObject* LayerInfo);
#endif
}
