// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LandscapeSpline.cpp
=============================================================================*/

#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformStackWalk.h"
#include "Misc/Guid.h"
#include "Math/RandomStream.h"
#include "SceneView.h"
#include "UObject/ObjectMacros.h"
#include "Templates/Casts.h"
#include "UObject/ConstructorHelpers.h"
#include "Engine/EngineTypes.h"
#include "HitProxies.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "LandscapeProxy.h"
#include "Engine/Texture2D.h"
#include "PrimitiveDrawingUtils.h"
#include "LandscapeComponent.h"
#include "LandscapeVersion.h"
#include "Components/SplineMeshComponent.h"
#include "Components/BillboardComponent.h"
#include "Engine/StaticMesh.h"
#include "LandscapeSplineProxies.h"
#include "LandscapeSplinesComponent.h"
#include "LandscapeSplineSegment.h"
#include "LandscapeSplineRaster.h"
#include "LandscapeSplineControlPoint.h"
#include "LandscapePrivate.h"
#include "MeshElementCollector.h"
#include "ILandscapeSplineInterface.h"
#include "ControlPointMeshComponent.h"
#include "Engine/CollisionProfile.h"
#include "Engine/Engine.h"
#include "PhysicsEngine/BodySetup.h"
#include "Engine/StaticMeshSocket.h"
#include "EngineGlobals.h"
#include "TextureResource.h"
#include "UObject/FortniteReleaseBranchCustomObjectVersion.h"
#include "UObject/UObjectIterator.h"
#include "UObject/ObjectSaveContext.h"
#if WITH_EDITOR
#include "Logging/TokenizedMessage.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Landscape.h"
#include "LandscapeLayerInfoObject.h"
#include "LandscapeInfoMap.h"
#include "LandscapeSplineActor.h"
#include "LandscapeSettings.h"
#endif

IMPLEMENT_HIT_PROXY(HLandscapeSplineProxy, HHitProxy);
IMPLEMENT_HIT_PROXY(HLandscapeSplineProxy_Segment, HLandscapeSplineProxy);
IMPLEMENT_HIT_PROXY(HLandscapeSplineProxy_ControlPoint, HLandscapeSplineProxy);
IMPLEMENT_HIT_PROXY(HLandscapeSplineProxy_Tangent, HLandscapeSplineProxy);

#define LOCTEXT_NAMESPACE "Landscape.Splines"

int32 SplinesAlwaysUseBlockAll = 0;
static FAutoConsoleVariableRef CVarSplinesAlwaysUseBlockAll(
	TEXT("splines.blockall"),
	SplinesAlwaysUseBlockAll,
	TEXT("Force splines to always use the BlockAll collision profile instead of whatever is stored in the CollisionProfileName property")
);

int32 LandscapeSplineToSplineComponentMaxIterations = 200;
static FAutoConsoleVariableRef CVarLandscapeSplineToSplineComponentMaxIterations(
	TEXT("Landscape.Splines.ApplyToSplineComponentMaxIterations"),
	LandscapeSplineToSplineComponentMaxIterations,
	TEXT("Max possible iterations when converting a landscape spline into a spline component")
);

#if WITH_EDITOR
static FAutoConsoleVariable CVarLandscapeSplineEnableFixedSpriteIcons(
	TEXT("landscape.Splines.EnableFixedSpriteIcons"),
	false,
	TEXT("Enables fixed scaling and Z-offset for spline icons. If disabled, configure dynamic scale and offset in project settings."));
#endif // WITH_EDITOR 

//////////////////////////////////////////////////////////////////////////
// LANDSCAPE SPLINES SCENE PROXY

/** Represents a ULandscapeSplinesComponent to the scene manager. */
#if WITH_EDITOR
struct FLandscapeFixSplines
{
	FLandscapeFixSplines()
		: FixSplinesConsoleCommand(
			TEXT("Landscape.FixSplines"),
			TEXT("One off fix for bad layer width"),
			FConsoleCommandDelegate::CreateRaw(this, &FLandscapeFixSplines::FixSplines))
	{
	}

	FAutoConsoleCommand FixSplinesConsoleCommand;

	void FixSplines()
	{
		bool bFixed = false;

		for (TObjectIterator<UWorld> It(/*AdditionalExclusionFlags = */RF_ClassDefaultObject, /*bIncludeDerivedClasses = */true, /*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
		{
			UWorld* CurrentWorld = *It;
			if (!CurrentWorld->IsGameWorld())
			{
				auto& LandscapeInfoMap = ULandscapeInfoMap::GetLandscapeInfoMap(CurrentWorld);
				for (auto& Pair : LandscapeInfoMap.Map)
				{
					if (Pair.Value && Pair.Value->SupportsLandscapeEditing())
					{
						Pair.Value->ForAllSplineActors([&bFixed](TScriptInterface<ILandscapeSplineInterface> SplineOwner)
						{
							if (SplineOwner && SplineOwner->GetSplinesComponent())
							{
								SplineOwner->GetSplinesComponent()->RebuildAllSplines();
								bFixed = true;
							}
						});

						if (Pair.Value->LandscapeActor.IsValid() && !Pair.Value->LandscapeActor->IsTemplate())
						{
							Pair.Value->LandscapeActor->RequestSplineLayerUpdate();
						}
					}
				}
			}
		}

		UE_LOGF(LogLandscape, Display, "Landscape.FixSplines: %ls", bFixed ? TEXT("Splines fixed") : TEXT("Nothing to fix"));
	}
};

FLandscapeFixSplines GLandscapeFixSplines;

class FLandscapeSplinesSceneProxy final : public FPrimitiveSceneProxy
{
private:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	const FLinearColor	SplineColor;

	const UTexture2D*	ControlPointSprite;
	const bool			bDrawControlPointSprite;
	const bool			bDrawFalloff;

	struct FSegmentProxy
	{
		ULandscapeSplineSegment* Owner;
		TRefCountPtr<HHitProxy> HitProxy;
		TArray<FLandscapeSplineInterpPoint> Points;
		uint32 bSelected : 1;
	};
	TArray<FSegmentProxy> Segments;

	struct FControlPointProxy
	{
		ULandscapeSplineControlPoint* Owner;
		TRefCountPtr<HHitProxy> HitProxy;
		FVector Location;
		TArray<FLandscapeSplineInterpPoint> Points;
		// Controls sprite scaling based on segment width, may be removed in favor of project settings as sole scale factor
		// Scale usage toggled by CVar command landscape.Splines.EnableFixedSpriteIcons
		float SpriteScale;
		uint32 bSelected : 1;
	};
	TArray<FControlPointProxy> ControlPoints;

public:

	~FLandscapeSplinesSceneProxy()
	{
	}

	FLandscapeSplinesSceneProxy(ULandscapeSplinesComponent* Component):
		FPrimitiveSceneProxy(Component),
		SplineColor(Component->SplineColor),
		ControlPointSprite(Component->ControlPointSprite),
		bDrawControlPointSprite(Component->bShowSplineEditorMesh),
		bDrawFalloff(Component->bShowSplineEditorMesh)
	{
		Segments.Reserve(Component->Segments.Num());
		for (ULandscapeSplineSegment* Segment : Component->Segments)
		{
			if (Segment)
			{
				FSegmentProxy SegmentProxy;
				SegmentProxy.Owner = Segment;
				SegmentProxy.HitProxy = nullptr;
				SegmentProxy.Points = Segment->GetPoints();
				SegmentProxy.bSelected = Segment->IsSplineSelected();
				Segments.Add(SegmentProxy);
			}
		}

		ControlPoints.Reserve(Component->ControlPoints.Num());
		for (ULandscapeSplineControlPoint* ControlPoint : Component->ControlPoints)
		{
			if (ControlPoint)
			{
				FControlPointProxy ControlPointProxy;
				ControlPointProxy.Owner = ControlPoint;
				ControlPointProxy.HitProxy = nullptr;
				ControlPointProxy.Location = ControlPoint->Location;
				ControlPointProxy.Points = ControlPoint->GetPoints();
				ControlPointProxy.SpriteScale = FMath::Clamp<float>(ControlPoint->Width != 0 ? ControlPoint->Width / 8 : ControlPoint->SideFalloff / 4, 10, 1000);
				ControlPointProxy.bSelected = ControlPoint->IsSplineSelected();
				ControlPoints.Add(ControlPointProxy);
			}
		}
	}

	virtual HHitProxy* CreateHitProxies(UPrimitiveComponent* Component, TArray<TRefCountPtr<HHitProxy> >& OutHitProxies) override
	{
		OutHitProxies.Reserve(OutHitProxies.Num() + Segments.Num() + ControlPoints.Num());
		for (FSegmentProxy& Segment : Segments)
		{
			Segment.HitProxy = new HLandscapeSplineProxy_Segment(Segment.Owner);
			OutHitProxies.Add(Segment.HitProxy);
		}
		for (FControlPointProxy& ControlPoint : ControlPoints)
		{
			ControlPoint.HitProxy = new HLandscapeSplineProxy_ControlPoint(ControlPoint.Owner);
			OutHitProxies.Add(ControlPoint.HitProxy);
		}
		return nullptr;
	}

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		// Slight Depth Bias so that the splines show up when they exactly match the target surface
		// e.g. someone playing with splines on a newly-created perfectly-flat landscape
		static const float DepthBias = 0.0001f;

		const FMatrix& MyLocalToWorld = GetLocalToWorld();

		const FLinearColor SelectedSplineColor = GEngine->GetSelectedMaterialColor();
		const FLinearColor SelectedControlPointSpriteColor = FLinearColor::White + (GEngine->GetSelectedMaterialColor() * GEngine->SelectionHighlightIntensityBillboards * 20);

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				const FSceneView* const View = Views[ViewIndex];
				FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);
				const uint8 DepthPriority = static_cast<uint8>(GetDepthPriorityGroup(View));

				for (const FSegmentProxy& Segment : Segments)
				{
					const FLinearColor SegmentColor = Segment.bSelected ? SelectedSplineColor : SplineColor;

					if (Segment.Points.Num() == 0 || !Segment.Points.IsValidIndex(0)) // for some reason the segment do not have valid points, prevent possible crash, by simply not rendering this segment
						continue;

					FLandscapeSplineInterpPoint OldPoint = Segment.Points[0];
					OldPoint.Center       = MyLocalToWorld.TransformPosition(OldPoint.Center);
					OldPoint.Left         = MyLocalToWorld.TransformPosition(OldPoint.Left);
					OldPoint.Right        = MyLocalToWorld.TransformPosition(OldPoint.Right);
					OldPoint.FalloffLeft  = MyLocalToWorld.TransformPosition(OldPoint.FalloffLeft);
					OldPoint.FalloffRight = MyLocalToWorld.TransformPosition(OldPoint.FalloffRight);
					for (int32 i = 1; i < Segment.Points.Num(); i++)
					{
						FLandscapeSplineInterpPoint NewPoint = Segment.Points[i];
						NewPoint.Center       = MyLocalToWorld.TransformPosition(NewPoint.Center);
						NewPoint.Left         = MyLocalToWorld.TransformPosition(NewPoint.Left);
						NewPoint.Right        = MyLocalToWorld.TransformPosition(NewPoint.Right);
						NewPoint.FalloffLeft  = MyLocalToWorld.TransformPosition(NewPoint.FalloffLeft);
						NewPoint.FalloffRight = MyLocalToWorld.TransformPosition(NewPoint.FalloffRight);

						// Draw lines from the last keypoint.
						PDI->SetHitProxy(Segment.HitProxy);

						// center line
						PDI->DrawLine(OldPoint.Center, NewPoint.Center, SegmentColor, DepthPriority, 0.0f, DepthBias);

						// draw sides
						PDI->DrawLine(OldPoint.Left, NewPoint.Left, SegmentColor, DepthPriority, 0.0f, DepthBias);
						PDI->DrawLine(OldPoint.Right, NewPoint.Right, SegmentColor, DepthPriority, 0.0f, DepthBias);

						PDI->SetHitProxy(nullptr);

						// draw falloff sides
						if (bDrawFalloff)
						{
							DrawDashedLine(PDI, OldPoint.FalloffLeft, NewPoint.FalloffLeft, SegmentColor, 100, DepthPriority, DepthBias);
							DrawDashedLine(PDI, OldPoint.FalloffRight, NewPoint.FalloffRight, SegmentColor, 100, DepthPriority, DepthBias);
						}

						OldPoint = NewPoint;
					}
				}

				for (const FControlPointProxy& ControlPoint : ControlPoints)
				{
					const FVector ControlPointLocation = MyLocalToWorld.TransformPosition(ControlPoint.Location);

					// Draw Sprite
					if (bDrawControlPointSprite)
					{
						// By default use values from Landscape Settings for scale and z-offset
						const ULandscapeSettings* Settings = GetDefault<ULandscapeSettings>();
						bool bEnableFixedSpriteIcons = CVarLandscapeSplineEnableFixedSpriteIcons->GetBool();

						float ControlPointSpriteZOffset = Settings->GetSplineIconWorldZOffset();
						float ControlPointSpriteScale = bEnableFixedSpriteIcons ? ControlPoint.SpriteScale : Settings->GetSplineIconScale();
						FVector ControlPointSpriteLocation = ControlPointLocation + FVector(0, 0, ControlPointSpriteZOffset);
					
						if (bEnableFixedSpriteIcons)
						{
							ControlPointSpriteScale *= MyLocalToWorld.GetScaleVector().X;
							const FMatrix& ProjectionMatrix = View->ViewMatrices.GetViewToClip();
							const double ZoomFactor = FMath::Min<double>(ProjectionMatrix.M[0][0], ProjectionMatrix.M[1][1]);
							const double Scale = View->WorldToScreen(ControlPointLocation).W * (4.0f / View->UnscaledViewRect.Width() / ZoomFactor);
							ControlPointSpriteScale *= Scale;
						}
						
						const FLinearColor ControlPointSpriteColor = ControlPoint.bSelected ? SelectedControlPointSpriteColor : FLinearColor::White;
						PDI->SetHitProxy(ControlPoint.HitProxy);
#if WITH_EDITORONLY_DATA
						ControlPointSpriteScale *= UBillboardComponent::EditorScale;
#endif
						if (bEnableFixedSpriteIcons)
						{
							// Clamping the scale between 10 and the ControlPoint initial scale 
							ControlPointSpriteScale = FMath::Clamp<double>(ControlPointSpriteScale, 10, ControlPoint.SpriteScale);
							ControlPointSpriteLocation = ControlPointLocation + FVector(0, 0, ControlPointSpriteScale * 0.75f);
						}

						PDI->DrawSprite(
							ControlPointSpriteLocation,
							static_cast<float>(ControlPointSpriteScale),
							static_cast<float>(ControlPointSpriteScale),
							ControlPointSprite->GetResource(),
							ControlPointSpriteColor,
							DepthPriority,
							0, static_cast<float>(ControlPointSprite->GetResource()->GetSizeX()),
							0, static_cast<float>(ControlPointSprite->GetResource()->GetSizeY()),
							SE_BLEND_Masked);
					}

					// Draw Lines
					const FLinearColor ControlPointColor = ControlPoint.bSelected ? SelectedSplineColor : SplineColor;

					if (ControlPoint.Points.Num() == 1)
					{
						FLandscapeSplineInterpPoint NewPoint = ControlPoint.Points[0];
						NewPoint.Center = MyLocalToWorld.TransformPosition(NewPoint.Center);
						NewPoint.Left   = MyLocalToWorld.TransformPosition(NewPoint.Left);
						NewPoint.Right  = MyLocalToWorld.TransformPosition(NewPoint.Right);
						NewPoint.FalloffLeft  = MyLocalToWorld.TransformPosition(NewPoint.FalloffLeft);
						NewPoint.FalloffRight = MyLocalToWorld.TransformPosition(NewPoint.FalloffRight);

						// draw end for spline connection
						PDI->DrawPoint(NewPoint.Center, ControlPointColor, 6.0f, DepthPriority);
						PDI->DrawLine(NewPoint.Left, NewPoint.Center, ControlPointColor, DepthPriority, 0.0f, DepthBias);
						PDI->DrawLine(NewPoint.Right, NewPoint.Center, ControlPointColor, DepthPriority, 0.0f, DepthBias);
						if (bDrawFalloff)
						{
							DrawDashedLine(PDI, NewPoint.FalloffLeft, NewPoint.Left, ControlPointColor, 100, DepthPriority, DepthBias);
							DrawDashedLine(PDI, NewPoint.FalloffRight, NewPoint.Right, ControlPointColor, 100, DepthPriority, DepthBias);
						}
					}
					else if (ControlPoint.Points.Num() >= 2)
					{
						FLandscapeSplineInterpPoint OldPoint = ControlPoint.Points.Last();
						//OldPoint.Left   = MyLocalToWorld.TransformPosition(OldPoint.Left);
						OldPoint.Right  = MyLocalToWorld.TransformPosition(OldPoint.Right);
						//OldPoint.FalloffLeft  = MyLocalToWorld.TransformPosition(OldPoint.FalloffLeft);
						OldPoint.FalloffRight = MyLocalToWorld.TransformPosition(OldPoint.FalloffRight);

						for (const FLandscapeSplineInterpPoint& Point : ControlPoint.Points)
						{
							FLandscapeSplineInterpPoint NewPoint = Point;
							NewPoint.Center = MyLocalToWorld.TransformPosition(NewPoint.Center);
							NewPoint.Left   = MyLocalToWorld.TransformPosition(NewPoint.Left);
							NewPoint.Right  = MyLocalToWorld.TransformPosition(NewPoint.Right);
							NewPoint.FalloffLeft  = MyLocalToWorld.TransformPosition(NewPoint.FalloffLeft);
							NewPoint.FalloffRight = MyLocalToWorld.TransformPosition(NewPoint.FalloffRight);

							PDI->SetHitProxy(ControlPoint.HitProxy);

							// center line
							PDI->DrawLine(ControlPointLocation, NewPoint.Center, ControlPointColor, DepthPriority, 0.0f, DepthBias);

							// draw sides
							PDI->DrawLine(OldPoint.Right, NewPoint.Left, ControlPointColor, DepthPriority, 0.0f, DepthBias);

							PDI->SetHitProxy(nullptr);

							// draw falloff sides
							if (bDrawFalloff)
							{
								DrawDashedLine(PDI, OldPoint.FalloffRight, NewPoint.FalloffLeft, ControlPointColor, 100, DepthPriority, DepthBias);
							}

							// draw end for spline connection
							PDI->DrawPoint(NewPoint.Center, ControlPointColor, 6.0f, DepthPriority);
							PDI->DrawLine(NewPoint.Left, NewPoint.Center, ControlPointColor, DepthPriority, 0.0f, DepthBias);
							PDI->DrawLine(NewPoint.Right, NewPoint.Center, ControlPointColor, DepthPriority, 0.0f, DepthBias);
							if (bDrawFalloff)
							{
								DrawDashedLine(PDI, NewPoint.FalloffLeft, NewPoint.Left, ControlPointColor, 100, DepthPriority, DepthBias);
								DrawDashedLine(PDI, NewPoint.FalloffRight, NewPoint.Right, ControlPointColor, 100, DepthPriority, DepthBias);
							}

							//OldPoint = NewPoint;
							OldPoint.Right = NewPoint.Right;
							OldPoint.FalloffRight = NewPoint.FalloffRight;
						}
					}
				}

				PDI->SetHitProxy(nullptr);
			}
		}
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = IsShown(View) && View->Family->EngineShowFlags.Splines;
		Result.bDynamicRelevance = true;
		return Result;
	}

	virtual uint32 GetMemoryFootprint() const override
	{
		return sizeof(*this) + GetAllocatedSize();
	}
	uint32 GetAllocatedSize() const
	{
		SIZE_T AllocatedSize = FPrimitiveSceneProxy::GetAllocatedSize() + Segments.GetAllocatedSize() + ControlPoints.GetAllocatedSize();
		for (const FSegmentProxy& Segment : Segments)
		{
			AllocatedSize += Segment.Points.GetAllocatedSize();
		}
		for (const FControlPointProxy& ControlPoint : ControlPoints)
		{
			AllocatedSize += ControlPoint.Points.GetAllocatedSize();
		}
		return IntCastChecked<uint32>(AllocatedSize);
	}
};
#endif

//////////////////////////////////////////////////////////////////////////
// LANDSCAPE SPLINE INTERFACE
ULandscapeSplineInterface::ULandscapeSplineInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

}

//////////////////////////////////////////////////////////////////////////
// SPLINE COMPONENT

ULandscapeSplinesComponent::ULandscapeSplinesComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Mobility = EComponentMobility::Static;

#if WITH_EDITORONLY_DATA
	SplineResolution = 512;
	SplineColor = FColor(0, 192, 48);

	if (!IsRunningCommandlet())
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinder<UTexture2D> SpriteTexture;
			FConstructorStatics()
				: SpriteTexture(TEXT("/Engine/EditorResources/S_Terrain.S_Terrain"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		ControlPointSprite = ConstructorStatics.SpriteTexture.Object;
	}

	// Another one-time initialization (non-conditional to whether we're in the editor, this time): unlike the sprite, which is purely cosmetic, SplineEditorMesh is needed 
	//  at all times because we often check if a spline segment/control point is using the SplineEditorMesh (on PostLoad, for example) :
	{
		// Structure to hold one-time initialization
		struct FConstructorStatics
		{
			ConstructorHelpers::FObjectFinder<UStaticMesh> SplineEditorMesh;
			FConstructorStatics()
				: SplineEditorMesh(TEXT("/Engine/EditorLandscapeResources/SplineEditorMesh"))
			{
			}
		};
		static FConstructorStatics ConstructorStatics;

		SplineEditorMesh = ConstructorStatics.SplineEditorMesh.Object;
	}
#endif
	//RelativeScale3D = FVector(1/100.0f, 1/100.0f, 1/100.0f); // cancel out landscape scale. The scale is set up when component is created, but for a default landscape it's this
}

TArray<USplineMeshComponent*> ULandscapeSplinesComponent::GetSplineMeshComponents()
{
	TArray<USplineMeshComponent*> SplineMeshComponents;
#if WITH_EDITOR
	for (ULandscapeSplineSegment* SplineSegment : Segments)
	{
		// local
		SplineMeshComponents.Append(SplineSegment->GetLocalMeshComponents());
		
		// foreign
		TMap<ULandscapeSplinesComponent*, TArray<USplineMeshComponent*>> ForeignMeshComponentsMap = SplineSegment->GetForeignMeshComponents();
		for (const auto& ForeignMeshComponentsPair : ForeignMeshComponentsMap)
		{
			for (USplineMeshComponent* ForeignMeshComponent : ForeignMeshComponentsPair.Value)
			{
				SplineMeshComponents.Add(ForeignMeshComponent);
			}
		}
	}
#endif

	return SplineMeshComponents;
}

ILandscapeSplineInterface* ULandscapeSplinesComponent::GetSplineOwner()
{
	return Cast<ILandscapeSplineInterface>(GetOwner());
}

void ULandscapeSplinesComponent::CheckSplinesValid()
{
	// Remove all null control points/segments.  Do this even if checks are disabled to protect against saved bad data.
	// Likely, some error occurred in a previous session and the package is saved with dud references.
	ensureMsgf(!ControlPoints.Remove(nullptr), TEXT("Error found and resolved in ULandscapeSplinesComponent in %s, actor: %s"), *GetPathName(), GetOwner() ? *GetOwner()->GetActorNameOrLabel() : TEXT("null"));
	ensureMsgf(!Segments.Remove(nullptr), TEXT("Error found and resolved in ULandscapeSplinesComponent in %s, actor: %s"), *GetPathName(), GetOwner() ? *GetOwner()->GetActorNameOrLabel() : TEXT("null"));

	// Check for cross-spline connections, as this is a potential source of nulls.  This can also happen if something went wrong
	// with a segment/point connectivity edit (while merging splines, undo/redo of deletions, etc).  These errors show up some times,
	// but the trigger conditions are not known.
	for (ULandscapeSplineControlPoint* ControlPoint : ControlPoints)
	{
		ensureMsgf(ControlPoint->GetOuterULandscapeSplinesComponent() == this, TEXT("Error in ULandscapeSplinesComponent in %s, actor: %s"), *GetPathName(), GetOwner() ? *GetOwner()->GetActorNameOrLabel() : TEXT("null"));

		for (auto ConnectionIt = ControlPoint->ConnectedSegments.CreateIterator(); ConnectionIt; ++ConnectionIt)
		{
			// HACK [jonathan.bard] : comment out this ensure temporarily while we fix the data : if (ensureMsgf((ConnectionIt->Segment != nullptr),
			//	TEXT("Error in ULandscapeSplinesComponent in %s, actor: %s : null segment : removing connection"), *GetPathName(), GetOwner() ? *GetOwner()->GetActorNameOrLabel() : TEXT("null")))
			if (ConnectionIt->Segment != nullptr)
			{
				ensureMsgf(ConnectionIt->Segment->GetOuterULandscapeSplinesComponent() == this, TEXT("Error in ULandscapeSplinesComponent in %s, actor: %s"), *GetPathName(), GetOwner() ? *GetOwner()->GetActorNameOrLabel() : TEXT("null"));
			}
			else
			{
				ConnectionIt.RemoveCurrent();
			}
		}
	}

	for (ULandscapeSplineSegment* Segment : Segments)
	{
		ensureMsgf(Segment->GetOuterULandscapeSplinesComponent() == this, TEXT("Error in ULandscapeSplinesComponent in %s, actor: %s"), *GetPathName(), GetOwner() ? *GetOwner()->GetActorNameOrLabel() : TEXT("null"));
		for (const FLandscapeSplineSegmentConnection& Connection : Segment->Connections)
		{
			if (Connection.ControlPoint)
			{
				ensureMsgf(Connection.ControlPoint->GetOuterULandscapeSplinesComponent() == this, TEXT("Error in ULandscapeSplinesComponent in %s, actor: %s"), *GetPathName(), GetOwner() ? *GetOwner()->GetActorNameOrLabel() : TEXT("null"));
			}
		}
	}
}

void ULandscapeSplinesComponent::OnRegister()
{
	CheckSplinesValid();

	Super::OnRegister();
}

#if WITH_EDITOR
FPrimitiveSceneProxy* ULandscapeSplinesComponent::CreateSceneProxy()
{
	CheckSplinesValid();

	return new FLandscapeSplinesSceneProxy(this);
}
#endif

FBoxSphereBounds ULandscapeSplinesComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	FBox NewBoundsCalc(ForceInit);

	for (ULandscapeSplineControlPoint* ControlPoint : ControlPoints)
	{
		// TTP #334549: Somehow we're getting nulls in the ControlPoints array
		if (ControlPoint)
		{
			NewBoundsCalc += ControlPoint->GetBounds();
		}
	}

	for (ULandscapeSplineSegment* Segment : Segments)
	{
		if (Segment)
		{
			NewBoundsCalc += Segment->GetBounds();
		}
	}

	FBoxSphereBounds NewBounds;
	if (NewBoundsCalc.IsValid)
	{
		NewBoundsCalc = NewBoundsCalc.TransformBy(LocalToWorld);
		NewBounds = FBoxSphereBounds(NewBoundsCalc);
	}
	else
	{
		// There's no such thing as an "invalid" FBoxSphereBounds (unlike FBox)
		// try to return something that won't modify the parent bounds
		if (GetAttachParent())
		{
			NewBounds = FBoxSphereBounds(GetAttachParent()->Bounds.Origin, FVector::ZeroVector, 0.0f);
		}
		else
		{
			NewBounds = FBoxSphereBounds(LocalToWorld.GetTranslation(), FVector::ZeroVector, 0.0f);
		}
	}
	return NewBounds;
}

bool ULandscapeSplinesComponent::ModifySplines(bool bAlwaysMarkDirty /*= true*/)
{
	bool bSavedToTransactionBuffer = Modify(bAlwaysMarkDirty);

	for (ULandscapeSplineControlPoint* ControlPoint : ControlPoints)
	{
		bSavedToTransactionBuffer = ControlPoint->Modify(bAlwaysMarkDirty) || bSavedToTransactionBuffer;
	}
	for (ULandscapeSplineSegment* Segment : Segments)
	{
		bSavedToTransactionBuffer = Segment->Modify(bAlwaysMarkDirty) || bSavedToTransactionBuffer;
	}

	return bSavedToTransactionBuffer;
}

#if WITH_EDITORONLY_DATA
// legacy ForeignWorldSplineDataMap serialization
FArchive& operator<<(FArchive& Ar, FForeignSplineSegmentData& Value)
{
	if (!Ar.IsFilterEditorOnly())
	{
		Ar << Value.ModificationKey << Value.MeshComponents;
	}
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FForeignWorldSplineData& Value)
{
	if (!Ar.IsFilterEditorOnly())
	{
		// note: ForeignControlPointDataMap is missing in legacy serialization
		Ar << Value.ForeignSplineSegmentDataMap_DEPRECATED;
	}
	return Ar;
}
#endif

void ULandscapeSplinesComponent::Serialize(FArchive& Ar)
{
#if WITH_EDITORONLY_DATA
	// Cooking is a save-time operation, so has to be done before Super::Serialize
	if (Ar.IsCooking())
	{
		CookedForeignMeshComponents.Reset();

		for (const auto& ForeignWorldSplineDataPair : ForeignWorldSplineDataMap)
		{
			const auto& ForeignWorldSplineData = ForeignWorldSplineDataPair.Value;

			for (const auto& ForeignControlPointData : ForeignWorldSplineData.ForeignControlPointData)
			{
				CookedForeignMeshComponents.Add(ForeignControlPointData.MeshComponent);
			}

			for (const auto& ForeignSplineSegmentData : ForeignWorldSplineData.ForeignSplineSegmentData)
			{
				CookedForeignMeshComponents.Append(ForeignSplineSegmentData.MeshComponents);
			}
		}
	}
#endif

	Super::Serialize(Ar);

#if WITH_EDITORONLY_DATA
	if (Ar.UEVer() >= VER_UE4_LANDSCAPE_SPLINE_CROSS_LEVEL_MESHES &&
		!Ar.IsFilterEditorOnly())
	{
		Ar.UsingCustomVersion(FLandscapeCustomVersion::GUID);

		if (Ar.CustomVer(FLandscapeCustomVersion::GUID) < FLandscapeCustomVersion::NewSplineCrossLevelMeshSerialization)
		{
			Ar << ForeignWorldSplineDataMap;
		}

		if (Ar.IsLoading() && Ar.CustomVer(FLandscapeCustomVersion::GUID) < FLandscapeCustomVersion::SplineForeignDataLazyObjectPtrFix)
		{
			for (auto& SplineData : ForeignWorldSplineDataMap)
			{
				for (auto& ControlPoint : SplineData.Value.ForeignControlPointDataMap_DEPRECATED)
				{
					ControlPoint.Value.Identifier = ControlPoint.Key;
					SplineData.Value.ForeignControlPointData.Add(ControlPoint.Value);
				}

				SplineData.Value.ForeignControlPointDataMap_DEPRECATED.Empty();

				for (auto& SegmentData : SplineData.Value.ForeignSplineSegmentDataMap_DEPRECATED)
				{
					SegmentData.Value.Identifier = SegmentData.Key;
					SplineData.Value.ForeignSplineSegmentData.Add(SegmentData.Value);
				}

				SplineData.Value.ForeignSplineSegmentDataMap_DEPRECATED.Empty();
			}
		}
	}

	if (!Ar.IsPersistent())
	{
		Ar << MeshComponentLocalOwnersMap;
		Ar << MeshComponentForeignOwnersMap;
	}
#endif
}

#if WITH_EDITOR
void ULandscapeSplinesComponent::AutoFixMeshComponentErrors(UWorld* OtherWorld)
{
	UWorld* ThisOuterWorld = GetTypedOuter<UWorld>();

	TSoftObjectPtr<UWorld> OtherWorldSoftPtr = OtherWorld;
	ULandscapeSplinesComponent* StreamingSplinesComponent = GetStreamingSplinesComponentForLevel(OtherWorld->PersistentLevel);
	auto* ForeignWorldSplineData = StreamingSplinesComponent ? StreamingSplinesComponent->ForeignWorldSplineDataMap.Find(ThisOuterWorld) : nullptr;

	// Fix control point meshes
	for (ULandscapeSplineControlPoint* ControlPoint : ControlPoints)
	{
		if (ControlPoint->GetForeignWorld() == OtherWorld)
		{
			auto* ForeignControlPointData = ForeignWorldSplineData ? ForeignWorldSplineData->FindControlPoint(ControlPoint) : nullptr;
			if (!ForeignControlPointData || ForeignControlPointData->ModificationKey != ControlPoint->GetModificationKey())
			{
				// We don't pass true for update segments to avoid them being updated multiple times
				ControlPoint->UpdateSplinePoints(true, false);
			}
		}
	}

	// Fix spline segment meshes
	for (ULandscapeSplineSegment* Segment : Segments)
	{
		if (Segment->GetForeignWorlds().Contains(OtherWorld))
		{
			auto* ForeignSplineSegmentData = ForeignWorldSplineData ? ForeignWorldSplineData->FindSegmentData(Segment) : nullptr;
			if (!ForeignSplineSegmentData || ForeignSplineSegmentData->ModificationKey != Segment->GetModificationKey())
			{
				Segment->UpdateSplinePoints(true);
			}
		}
	}

	if (StreamingSplinesComponent)
	{
		StreamingSplinesComponent->DestroyOrphanedForeignSplineMeshComponents(ThisOuterWorld);
		StreamingSplinesComponent->DestroyOrphanedForeignControlPointMeshComponents(ThisOuterWorld);
	}
}

bool ULandscapeSplinesComponent::IsUsingEditorMesh(const USplineMeshComponent* SplineMeshComponent) const
{
	return SplineMeshComponent->GetStaticMesh() == SplineEditorMesh && SplineMeshComponent->bHiddenInGame;
}

void ULandscapeSplinesComponent::ForEachControlPoint(TFunctionRef<void(ULandscapeSplineControlPoint*)> Func)
{
	// Copy in case iteration modifies the list
	TArray<ULandscapeSplineControlPoint*> CopyControlPoints(ControlPoints);
	for (ULandscapeSplineControlPoint* ControlPoint : CopyControlPoints)
	{
		Func(ControlPoint);
	}
}

void ULandscapeSplinesComponent::CopyToSplineComponent(USplineComponent* SplineComponent)
{
	if (SplineComponent == nullptr)
	{
		return;
	}

	if (Segments.IsEmpty())
	{
		return;
	}
	
	SplineComponent->ClearSplinePoints(false);

	// Incremented index for spline component points
	int32 CurrentPointIndex = 0;

	// Traverse up to this many segments
	// For safety.  Ensures we don't get caught in an infinite loop.
	const int32 MAX_ITERATIONS = LandscapeSplineToSplineComponentMaxIterations;
	int32 NumIterations = 0;

	// List of segments we've visited.  Used to ensure we don't visit the same segment twice.
	TSet<ULandscapeSplineSegment*> VisitedSegments;

	// The last control point we used
	// 2 control points per segment
	// 2 segments can share the same control point - one being the arrive control point and one being the exit
	ULandscapeSplineControlPoint* LastControlPoint = nullptr;

	// Arrive Tangent to apply to the next spline point.  Populated as the previous spline mesh's End Tangent
	FVector ArriveTangent(0,0,0);

	ULandscapeSplineSegment* FirstSegment = Segments[0];
	ULandscapeSplineSegment* CurrentSegment = FirstSegment;
	do
	{
		if (!ensure(NumIterations < MAX_ITERATIONS))
		{
			UE_LOGF(LogLandscape, Error, "%ls Reached Max Iterations. Exiting.", ANSI_TO_TCHAR(__FUNCTION__));
			break;
		}
		
		if (!ensure(CurrentSegment != nullptr))
		{
			UE_LOGF(LogLandscape, Error, "%ls Current Segment is nullptr. Exiting.", ANSI_TO_TCHAR(__FUNCTION__));
			break;
		}

		if (!ensure(!VisitedSegments.Contains(CurrentSegment)))
		{
			UE_LOGF(LogLandscape, Error, "%ls Current Segment was already visited. Exiting.", ANSI_TO_TCHAR(__FUNCTION__));
			break;
		}

		VisitedSegments.Add(CurrentSegment);

		// No guarantee if the control point we're looking at is the segment's start or end control point
		// If it's the end control point, we must reverse the direction we traverse over this segment
		// Connections[0] == Start; Connections[1] == End;
		const bool bReverseTraversal = CurrentSegment->Connections[1].ControlPoint == LastControlPoint;
		const FLandscapeSplineSegmentConnection& CurrentConnection = bReverseTraversal ? CurrentSegment->Connections[1] : CurrentSegment->Connections[0];
		const FLandscapeSplineSegmentConnection& NextConnection = bReverseTraversal ? CurrentSegment->Connections[0] : CurrentSegment->Connections[1];

		TArray<USplineMeshComponent*> SegmentSplineMeshComponents = CurrentSegment->GetLocalMeshComponents();

		// No guarantee on the direction of individual spline mesh component directions
		// Test the first one and determine if the start or end location is closer
		bool bReverseSplineMeshes = false;
		const FVector ControlPointWorldLocation = GetComponentTransform().TransformPosition(CurrentConnection.ControlPoint->Location);
		if (const USplineMeshComponent* FirstSplineMesh = SegmentSplineMeshComponents[bReverseTraversal ? SegmentSplineMeshComponents.Num() - 1 : 0])
		{
			const float StartDistanceSqr = FVector::DistSquared(FirstSplineMesh->GetStartPosition() + FirstSplineMesh->GetComponentLocation(), ControlPointWorldLocation);
			const float EndDistanceSqr = FVector::DistSquared(FirstSplineMesh->GetEndPosition() + FirstSplineMesh->GetComponentLocation(), ControlPointWorldLocation);
			bReverseSplineMeshes = EndDistanceSqr < StartDistanceSqr;
		}
		
		// Walk down the spline meshes
		for (int32 Index = 0; Index < SegmentSplineMeshComponents.Num(); ++Index)
		{
			const int32 ActualIndex = bReverseTraversal ? SegmentSplineMeshComponents.Num() - (1 + Index) : Index;
			
			USplineMeshComponent* SplineMesh = SegmentSplineMeshComponents[ActualIndex];
			if (SplineMesh == nullptr)
			{
				continue;
			}

			FVector Position = bReverseSplineMeshes ? SplineMesh->GetEndPosition() : SplineMesh->GetStartPosition();
			Position += SplineMesh->GetComponentLocation();
			
			FVector LeaveTangent = bReverseSplineMeshes ? -SplineMesh->GetEndTangent() : SplineMesh->GetStartTangent();

			// Create a spline component point from each spline mesh in the segment
			SplineComponent->AddSplinePoint(Position, ESplineCoordinateSpace::World, false);
			SplineComponent->SetTangentsAtSplinePoint(CurrentPointIndex, ArriveTangent, LeaveTangent, ESplineCoordinateSpace::World, false);
			
			ArriveTangent = bReverseSplineMeshes ? SplineMesh->GetStartPosition() : SplineMesh->GetEndTangent();
				
			++CurrentPointIndex;
		}
		
		// Search for the next segment
		bool bFoundNextSegment = false;
		bool bLoopingSpline = false;
		if (NextConnection.ControlPoint)
		{
			for (FLandscapeSplineConnection& ControlPointConnection : NextConnection.ControlPoint->ConnectedSegments)
			{
				if (ControlPointConnection.Segment == nullptr || VisitedSegments.Contains(ControlPointConnection.Segment))
				{
					// Check for loop
					if (ControlPointConnection.Segment == FirstSegment)
					{
						bLoopingSpline = true;
					}
					
					continue;
				}
				
				CurrentSegment = ControlPointConnection.Segment;
				LastControlPoint = NextConnection.ControlPoint;
				bFoundNextSegment = true;
				break;
			}
		}

		if (!bFoundNextSegment)
		{
			SplineComponent->SetClosedLoop(bLoopingSpline);
			
			if (bLoopingSpline)
			{
				// Populate first point's arrive tangent
				if (SplineComponent->GetNumberOfSplinePoints() > 0)
				{
					const FVector LeaveTangent = SplineComponent->GetLeaveTangentAtSplinePoint(0, ESplineCoordinateSpace::Local);
					SplineComponent->SetTangentsAtSplinePoint(0, ArriveTangent, LeaveTangent, ESplineCoordinateSpace::World, false);
				}
			}
			
			UE_LOGF(LogLandscape, Verbose, "%ls Could not find next segment or all segments have been traversed.", ANSI_TO_TCHAR(__FUNCTION__));
			break;
		}

		++NumIterations;
	}
	while (NumIterations < MAX_ITERATIONS);

	// Finish by updating the spline with all of our work
	SplineComponent->UpdateSpline();
}

bool ULandscapeSplinesComponent::IsUsingLayerInfo(const ULandscapeLayerInfoObject* LayerInfo) const
{
	for (ULandscapeSplineControlPoint* ControlPoint : ControlPoints)
	{
		if (ControlPoint->LayerName == LayerInfo->GetLayerName())
		{
			return true;
		}
	}

	for (ULandscapeSplineSegment* Segment : Segments)
	{
		if (Segment->LayerName == LayerInfo->GetLayerName())
		{
			return true;
		}
	}

	return false;
}

void ULandscapeSplinesComponent::CheckForErrors()
{
	Super::CheckForErrors();

	UWorld* ThisOuterWorld = GetTypedOuter<UWorld>();
	check(IsRunningCommandlet() || ThisOuterWorld->WorldType == EWorldType::Editor);

	TSet<UWorld*> OutdatedWorlds;
	TMap<UWorld*, FForeignWorldSplineData*> ForeignWorldSplineDataMapCache;

	// Check for internally-inconsistent data in ControlPoints and Segments, likely due to a code error.
	CheckSplinesValid();

	// Check control point meshes
	for (ULandscapeSplineControlPoint* ControlPoint : ControlPoints)
	{
		UWorld* ForeignWorld = ControlPoint->GetForeignWorld().Get();
		if (ForeignWorld && !OutdatedWorlds.Contains(ForeignWorld))
		{
			auto** ForeignWorldSplineDataCachedPtr = ForeignWorldSplineDataMapCache.Find(ForeignWorld);
			auto* ForeignWorldSplineData = ForeignWorldSplineDataCachedPtr ? *ForeignWorldSplineDataCachedPtr : nullptr;
			if (!ForeignWorldSplineDataCachedPtr)
			{
				ULandscapeSplinesComponent* StreamingSplinesComponent = GetStreamingSplinesComponentForLevel(ForeignWorld->PersistentLevel);
				ForeignWorldSplineData = StreamingSplinesComponent ? StreamingSplinesComponent->ForeignWorldSplineDataMap.Find(ThisOuterWorld) : nullptr;
				ForeignWorldSplineDataMapCache.Add(ForeignWorld, ForeignWorldSplineData);
			}
			auto* ForeignControlPointData = ForeignWorldSplineData ? ForeignWorldSplineData->FindControlPoint(ControlPoint) : nullptr;
			if (!ForeignControlPointData || ForeignControlPointData->ModificationKey != ControlPoint->GetModificationKey())
			{
				OutdatedWorlds.Add(ForeignWorld);
			}
		}
	}

	// Check spline segment meshes
	for (ULandscapeSplineSegment* Segment : Segments)
	{
		for (auto& ForeignWorldSoftPtr : Segment->GetForeignWorlds())
		{
			UWorld* ForeignWorld = ForeignWorldSoftPtr.Get();

			if (ForeignWorld && !OutdatedWorlds.Contains(ForeignWorld))
			{
				auto** ForeignWorldSplineDataCachedPtr = ForeignWorldSplineDataMapCache.Find(ForeignWorld);
				auto* ForeignWorldSplineData = ForeignWorldSplineDataCachedPtr ? *ForeignWorldSplineDataCachedPtr : nullptr;
				if (!ForeignWorldSplineDataCachedPtr)
				{
					ULandscapeSplinesComponent* StreamingSplinesComponent = GetStreamingSplinesComponentForLevel(ForeignWorld->PersistentLevel);
					ForeignWorldSplineData = StreamingSplinesComponent ? StreamingSplinesComponent->ForeignWorldSplineDataMap.Find(ThisOuterWorld) : nullptr;
					ForeignWorldSplineDataMapCache.Add(ForeignWorld, ForeignWorldSplineData);
				}
				auto* ForeignSplineSegmentData = ForeignWorldSplineData ? ForeignWorldSplineData->FindSegmentData(Segment) : nullptr;
				if (!ForeignSplineSegmentData || ForeignSplineSegmentData->ModificationKey != Segment->GetModificationKey())
				{
					OutdatedWorlds.Add(ForeignWorld);
				}
			}
		}
	}
	ForeignWorldSplineDataMapCache.Empty();

	for (UWorld* OutdatedWorld : OutdatedWorlds)
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("MeshMap"), FText::FromName(OutdatedWorld->GetFName()));
		Arguments.Add(TEXT("SplineMap"), FText::FromName(ThisOuterWorld->GetFName()));

		FMessageLog("MapCheck").Error()
			->AddToken(FUObjectToken::Create(GetOwner()))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_MeshesOutDated", "Meshes in {MeshMap} out of date compared to landscape spline in {SplineMap}"), Arguments)))
			->AddToken(FActionToken::Create(LOCTEXT("MapCheck_ActionName_MeshesOutDated", "Rebuild landscape splines"), FText(),
			FOnActionTokenExecuted::CreateUObject(this, &ULandscapeSplinesComponent::AutoFixMeshComponentErrors, OutdatedWorld), true));
	}

	// check for orphaned components
	for (auto& ForeignWorldSplineDataPair : ForeignWorldSplineDataMap)
	{
		auto& ForeignWorldSoftPtr = ForeignWorldSplineDataPair.Key;
		auto& ForeignWorldSplineData = ForeignWorldSplineDataPair.Value;

		// World is not loaded
		if (ForeignWorldSoftPtr.IsPending())
		{
			continue;
		}

		UWorld* ForeignWorld = ForeignWorldSoftPtr.Get();
		for (auto& ForeignSplineSegmentData : ForeignWorldSplineData.ForeignSplineSegmentData)
		{
			const ULandscapeSplineSegment* ForeignSplineSegment = ForeignSplineSegmentData.Identifier.Get();

			// No such segment or segment doesn't match our meshes
			if (!ForeignSplineSegment)
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("MeshMap"),   FText::FromName(ThisOuterWorld->GetFName()));
				Arguments.Add(TEXT("SplineMap"), FText::FromName(ForeignWorld->GetFName()));

				FMessageLog("MapCheck").Error()
					->AddToken(FUObjectToken::Create(GetOwner()))
					->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_OrphanedSplineMeshes", "{MeshMap} contains orphaned spline meshes due to mismatch with landscape splines in {SplineMap}"), Arguments)))
					->AddToken(FActionToken::Create(LOCTEXT("MapCheck_ActionName_OrphanedSplineMeshes", "Clean up orphaned spline meshes"), FText(),
						FOnActionTokenExecuted::CreateUObject(this, &ULandscapeSplinesComponent::DestroyOrphanedForeignSplineMeshComponents, ForeignWorld), true));

				break;
			}
		}

		for (auto& ForeignControlPointData : ForeignWorldSplineData.ForeignControlPointData)
		{
			const ULandscapeSplineControlPoint* ForeignControlPoint = ForeignControlPointData.Identifier.Get();

			// No such control point
			if (!ForeignControlPoint)
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("MeshMap"), FText::FromName(ThisOuterWorld->GetFName()));
				Arguments.Add(TEXT("SplineMap"), FText::FromName(ForeignWorld->GetFName()));

				FMessageLog("MapCheck").Error()
					->AddToken(FUObjectToken::Create(GetOwner()))
					->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_OrphanedControlPointMeshes", "{MeshMap} contains orphaned control point meshes due to mismatch with landscape control points in {SplineMap}"), Arguments)))
					->AddToken(FActionToken::Create(LOCTEXT("MapCheck_ActionName_OrphanedControlPointMeshes", "Clean up orphaned control point meshes"), FText(),
						FOnActionTokenExecuted::CreateUObject(this, &ULandscapeSplinesComponent::DestroyOrphanedForeignControlPointMeshComponents, ForeignWorld), true));

				break;
			}
		}
	}

	// Find Unreferenced Foreign Mesh Components
	ForEachUnreferencedForeignMeshComponent([this, ThisOuterWorld](ULandscapeSplineSegment*, USplineMeshComponent*, ULandscapeSplineControlPoint*, UControlPointMeshComponent*)
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("MeshMap"), FText::FromName(ThisOuterWorld->GetFName()));

		FMessageLog("MapCheck").Error()
			->AddToken(FUObjectToken::Create(GetOwner()))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT("MapCheck_Message_UnreferencedMeshes", "{MeshMap} contains meshes with no owners"), Arguments)))
			->AddToken(FActionToken::Create(LOCTEXT("MapCheck_ActionName_UnreferencedMeshes", "Clean up meshes that are not referenced by their owner segments/control points"), FText(),
				FOnActionTokenExecuted::CreateUObject(this, &ULandscapeSplinesComponent::DestroyUnreferencedForeignMeshComponents), true));

		return true;
	});
}
#endif

void ULandscapeSplinesComponent::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (GIsEditor && GetWorld() && GetWorld()->WorldType == EWorldType::Editor)
	{
		// Build MeshComponentForeignOwnersMap (Component->Spline) from ForeignWorldSplineDataMap (World->Spline->Component)
		for (auto& ForeignWorldSplineDataPair : ForeignWorldSplineDataMap)
		{
			auto& ForeignWorld = ForeignWorldSplineDataPair.Key;
			auto& ForeignWorldSplineData = ForeignWorldSplineDataPair.Value;

			for (auto& ForeignControlPointData : ForeignWorldSplineData.ForeignControlPointData)
			{
				MeshComponentForeignOwnersMap.Add(ForeignControlPointData.MeshComponent, ForeignControlPointData.Identifier);
			}

			for (auto& ForeignSplineSegmentData : ForeignWorldSplineData.ForeignSplineSegmentData)
			{
				for (auto& MeshComponent : ForeignSplineSegmentData.MeshComponents)
				{
					MeshComponentForeignOwnersMap.Add(MeshComponent, ForeignSplineSegmentData.Identifier);
				}
			}
		}
	}
#endif

	CheckSplinesValid();
}

void ULandscapeSplinesComponent::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	Super::PreSave(ObjectSaveContext);

	CheckSplinesValid();
}

#if WITH_EDITOR
static bool bHackIsUndoingSplines = false;
void ULandscapeSplinesComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Don't update splines when undoing, not only is it unnecessary and expensive,
	// it also causes failed asserts in debug builds when trying to register components
	// (because the actor hasn't reset its OwnedComponents array yet)
	if (!bHackIsUndoingSplines)
	{
		const bool bUpdateCollision = PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive;
		RebuildAllSplines(bUpdateCollision);
	}
}
void ULandscapeSplinesComponent::PostEditUndo()
{
	bHackIsUndoingSplines = true;
	Super::PostEditUndo();
	bHackIsUndoingSplines = false;

	CheckSplinesValid();

	MarkRenderStateDirty();
}

void ULandscapeSplinesComponent::RebuildAllSplines(bool bUpdateCollision)
{
	for (ULandscapeSplineControlPoint* ControlPoint : ControlPoints)
	{
		ControlPoint->UpdateSplinePoints(true, false);
	}

	for (ULandscapeSplineSegment* Segment : Segments)
	{
		Segment->UpdateSplinePoints(true);
	}
}

void ULandscapeSplinesComponent::RequestSplineLayerUpdate()
{
	if (IsValidChecked(this))
	{
		ILandscapeSplineInterface* SplineOwner = GetSplineOwner();
		//  This can get called on a deleted spline, when redoing a deletion.
		if (SplineOwner)
		{
			SplineOwner->GetLandscapeInfo()->RequestSplineLayerUpdate();
		}
	}
}

void ULandscapeSplinesComponent::ShowSplineEditorMesh(bool bShow)
{
	bShowSplineEditorMesh = bShow;

	for (ULandscapeSplineSegment* Segment : Segments)
	{
		Segment->UpdateSplineEditorMesh();
	}

	MarkRenderStateDirty();
}

bool FForeignWorldSplineData::IsEmpty()
{
	return ForeignControlPointData.Num() == 0 && ForeignSplineSegmentData.Num() == 0;
}

FForeignControlPointData* FForeignWorldSplineData::FindControlPoint(ULandscapeSplineControlPoint* InIdentifer)
{
	for (auto& ControlPoint : ForeignControlPointData)
	{
		if (ControlPoint.Identifier == InIdentifer)
		{
			return &ControlPoint;
		}
	}

	return nullptr;
}

FForeignSplineSegmentData* FForeignWorldSplineData::FindSegmentData(ULandscapeSplineSegment* InIdentifer)
{
	for (auto& SegmentData : ForeignSplineSegmentData)
	{
		if (SegmentData.Identifier == InIdentifer)
		{
			return &SegmentData;
		}
	}

	return nullptr;
}

ULandscapeSplinesComponent* ULandscapeSplinesComponent::GetStreamingSplinesComponentByLocation(const FVector& LocalLocation, bool bCreate /* = true*/)
{
	ALandscapeProxy* OuterLandscape = Cast<ALandscapeProxy>(GetOwner());
	if (OuterLandscape &&
		// when copy/pasting this can get called with a null guid on the parent landscape
		// this is fine, we won't have any cross-level meshes in this case anyway
		OuterLandscape->GetLandscapeGuid().IsValid() &&
		OuterLandscape->GetLandscapeInfo())
	{
		FVector LandscapeLocalLocation = GetComponentTransform().GetRelativeTransform(OuterLandscape->LandscapeActorToWorld()).TransformPosition(LocalLocation);
		const int32 ComponentIndexX = (LandscapeLocalLocation.X >= 0.0f) ? FMath::FloorToInt32(LandscapeLocalLocation.X / OuterLandscape->ComponentSizeQuads) : FMath::CeilToInt32(LandscapeLocalLocation.X / OuterLandscape->ComponentSizeQuads);
		const int32 ComponentIndexY = (LandscapeLocalLocation.Y >= 0.0f) ? FMath::FloorToInt32(LandscapeLocalLocation.Y / OuterLandscape->ComponentSizeQuads) : FMath::CeilToInt32(LandscapeLocalLocation.Y / OuterLandscape->ComponentSizeQuads);
		ULandscapeComponent* LandscapeComponent = OuterLandscape->GetLandscapeInfo()->XYtoComponentMap.FindRef(FIntPoint(ComponentIndexX, ComponentIndexY));
		if (LandscapeComponent)
		{
			ALandscapeProxy* ComponentLandscapeProxy = LandscapeComponent->GetLandscapeProxy();
			if (!ComponentLandscapeProxy->GetSplinesComponent() && bCreate)
			{
				ComponentLandscapeProxy->CreateSplineComponent(GetRelativeScale3D());
			}

			if (ULandscapeSplinesComponent* SplineComponent = ComponentLandscapeProxy->GetSplinesComponent())
			{
				return SplineComponent;
			}
		}
	}

	return this;
}

ULandscapeSplinesComponent* ULandscapeSplinesComponent::GetStreamingSplinesComponentForLevel(ULevel* Level, bool bCreate /* = true*/)
{
	ALandscapeProxy* OuterLandscape = Cast<ALandscapeProxy>(GetOwner());
	if (OuterLandscape)
	{
		ULandscapeInfo* LandscapeInfo = OuterLandscape->GetLandscapeInfo();
		check(LandscapeInfo);

		ALandscapeProxy* Proxy = LandscapeInfo->GetLandscapeProxyForLevel(Level);
		if (Proxy)
		{
			if (!Proxy->GetSplinesComponent() && bCreate)
			{
				Proxy->CreateSplineComponent(GetRelativeScale3D());
			}
			return Proxy->GetSplinesComponent();
		}
	}

	return nullptr;
}

TArray<ULandscapeSplinesComponent*> ULandscapeSplinesComponent::GetAllStreamingSplinesComponents()
{
	ALandscapeProxy* OuterLandscape = Cast<ALandscapeProxy>(GetOwner());
	if (OuterLandscape &&
		// when copy/pasting this can get called with a null guid on the parent landscape
		// this is fine, we won't have any cross-level meshes in this case anyway
		OuterLandscape->GetLandscapeGuid().IsValid())
	{
		ULandscapeInfo* LandscapeInfo = OuterLandscape->GetLandscapeInfo();

		if (LandscapeInfo)
		{
			TArray<ULandscapeSplinesComponent*> SplinesComponents;
			LandscapeInfo->ForAllSplineActors([&SplinesComponents](TScriptInterface<ILandscapeSplineInterface> SplineOwner)
			{
				if (ULandscapeSplinesComponent* SplineComponent = SplineOwner->GetSplinesComponent())
				{
					SplinesComponents.Add(SplineComponent);
				}
			});
			return SplinesComponents;
		}
	}

	return {};
}

void ULandscapeSplinesComponent::UpdateModificationKey(ULandscapeSplineSegment* Owner)
{
	UWorld* OwnerWorld = Owner->GetTypedOuter<UWorld>();
	checkSlow(OwnerWorld != GetTypedOuter<UWorld>());

	auto* ForeignWorldSplineData = ForeignWorldSplineDataMap.Find(OwnerWorld);
	checkSlow(ForeignWorldSplineData);

	if (ForeignWorldSplineData)
	{
		auto* ForeignSplineSegmentData = ForeignWorldSplineData->FindSegmentData(Owner);
		
		if (ForeignSplineSegmentData != nullptr)
		{
			ForeignSplineSegmentData->ModificationKey = Owner->GetModificationKey();
		}
	}
}

void ULandscapeSplinesComponent::UpdateModificationKey(ULandscapeSplineControlPoint* Owner)
{
	UWorld* OwnerWorld = Owner->GetTypedOuter<UWorld>();
	checkSlow(OwnerWorld != GetTypedOuter<UWorld>());

	auto* ForeignWorldSplineData = ForeignWorldSplineDataMap.Find(OwnerWorld);
	checkSlow(ForeignWorldSplineData);

	if (ForeignWorldSplineData)
	{
		auto* ForeignControlPointData = ForeignWorldSplineData->FindControlPoint(Owner);
		
		if (ForeignControlPointData != nullptr)
		{
			ForeignControlPointData->ModificationKey = Owner->GetModificationKey();
		}
	}
}

void ULandscapeSplinesComponent::AddForeignMeshComponent(ULandscapeSplineSegment* Owner, USplineMeshComponent* Component)
{
	UWorld* OwnerWorld = Owner->GetTypedOuter<UWorld>();

#if DO_GUARD_SLOW
	UWorld* ThisOuterWorld = GetTypedOuter<UWorld>();
	UWorld* ComponentOuterWorld = Component->GetTypedOuter<UWorld>();
	checkSlow(ComponentOuterWorld == ThisOuterWorld);
	checkSlow(OwnerWorld != ThisOuterWorld);
#endif

	auto& ForeignWorldSplineData = ForeignWorldSplineDataMap.FindOrAdd(OwnerWorld);
	FForeignSplineSegmentData* ForeignSplineSegmentData = ForeignWorldSplineData.FindSegmentData(Owner);

	if (ForeignSplineSegmentData == nullptr)
	{
		int32 AddedIndex = ForeignWorldSplineData.ForeignSplineSegmentData.Add(FForeignSplineSegmentData());
		ForeignSplineSegmentData = &ForeignWorldSplineData.ForeignSplineSegmentData[AddedIndex];
	}

	ForeignSplineSegmentData->MeshComponents.Add(Component);
	ForeignSplineSegmentData->ModificationKey = Owner->GetModificationKey();
	ForeignSplineSegmentData->Identifier = Owner;

	MeshComponentForeignOwnersMap.Add(Component, Owner);
}

void ULandscapeSplinesComponent::RemoveForeignMeshComponent(ULandscapeSplineSegment* Owner, USplineMeshComponent* Component)
{
	UWorld* OwnerWorld = Owner->GetTypedOuter<UWorld>();

#if DO_GUARD_SLOW
	UWorld* ThisOuterWorld = GetTypedOuter<UWorld>();
	UWorld* ComponentOuterWorld = Component->GetTypedOuter<UWorld>();
	checkSlow(ComponentOuterWorld == ThisOuterWorld);
	checkSlow(OwnerWorld != ThisOuterWorld);
#endif

	auto* ForeignWorldSplineData = ForeignWorldSplineDataMap.Find(OwnerWorld);
	checkSlow(ForeignWorldSplineData);
	checkSlow(MeshComponentForeignOwnersMap.FindRef(Component) == Owner);
	verifySlow(MeshComponentForeignOwnersMap.Remove(Component) == 1);

	if (ForeignWorldSplineData)
	{
		FForeignSplineSegmentData* SegmentData = ForeignWorldSplineData->FindSegmentData(Owner);
		verifySlow(SegmentData->MeshComponents.RemoveSingle(Component) == 1);
		if (SegmentData->MeshComponents.Num() == 0)
		{
			verifySlow(ForeignWorldSplineData->ForeignSplineSegmentData.RemoveSingle(*SegmentData) == 1);

			if (ForeignWorldSplineData->IsEmpty())
			{
				verifySlow(ForeignWorldSplineDataMap.Remove(OwnerWorld) == 1);
			}
		}
		else
		{
			SegmentData->ModificationKey = Owner->GetModificationKey();
		}
	}
}

void ULandscapeSplinesComponent::RemoveAllForeignMeshComponents(ULandscapeSplineSegment* Owner)
{
	UWorld* OwnerWorld = Owner->GetTypedOuter<UWorld>();
	checkSlow(OwnerWorld != GetTypedOuter<UWorld>());

	auto* ForeignWorldSplineData = ForeignWorldSplineDataMap.Find(OwnerWorld);
	checkSlow(ForeignWorldSplineData);

	if (ForeignWorldSplineData)
	{
		auto* ForeignSplineSegmentData = ForeignWorldSplineData->FindSegmentData(Owner);
		if (ForeignSplineSegmentData)
		{
			for (auto& MeshComponent : ForeignSplineSegmentData->MeshComponents)
			{
				checkSlow(MeshComponentForeignOwnersMap.FindRef(MeshComponent) == Owner);
				verifySlow(MeshComponentForeignOwnersMap.Remove(MeshComponent) == 1);
			}
			ForeignSplineSegmentData->MeshComponents.Empty();
			verifySlow(ForeignWorldSplineData->ForeignSplineSegmentData.RemoveSingle(*ForeignSplineSegmentData) == 1);
		}

		if (ForeignWorldSplineData->IsEmpty())
		{
			verifySlow(ForeignWorldSplineDataMap.Remove(OwnerWorld) == 1);
		}
	}
}

void ULandscapeSplinesComponent::AddForeignMeshComponent(ULandscapeSplineControlPoint* Owner, UControlPointMeshComponent* Component)
{
	UWorld* OwnerWorld = Owner->GetTypedOuter<UWorld>();

#if DO_GUARD_SLOW
	UWorld* ThisOuterWorld = GetTypedOuter<UWorld>();
	UWorld* ComponentOuterWorld = Component->GetTypedOuter<UWorld>();
	checkSlow(ComponentOuterWorld == ThisOuterWorld);
	checkSlow(OwnerWorld != ThisOuterWorld);
#endif

	auto& ForeignWorldSplineData = ForeignWorldSplineDataMap.FindOrAdd(OwnerWorld);
	checkSlow(ForeignWorldSplineData.FindControlPoint(Owner) == nullptr);
	int32 AddedIndex = ForeignWorldSplineData.ForeignControlPointData.Add(FForeignControlPointData());
	auto& ForeignControlPointData = ForeignWorldSplineData.ForeignControlPointData[AddedIndex];

	ForeignControlPointData.MeshComponent = Component;
	ForeignControlPointData.ModificationKey = Owner->GetModificationKey();
	ForeignControlPointData.Identifier = Owner;

	MeshComponentForeignOwnersMap.Add(Component, Owner);
}

void ULandscapeSplinesComponent::RemoveForeignMeshComponent(ULandscapeSplineControlPoint* Owner, UControlPointMeshComponent* Component)
{
	UWorld* OwnerWorld = Owner->GetTypedOuter<UWorld>();

#if DO_GUARD_SLOW
	UWorld* ThisOuterWorld = GetTypedOuter<UWorld>();
	UWorld* ComponentOuterWorld = Component->GetTypedOuter<UWorld>();
	checkSlow(ComponentOuterWorld == ThisOuterWorld);
	checkSlow(OwnerWorld != ThisOuterWorld);
#endif

	auto* ForeignWorldSplineData = ForeignWorldSplineDataMap.Find(OwnerWorld);
	checkSlow(ForeignWorldSplineData);
	checkSlow(MeshComponentForeignOwnersMap.FindRef(Component) == Owner);
	verifySlow(MeshComponentForeignOwnersMap.Remove(Component) == 1);

	if (ForeignWorldSplineData)
	{
		auto* ForeignControlPointData = ForeignWorldSplineData->FindControlPoint(Owner);
		checkSlow(ForeignControlPointData);
		checkSlow(ForeignControlPointData->MeshComponent == Component);

		verifySlow(ForeignWorldSplineData->ForeignControlPointData.RemoveSingle(*ForeignControlPointData) == 1);
		if (ForeignWorldSplineData->IsEmpty())
		{
			verifySlow(ForeignWorldSplineDataMap.Remove(OwnerWorld) == 1);
		}
	}
}

void ULandscapeSplinesComponent::ForEachUnreferencedForeignMeshComponent(TFunctionRef<bool(ULandscapeSplineSegment*, USplineMeshComponent*, ULandscapeSplineControlPoint*, UControlPointMeshComponent*)> Func)
{
	TArray<UObject*> ObjectsWithOuter;
	GetObjectsWithOuter(GetOwner(), ObjectsWithOuter);
	for (UObject* Obj : ObjectsWithOuter)
	{
		if (USplineMeshComponent* SplineMeshComponent = Cast<USplineMeshComponent>(Obj))
		{
            // Find Mesh in the Spline Component local map
			if (TLazyObjectPtr<UObject>* ForeignOwner = MeshComponentForeignOwnersMap.Find(SplineMeshComponent))
			{
				if (ULandscapeSplineSegment* OwnerForMesh = Cast<ULandscapeSplineSegment>(ForeignOwner->Get()))
				{
                    // Try to find the mesh through the global map from the owner we found in local map
					TArray<USplineMeshComponent*> SplineMeshComponents = GetForeignMeshComponents(OwnerForMesh);     
					if (!SplineMeshComponents.Contains(SplineMeshComponent))
					{
						// It wasn't found. Link is broken, cleanup is needed.
						if (Func(OwnerForMesh, SplineMeshComponent, nullptr, nullptr))
						{
							break;
						}
					}
				}
			} 
		}
		else if (UControlPointMeshComponent* ControlPointMeshComponent = Cast<UControlPointMeshComponent>(Obj))
		{
			if (TLazyObjectPtr<UObject>* ForeignOwner = MeshComponentForeignOwnersMap.Find(SplineMeshComponent))
			{
				if (ULandscapeSplineControlPoint* OwnerForMesh = Cast<ULandscapeSplineControlPoint>(ForeignOwner->Get()))
				{
					UControlPointMeshComponent* ForeignControlPointMeshComponent = GetForeignMeshComponent(OwnerForMesh);
					if (ControlPointMeshComponent != ForeignControlPointMeshComponent)
					{
						if (Func(nullptr, nullptr, OwnerForMesh, ControlPointMeshComponent))
						{
							break;
						}
					}
				}
			}
		}
	}
}

void ULandscapeSplinesComponent::DestroyUnreferencedForeignMeshComponents()
{
	ForEachUnreferencedForeignMeshComponent([this](ULandscapeSplineSegment* Segment, USplineMeshComponent* SplineMeshComponent, ULandscapeSplineControlPoint* ControlPoint, UControlPointMeshComponent* ControlPointMeshComponent)
	{
		TArray<UWorld*> EmptyWorlds;

		if (SplineMeshComponent != nullptr)
		{
			SplineMeshComponent->DestroyComponent();
			
			// We should be removing something here
			verify(MeshComponentForeignOwnersMap.Remove(SplineMeshComponent));

			for (auto Pair : ForeignWorldSplineDataMap)
			{
				for (int32 i = Pair.Value.ForeignSplineSegmentData.Num() - 1; i >= 0; --i)
				{
					FForeignSplineSegmentData& SegmentData = Pair.Value.ForeignSplineSegmentData[i];
					if (SegmentData.Identifier == Segment)
					{
						int32 RemoveCount = SegmentData.MeshComponents.Remove(SplineMeshComponent);
						// if remove count is not 0, then we are expecting worlds not to match.
						check(RemoveCount == 0 || Segment->GetTypedOuter<UWorld>() != Pair.Key.Get());
					}

					if (SegmentData.MeshComponents.Num() == 0)
					{
						Pair.Value.ForeignSplineSegmentData.RemoveSingle(SegmentData);
					}

					if (Pair.Value.IsEmpty())
					{
						EmptyWorlds.Add(Pair.Key.Get());
					}
				}
			}
		}

		if (ControlPointMeshComponent != nullptr)
		{
			ControlPointMeshComponent->DestroyComponent();
			
			// We should be removing something here
			verify(MeshComponentForeignOwnersMap.Remove(ControlPointMeshComponent));

			for (auto Pair : ForeignWorldSplineDataMap)
			{
				for (int32 i = Pair.Value.ForeignControlPointData.Num() - 1; i >= 0; --i)
				{
					FForeignControlPointData& ControlPointData = Pair.Value.ForeignControlPointData[i];
					if (ControlPointData.Identifier == ControlPoint && ControlPointData.MeshComponent == ControlPointMeshComponent)
					{
						check(ControlPoint->GetTypedOuter<UWorld>() != Pair.Key.Get());
						Pair.Value.ForeignControlPointData.RemoveSingle(ControlPointData);
					}

					if (Pair.Value.IsEmpty())
					{
						EmptyWorlds.Add(Pair.Key.Get());
					}
				}
			}
		}

		for (UWorld* EmptyWorld : EmptyWorlds)
		{
			ForeignWorldSplineDataMap.Remove(EmptyWorld);
		}

		return false;
	});
}

void ULandscapeSplinesComponent::DestroyOrphanedForeignSplineMeshComponents(UWorld* OwnerWorld)
{
	auto* ForeignWorldSplineData = ForeignWorldSplineDataMap.Find(OwnerWorld);

	if (ForeignWorldSplineData)
	{
		for (int32 i = ForeignWorldSplineData->ForeignSplineSegmentData.Num() - 1; i >= 0; --i)
		{
			FForeignSplineSegmentData& SegmentData = ForeignWorldSplineData->ForeignSplineSegmentData[i];
			const auto& ForeignSplineSegment = SegmentData.Identifier;

			if (!ForeignSplineSegment)
			{
				for (auto& MeshComponent : SegmentData.MeshComponents)
				{
					if (MeshComponent)
					{
						checkSlow(!MeshComponentForeignOwnersMap.FindRef(MeshComponent).IsValid());
						verifySlow(MeshComponentForeignOwnersMap.Remove(MeshComponent) == 1);
						MeshComponent->DestroyComponent();
					}
				}
				SegmentData.MeshComponents.Empty();

				ForeignWorldSplineData->ForeignSplineSegmentData.RemoveSingle(SegmentData);
			}
		}

		if (ForeignWorldSplineData->IsEmpty())
		{
			verifySlow(ForeignWorldSplineDataMap.Remove(OwnerWorld) == 1);
		}
	}
}

void ULandscapeSplinesComponent::DestroyOrphanedForeignControlPointMeshComponents(UWorld* OwnerWorld)
{
	auto* ForeignWorldSplineData = ForeignWorldSplineDataMap.Find(OwnerWorld);

	if (ForeignWorldSplineData)
	{
		for (int32 i = ForeignWorldSplineData->ForeignControlPointData.Num() - 1; i >= 0; --i)
		{
			FForeignControlPointData& ControlPointData = ForeignWorldSplineData->ForeignControlPointData[i];
			const auto& ForeignControlPoint = ControlPointData.Identifier;

			if (!ForeignControlPoint && ControlPointData.MeshComponent)
			{
				ControlPointData.MeshComponent->DestroyComponent();
				ForeignWorldSplineData->ForeignControlPointData.RemoveSingle(ControlPointData);
				MeshComponentForeignOwnersMap.Remove(ControlPointData.MeshComponent);
			}
		}

		if (ForeignWorldSplineData->IsEmpty())
		{
			verifySlow(ForeignWorldSplineDataMap.Remove(OwnerWorld) == 1);
		}
	}
}

UControlPointMeshComponent* ULandscapeSplinesComponent::GetForeignMeshComponent(ULandscapeSplineControlPoint* Owner)
{
	UWorld* OwnerWorld = Owner->GetTypedOuter<UWorld>();
	checkSlow(OwnerWorld != GetTypedOuter<UWorld>());

	FForeignWorldSplineData* ForeignWorldSplineData = ForeignWorldSplineDataMap.Find(OwnerWorld);
	if (ForeignWorldSplineData)
	{
		FForeignControlPointData* ForeignControlPointData = ForeignWorldSplineData->FindControlPoint(Owner);
		if (ForeignControlPointData)
		{
			return ForeignControlPointData->MeshComponent;
		}
	}

	return nullptr;
}

TArray<USplineMeshComponent*> ULandscapeSplinesComponent::GetForeignMeshComponents(ULandscapeSplineSegment* Owner)
{
	UWorld* OwnerWorld = Owner->GetTypedOuter<UWorld>();
	checkSlow(OwnerWorld != GetTypedOuter<UWorld>());

	auto* ForeignWorldSplineData = ForeignWorldSplineDataMap.Find(OwnerWorld);
	if (ForeignWorldSplineData)
	{
		auto* ForeignSplineSegmentData = ForeignWorldSplineData->FindSegmentData(Owner);
		if (ForeignSplineSegmentData)
		{
			return ForeignSplineSegmentData->MeshComponents;
		}
	}

	return {};
}

UObject* ULandscapeSplinesComponent::GetOwnerForMeshComponent(const UMeshComponent* SplineMeshComponent)
{
	UObject* LocalOwner = MeshComponentLocalOwnersMap.FindRef(SplineMeshComponent);
	if (LocalOwner)
	{
		return LocalOwner;
	}

	TLazyObjectPtr<UObject>* ForeignOwner = MeshComponentForeignOwnersMap.Find(SplineMeshComponent);
	if (ForeignOwner)
	{
		// this will be null if ForeignOwner isn't currently loaded
		return ForeignOwner->Get();
	}

	return nullptr;
}
#endif // WITH_EDITOR

//////////////////////////////////////////////////////////////////////////
// CONTROL POINT MESH COMPONENT

UControlPointMeshComponent::UControlPointMeshComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SetCollisionProfileName(UCollisionProfile::BlockAll_ProfileName);
	Mobility = EComponentMobility::Static;

#if WITH_EDITORONLY_DATA
	bSelected = false;
#endif
}

#if WITH_EDITOR
bool UControlPointMeshComponent::IsRelevantForSplinePartitioning() const
{
	// Ignore editor only splines
	// Avoid using USplineMeshComponent::IsEditorOnly() as that would exclude all spline mesh components that are partitioned already
	if (Super::IsEditorOnly())
	{
		return false;
	}

	if (GetStaticMesh() == nullptr)
	{
		return false;
	}

	// Ignore editor splines
	if (GetStaticMesh() == GetDefault<ULandscapeSplinesComponent>()->SplineEditorMesh)
	{
		return false;
	}

	return true;
}

UHLODLayer* UControlPointMeshComponent::GetHLODLayerForSplinePartitioning() const
{
	if (HLODLayerOverrideForSplinePartitioning)
	{
		return HLODLayerOverrideForSplinePartitioning;
	}

	if (GetOwner())
	{
		return GetOwner()->GetHLODLayer();
	}

	return nullptr;
}

bool UControlPointMeshComponent::IsEditorOnly() const
{
	if (Super::IsEditorOnly())
	{
		return true;
	}

	// If landscape uses generated LandscapeSplineMeshesActors, ControlPointMeshComponents is removed from cooked build  
	ALandscapeSplineActor* SplineActor = Cast<ALandscapeSplineActor>(GetOwner());
	if (SplineActor && SplineActor->HasGeneratedLandscapeSplineMeshesActors())
	{
		// Treat as editor only if the component was included in the spline partitioning
		return IsRelevantForSplinePartitioning();
	}

	return false;
}
#endif

//////////////////////////////////////////////////////////////////////////
// SPLINE CONTROL POINT

ULandscapeSplineControlPoint::ULandscapeSplineControlPoint(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Width = 1000;
	SideFalloff = 1000;
	EndFalloff = 2000;

#if WITH_EDITORONLY_DATA
	Mesh = nullptr;
	MeshScale = FVector(1);
	bHiddenInGame = false;
	LDMaxDrawDistance = 0;
	TranslucencySortPriority = 0;

	bEnableCollision_DEPRECATED = true;
	BodyInstance.SetCollisionProfileName(UCollisionProfile::BlockAll_ProfileName);

	LayerName = NAME_None;
	bRaiseTerrain = true;
	bLowerTerrain = true;

	LocalMeshComponent = nullptr;
	bPlaceSplineMeshesInStreamingLevels = true;
	bCastShadow = true;

	bRenderCustomDepth = false;
	CustomDepthStencilWriteMask = ERendererStencilMask::ERSM_Default;
	CustomDepthStencilValue = 0;

	// transients
	bSelected = false;
#endif
}

void ULandscapeSplineControlPoint::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FLandscapeCustomVersion::GUID);
	Ar.UsingCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID);

#if WITH_EDITOR
	if (Ar.UEVer() < VER_UE4_LANDSCAPE_SPLINE_CROSS_LEVEL_MESHES)
	{
		bPlaceSplineMeshesInStreamingLevels = false;
	}
#endif

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading() && Ar.CustomVer(FLandscapeCustomVersion::GUID) < FLandscapeCustomVersion::AddingBodyInstanceToSplinesElements)
	{
		BodyInstance.SetCollisionProfileName(bEnableCollision_DEPRECATED ? UCollisionProfile::BlockAll_ProfileName : UCollisionProfile::NoCollision_ProfileName);
	}

	if(Ar.IsLoading() && Ar.CustomVer(FLandscapeCustomVersion::GUID) < FLandscapeCustomVersion::AddSplineLayerFalloff)
	{
		LeftSideLayerFalloffFactor = LeftSideFalloffFactor;
		RightSideLayerFalloffFactor = RightSideFalloffFactor;

		for (FLandscapeSplineInterpPoint& Point : Points)
		{
			Point.LayerFalloffLeft = Point.FalloffLeft;
			Point.LayerFalloffRight = Point.FalloffRight;
		}
	}

	if (Ar.IsLoading() && Ar.CustomVer(FLandscapeCustomVersion::GUID) < FLandscapeCustomVersion::AddSplineLayerWidth)
	{
		for (FLandscapeSplineInterpPoint& Point : Points)
		{
			Point.LayerLeft = Point.Left;
			Point.LayerRight = Point.Right;
		}
	}
#endif
}

void ULandscapeSplineControlPoint::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (GIsEditor)
	{
		if (LocalMeshComponent != nullptr)
		{
			ULandscapeSplinesComponent* OuterSplines = GetOuterULandscapeSplinesComponent();
			OuterSplines->MeshComponentLocalOwnersMap.Add(LocalMeshComponent, this);
		}
	}

	if (GetLinkerUEVersion() < VER_UE4_LANDSCAPE_SPLINE_CROSS_LEVEL_MESHES)
	{
		// Fix collision profile
		if (LocalMeshComponent != nullptr) // ForeignMeshComponents didn't exist yet
		{
#if WITH_EDITORONLY_DATA
			const FName DesiredCollisionProfileName = SplinesAlwaysUseBlockAll ? UCollisionProfile::BlockAll_ProfileName : CollisionProfileName_DEPRECATED;
			const FName CollisionProfile = bEnableCollision_DEPRECATED ? DesiredCollisionProfileName : UCollisionProfile::NoCollision_ProfileName;
#else
			const FName CollisionProfile = UCollisionProfile::BlockAll_ProfileName;
#endif
			if (LocalMeshComponent->GetCollisionProfileName() != CollisionProfile)
			{
				LocalMeshComponent->SetCollisionProfileName(CollisionProfile);
			}

			LocalMeshComponent->SetFlags(RF_TextExportTransient);
		}
	}

	if (GIsEditor 
		&& ((GetLinkerCustomVersion(FLandscapeCustomVersion::GUID) < FLandscapeCustomVersion::AddingBodyInstanceToSplinesElements)
			|| (GetLinkerCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID) < FFortniteReleaseBranchCustomObjectVersion::RemoveUselessLandscapeMeshesCookedCollisionData)))
	{
		auto ForeignMeshComponentsMap = GetForeignMeshComponents();

		for (auto& ForeignMeshComponentsPair : ForeignMeshComponentsMap)
		{
			const bool bUsingEditorMesh = ForeignMeshComponentsPair.Value->bHiddenInGame;

			if (!bUsingEditorMesh)
			{
				ForeignMeshComponentsPair.Value->BodyInstance = BodyInstance;
			}
			else
			{
				ForeignMeshComponentsPair.Value->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
			}

			// We won't ever enable collisions when using the editor mesh, ensure we don't even cook or load any collision data on this mesh  :
			if (UBodySetup* BodySetup = ForeignMeshComponentsPair.Value->GetBodySetup())
			{
				BodySetup->bNeverNeedsCookedCollisionData = bUsingEditorMesh;
			}
		}

		if (LocalMeshComponent != nullptr)
		{
			const bool bUsingEditorMesh = LocalMeshComponent->bHiddenInGame;

			if (!bUsingEditorMesh)
			{
				LocalMeshComponent->BodyInstance = BodyInstance;
			}
			else
			{
				LocalMeshComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
			}

			// We won't ever enable collisions when using the editor mesh, ensure we don't even cook or load any collision data on this mesh  :
			if (UBodySetup* BodySetup = LocalMeshComponent->GetBodySetup())
			{
				BodySetup->bNeverNeedsCookedCollisionData = bUsingEditorMesh;
			}
		}
	}
#endif
}

FLandscapeSplineSegmentConnection& FLandscapeSplineConnection::GetNearConnection() const
{
	return Segment->Connections[End];
}

FLandscapeSplineSegmentConnection& FLandscapeSplineConnection::GetFarConnection() const
{
	return Segment->Connections[1 - End];
}

#if WITH_EDITOR
bool ULandscapeSplineControlPoint::SupportsForeignSplineMesh() const
{
	ULandscapeSplinesComponent* SplineComponent = GetOuterSafe();
	return bPlaceSplineMeshesInStreamingLevels && SplineComponent && SplineComponent->GetSplineOwner()->SupportsForeignSplineMesh();
}

FName ULandscapeSplineControlPoint::GetBestConnectionTo(FVector Destination) const
{
	FName BestSocket = NAME_None;
	double BestScore = -DBL_MAX;

	if (Mesh != nullptr)
	{
		for (const UStaticMeshSocket* Socket : Mesh->Sockets)
		{
			FTransform SocketTransform = FTransform(Socket->RelativeRotation, Socket->RelativeLocation) * FTransform(Rotation, Location, MeshScale);
			FVector SocketLocation = SocketTransform.GetTranslation();
			FRotator SocketRotation = SocketTransform.GetRotation().Rotator();

			const double DistanceToCP = (Destination - Location).Size();

			// score closer locations higher
			const FVector SocketDelta = Destination - SocketLocation;
			const double DistanceToSocket = SocketDelta.Size();

			// score closer rotations higher
			const double DirectionWeight = FMath::Abs(FVector::DotProduct(SocketDelta, SocketRotation.Vector()));

			const double Score = (DistanceToCP - DistanceToSocket) * DirectionWeight;

			if (Score > BestScore)
			{
				BestSocket = Socket->SocketName;
				BestScore = Score;
			}
		}
	}

	return BestSocket;
}

void ULandscapeSplineControlPoint::GetConnectionLocalLocationAndRotation(FName SocketName, OUT FVector& OutLocation, OUT FRotator& OutRotation) const
{
	OutLocation = FVector::ZeroVector;
	OutRotation = FRotator::ZeroRotator;

	if (Mesh != nullptr)
	{
		const UStaticMeshSocket* Socket = Mesh->FindSocket(SocketName);
		if (Socket != nullptr)
		{
			OutLocation = Socket->RelativeLocation;
			OutRotation = Socket->RelativeRotation;
		}
	}
}

void ULandscapeSplineControlPoint::GetConnectionLocationAndRotation(FName SocketName, OUT FVector& OutLocation, OUT FRotator& OutRotation) const
{
	OutLocation = Location;
	OutRotation = Rotation;

	if (Mesh != nullptr)
	{
		const UStaticMeshSocket* Socket = Mesh->FindSocket(SocketName);
		if (Socket != nullptr)
		{
			FTransform SocketTransform = FTransform(Socket->RelativeRotation, Socket->RelativeLocation) * FTransform(Rotation, Location, MeshScale);
			OutLocation = SocketTransform.GetTranslation();
			OutRotation = SocketTransform.GetRotation().Rotator().GetNormalized();
		}
	}
}

void ULandscapeSplineControlPoint::SetSplineSelected(bool bInSelected)
{
	if (bSelected == bInSelected)
	{
		return;
	}
	bSelected = bInSelected;
	ULandscapeSplinesComponent* SplineComponent = GetOuterSafe();
	if (SplineComponent)
	{
		SplineComponent->MarkRenderStateDirty();
	}

	UpdateSplineMeshSelectionState();
}

void ULandscapeSplineControlPoint::AutoCalcRotation(bool bAlwaysRotateForward)
{
	Modify();

	// always rotate forward only applies when there is exactly 2 segments connected
	if (bAlwaysRotateForward && ConnectedSegments.Num() == 2)
	{
		const FLandscapeSplineSegmentConnection& Near0 = ConnectedSegments[0].GetNearConnection();
		const FLandscapeSplineSegmentConnection& Far0 = ConnectedSegments[0].GetFarConnection();

		// we modify the rotation so it is halfway between the two connected segments
		// Get the start and end location/rotation of this connection
		FVector StartLocation0; FRotator StartRotation0;
		this->GetConnectionLocationAndRotation(Near0.SocketName, StartLocation0, StartRotation0);
		FVector EndLocation0; FRotator EndRotation0;
		Far0.ControlPoint->GetConnectionLocationAndRotation(Far0.SocketName, EndLocation0, EndRotation0);
		const FVector DesiredDirection0 = (EndLocation0 - StartLocation0);

		const FLandscapeSplineSegmentConnection& Near1 = ConnectedSegments[1].GetNearConnection();
		const FLandscapeSplineSegmentConnection& Far1 = ConnectedSegments[1].GetFarConnection();
		FVector StartLocation1; FRotator StartRotation1;
		this->GetConnectionLocationAndRotation(Near1.SocketName, StartLocation1, StartRotation1);
		FVector EndLocation1; FRotator EndRotation1;
		Far1.ControlPoint->GetConnectionLocationAndRotation(Far1.SocketName, EndLocation1, EndRotation1);
		const FVector DesiredDirection1 = (EndLocation1 - StartLocation1);

		// compute orientations for incoming and outgoing segments from their direction
		FRotator Rot0 = DesiredDirection0.Rotation();
		FRotator Rot1 = (-DesiredDirection1).Rotation();

		// determine the midpoint orientation via slerp, then convert back to rotator representation
		FRotator DesiredRot = FQuat::Slerp(Rot0.Quaternion(), Rot1.Quaternion(), 0.5).Rotator();

		// enforce pitch as the average of the incoming and outgoing orientations
		DesiredRot.Pitch = (Rot0.Pitch + Rot1.Pitch) * 0.5;
		
		// Remove socket local rotation to calculate the Rotation setting
		FVector StartLocalLocation; FRotator StartLocalRotation;
		this->GetConnectionLocalLocationAndRotation(Near0.SocketName, StartLocalLocation, StartLocalRotation);
		FQuat SocketLocalRotation = StartLocalRotation.Quaternion();
		if (FMath::Sign(Near0.TangentLen) < 0)	// flip 180 if the tangent is inverted
		{
			SocketLocalRotation = SocketLocalRotation * FRotator(0, 180, 0).Quaternion();
		}

		FQuat LocalQuat = DesiredRot.Quaternion() * SocketLocalRotation.Inverse();
		Rotation = LocalQuat.Rotator().GetNormalized();

		AutoFlipTangents();

		return;
	}

	FRotator Delta = FRotator::ZeroRotator;

	// check all connections to this control point
	for (const FLandscapeSplineConnection& Connection : ConnectedSegments)
	{
		// Get the start and end location/rotation of this connection
		FVector StartLocation; FRotator StartRotation;
		this->GetConnectionLocationAndRotation(Connection.GetNearConnection().SocketName, StartLocation, StartRotation);
		FVector StartLocalLocation; FRotator StartLocalRotation;
		this->GetConnectionLocalLocationAndRotation(Connection.GetNearConnection().SocketName, StartLocalLocation, StartLocalRotation);
		FVector EndLocation; FRotator EndRotation;
		Connection.GetFarConnection().ControlPoint->GetConnectionLocationAndRotation(Connection.GetFarConnection().SocketName, EndLocation, EndRotation);

		// Find the delta between the direction of the tangent at the connection point and
		// the direction to the other end's control point
		FQuat SocketLocalRotation = StartLocalRotation.Quaternion();
		if (FMath::Sign(Connection.GetNearConnection().TangentLen) < 0)
		{
			SocketLocalRotation = SocketLocalRotation * FRotator(0, 180, 0).Quaternion();
		}
		const FVector  DesiredDirection = (EndLocation - StartLocation);
		const FQuat    DesiredSocketRotation = DesiredDirection.Rotation().Quaternion();
		const FRotator DesiredRotation = (DesiredSocketRotation * SocketLocalRotation.Inverse()).Rotator().GetNormalized();
		const FRotator DesiredRotationDelta = (DesiredRotation - Rotation).GetNormalized();

		Delta += DesiredRotationDelta;
	}

	// Average delta of all connections
	Delta *= 1.0f / ConnectedSegments.Num();

	// Apply Delta and normalize
	Rotation = (Rotation + Delta).GetNormalized();
}

void ULandscapeSplineControlPoint::AutoFlipTangents()
{
	for (const FLandscapeSplineConnection& Connection : ConnectedSegments)
	{
		Connection.Segment->AutoFlipTangents();
	}
}

void ULandscapeSplineControlPoint::AutoSetConnections(bool bIncludingValid)
{
	for (const FLandscapeSplineConnection& Connection : ConnectedSegments)
	{
		FLandscapeSplineSegmentConnection& NearConnection = Connection.GetNearConnection();
		if (bIncludingValid ||
			(Mesh != nullptr && Mesh->FindSocket(NearConnection.SocketName) == nullptr) ||
			(Mesh == nullptr && NearConnection.SocketName != NAME_None))
		{
			FLandscapeSplineSegmentConnection& FarConnection = Connection.GetFarConnection();
			FVector EndLocation; FRotator EndRotation;
			FarConnection.ControlPoint->GetConnectionLocationAndRotation(FarConnection.SocketName, EndLocation, EndRotation);

			NearConnection.SocketName = GetBestConnectionTo(EndLocation);
			NearConnection.TangentLen = FMath::Abs(NearConnection.TangentLen);

			// Allow flipping tangent on the null connection
			if (NearConnection.SocketName == NAME_None)
			{
				FVector StartLocation; FRotator StartRotation;
				NearConnection.ControlPoint->GetConnectionLocationAndRotation(NearConnection.SocketName, StartLocation, StartRotation);

				if (FVector::DotProduct((EndLocation - StartLocation).GetSafeNormal(), StartRotation.Vector()) < 0)
				{
					NearConnection.TangentLen = -NearConnection.TangentLen;
				}
			}
		}
	}
}
#endif

#if WITH_EDITOR
TMap<ULandscapeSplinesComponent*, UControlPointMeshComponent*> ULandscapeSplineControlPoint::GetForeignMeshComponents()
{
	TMap<ULandscapeSplinesComponent*, UControlPointMeshComponent*> ForeignMeshComponentsMap;

	ULandscapeSplinesComponent* OuterSplines = GetOuterSafe();
	if (OuterSplines)
	{
		TArray<ULandscapeSplinesComponent*> SplineComponents = OuterSplines->GetAllStreamingSplinesComponents();

		for (ULandscapeSplinesComponent* SplineComponent : SplineComponents)
		{
			if (SplineComponent != OuterSplines)
			{
				auto* ForeignMeshComponent = SplineComponent->GetForeignMeshComponent(this);
				if (ForeignMeshComponent)
				{
					ForeignMeshComponent->Modify(false);
					ForeignMeshComponentsMap.Add(SplineComponent, ForeignMeshComponent);
				}
			}
		}
	}

	return ForeignMeshComponentsMap;
}

void ULandscapeSplineControlPoint::UpdateSplinePoints(bool bUpdateCollision, bool bUpdateAttachedSegments, bool bUpdateMeshLevel)
{
	Modify();

	ULandscapeSplinesComponent* OuterSplines = GetOuterChecked();

	auto ForeignMeshComponentsMap = GetForeignMeshComponents();

	ModificationKey = FGuid::NewGuid();
	// Clear Foreign world (will get updated if MeshComponent lives outside of Spline world)
	ForeignWorld = nullptr;

	UControlPointMeshComponent* MeshComponent = LocalMeshComponent;
	ULandscapeSplinesComponent* MeshComponentOuterSplines = OuterSplines;
		
	if (Mesh != nullptr)
	{
		// Attempt to place mesh components into the appropriate landscape streaming levels based on the components under the spline
		if (SupportsForeignSplineMesh())
		{
			MeshComponentOuterSplines = OuterSplines->GetStreamingSplinesComponentByLocation(Location);

			if (MeshComponentOuterSplines != OuterSplines)
			{
				MeshComponent = MeshComponentOuterSplines->GetForeignMeshComponent(this);
				if (MeshComponent)
				{
					MeshComponentOuterSplines->Modify();
					MeshComponentOuterSplines->UpdateModificationKey(this);
				}
			}
		}

		// Create mesh component if needed
		bool bComponentNeedsRegistering = false;
		bool bUpdateLocalForeign = false;
		if (MeshComponent == nullptr)
		{
			AActor* MeshComponentOuterActor = MeshComponentOuterSplines->GetOwner();
			MeshComponentOuterSplines->Modify();
			MeshComponentOuterActor->Modify();
			MeshComponent = NewObject<UControlPointMeshComponent>(MeshComponentOuterActor, NAME_None, RF_Transactional | RF_TextExportTransient);
			MeshComponent->bSelected = bSelected;
			MeshComponent->AttachToComponent(MeshComponentOuterSplines, FAttachmentTransformRules::KeepRelativeTransform);
			bComponentNeedsRegistering = true;
			bUpdateLocalForeign = true;
		}
		else if(bUpdateMeshLevel)// Update Foreign/Local if necessary
		{
			AActor* CurrentMeshComponentOuterActor = MeshComponent->GetTypedOuter<AActor>();
			AActor* MeshComponentOuterActor = MeshComponentOuterSplines->GetOwner();
			ILandscapeSplineInterface* SplineOwner = Cast<ILandscapeSplineInterface>(MeshComponentOuterActor);

			// Needs updating
			if (MeshComponentOuterActor != CurrentMeshComponentOuterActor)
			{
				MeshComponentOuterActor->Modify();
				CurrentMeshComponentOuterActor->Modify();			
				
				MeshComponent->Modify();
				MeshComponent->UnregisterComponent();
				MeshComponent->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);
				MeshComponent->InvalidateLightingCache();
				MeshComponent->Rename(nullptr, MeshComponentOuterActor);
				MeshComponent->AttachToComponent(SplineOwner->GetSplinesComponent(), FAttachmentTransformRules::KeepWorldTransform);
				bComponentNeedsRegistering = true;
				bUpdateLocalForeign = true;
			}
		}

		// If something changed add MeshComponent to proper map
		if (bUpdateLocalForeign)
		{
			if (MeshComponentOuterSplines == OuterSplines)
			{
				UObject*& ValueRef = MeshComponentOuterSplines->MeshComponentLocalOwnersMap.FindOrAdd(MeshComponent);
				ValueRef = this;
				LocalMeshComponent = MeshComponent;
			}
			else
			{
				MeshComponentOuterSplines->AddForeignMeshComponent(this, MeshComponent);
			}
		}
		
		// Update Foreign World
		if (MeshComponent && MeshComponentOuterSplines != OuterSplines)
		{
			ForeignWorld = MeshComponentOuterSplines->GetTypedOuter<UWorld>();
		}

		FVector MeshLocation = Location;
		FRotator MeshRotation = Rotation;
		if (MeshComponentOuterSplines != OuterSplines)
		{
			const FTransform RelativeTransform = OuterSplines->GetComponentTransform().GetRelativeTransform(MeshComponentOuterSplines->GetComponentTransform());
			MeshLocation = RelativeTransform.TransformPosition(MeshLocation);
		}

		const float LocationErrorTolerance = 0.01f;
		if (!MeshComponent->GetRelativeLocation().Equals(MeshLocation, LocationErrorTolerance) ||
			!MeshComponent->GetRelativeRotation().Equals(MeshRotation) ||
			!MeshComponent->GetRelativeScale3D().Equals(MeshScale))
		{
			MeshComponent->Modify();
			MeshComponent->SetRelativeTransform(FTransform(MeshRotation, MeshLocation, MeshScale));
			MeshComponent->InvalidateLightingCache();
		}

		if (MeshComponent->GetStaticMesh() != Mesh)
		{
			MeshComponent->Modify();
			MeshComponent->UnregisterComponent();
			bComponentNeedsRegistering = true;
			MeshComponent->SetStaticMesh(Mesh);

			AutoSetConnections(false);
		}

		if (MeshComponent->OverrideMaterials != MaterialOverrides)
		{
			MeshComponent->Modify();
			MeshComponent->OverrideMaterials = MaterialOverrides;
			MeshComponent->MarkRenderStateDirty();
			if (MeshComponent->BodyInstance.IsValidBodyInstance())
			{
				MeshComponent->BodyInstance.UpdatePhysicalMaterials();
			}
		}

		if (MeshComponent->TranslucencySortPriority != TranslucencySortPriority)
		{
			MeshComponent->Modify();
			MeshComponent->TranslucencySortPriority = TranslucencySortPriority;
			MeshComponent->MarkRenderStateDirty();
		}

		if (MeshComponent->bHiddenInGame != bHiddenInGame)
		{
			MeshComponent->Modify();
			MeshComponent->SetHiddenInGame(bHiddenInGame);
		}

		if (UBodySetup* BodySetup = MeshComponent->GetBodySetup())
		{
			BodySetup->bNeverNeedsCookedCollisionData = bHiddenInGame;
		}

		if (MeshComponent->LDMaxDrawDistance != LDMaxDrawDistance)
		{
			MeshComponent->Modify();
			MeshComponent->LDMaxDrawDistance = LDMaxDrawDistance;
			MeshComponent->CachedMaxDrawDistance = 0;
			MeshComponent->MarkRenderStateDirty();
		}

#if WITH_EDITORONLY_DATA
		if (MeshComponent->BodyInstance.GetCollisionProfileName() != BodyInstance.GetCollisionProfileName())
		{
			MeshComponent->Modify();
			MeshComponent->BodyInstance = BodyInstance;
			MeshComponent->RecreatePhysicsState();
			MeshComponent->MarkRenderStateDirty();
		}
#endif

		if (MeshComponent->CastShadow != bCastShadow)
		{
			MeshComponent->Modify();
			MeshComponent->SetCastShadow(bCastShadow);
		}

		if (MeshComponent->RuntimeVirtualTextures != RuntimeVirtualTextures)
		{
			MeshComponent->Modify();
			MeshComponent->RuntimeVirtualTextures = RuntimeVirtualTextures;
			MeshComponent->MarkRenderStateDirty();
		}

		if (MeshComponent->VirtualTextureLodBias != VirtualTextureLodBias)
		{
			MeshComponent->Modify();
			MeshComponent->VirtualTextureLodBias = static_cast<int8>(VirtualTextureLodBias);
			MeshComponent->MarkRenderStateDirty();
		}
		
		if (MeshComponent->VirtualTextureCullMips != VirtualTextureCullMips)
		{
			MeshComponent->Modify();
			MeshComponent->VirtualTextureCullMips = static_cast<int8>(VirtualTextureCullMips);
			MeshComponent->MarkRenderStateDirty();
		}

		if (MeshComponent->VirtualTextureMainPassMaxDrawDistance != VirtualTextureMainPassMaxDrawDistance)
		{
			MeshComponent->Modify();
			MeshComponent->VirtualTextureMainPassMaxDrawDistance = VirtualTextureMainPassMaxDrawDistance;
			MeshComponent->MarkRenderStateDirty();
		}

		if (MeshComponent->VirtualTextureRenderPassType != VirtualTextureRenderPassType)
		{
			MeshComponent->Modify();
			MeshComponent->VirtualTextureRenderPassType = VirtualTextureRenderPassType;
			MeshComponent->MarkRenderStateDirty();
		}

		if (MeshComponent->bRenderCustomDepth != bRenderCustomDepth)
		{
			MeshComponent->Modify();
			MeshComponent->bRenderCustomDepth = bRenderCustomDepth;
			MeshComponent->MarkRenderStateDirty();
		}

		if (MeshComponent->CustomDepthStencilWriteMask != CustomDepthStencilWriteMask)
		{
			MeshComponent->Modify();
			MeshComponent->CustomDepthStencilWriteMask = CustomDepthStencilWriteMask;
			MeshComponent->MarkRenderStateDirty();
		}

		if (MeshComponent->CustomDepthStencilValue != CustomDepthStencilValue)
		{
			MeshComponent->Modify();
			MeshComponent->CustomDepthStencilValue = CustomDepthStencilValue;
			MeshComponent->MarkRenderStateDirty();
		}

		if (MeshComponent->HLODLayerOverrideForSplinePartitioning != HLODLayerOverrideForSplinePartitioning)
		{
			MeshComponent->Modify();
			MeshComponent->HLODLayerOverrideForSplinePartitioning = HLODLayerOverrideForSplinePartitioning;
		}

		if (bComponentNeedsRegistering)
		{
			MeshComponent->RegisterComponent();
		}
	}
	else
	{
		MeshComponent = nullptr;
		ForeignWorld = nullptr;

		AutoSetConnections(false);
	}

	// Destroy any unused components
	if (LocalMeshComponent && LocalMeshComponent != MeshComponent)
	{
		OuterSplines->Modify();
		LocalMeshComponent->Modify();
		checkSlow(OuterSplines->MeshComponentLocalOwnersMap.FindRef(LocalMeshComponent) == this);
		verifySlow(OuterSplines->MeshComponentLocalOwnersMap.Remove(LocalMeshComponent) == 1);
		LocalMeshComponent->DestroyComponent();
		LocalMeshComponent = nullptr;
	}
	for (auto& ForeignMeshComponentsPair : ForeignMeshComponentsMap)
	{
		ULandscapeSplinesComponent* ForeignMeshComponentOuterSplines = ForeignMeshComponentsPair.Key;
		auto* ForeignMeshComponent = ForeignMeshComponentsPair.Value;
		if (ForeignMeshComponent != MeshComponent)
		{
			ForeignMeshComponentOuterSplines->Modify();
			ForeignMeshComponent->Modify();
			ForeignMeshComponentOuterSplines->RemoveForeignMeshComponent(this, ForeignMeshComponent);
			ForeignMeshComponent->DestroyComponent();
		}
	}
	ForeignMeshComponentsMap.Empty();
	
	const float LeftSideFalloff = LeftSideFalloffFactor * SideFalloff;
	const float RightSideFalloff = RightSideFalloffFactor * SideFalloff;
	const float LeftSideLayerFalloff = LeftSideLayerFalloffFactor * SideFalloff;
	const float RightSideLayerFalloff = RightSideLayerFalloffFactor * SideFalloff;
	const float LayerWidth = Width * LayerWidthRatio;

	// Update "Points" array
	if (Mesh != nullptr)
	{
		Points.Reset(ConnectedSegments.Num());

		for (const FLandscapeSplineConnection& Connection : ConnectedSegments)
		{
			FVector StartLocation; FRotator StartRotation;
			GetConnectionLocationAndRotation(Connection.GetNearConnection().SocketName, StartLocation, StartRotation);

			const double Roll = FMath::DegreesToRadians(StartRotation.Roll);
			const FVector Tangent = StartRotation.Vector();
			const FVector BiNormal = FQuat(Tangent, -Roll).RotateVector((Tangent ^ FVector(0, 0, -1)).GetSafeNormal());
			const FVector LeftPos = StartLocation - BiNormal * Width;
			const FVector RightPos = StartLocation + BiNormal * Width;
			const FVector FalloffLeftPos = StartLocation - BiNormal * (Width + LeftSideFalloff);
			const FVector FalloffRightPos = StartLocation + BiNormal * (Width + RightSideFalloff);
			
			const FVector LayerLeftPos = StartLocation - BiNormal * LayerWidth;
			const FVector LayerRightPos = StartLocation + BiNormal * LayerWidth;
			const FVector LayerFalloffLeftPos = StartLocation - BiNormal * (LayerWidth + LeftSideLayerFalloff);
			const FVector LayerFalloffRightPos = StartLocation + BiNormal * (LayerWidth + RightSideLayerFalloff);

			Points.Emplace(StartLocation, LeftPos, RightPos, FalloffLeftPos, FalloffRightPos, LayerLeftPos, LayerRightPos, LayerFalloffLeftPos, LayerFalloffRightPos, 1.0f);
		}

		const FVector CPLocation = Location;
		Points.Sort([&CPLocation](const FLandscapeSplineInterpPoint& x, const FLandscapeSplineInterpPoint& y){return (x.Center - CPLocation).Rotation().Yaw < (y.Center - CPLocation).Rotation().Yaw;});
	}
	else
	{
		Points.Reset(1);

		FVector StartLocation; FRotator StartRotation;
		GetConnectionLocationAndRotation(NAME_None, StartLocation, StartRotation);

		const double Roll = FMath::DegreesToRadians(StartRotation.Roll);
		const FVector Tangent = StartRotation.Vector();
		const FVector BiNormal = FQuat(Tangent, -Roll).RotateVector((Tangent ^ FVector(0, 0, -1)).GetSafeNormal());
		const FVector LeftPos = StartLocation - BiNormal * Width;
		const FVector RightPos = StartLocation + BiNormal * Width;
		const FVector FalloffLeftPos = StartLocation - BiNormal * (Width + LeftSideFalloff);
		const FVector FalloffRightPos = StartLocation + BiNormal * (Width + RightSideFalloff);

		const FVector LayerLeftPos = StartLocation - BiNormal * LayerWidth;
		const FVector LayerRightPos = StartLocation + BiNormal * LayerWidth;
		const FVector LayerFalloffLeftPos = StartLocation - BiNormal * (LayerWidth + LeftSideLayerFalloff);
		const FVector LayerFalloffRightPos = StartLocation + BiNormal * (LayerWidth + RightSideLayerFalloff);

		Points.Emplace(StartLocation, LeftPos, RightPos, FalloffLeftPos, FalloffRightPos, LayerLeftPos, LayerRightPos, LayerFalloffLeftPos, LayerFalloffRightPos, 1.0f);
	}

	// Update bounds
	Bounds = FBox(ForceInit);

	// Sprite bounds
	float SpriteScale = FMath::Clamp<float>(Width != 0 ? Width / 2 : SideFalloff / 4, 10, 1000);
	Bounds += Location + FVector(0, 0, 0.75f * SpriteScale);
	Bounds = Bounds.ExpandBy(SpriteScale);

	// Points bounds
	for (const FLandscapeSplineInterpPoint& Point : Points)
	{
		Bounds += Point.FalloffLeft;
		Bounds += Point.FalloffRight;
	}

	OuterSplines->MarkRenderStateDirty();

	if (bUpdateAttachedSegments)
	{
		for (const FLandscapeSplineConnection& Connection : ConnectedSegments)
		{
			Connection.Segment->UpdateSplinePoints(bUpdateCollision);
		}
	}
}

void ULandscapeSplineControlPoint::UpdateSplineMeshSelectionState()
{
	if (LocalMeshComponent != nullptr)
	{
		LocalMeshComponent->bSelected = bSelected;
		LocalMeshComponent->PushSelectionToProxy();
	}

	auto ForeignMeshComponentsMap = GetForeignMeshComponents();
	for (auto& ForeignMeshComponentsPair : ForeignMeshComponentsMap)
	{
		ULandscapeSplinesComponent* MeshComponentOuterSplines = ForeignMeshComponentsPair.Key;
		auto* MeshComponent = ForeignMeshComponentsPair.Value;
		MeshComponent->bSelected = bSelected;
		MeshComponent->PushSelectionToProxy();
	}
}

void ULandscapeSplineControlPoint::DeleteSplinePoints()
{
	Modify();

	ULandscapeSplinesComponent* OuterSplines = GetOuterChecked();
	UE_LOGF(LogLandscape, Verbose, "ULandscapeSplineControlPoint::DeleteSplinePoints removing %ls from spline actor %ls", *GetName(), *OuterSplines->GetOwner()->GetActorNameOrLabel());

	Points.Reset();
	Bounds = FBox(ForceInit);

	// Make this control point transient to prevent duplicate from picking the disconnected, but un-garbage-collected
	// instance and exporting it.  Otherwise the non-standard code in PostEditImport will re-attach the point to the
	// component and it will re-appear as part of the copy.
	Rename(nullptr, GetTransientPackage());

	OuterSplines->MarkRenderStateDirty();

	if (LocalMeshComponent != nullptr)
	{
		OuterSplines->Modify();
		LocalMeshComponent->Modify();
		checkSlow(OuterSplines->MeshComponentLocalOwnersMap.FindRef(LocalMeshComponent) == this);
		verifySlow(OuterSplines->MeshComponentLocalOwnersMap.Remove(LocalMeshComponent) == 1);
		LocalMeshComponent->DestroyComponent();
		LocalMeshComponent = nullptr;
	}

	auto ForeignMeshComponentsMap = GetForeignMeshComponents();
	for (auto& ForeignMeshComponentsPair : ForeignMeshComponentsMap)
	{
		ULandscapeSplinesComponent* MeshComponentOuterSplines = ForeignMeshComponentsPair.Key;
		MeshComponentOuterSplines->Modify();
		auto* MeshComponent = ForeignMeshComponentsPair.Value;
		MeshComponent->Modify();
		MeshComponentOuterSplines->RemoveForeignMeshComponent(this, MeshComponent);
		MeshComponent->DestroyComponent();
	}
}

FName ULandscapeSplineControlPoint::GetCollisionProfileName() const
{
#if WITH_EDITORONLY_DATA
	return BodyInstance.GetCollisionProfileName();
#else
	return UCollisionProfile::BlockAll_ProfileName;
#endif
}

void ULandscapeSplineControlPoint::PostEditUndo()
{
	bHackIsUndoingSplines = true;
	Super::PostEditUndo();
	bHackIsUndoingSplines = false;

	// Update mesh components to make sure the selection state stays in sync
	UpdateSplineMeshSelectionState();

	ULandscapeSplinesComponent* SplineComponent = GetOuterSafe();
	if (SplineComponent)
	{
		SplineComponent->MarkRenderStateDirty();
		SplineComponent->RequestSplineLayerUpdate();
	}
}

void ULandscapeSplineControlPoint::PostDuplicate(bool bDuplicateForPIE)
{
	if (!bDuplicateForPIE)
	{
		// if we get duplicated but our local mesh doesn't, then clear our reference to the mesh - it's not ours
		if (LocalMeshComponent != nullptr)
		{
			ULandscapeSplinesComponent* OuterSplines = GetOuterChecked();
			if (LocalMeshComponent->GetOuter() != OuterSplines->GetOwner())
			{
				LocalMeshComponent = nullptr;
			}
			else
			{
				// If LocalMeshComponent is still valid make sure its added to the transient map that normally gets populated on PostLoad
				OuterSplines->MeshComponentLocalOwnersMap.Add(LocalMeshComponent, this);
			}
		}

		UpdateSplinePoints();
	}

	Super::PostDuplicate(bDuplicateForPIE);
}

void ULandscapeSplineControlPoint::PostEditImport()
{
	Super::PostEditImport();

	GetOuterULandscapeSplinesComponent()->ControlPoints.AddUnique(this);
}

void ULandscapeSplineControlPoint::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	Width = FMath::Max(Width, 0.001f);
	LayerWidthRatio = FMath::Max(LayerWidthRatio, 0.01f);
	SideFalloff = FMath::Max(SideFalloff, 0.0f);
	LeftSideFalloffFactor = FMath::Clamp(LeftSideFalloffFactor, 0.0f, 1.0f);
	RightSideFalloffFactor = FMath::Clamp(RightSideFalloffFactor, 0.0f, 1.0f);
	LeftSideLayerFalloffFactor = FMath::Clamp(LeftSideLayerFalloffFactor, 0.0f, 1.0f);
	RightSideLayerFalloffFactor = FMath::Clamp(RightSideLayerFalloffFactor, 0.0f, 1.0f);
	EndFalloff = FMath::Max(EndFalloff, 0.0f);

	// Don't update splines when undoing, not only is it unnecessary and expensive,
	// it also causes failed asserts in debug builds when trying to register components
	// (because the actor hasn't reset its OwnedComponents array yet)
	if (!bHackIsUndoingSplines)
	{
		const bool bUpdateCollision = PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive;
		UpdateSplinePoints(bUpdateCollision);
	}

	if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet)
	{
		GetOuterChecked()->RequestSplineLayerUpdate();
	}
}
#endif // WITH_EDITOR

//////////////////////////////////////////////////////////////////////////
// SPLINE SEGMENT

ULandscapeSplineSegment::ULandscapeSplineSegment(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{

	Connections[0].ControlPoint = nullptr;
	Connections[0].TangentLen = 0;
	Connections[1].ControlPoint = nullptr;
	Connections[1].TangentLen = 0;

#if WITH_EDITORONLY_DATA
	LayerName = NAME_None;
	bRaiseTerrain = true;
	bLowerTerrain = true;

	// SplineMesh properties
	SplineMeshes.Empty();
	LDMaxDrawDistance = 0;
	bHiddenInGame = false;
	TranslucencySortPriority = 0;
	bPlaceSplineMeshesInStreamingLevels = true;
	bCastShadow = true;

	bRenderCustomDepth = false;
	CustomDepthStencilWriteMask = ERendererStencilMask::ERSM_Default;
	CustomDepthStencilValue = 0;

	bEnableCollision_DEPRECATED = true;
	BodyInstance.SetCollisionProfileName(UCollisionProfile::BlockAll_ProfileName);

	// transients
	bSelected = false;
#endif
}

void ULandscapeSplineSegment::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITORONLY_DATA
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_NeedLoad) &&
		!HasAnyInternalFlags(EInternalObjectFlags_AsyncLoading))
	{
		// create a new random seed for all new objects
		RandomSeed = FMath::Rand();
	}
#endif
}

void ULandscapeSplineSegment::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FLandscapeCustomVersion::GUID);
	Ar.UsingCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID);

#if WITH_EDITOR
	if (Ar.UEVer() < VER_UE4_SPLINE_MESH_ORIENTATION)
	{
		for (FLandscapeSplineMeshEntry& MeshEntry : SplineMeshes)
		{
			switch (MeshEntry.Orientation_DEPRECATED)
			{
			case LSMO_XUp:
				MeshEntry.ForwardAxis = ESplineMeshAxis::Z;
				MeshEntry.UpAxis = ESplineMeshAxis::X;
				break;
			case LSMO_YUp:
				MeshEntry.ForwardAxis = ESplineMeshAxis::Z;
				MeshEntry.UpAxis = ESplineMeshAxis::Y;
				break;
			}
		}
	}

	if (Ar.UEVer() < VER_UE4_LANDSCAPE_SPLINE_CROSS_LEVEL_MESHES)
	{
		bPlaceSplineMeshesInStreamingLevels = false;
	}
#endif

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading() && Ar.CustomVer(FLandscapeCustomVersion::GUID) < FLandscapeCustomVersion::AddingBodyInstanceToSplinesElements)
	{
		BodyInstance.SetCollisionProfileName(bEnableCollision_DEPRECATED ? UCollisionProfile::BlockAll_ProfileName : UCollisionProfile::NoCollision_ProfileName);
	}

	if (Ar.IsLoading() && Ar.CustomVer(FLandscapeCustomVersion::GUID) < FLandscapeCustomVersion::AddSplineLayerFalloff)
	{
		for (FLandscapeSplineInterpPoint& Point : Points)
		{
			Point.LayerFalloffLeft = Point.FalloffLeft;
			Point.LayerFalloffRight = Point.FalloffRight;
		}
	}

	if (Ar.IsLoading() && Ar.CustomVer(FLandscapeCustomVersion::GUID) < FLandscapeCustomVersion::AddSplineLayerWidth)
	{
		for (FLandscapeSplineInterpPoint& Point : Points)
		{
			Point.LayerLeft = Point.Left;
			Point.LayerRight = Point.Right;
		}
	}
#endif
}

void ULandscapeSplineSegment::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (GIsEditor)
	{
		if (GetLinkerUEVersion() < VER_UE4_ADDED_LANDSCAPE_SPLINE_EDITOR_MESH &&
			LocalMeshComponents.Num() == 0) // ForeignMeshComponents didn't exist yet
		{
			UpdateSplinePoints();
		}

		// Replace null meshes with the editor mesh
		// Also make sure that we update their visibility and collision profile if they are editor mesh
		ULandscapeSplinesComponent* OuterSplines = GetOuterULandscapeSplinesComponent();
		if (OuterSplines->SplineEditorMesh != nullptr)
		{
			for (auto& LocalMeshComponent : LocalMeshComponents)
			{
				if (LocalMeshComponent->GetStaticMesh() == nullptr || LocalMeshComponent->GetStaticMesh() == OuterSplines->SplineEditorMesh)
				{
					LocalMeshComponent->ConditionalPostLoad();
					
					if (LocalMeshComponent->GetStaticMesh() == nullptr)
					{
						LocalMeshComponent->SetStaticMesh(OuterSplines->SplineEditorMesh);
					}

					LocalMeshComponent->SetHiddenInGame(true);
					LocalMeshComponent->SetVisibility(OuterSplines->bShowSplineEditorMesh);
					LocalMeshComponent->BodyInstance.SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
				}
			}
		}

		for (auto& LocalMeshComponent : LocalMeshComponents)
		{
			OuterSplines->MeshComponentLocalOwnersMap.Add(LocalMeshComponent, this);
		}
	}

	if (GetLinkerUEVersion() < VER_UE4_LANDSCAPE_SPLINE_CROSS_LEVEL_MESHES)
	{
		// Fix collision profile
		for (auto& LocalMeshComponent : LocalMeshComponents) // ForeignMeshComponents didn't exist yet
		{
			UpdateMeshCollisionProfile(LocalMeshComponent);
			LocalMeshComponent->SetFlags(RF_TextExportTransient);
		}
	}

	if (GIsEditor  
		&& ((GetLinkerCustomVersion(FLandscapeCustomVersion::GUID) < FLandscapeCustomVersion::AddingBodyInstanceToSplinesElements)
			|| (GetLinkerCustomVersion(FFortniteReleaseBranchCustomObjectVersion::GUID) < FFortniteReleaseBranchCustomObjectVersion::RemoveUselessLandscapeMeshesCookedCollisionData)))
	{
		ULandscapeSplinesComponent* OuterSplines = GetOuterULandscapeSplinesComponent();

		auto ForeignMeshComponentsMap = GetForeignMeshComponents();

		for (auto& ForeignMeshComponentsPair : ForeignMeshComponentsMap)
		{
			for (auto* ForeignMeshComponent : ForeignMeshComponentsPair.Value)
			{
				const bool bUsingEditorMesh = OuterSplines->IsUsingEditorMesh(ForeignMeshComponent);

				if (!bUsingEditorMesh)
				{
					ForeignMeshComponent->BodyInstance = BodyInstance;
				}
				else
				{
					ForeignMeshComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
				}

				// We won't ever enable collisions when using the editor mesh, ensure we don't even cook or load any collision data on this mesh  :
				ForeignMeshComponent->SetbNeverNeedsCookedCollisionData(bUsingEditorMesh);
			}
		}

		for (auto& LocalMeshComponent : LocalMeshComponents)
		{
			if (LocalMeshComponent != nullptr)
			{
				const bool bUsingEditorMesh = OuterSplines->IsUsingEditorMesh(LocalMeshComponent);

				if (!bUsingEditorMesh)
				{
					LocalMeshComponent->BodyInstance = BodyInstance;
				}
				else
				{
					LocalMeshComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
				}

				// We won't ever enable collisions when using the editor mesh, ensure we don't even cook or load any collision data on this mesh  :
				LocalMeshComponent->SetbNeverNeedsCookedCollisionData(bUsingEditorMesh);
			}
		}
	}
#endif

	// If the "spline.blockall" cvar is on we might need to rebuild collision
	// for our spline meshes.
	if (SplinesAlwaysUseBlockAll)
	{
		for (auto& LocalMeshComponent : LocalMeshComponents)
		{
			UpdateMeshCollisionProfile(LocalMeshComponent);
			LocalMeshComponent->Modify();
		}
	}
}

void ULandscapeSplineSegment::UpdateMeshCollisionProfile(USplineMeshComponent* MeshComponent)
{
#if WITH_EDITORONLY_DATA
	ULandscapeSplinesComponent* OuterSplines = GetOuterChecked();
	const bool bUsingEditorMesh = OuterSplines->IsUsingEditorMesh(MeshComponent);

	const FName DesiredCollisionProfileName = SplinesAlwaysUseBlockAll ? UCollisionProfile::BlockAll_ProfileName : GetCollisionProfileName();
	const FName CollisionProfile = (bEnableCollision_DEPRECATED && !bUsingEditorMesh) ? DesiredCollisionProfileName : UCollisionProfile::NoCollision_ProfileName;

	// We won't ever enable collisions when using the editor mesh, ensure we don't even cook or load any collision data on this mesh  :
	MeshComponent->SetbNeverNeedsCookedCollisionData(bUsingEditorMesh);
#else
	const FName CollisionProfile = UCollisionProfile::BlockAll_ProfileName;
#endif
	if (MeshComponent->GetCollisionProfileName() != CollisionProfile)
	{
		MeshComponent->SetCollisionProfileName(CollisionProfile);
		MeshComponent->Modify();
	}
}

/**  */
#if WITH_EDITOR
void ULandscapeSplineSegment::SetSplineSelected(bool bInSelected)
{
	if (bInSelected == bSelected)
	{
		return;
	}

	bSelected = bInSelected;
	ULandscapeSplinesComponent* SplineComponent = GetOuterSafe();
	if (SplineComponent)
	{
		SplineComponent->MarkRenderStateDirty();
	}

	UpdateSplineMeshSelectionState();
}

void ULandscapeSplineSegment::AutoFlipTangents()
{
	FVector StartLocation; FRotator StartRotation;
	Connections[0].ControlPoint->GetConnectionLocationAndRotation(Connections[0].SocketName, StartLocation, StartRotation);
	FVector EndLocation; FRotator EndRotation;
	Connections[1].ControlPoint->GetConnectionLocationAndRotation(Connections[1].SocketName, EndLocation, EndRotation);

	// Flipping the tangent is only allowed if not using a socket
	if (Connections[0].SocketName == NAME_None && FVector::DotProduct((EndLocation - StartLocation).GetSafeNormal() * Connections[0].TangentLen, StartRotation.Vector()) < 0)
	{
		Connections[0].TangentLen = -Connections[0].TangentLen;
	}
	if (Connections[1].SocketName == NAME_None && FVector::DotProduct((StartLocation - EndLocation).GetSafeNormal() * Connections[1].TangentLen, EndRotation.Vector()) < 0)
	{
		Connections[1].TangentLen = -Connections[1].TangentLen;
	}
}
#endif

static float ApproxLength(const FInterpCurveVector& SplineInfo, const float Start = 0.0f, const float End = 1.0f, const int32 ApproxSections = 4)
{
	double SplineLength = 0;
	FVector OldPos = SplineInfo.Eval(Start, FVector::ZeroVector);
	for (int32 i = 1; i <= ApproxSections; i++)
	{
		FVector NewPos = SplineInfo.Eval(FMath::Lerp(Start, End, (float)i / (float)ApproxSections), FVector::ZeroVector);
		SplineLength += static_cast<float>((NewPos - OldPos).Size());
		OldPos = NewPos;
	}

	return static_cast<float>(SplineLength);
}

static ESplineMeshAxis::Type CrossAxis(ESplineMeshAxis::Type InForwardAxis, ESplineMeshAxis::Type InUpAxis)
{
	check(InForwardAxis != InUpAxis);
	return (ESplineMeshAxis::Type)(3 ^ InForwardAxis ^ InUpAxis);
}

bool FLandscapeSplineMeshEntry::IsValid() const
{
	return Mesh != nullptr && ForwardAxis != UpAxis && Scale.GetAbsMin() > KINDA_SMALL_NUMBER;
}

#if WITH_EDITOR
TMap<ULandscapeSplinesComponent*, TArray<USplineMeshComponent*>> ULandscapeSplineSegment::GetForeignMeshComponents()
{
	TMap<ULandscapeSplinesComponent*, TArray<USplineMeshComponent*>> ForeignMeshComponentsMap;

	ULandscapeSplinesComponent* OuterSplines = GetOuterSafe();
	if (OuterSplines)
	{
		TArray<ULandscapeSplinesComponent*> SplineComponents = OuterSplines->GetAllStreamingSplinesComponents();

		for (ULandscapeSplinesComponent* SplineComponent : SplineComponents)
		{
			if (SplineComponent != OuterSplines)
			{
				auto ForeignMeshComponents = SplineComponent->GetForeignMeshComponents(this);
				if (ForeignMeshComponents.Num() > 0)
				{
					for (auto* ForeignMeshComponent : ForeignMeshComponents)
					{
						ForeignMeshComponent->Modify(false);
					}
					ForeignMeshComponentsMap.Add(SplineComponent, MoveTemp(ForeignMeshComponents));
				}
			}
		}
	}

	return ForeignMeshComponentsMap;
}

TArray<USplineMeshComponent*> ULandscapeSplineSegment::GetLocalMeshComponents() const
{
	return LocalMeshComponents;
}

bool ULandscapeSplineSegment::SupportsForeignSplineMesh() const
{
	ULandscapeSplinesComponent* SplineComponent = GetOuterSafe();
	return bPlaceSplineMeshesInStreamingLevels && SplineComponent && SplineComponent->GetSplineOwner()->SupportsForeignSplineMesh();
}

void ULandscapeSplineSegment::UpdateSplinePoints(bool bUpdateCollision, bool bUpdateMeshLevel)
{
	Modify();

	ULandscapeSplinesComponent* OuterSplines = GetOuterChecked();

	SplineInfo.Points.Empty(2);
	Points.Reset();

	if (Connections[0].ControlPoint == nullptr
		|| Connections[1].ControlPoint == nullptr)
	{
		return;
	}

	// Set up BSpline
	FVector StartLocation; FRotator StartRotation;
	Connections[0].ControlPoint->GetConnectionLocationAndRotation(Connections[0].SocketName, StartLocation, StartRotation);
	SplineInfo.Points.Emplace(0.0f, StartLocation, StartRotation.Vector() * Connections[0].TangentLen, StartRotation.Vector() * Connections[0].TangentLen, CIM_CurveUser);
	FVector EndLocation; FRotator EndRotation;
	Connections[1].ControlPoint->GetConnectionLocationAndRotation(Connections[1].SocketName, EndLocation, EndRotation);
	SplineInfo.Points.Emplace(1.0f, EndLocation, EndRotation.Vector() * -Connections[1].TangentLen, EndRotation.Vector() * -Connections[1].TangentLen, CIM_CurveUser);

	// Pointify

	// Calculate spline length
	const float SplineLength = ApproxLength(SplineInfo, 0.0f, 1.0f, 4);

	const float StartFalloffFraction = ((Connections[0].ControlPoint->ConnectedSegments.Num() > 1) ? 0 : (Connections[0].ControlPoint->EndFalloff / SplineLength));
	const float EndFalloffFraction = ((Connections[1].ControlPoint->ConnectedSegments.Num() > 1) ? 0 : (Connections[1].ControlPoint->EndFalloff / SplineLength));
	const float StartWidth = Connections[0].ControlPoint->Width;
	const float EndWidth = Connections[1].ControlPoint->Width;
	const float StartLayerWidth = StartWidth * Connections[0].ControlPoint->LayerWidthRatio;
	const float EndLayerWidth = EndWidth * Connections[1].ControlPoint->LayerWidthRatio;

	LandscapeSplineRaster::FPointifyFalloffs Falloffs;
	Falloffs.StartLeftSide = Connections[0].ControlPoint->LeftSideFalloffFactor * Connections[0].ControlPoint->SideFalloff;
	Falloffs.EndLeftSide = Connections[1].ControlPoint->LeftSideFalloffFactor * Connections[1].ControlPoint->SideFalloff;
	Falloffs.StartRightSide = Connections[0].ControlPoint->RightSideFalloffFactor * Connections[0].ControlPoint->SideFalloff;
	Falloffs.EndRightSide = Connections[1].ControlPoint->RightSideFalloffFactor * Connections[1].ControlPoint->SideFalloff;
	Falloffs.StartLeftSideLayer = Connections[0].ControlPoint->LeftSideLayerFalloffFactor * Connections[0].ControlPoint->SideFalloff;
	Falloffs.EndLeftSideLayer = Connections[1].ControlPoint->LeftSideLayerFalloffFactor * Connections[1].ControlPoint->SideFalloff;
	Falloffs.StartRightSideLayer = Connections[0].ControlPoint->RightSideLayerFalloffFactor * Connections[0].ControlPoint->SideFalloff;
	Falloffs.EndRightSideLayer = Connections[1].ControlPoint->RightSideLayerFalloffFactor * Connections[1].ControlPoint->SideFalloff;
	const float StartRollDegrees = static_cast<float>(StartRotation.Roll * (Connections[0].TangentLen > 0 ? 1 : -1));
	const float EndRollDegrees = static_cast<float>(EndRotation.Roll * (Connections[1].TangentLen > 0 ? -1 : 1));
	const float StartRoll = FMath::DegreesToRadians(StartRollDegrees);
	const float EndRoll = FMath::DegreesToRadians(EndRollDegrees);
	const float StartMeshOffset = Connections[0].ControlPoint->SegmentMeshOffset;
	const float EndMeshOffset = Connections[1].ControlPoint->SegmentMeshOffset;

	int32 NumPoints = FMath::CeilToInt(SplineLength / OuterSplines->SplineResolution);
	NumPoints = FMath::Clamp(NumPoints, 1, 1000);

	LandscapeSplineRaster::Pointify(SplineInfo, Points, NumPoints, StartFalloffFraction, EndFalloffFraction, StartWidth, EndWidth, StartLayerWidth, EndLayerWidth, Falloffs, StartRollDegrees, EndRollDegrees);

	// Update Bounds
	Bounds = FBox(ForceInit);
	for (const FLandscapeSplineInterpPoint& Point : Points)
	{
		Bounds += Point.FalloffLeft;
		Bounds += Point.FalloffRight;
	}

	OuterSplines->MarkRenderStateDirty();

	// Spline mesh components
	TArray<const FLandscapeSplineMeshEntry*> UsableMeshes;
	UsableMeshes.Reserve(SplineMeshes.Num());
	for (const FLandscapeSplineMeshEntry& MeshEntry : SplineMeshes)
	{
		if (MeshEntry.IsValid())
		{
			UsableMeshes.Add(&MeshEntry);
		}
	}

	// Editor mesh
	bool bUsingEditorMesh = false;
	FLandscapeSplineMeshEntry SplineEditorMeshEntry;
	if (UsableMeshes.Num() == 0 && OuterSplines->SplineEditorMesh != nullptr)
	{
		SplineEditorMeshEntry.Mesh = OuterSplines->SplineEditorMesh;
		SplineEditorMeshEntry.MaterialOverrides = {};
		SplineEditorMeshEntry.bCenterH = true;
		SplineEditorMeshEntry.CenterAdjust = {0.0f, 0.5f};
		SplineEditorMeshEntry.bScaleToWidth = true;
		SplineEditorMeshEntry.Scale = {3, 1, 1};
		SplineEditorMeshEntry.ForwardAxis = ESplineMeshAxis::X;
		SplineEditorMeshEntry.UpAxis = ESplineMeshAxis::Z;
		UsableMeshes.Add(&SplineEditorMeshEntry);
		bUsingEditorMesh = true;
	}

	OuterSplines->Modify();

	TArray<USplineMeshComponent*> MeshComponents;

	TArray<TObjectPtr<USplineMeshComponent>> OldLocalMeshComponents;
	Swap(LocalMeshComponents, OldLocalMeshComponents);
	LocalMeshComponents.Reserve(20);

	// Gather all ForeignMesh associated with this Segment
	TMap<ULandscapeSplinesComponent*, TArray<USplineMeshComponent*>> ForeignMeshComponentsMap = GetForeignMeshComponents();
	
	// Unregister components, Remove Foreign/Local Associations
	for (TObjectPtr<USplineMeshComponent>& LocalMeshComponent : OldLocalMeshComponents)
	{
		USplineMeshComponent* Comp = LocalMeshComponent.Get();
		checkSlow(OuterSplines->MeshComponentLocalOwnersMap.FindRef(Comp) == this);
		verifySlow(OuterSplines->MeshComponentLocalOwnersMap.Remove(Comp) == 1);
		Comp->Modify();
		Comp->UnregisterComponent();
	}
	for (auto& ForeignMeshComponentsPair : ForeignMeshComponentsMap)
	{
		ForeignMeshComponentsPair.Key->Modify();
		ForeignMeshComponentsPair.Key->GetOwner()->Modify();
		ForeignMeshComponentsPair.Key->RemoveAllForeignMeshComponents(this);
		for (auto* ForeignMeshComponent : ForeignMeshComponentsPair.Value)
		{
			ForeignMeshComponent->Modify();
			ForeignMeshComponent->UnregisterComponent();
		}
	}

	ModificationKey = FGuid::NewGuid();
	ForeignWorlds.Reset();

	if (SplineLength > 0 && (StartWidth > 0 || EndWidth > 0) && UsableMeshes.Num() > 0)
	{
		float T = 0;
		int32 iMesh = 0;

		struct FMeshSettings
		{
			const float T;
			const FLandscapeSplineMeshEntry* const MeshEntry;

			FMeshSettings(float InT, const FLandscapeSplineMeshEntry* const InMeshEntry) :
				T(InT), MeshEntry(InMeshEntry) { }
		};
		TArray<FMeshSettings> MeshSettings;
		MeshSettings.Reserve(21);

		FRandomStream Random(RandomSeed);

		// First pass:
		// Choose meshes, create components, calculate lengths
		while (T < 1.0f && iMesh < 20) // Max 20 meshes per spline segment
		{
			const float CosInterp = 0.5f - 0.5f * FMath::Cos(T * PI);
			const float Width = FMath::Lerp(StartWidth, EndWidth, CosInterp);

			const FLandscapeSplineMeshEntry* MeshEntry = UsableMeshes[Random.RandHelper(UsableMeshes.Num())];
			UStaticMesh* Mesh = MeshEntry->Mesh;
			const FBoxSphereBounds MeshBounds = Mesh->GetBounds();

			FVector Scale = MeshEntry->Scale;
			if (MeshEntry->bScaleToWidth)
			{
				float ScaleFactor = Width / USplineMeshComponent::GetAxisValueRef(MeshBounds.BoxExtent, CrossAxis(MeshEntry->ForwardAxis, MeshEntry->UpAxis));
				if (MeshEntry->bNoZScaling)
				{
					Scale.X *= ScaleFactor;
					Scale.Y *= ScaleFactor;
				}
				else
				{
					Scale *= ScaleFactor;
				}
			}

			const float MeshLength = static_cast<float>(FMath::Abs(USplineMeshComponent::GetAxisValueRef(MeshBounds.BoxExtent, MeshEntry->ForwardAxis) * 2.0 *
				USplineMeshComponent::GetAxisValueRef(Scale, MeshEntry->ForwardAxis)));
			float MeshT = (MeshLength / SplineLength);

			// Improve our approximation if we're not going off the end of the spline
			if (T + MeshT <= 1.0f)
			{
				MeshT *= (MeshLength / ApproxLength(SplineInfo, T, T + MeshT, 4));
				MeshT *= (MeshLength / ApproxLength(SplineInfo, T, T + MeshT, 4));
			}

			// If it's smaller to round up than down, don't add another component
			if (iMesh != 0 && (1.0f - T) < (T + MeshT - 1.0f))
			{
				break;
			}

			ULandscapeSplinesComponent* MeshComponentOuterSplines = OuterSplines;

			// Attempt to place mesh components into the appropriate landscape streaming levels based on the components under the spline
			if (SupportsForeignSplineMesh() && !bUsingEditorMesh)
			{
				// Only "approx" because we rescale T for the 2nd pass based on how well our chosen meshes fit, but it should be good enough
				FVector ApproxMeshLocation = SplineInfo.Eval(T + MeshT / 2, FVector::ZeroVector);
				MeshComponentOuterSplines = OuterSplines->GetStreamingSplinesComponentByLocation(ApproxMeshLocation);
				MeshComponentOuterSplines->Modify();
			}

			USplineMeshComponent* MeshComponent = nullptr;
			if (MeshComponentOuterSplines == OuterSplines)
			{
				if (OldLocalMeshComponents.Num() > 0)
				{
					MeshComponent = OldLocalMeshComponents.Pop(EAllowShrinking::No).Get();
				}
			}
			else
			{
				TArray<USplineMeshComponent*>* ForeignMeshComponents = ForeignMeshComponentsMap.Find(MeshComponentOuterSplines);
				if (ForeignMeshComponents && ForeignMeshComponents->Num() > 0)
				{
					MeshComponentOuterSplines->UpdateModificationKey(this);
					MeshComponent = ForeignMeshComponents->Pop(EAllowShrinking::No);
				}
			}

			if (MeshComponent == nullptr)
			{
				AActor* MeshComponentOuterActor = MeshComponentOuterSplines->GetOwner();
				MeshComponentOuterActor->Modify();
				MeshComponent = NewObject<USplineMeshComponent>(MeshComponentOuterActor, NAME_None, RF_Transactional | RF_TextExportTransient);
				MeshComponent->bSelected = bSelected;
				MeshComponent->AttachToComponent(MeshComponentOuterSplines, FAttachmentTransformRules::KeepRelativeTransform);
			}
			else if (bUpdateMeshLevel)// Update Foreign/Local if necessary
			{
				AActor* CurrentMeshComponentOuterActor = MeshComponent->GetTypedOuter<AActor>();
				AActor* MeshComponentOuterActor = MeshComponentOuterSplines->GetOwner();
				ILandscapeSplineInterface* SplineOwner = Cast<ILandscapeSplineInterface>(MeshComponentOuterActor);

				// Needs updating
				if (MeshComponentOuterActor != CurrentMeshComponentOuterActor)
				{
					MeshComponentOuterActor->Modify();
					CurrentMeshComponentOuterActor->Modify();
							
					MeshComponent->Modify();
					MeshComponent->DetachFromComponent(FDetachmentTransformRules::KeepWorldTransform);
					MeshComponent->InvalidateLightingCache();
					MeshComponent->Rename(nullptr, MeshComponentOuterActor);
					MeshComponent->AttachToComponent(SplineOwner->GetSplinesComponent(), FAttachmentTransformRules::KeepWorldTransform);
				}
			}

			// New Foreign/Local Mapping
			if (MeshComponentOuterSplines == OuterSplines)
			{
				LocalMeshComponents.Add(MeshComponent);
				MeshComponentOuterSplines->MeshComponentLocalOwnersMap.Add(MeshComponent, this);
			}
			else
			{
				MeshComponentOuterSplines->AddForeignMeshComponent(this, MeshComponent);
				ForeignWorlds.AddUnique(MeshComponentOuterSplines->GetTypedOuter<UWorld>());
			}

			MeshComponents.Add(MeshComponent);

			MeshComponent->SetStaticMesh(Mesh);

			MeshComponent->OverrideMaterials = MeshEntry->MaterialOverrides;
			MeshComponent->MarkRenderStateDirty();
			if (MeshComponent->BodyInstance.IsValidBodyInstance())
			{
				MeshComponent->BodyInstance.UpdatePhysicalMaterials();
			}

			MeshComponent->SetHiddenInGame(bUsingEditorMesh || bHiddenInGame);
			MeshComponent->SetVisibility(!bUsingEditorMesh || OuterSplines->bShowSplineEditorMesh);

			MeshComponent->HLODLayerOverrideForSplinePartitioning = HLODLayerOverrideForSplinePartitioning;

			MeshSettings.Add(FMeshSettings(T, MeshEntry));
			iMesh++;
			T += MeshT;
		}
		// Add terminating key
		MeshSettings.Add(FMeshSettings(T, nullptr));

		// Second pass:
		// Rescale components to fit a whole number to the spline, set up final parameters
		const float Rescale = 1.0f / T;
		for (int32 i = 0; i < MeshComponents.Num(); i++)
		{
			USplineMeshComponent* const MeshComponent = MeshComponents[i];
			const UStaticMesh* const Mesh = MeshComponent->GetStaticMesh();
			const FBoxSphereBounds MeshBounds = Mesh->GetBounds();

			const float RescaledT = MeshSettings[i].T * Rescale;
			const FLandscapeSplineMeshEntry* MeshEntry = MeshSettings[i].MeshEntry;
			const ESplineMeshAxis::Type SideAxis = CrossAxis(MeshEntry->ForwardAxis, MeshEntry->UpAxis);

			const float TEnd = MeshSettings[i + 1].T * Rescale;

			const float CosInterp = 0.5f - 0.5f * FMath::Cos(RescaledT * PI);
			const float Width = FMath::Lerp(StartWidth, EndWidth, CosInterp);
			const bool bDoOrientationRoll = (MeshEntry->ForwardAxis == ESplineMeshAxis::X && MeshEntry->UpAxis == ESplineMeshAxis::Y) ||
			                                (MeshEntry->ForwardAxis == ESplineMeshAxis::Y && MeshEntry->UpAxis == ESplineMeshAxis::Z) ||
			                                (MeshEntry->ForwardAxis == ESplineMeshAxis::Z && MeshEntry->UpAxis == ESplineMeshAxis::X);
			const float Roll = FMath::Lerp(StartRoll, EndRoll, CosInterp) + (bDoOrientationRoll ? -HALF_PI : 0);
			const float MeshOffset = FMath::Lerp(StartMeshOffset, EndMeshOffset, CosInterp);

			FVector Scale = MeshEntry->Scale;
			if (MeshEntry->bScaleToWidth)
			{
				float ScaleFactor = Width / USplineMeshComponent::GetAxisValueRef(MeshBounds.BoxExtent, SideAxis);
				if (MeshEntry->bNoZScaling)
				{
					Scale.X *= ScaleFactor;
					Scale.Y *= ScaleFactor;
				}
				else
				{
					Scale *= ScaleFactor;
				}
			}

			FVector2D Offset = MeshEntry->CenterAdjust;
			if (MeshEntry->bCenterH)
			{
				if (bDoOrientationRoll)
				{
					Offset.Y -= USplineMeshComponent::GetAxisValueRef(MeshBounds.Origin, SideAxis);
				}
				else
				{
					Offset.X -= USplineMeshComponent::GetAxisValueRef(MeshBounds.Origin, SideAxis);
				}
			}

			FVector2D Scale2D;
			switch (MeshEntry->ForwardAxis)
			{
			case ESplineMeshAxis::X:
				Scale2D = FVector2D(Scale.Y, Scale.Z);
				break;
			case ESplineMeshAxis::Y:
				Scale2D = FVector2D(Scale.Z, Scale.X);
				break;
			case ESplineMeshAxis::Z:
				Scale2D = FVector2D(Scale.X, Scale.Y);
				break;
			default:
				check(0);
				break;
			}
			Offset *= Scale2D;
			Offset.Y += MeshOffset;
			Offset = Offset.GetRotated(-Roll);

			MeshComponent->SplineParams.StartPos = SplineInfo.Eval(RescaledT, FVector::ZeroVector);
			MeshComponent->SplineParams.StartTangent = SplineInfo.EvalDerivative(RescaledT, FVector::ZeroVector) * (TEnd - RescaledT);
			MeshComponent->SplineParams.StartScale = Scale2D;
			MeshComponent->SplineParams.StartRoll = Roll;
			MeshComponent->SplineParams.StartOffset = Offset;

			const float CosInterpEnd = 0.5f - 0.5f * FMath::Cos(TEnd * PI);
			const float WidthEnd = FMath::Lerp(StartWidth, EndWidth, CosInterpEnd);
			const float RollEnd = FMath::Lerp(StartRoll, EndRoll, CosInterpEnd) + (bDoOrientationRoll ? -HALF_PI : 0);
			const float MeshOffsetEnd = FMath::Lerp(StartMeshOffset, EndMeshOffset, CosInterpEnd);

			FVector ScaleEnd = MeshEntry->Scale;
			if (MeshEntry->bScaleToWidth)
			{
				float ScaleFactor = WidthEnd / USplineMeshComponent::GetAxisValueRef(MeshBounds.BoxExtent, SideAxis);
				if (MeshEntry->bNoZScaling)
				{
					ScaleEnd.X *= ScaleFactor;
					ScaleEnd.Y *= ScaleFactor;
				}
				else
				{
					ScaleEnd *= ScaleFactor;
				}
			}

			FVector2D OffsetEnd = MeshEntry->CenterAdjust;
			if (MeshEntry->bCenterH)
			{
				if (bDoOrientationRoll)
				{
					OffsetEnd.Y -= USplineMeshComponent::GetAxisValueRef(MeshBounds.Origin, SideAxis);
				}
				else
				{
					OffsetEnd.X -= USplineMeshComponent::GetAxisValueRef(MeshBounds.Origin, SideAxis);
				}
			}

			FVector2D Scale2DEnd;
			switch (MeshEntry->ForwardAxis)
			{
			case ESplineMeshAxis::X:
				Scale2DEnd = FVector2D(ScaleEnd.Y, ScaleEnd.Z);
				break;
			case ESplineMeshAxis::Y:
				Scale2DEnd = FVector2D(ScaleEnd.Z, ScaleEnd.X);
				break;
			case ESplineMeshAxis::Z:
				Scale2DEnd = FVector2D(ScaleEnd.X, ScaleEnd.Y);
				break;
			default:
				check(0);
				break;
			}
			OffsetEnd *= Scale2DEnd;
			OffsetEnd.Y += MeshOffsetEnd;
			OffsetEnd = OffsetEnd.GetRotated(-RollEnd);

			MeshComponent->SplineParams.EndPos = SplineInfo.Eval(TEnd, FVector::ZeroVector);
			MeshComponent->SplineParams.EndTangent = SplineInfo.EvalDerivative(TEnd, FVector::ZeroVector) * (TEnd - RescaledT);
			MeshComponent->SplineParams.EndScale = Scale2DEnd;
			MeshComponent->SplineParams.EndRoll = RollEnd;
			MeshComponent->SplineParams.EndOffset = OffsetEnd;

			MeshComponent->SplineUpDir = FVector(0,0,1); // Up, to be consistent between joined meshes. We rotate it to horizontal using roll if using Z Forward X Up or X Forward Y Up
			MeshComponent->ForwardAxis = MeshEntry->ForwardAxis;

			auto* const MeshComponentOuterSplines = MeshComponent->GetAttachParent();
			if (MeshComponentOuterSplines != nullptr && MeshComponentOuterSplines != OuterSplines)
			{
				const FTransform RelativeTransform = OuterSplines->GetComponentTransform().GetRelativeTransform(MeshComponentOuterSplines->GetComponentTransform());
				MeshComponent->SplineParams.StartPos = RelativeTransform.TransformPosition(MeshComponent->SplineParams.StartPos);
				MeshComponent->SplineParams.EndPos   = RelativeTransform.TransformPosition(MeshComponent->SplineParams.EndPos);
			}

			if (USplineMeshComponent::GetAxisValueRef(MeshEntry->Scale, MeshEntry->ForwardAxis) < 0)
			{
				Swap(MeshComponent->SplineParams.StartPos, MeshComponent->SplineParams.EndPos);
				Swap(MeshComponent->SplineParams.StartTangent, MeshComponent->SplineParams.EndTangent);
				Swap(MeshComponent->SplineParams.StartScale, MeshComponent->SplineParams.EndScale);
				Swap(MeshComponent->SplineParams.StartRoll, MeshComponent->SplineParams.EndRoll);
				Swap(MeshComponent->SplineParams.StartOffset, MeshComponent->SplineParams.EndOffset);

				MeshComponent->SplineParams.StartTangent = -MeshComponent->SplineParams.StartTangent;
				MeshComponent->SplineParams.EndTangent = -MeshComponent->SplineParams.EndTangent;
				MeshComponent->SplineParams.StartScale.X = -MeshComponent->SplineParams.StartScale.X;
				MeshComponent->SplineParams.EndScale.X = -MeshComponent->SplineParams.EndScale.X;
				MeshComponent->SplineParams.StartRoll = -MeshComponent->SplineParams.StartRoll;
				MeshComponent->SplineParams.EndRoll = -MeshComponent->SplineParams.EndRoll;
				MeshComponent->SplineParams.StartOffset.X = -MeshComponent->SplineParams.StartOffset.X;
				MeshComponent->SplineParams.EndOffset.X = -MeshComponent->SplineParams.EndOffset.X;
			}

			// Set Mesh component's location to half way between the start and end points. Improves the bounds and allows LDMaxDrawDistance to work
			MeshComponent->SetRelativeLocation_Direct((MeshComponent->SplineParams.StartPos + MeshComponent->SplineParams.EndPos) / 2);
			MeshComponent->SplineParams.StartPos -= MeshComponent->GetRelativeLocation();
			MeshComponent->SplineParams.EndPos -= MeshComponent->GetRelativeLocation();

			if (MeshComponent->LDMaxDrawDistance != LDMaxDrawDistance)
			{
				MeshComponent->LDMaxDrawDistance = LDMaxDrawDistance;
				MeshComponent->CachedMaxDrawDistance = 0;
			}
			MeshComponent->TranslucencySortPriority = TranslucencySortPriority;

			MeshComponent->RuntimeVirtualTextures = RuntimeVirtualTextures;
			MeshComponent->VirtualTextureLodBias = static_cast<int8>(VirtualTextureLodBias);
			MeshComponent->VirtualTextureCullMips = static_cast<int8>(VirtualTextureCullMips);
			MeshComponent->VirtualTextureMainPassMaxDrawDistance = VirtualTextureMainPassMaxDrawDistance;
			MeshComponent->VirtualTextureRenderPassType = VirtualTextureRenderPassType;

			MeshComponent->SetRenderCustomDepth(bRenderCustomDepth);
			MeshComponent->SetCustomDepthStencilWriteMask(CustomDepthStencilWriteMask);
 			MeshComponent->SetCustomDepthStencilValue(CustomDepthStencilValue);

			MeshComponent->SetCastShadow(bCastShadow);
			MeshComponent->InvalidateLightingCache();

#if WITH_EDITOR
			if (!bUsingEditorMesh)
			{
				MeshComponent->BodyInstance = BodyInstance;
			}
			else
			{
				MeshComponent->BodyInstance.SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
			}
			// If we won't ever enable collisions, ensure we don't even cook or load any collision data on this mesh :
			MeshComponent->SetbNeverNeedsCookedCollisionData(bUsingEditorMesh);

			if (bUpdateCollision)
			{
				MeshComponent->RecreateCollision();
			}
			else
			{
				if (MeshComponent->BodySetup)
				{
					MeshComponent->BodySetup->InvalidatePhysicsData();
					MeshComponent->BodySetup->AggGeom.EmptyElements();
				}
			}
#endif
		}

		// Finally, register components if the world is initialized
		UWorld* World = GetWorld();
		if (World && World->bIsWorldInitialized)
		{
			for (USplineMeshComponent* MeshComponent : MeshComponents)
			{
				MeshComponent->RegisterComponent();
			}
		}
	}
	
	// Clean up unused components
	for (TObjectPtr<USplineMeshComponent>& LocalMeshComponent : OldLocalMeshComponents)
	{
		LocalMeshComponent->DestroyComponent();
	}
	OldLocalMeshComponents.Empty();
	for (auto& ForeignMeshComponentsPair : ForeignMeshComponentsMap)
	{
		for (USplineMeshComponent* MeshComponent : ForeignMeshComponentsPair.Value)
		{
			MeshComponent->DestroyComponent();
		}
	}
	ForeignMeshComponentsMap.Empty();
}

void ULandscapeSplineSegment::UpdateSplineMeshSelectionState()
{
	for (auto& LocalMeshComponent : LocalMeshComponents)
	{
		if (LocalMeshComponent)
		{
			LocalMeshComponent->bSelected = bSelected;
			LocalMeshComponent->PushSelectionToProxy();
		}
	}

	auto ForeignMeshComponentsMap = GetForeignMeshComponents();
	for (auto& ForeignMeshComponentsPair : ForeignMeshComponentsMap)
	{
		ULandscapeSplinesComponent* MeshComponentOuterSplines = ForeignMeshComponentsPair.Key;
		for (auto* ForeignMeshComponent : ForeignMeshComponentsPair.Value)
		{
			ForeignMeshComponent->bSelected = bSelected;
			ForeignMeshComponent->PushSelectionToProxy();
		}
	}
}

void ULandscapeSplineSegment::UpdateSplineEditorMesh()
{
	ULandscapeSplinesComponent* OuterSplines = GetOuterChecked();

	for (auto& LocalMeshComponent : LocalMeshComponents)
	{
		if (OuterSplines->IsUsingEditorMesh(LocalMeshComponent))
		{
			LocalMeshComponent->SetVisibility(OuterSplines->bShowSplineEditorMesh);
		}
	}

	auto ForeignMeshComponentsMap = GetForeignMeshComponents();
	for (auto& ForeignMeshComponentsPair : ForeignMeshComponentsMap)
	{
		for (auto* ForeignMeshComponent : ForeignMeshComponentsPair.Value)
		{
			if (OuterSplines->IsUsingEditorMesh(ForeignMeshComponent))
			{
				ForeignMeshComponent->SetVisibility(OuterSplines->bShowSplineEditorMesh);
			}
		}
	}
}

void ULandscapeSplineSegment::DeleteSplinePoints()
{
	Modify();

	ULandscapeSplinesComponent* OuterSplines = GetOuterChecked();
	UE_LOGF(LogLandscape, Verbose, "ULandscapeSplineSegment::DeleteSplinePoints removing %ls from spline actor %ls", *GetName(), *OuterSplines->GetOwner()->GetActorNameOrLabel());

	SplineInfo.Reset();
	Points.Reset();
	Bounds = FBox(ForceInit);

	// Make this segment transient to prevent duplicate from picking the disconnected, but un-garbage-collected
	// instance and exporting it.  Otherwise the non-standard code in PostEditImport will re-attach the segment to the
	// component and it will re-appear as part of the copy.
	Rename(nullptr, GetTransientPackage());

	OuterSplines->MarkRenderStateDirty();

	// Destroy mesh components
	if (LocalMeshComponents.Num() > 0)
	{
		OuterSplines->Modify();
		for (auto& LocalMeshComponent : LocalMeshComponents)
		{
			checkSlow(OuterSplines->MeshComponentLocalOwnersMap.FindRef(LocalMeshComponent) == this);
			verifySlow(OuterSplines->MeshComponentLocalOwnersMap.Remove(LocalMeshComponent) == 1);
			LocalMeshComponent->Modify();
			LocalMeshComponent->DestroyComponent();
		}
		LocalMeshComponents.Empty();
	}

	auto ForeignMeshComponentsMap = GetForeignMeshComponents();
	for (auto& ForeignMeshComponentsPair : ForeignMeshComponentsMap)
	{
		ULandscapeSplinesComponent* MeshComponentOuterSplines = ForeignMeshComponentsPair.Key;
		MeshComponentOuterSplines->Modify();
		MeshComponentOuterSplines->GetOwner()->Modify();
		for (auto* ForeignMeshComponent : ForeignMeshComponentsPair.Value)
		{
			ForeignMeshComponent->Modify();
			MeshComponentOuterSplines->RemoveForeignMeshComponent(this, ForeignMeshComponent);
			ForeignMeshComponent->DestroyComponent();
		}
	}

	ModificationKey.Invalidate();
	ForeignWorlds.Empty();
}

FName ULandscapeSplineSegment::GetCollisionProfileName() const
{
#if WITH_EDITORONLY_DATA
	return BodyInstance.GetCollisionProfileName();
#else
	return UCollisionProfile::BlockAll_ProfileName;
#endif
}

#endif

void ULandscapeSplineSegment::FindNearest( const FVector& InLocation, float& t, FVector& OutLocation, FVector& OutTangent )
{
	float TempOutDistanceSq;
	t = SplineInfo.FindNearest(InLocation, TempOutDistanceSq);
	OutLocation = SplineInfo.Eval(t, FVector::ZeroVector);
	OutTangent = SplineInfo.EvalDerivative(t, FVector::ZeroVector);
}

FFloatInterval ULandscapeSplineSegment::GetKeyInterval() const
{
	return SplineInfo.GetKeyInterval();
}

FVector ULandscapeSplineSegment::Evaluate(const float Key, const FVector& Default) const
{
	return SplineInfo.Eval(Key, Default);
}

#if WITH_EDITOR
void ULandscapeSplineSegment::PostEditUndo()
{
	bHackIsUndoingSplines = true;
	Super::PostEditUndo();
	bHackIsUndoingSplines = false;

	// Update mesh components to make sure the selection state stays in sync
	UpdateSplineMeshSelectionState();

	ULandscapeSplinesComponent* SplineComponent = GetOuterSafe();
	if (SplineComponent)
	{
		SplineComponent->MarkRenderStateDirty();
		SplineComponent->RequestSplineLayerUpdate();
	}
}

void ULandscapeSplineSegment::PostDuplicate(bool bDuplicateForPIE)
{
	if (!bDuplicateForPIE)
	{
		// if we get duplicated but our local meshes don't, then clear our reference to the meshes - they're not ours
		if (LocalMeshComponents.Num() > 0)
		{
			ULandscapeSplinesComponent* OuterSplines = GetOuterChecked();

			// we assume all meshes are duplicated or none are, to avoid testing every one
			if (LocalMeshComponents[0]->GetOuter() != OuterSplines->GetOwner())
			{
				LocalMeshComponents.Empty();
			}

			// If LocalMeshComponents are still valid make sure they are added to the transient map that normally gets populated PostLoad
			for (auto& LocalMeshComponent : LocalMeshComponents)
			{
				OuterSplines->MeshComponentLocalOwnersMap.Add(LocalMeshComponent, this);
			}
		}
		
		UpdateSplinePoints();
	}

	Super::PostDuplicate(bDuplicateForPIE);
}

void ULandscapeSplineSegment::PostEditImport()
{
	Super::PostEditImport();

	GetOuterULandscapeSplinesComponent()->Segments.AddUnique(this);

	if (Connections[0].ControlPoint != nullptr)
	{
		Connections[0].ControlPoint->ConnectedSegments.AddUnique(FLandscapeSplineConnection(this, 0));
		Connections[1].ControlPoint->ConnectedSegments.AddUnique(FLandscapeSplineConnection(this, 1));
	}
}

void ULandscapeSplineSegment::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Flipping the tangent is only allowed if not using a socket
	if (Connections[0].SocketName != NAME_None)
	{
		Connections[0].TangentLen = FMath::Abs(Connections[0].TangentLen);
	}
	if (Connections[1].SocketName != NAME_None)
	{
		Connections[1].TangentLen = FMath::Abs(Connections[1].TangentLen);
	}

	// Don't update splines when undoing, not only is it unnecessary and expensive,
	// it also causes failed asserts in debug builds when trying to register components
	// (because the actor hasn't reset its OwnedComponents array yet)
	if (!bHackIsUndoingSplines)
	{
		const bool bUpdateCollision = PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive;
		UpdateSplinePoints(bUpdateCollision);
	}

	if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet)
	{
		GetOuterChecked()->RequestSplineLayerUpdate();
	}
}

ULandscapeSplinesComponent* ULandscapeSplineSegment::GetOuterSafe() const
{
	return !HasAnyFlags(RF_ClassDefaultObject) ? Cast<ULandscapeSplinesComponent>(GetOuter()) : nullptr;
}

ULandscapeSplinesComponent* ULandscapeSplineSegment::GetOuterChecked() const
{
	check(!HasAnyFlags(RF_ClassDefaultObject));
	return CastChecked<ULandscapeSplinesComponent>(GetOuter());
}

ULandscapeSplinesComponent* ULandscapeSplineControlPoint::GetOuterSafe() const
{
	return !HasAnyFlags(RF_ClassDefaultObject) ? Cast<ULandscapeSplinesComponent>(GetOuter()) : nullptr;
}

ULandscapeSplinesComponent* ULandscapeSplineControlPoint::GetOuterChecked() const
{
	check(!HasAnyFlags(RF_ClassDefaultObject));
	return CastChecked<ULandscapeSplinesComponent>(GetOuter());
}

void ALandscapeProxy::CreateSplineComponent()
{
	CreateSplineComponent(FVector(1.0f) / GetRootComponent()->GetRelativeScale3D());
}

void ALandscapeProxy::CreateSplineComponent(const FVector& Scale3D)
{
	check(SplineComponent == nullptr);
	Modify();
	SplineComponent = NewObject<ULandscapeSplinesComponent>(this, NAME_None, RF_Transactional);
	SplineComponent->SetRelativeScale3D_Direct(Scale3D);
	SplineComponent->AttachToComponent(GetRootComponent(), FAttachmentTransformRules::KeepRelativeTransform);
	SplineComponent->ShowSplineEditorMesh(true);

	if (RootComponent && RootComponent->IsRegistered())
	{
		SplineComponent->RegisterComponent();
	}
}

void ULandscapeInfo::MoveControlPoint(ULandscapeSplineControlPoint* InControlPoint,  TScriptInterface<ILandscapeSplineInterface> From, TScriptInterface<ILandscapeSplineInterface> To)
{
	ULandscapeSplinesComponent* ToSplineComponent = To->GetSplinesComponent();
	ULandscapeSplinesComponent* FromSplineComponent = From->GetSplinesComponent();
	if (ToSplineComponent == nullptr)
	{
		To->CreateSplineComponent(FromSplineComponent->GetRelativeScale3D());
		ToSplineComponent = To->GetSplinesComponent();
		check(ToSplineComponent);
	}

	if (InControlPoint->GetOuterULandscapeSplinesComponent() == ToSplineComponent)
	{
		return;
	}
	check(InControlPoint->GetOuterULandscapeSplinesComponent() == FromSplineComponent);

	UE_LOGF(LogLandscape, Display, "ULandscapeInfo::MoveControlPoint:  move %ls, from %ls, to %ls.",
		*InControlPoint->GetName(), *FromSplineComponent->GetOwner()->GetActorNameOrLabel(), *ToSplineComponent->GetOwner()->GetActorNameOrLabel());

	ToSplineComponent->Modify();
	InControlPoint->Modify();

	const FTransform FromComponentTransform = FromSplineComponent->GetComponentTransform();	// (From-->World)
	const FTransform ToComponentTransform = ToSplineComponent->GetComponentTransform();		// (To-->World)

	const FTransform OldToNewTransform =		// (From-->To) == (From-->World) * (World-->To)  == From * Inverse(To)
		FromComponentTransform.GetRelativeTransform(ToComponentTransform);
		
	// Delete all Mesh Components associated with the ControlPoint. (Will get recreated in UpdateSplinePoints)
	if (InControlPoint->LocalMeshComponent)
	{
		InControlPoint->LocalMeshComponent->Modify();
		InControlPoint->LocalMeshComponent->UnregisterComponent();
		InControlPoint->LocalMeshComponent->DestroyComponent();
		FromSplineComponent->Modify();
		FromSplineComponent->MeshComponentLocalOwnersMap.Remove(InControlPoint->LocalMeshComponent);
		InControlPoint->LocalMeshComponent = nullptr;
	}

	TMap<ULandscapeSplinesComponent*, UControlPointMeshComponent*> ForeignMeshComponents = InControlPoint->GetForeignMeshComponents();
	for (auto Pair : ForeignMeshComponents)
	{
		Pair.Key->RemoveForeignMeshComponent(InControlPoint, Pair.Value);
		Pair.Value->Modify();
		Pair.Value->UnregisterComponent();
		Pair.Value->DestroyComponent();
	}

	// Move control point to new level
	FromSplineComponent->ControlPoints.Remove(InControlPoint);
	InControlPoint->Rename(nullptr, ToSplineComponent);
	ToSplineComponent->ControlPoints.Add(InControlPoint);

	InControlPoint->Location = OldToNewTransform.TransformPosition(InControlPoint->Location);
	InControlPoint->Rotation = FRotator(OldToNewTransform.TransformRotation(InControlPoint->Rotation.Quaternion()));

	const bool bUpdateCollision = true; // default value
	const bool bUpdateSegments = false; // done in next loop
	const bool bUpdateMeshLevel = false; // no need because mesh have been deleted
	InControlPoint->UpdateSplinePoints(bUpdateCollision, bUpdateSegments, bUpdateMeshLevel);

	// Continue
	for (const FLandscapeSplineConnection& Connection : InControlPoint->ConnectedSegments)
	{
		// Continue both directions anyways it will early out for ControlPoints that already have moved
		MoveSegment(Connection.Segment, From, To);
	}
}

void ULandscapeInfo::MoveSegment(ULandscapeSplineSegment* InSegment, TScriptInterface<ILandscapeSplineInterface> From, TScriptInterface<ILandscapeSplineInterface> To)
{
	AActor* ToActor = CastChecked<AActor>(To.GetObject());
	ToActor->Modify();

	AActor* FromActor = CastChecked<AActor>(From.GetObject());
	ULandscapeSplinesComponent* ToSplineComponent = To->GetSplinesComponent();
	ULandscapeSplinesComponent* FromSplineComponent = From->GetSplinesComponent();
	
	if (ToSplineComponent == nullptr)
	{
		To->CreateSplineComponent(FromSplineComponent->GetRelativeScale3D());
		ToSplineComponent = To->GetSplinesComponent();
		check(ToSplineComponent);
	}
		
	if (InSegment->GetOuterULandscapeSplinesComponent() == ToSplineComponent)
	{
		return;
	}
	check(InSegment->GetOuterULandscapeSplinesComponent() == FromSplineComponent);

	UE_LOGF(LogLandscape, Display, "ULandscapeInfo::MoveSegment:  move %ls, from %ls, to %ls.",
		*InSegment->GetName(), *FromSplineComponent->GetOwner()->GetActorNameOrLabel(), *ToSplineComponent->GetOwner()->GetActorNameOrLabel());

	ToSplineComponent->Modify();
	InSegment->Modify();
		
	// Delete all Mesh Components associated with the Segment. (Will get recreated in UpdateSplinePoints)
	for (auto& MeshComponent : InSegment->LocalMeshComponents)
	{
		MeshComponent->Modify();
		MeshComponent->UnregisterComponent();
		MeshComponent->DestroyComponent();
		FromActor->Modify();
		FromSplineComponent->MeshComponentLocalOwnersMap.Remove(MeshComponent);
	}
	InSegment->LocalMeshComponents.Empty();

	TMap<ULandscapeSplinesComponent*, TArray<USplineMeshComponent*>> ForeignMeshComponents = InSegment->GetForeignMeshComponents();
	for (auto Pair : ForeignMeshComponents)
	{
		Pair.Key->RemoveAllForeignMeshComponents(InSegment);
		for (auto MeshComponent : Pair.Value)
		{
			MeshComponent->Modify();
			MeshComponent->UnregisterComponent();
			MeshComponent->DestroyComponent();
		}
	}

	// Move segment to new level
	FromSplineComponent->Segments.Remove(InSegment);
	InSegment->Rename(nullptr, ToSplineComponent);
	ToSplineComponent->Segments.Add(InSegment);
		
	// Continue both directions anyways it will early out for ControlPoints that already have moved
	MoveControlPoint(InSegment->Connections[0].ControlPoint, From, To);
	MoveControlPoint(InSegment->Connections[1].ControlPoint, From, To);

	const bool bUpdateCollision = true; // default value
	const bool bUpdateMeshLevel = false; // no need because mesh have been deleted 
	InSegment->UpdateSplinePoints(bUpdateCollision, bUpdateMeshLevel);
}

void ULandscapeInfo::MoveSplineToLevel(ULandscapeSplineControlPoint* InControlPoint, ULevel* TargetLevel)
{
	ALandscapeProxy* FromProxy = InControlPoint->GetTypedOuter<ALandscapeProxy>();
	if (FromProxy->GetLevel() == TargetLevel)
	{
		return;
	}

	ALandscapeProxy* ToProxy = GetLandscapeProxyForLevel(TargetLevel);
	if (!ToProxy)
	{
		return;
	}
	
	MoveSplineToProxy(InControlPoint, ToProxy);
}

void ULandscapeInfo::MoveSplineToProxy(ULandscapeSplineControlPoint* InControlPoint, ALandscapeProxy* InLandscapeProxy)
{
	MoveSpline(InControlPoint, InLandscapeProxy);
}

void ULandscapeInfo::MoveSpline(ULandscapeSplineControlPoint* InControlPoint, TScriptInterface<ILandscapeSplineInterface> InNewOwner)
{
	bool bUpdateBounds = false;

	ULandscapeSplinesComponent* FromSplineComponent = InControlPoint->GetOuterChecked();
	TScriptInterface<ILandscapeSplineInterface> From(FromSplineComponent->GetOwner());
	if (From == InNewOwner)
	{
		return;
	}

	UE_LOGF(LogLandscape, Display, "ULandscapeInfo::MoveSpline - ControlPoint %ls. Deleting actor %ls. Merging spline data to %ls.",
		*InControlPoint->GetName(), *FromSplineComponent->GetOwner()->GetActorNameOrLabel(), *Cast<AActor>(InNewOwner.GetObject())->GetActorNameOrLabel());

	bUpdateBounds = true;
	From.GetObject()->Modify();
	FromSplineComponent->Modify();
	FromSplineComponent->MarkRenderStateDirty();
	MoveControlPoint(InControlPoint, From, InNewOwner);
	FromSplineComponent->UpdateBounds();
	
	if (bUpdateBounds)
	{
		InNewOwner->GetSplinesComponent()->UpdateBounds();
	}
}

void ULandscapeInfo::MoveSplinesToLevel(ULandscapeSplinesComponent* InSplineComponent, ULevel * TargetLevel)
{
	ALandscapeProxy* FromProxy = InSplineComponent->GetTypedOuter<ALandscapeProxy>();
	if (FromProxy->GetLevel() == TargetLevel)
	{
		return;
	}

	ALandscapeProxy* ToProxy = GetLandscapeProxyForLevel(TargetLevel);
	if (!ToProxy)
	{
		return;
	}
	
	MoveSplines(InSplineComponent, ToProxy);
}

/** Moves all Splines to target Proxy. Creates ULandscapeSplineComponent if needed */
void ULandscapeInfo::MoveSplinesToProxy(ULandscapeSplinesComponent* InSplineComponent, ALandscapeProxy* InLandscapeProxy)
{
	MoveSplines(InSplineComponent, InLandscapeProxy);

	check(InSplineComponent->ControlPoints.Num() == 0 && InSplineComponent->Segments.Num() == 0);
}


/** Moves all Splines to target Spline owner */
void ULandscapeInfo::MoveSplines(ULandscapeSplinesComponent* InSplineComponent, TScriptInterface<ILandscapeSplineInterface> InNewOwner)
{
	check(InSplineComponent != InNewOwner->GetSplinesComponent());

	InSplineComponent->ForEachControlPoint([this, InNewOwner](ULandscapeSplineControlPoint* ControlPoint)
	{
		// Even if ControlPoints are part of same connected spline they will be skipped if already moved
		MoveSpline(ControlPoint, InNewOwner);
	});
}
#endif // WITH_EDITOR


#undef LOCTEXT_NAMESPACE
