// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeSubsystem.h"
#include "Engine/Engine.h"
#include "UObject/UObjectGlobals.h"
#include "Engine/EngineBaseTypes.h"
#include "Engine/World.h"
#include "Modules/ModuleManager.h"
#include "ContentStreaming.h"
#include "HAL/IConsoleManager.h"
#include "Landscape.h"
#include "LandscapeEditTypes.h"
#include "LandscapeProxy.h"
#include "LandscapeStreamingProxy.h"
#include "LandscapeInfo.h"
#include "LandscapeInfoMap.h"
#include "LandscapeModule.h"
#include "LandscapeRender.h"
#include "LandscapePrivate.h"
#include "LandscapeSettings.h"
#include "LandscapeGrassMapsBuilder.h"
#include "LandscapeTextureStorageProvider.h"
#include "LandscapeComponent.h"
#include "LandscapeGroup.h"
#include "Logging/StructuredLog.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "ActorPartition/ActorPartitionSubsystem.h"
#include "Engine/World.h"
#include "Math/IntRect.h"
#include "LandscapeNotification.h"
#include "LandscapeConfigHelper.h"
#include "Engine/Canvas.h"
#include "EngineUtils.h"
#include "Misc/ScopedSlowTask.h"
#include "Algo/ForEach.h"
#include "Algo/Transform.h"
#include "Algo/RemoveIf.h"
#include "Algo/Unique.h"
#include "Algo/Partition.h"
#include "Misc/App.h"
#include "Misc/DateTime.h"
#include "AssetCompilingManager.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapeSubsystem)

// enables debug spew
//#define ENABLE_LANDSCAPE_SUBSYSTEM_DEBUG_SPEW 1
#ifdef ENABLE_LANDSCAPE_SUBSYSTEM_DEBUG_SPEW
#define SUBSYSTEM_DEBUG_LOG(...) UE_LOG(LogLandscape, Warning, __VA_ARGS__)
#define SUBSYSTEM_DEBUG_LOG_REGISTER(...) UE_LOG(LogLandscape, Warning, __VA_ARGS__)
#else
#define SUBSYSTEM_DEBUG_LOG(...) UE_LOG(LogLandscape, Verbose, __VA_ARGS__)
#define SUBSYSTEM_DEBUG_LOG_REGISTER(...) do {} while(0)
#endif // ENABLE_LANDSCAPE_EDGE_FIXUP_DEBUG_SPEW

#if WITH_EDITOR
#include "ActionableMessageSubsystem.h"
#include "Async/ParallelFor.h"
#include "FileHelpers.h"
#include "Editor.h"
#endif

static int32 GUseStreamingManagerForCameras = 1;
static FAutoConsoleVariableRef CVarUseStreamingManagerForCameras(
	TEXT("grass.UseStreamingManagerForCameras"),
	GUseStreamingManagerForCameras,
	TEXT("1: Use Streaming Manager; 0: Use ViewLocationsRenderedLastFrame"));

static TAutoConsoleVariable<bool> CVarGrassVisualLogCameraLocations(
	TEXT("grass.VisualLog.CameraLocations"),
	false,
	TEXT("Allows to visualize the cameras used for landscape grass in the visual logger "));

static FAutoConsoleVariable CVarMaxAsyncNaniteProxiesPerSecond(
	TEXT("landscape.Nanite.MaxAsyncProxyBuildsPerSecond"),
	6.0f,
	TEXT("Number of Async nanite proxies to dispatch per second"));

int32 LiveRebuildNaniteOnModification = 0;
static FAutoConsoleVariableRef CVarLiveRebuildNaniteOnModification(
	TEXT("landscape.Nanite.LiveRebuildOnModification"),
	LiveRebuildNaniteOnModification,
	TEXT("Trigger a rebuild of Nanite representation immediately when a modification is performed (World Partition Maps Only)"));

int32 LandscapeMultithreadNaniteBuild = 1;
static FAutoConsoleVariableRef CVarLandscapeMultithreadNaniteBuild(
	TEXT("landscape.Nanite.MultithreadBuild"),
	LandscapeMultithreadNaniteBuild,
	TEXT("Multithread nanite landscape build in (World Partition Maps Only)"));

int32 LandscapeMaxSimultaneousMultithreadNaniteBuilds = -1;
static FAutoConsoleVariableRef CVarLandscapeMaxSimultaneousMultithreadNaniteBuilds(
	TEXT("landscape.Nanite.MaxSimultaneousMultithreadBuilds"),
	LandscapeMaxSimultaneousMultithreadNaniteBuilds,
	TEXT("Max number of simultaneous Nanite static mesh tasks (-1 = unlimited )"));

extern int32 GGrassEnable;
extern int32 GGrassMapUseRuntimeGeneration;

DECLARE_CYCLE_STAT(TEXT("LandscapeSubsystem Tick"), STAT_LandscapeSubsystemTick, STATGROUP_Landscape);

#define LOCTEXT_NAMESPACE "LandscapeSubsystem"

namespace UE::Landscape
{
	static void DumpLandscapeWeightmapAllocations(const TArray<FString>& Args)
	{
		bool bDumpDetails = false;

		for (const FString& Arg : Args)
		{
			if (FParse::Param(*Arg, TEXT("details")))
			{
				bDumpDetails = true;
			}
		}

		// Command supports editor and cooked games, so get worlds from Engine instead of Editor
		for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
		{
			if (UWorld* World = WorldContext.World())
			{
				UE_LOGF(LogLandscape, Log, "In World: %ls", *World->GetFullName());

				if (ULandscapeSubsystem* LandscapeSubsystem = World->GetSubsystem<ULandscapeSubsystem>())
				{
					LandscapeSubsystem->ForEachLandscapeInfo([&](ULandscapeInfo* LandscapeInfo)
					{
						TMap<FName, int32> TargetLayerToNumComponents;

						const FString& LandscapeName = LandscapeInfo->GetLandscapeProxy()->GetActorNameOrLabel();
						UE_LOGF(LogLandscape, Log, "- Landscape: %ls", *LandscapeName);

						LandscapeInfo->ForEachLandscapeProxy([&TargetLayerToNumComponents, &bDumpDetails, &LandscapeName](ALandscapeProxy* LandscapeProxy)
						{
							for (ULandscapeComponent* Component : LandscapeProxy->LandscapeComponents)
							{
								TSet<FName> ComponentLayerNameSet;
								for (const FWeightmapLayerAllocationInfo& AllocInfo : Component->GetWeightmapLayerAllocations())
								{
									if (AllocInfo.IsAllocated())
									{
										check(AllocInfo.LayerInfo != nullptr);
										FName LayerName = AllocInfo.LayerInfo->GetLayerName();
										TargetLayerToNumComponents.FindOrAdd(LayerName)++;
										ComponentLayerNameSet.Add(LayerName);
									}
								}

								// Detailed print option
								if (bDumpDetails)
								{
									const FString& ComponentName = Component->GetName();
									const FString& ProxyName = LandscapeProxy->GetActorNameOrLabel();
									UE_LOGF(LogLandscape, Log, "	Proxy: %ls, Component: %ls [%ls]", *ProxyName, *ComponentName, *Component->GetSectionBase().ToString());
									UE_LOGF(LogLandscape, Log, "	- Target Layers: %d", ComponentLayerNameSet.Num());

									ComponentLayerNameSet.Sort(FNameLexicalLess());

									for (const FName& LayerName : ComponentLayerNameSet)
									{
										UE_LOGF(LogLandscape, Log, "	- %ls", *LayerName.ToString());
									}
								}
							}
							return true;
						});

						// Print the total layers and their names
						UE_LOGF(LogLandscape, Log, "Number of unique Target Layers: %d", TargetLayerToNumComponents.Num());

						TargetLayerToNumComponents.KeySort(FNameLexicalLess());

						for (const TPair<FName, int32>& Pair : TargetLayerToNumComponents)
						{
							UE_LOGF(LogLandscape, Log, "- %ls, used in %d landscape components", *Pair.Key.ToString(), Pair.Value);
						}

						return true;
					});
				}
			}
		}
	}

	static FAutoConsoleCommand CmdDumpLandscapeWeightmapAllocations(
		TEXT("landscape.DumpTargetLayerAllocations"),
		TEXT("[optional: -details] - Dumps a report of target layers allocated for every landscape. \n"
			"-details shows a detailed report of allocated target layers for each individual landscape component. \n"),
		FConsoleCommandWithArgsDelegate::CreateStatic(&DumpLandscapeWeightmapAllocations)
		);

#if WITH_EDITOR
	bool HasModifiedLandscapes()
	{
		if (GEditor)
		{
			if (UWorld* World = GEditor->GetEditorWorldContext().World())
			{
				if (ULandscapeSubsystem* LandscapeSubsystem = World->GetSubsystem<ULandscapeSubsystem>())
				{
					return LandscapeSubsystem->HasModifiedLandscapes();
				}
			}
		}

		return false;
	}

	void SaveModifiedLandscapes(UE::Landscape::EBuildFlags InBuildFlags)
	{
		if (GEditor)
		{
			if (UWorld* World = GEditor->GetEditorWorldContext().World())
			{
				if (ULandscapeSubsystem* LandscapeSubsystem = World->GetSubsystem<ULandscapeSubsystem>())
				{
					LandscapeSubsystem->SaveModifiedLandscapes(InBuildFlags);
				}
			}
		}
	}

	void MarkModifiedLandscapesAsDirty(EBuildFlags InBuildFlags)
	{
		if (GEditor)
		{
			if (UWorld* World = GEditor->GetEditorWorldContext().World())
			{
				if (ULandscapeSubsystem* LandscapeSubsystem = World->GetSubsystem<ULandscapeSubsystem>())
				{
					LandscapeSubsystem->MarkModifiedLandscapesAsDirty(InBuildFlags);
				}
			}
		}
	}

	void BuildGrassMaps(EBuildFlags InBuildFlags)
	{
		if (GEditor)
		{
			if (UWorld* World = GEditor->GetEditorWorldContext().World())
			{
				if (ULandscapeSubsystem* LandscapeSubsystem = World->GetSubsystem<ULandscapeSubsystem>())
				{
					LandscapeSubsystem->BuildGrassMaps(InBuildFlags);
				}
			}
		}
	}

	void BuildPhysicalMaterial(EBuildFlags InBuildFlags)
	{
		if (GEditor)
		{
			if (UWorld* World = GEditor->GetEditorWorldContext().World())
			{
				if (ULandscapeSubsystem* LandscapeSubsystem = World->GetSubsystem<ULandscapeSubsystem>())
				{
					LandscapeSubsystem->BuildPhysicalMaterial(InBuildFlags);
				}
			}
		}
	}

	void BuildNanite(EBuildFlags InBuildFlags)
	{
		if (GEditor)
		{
			if (UWorld* World = GEditor->GetEditorWorldContext().World())
			{
				if (ULandscapeSubsystem* LandscapeSubsystem = World->GetSubsystem<ULandscapeSubsystem>())
				{
					LandscapeSubsystem->BuildNanite(InBuildFlags);
				}
			}
		}
	}

	void BuildAll(EBuildFlags InBuildFlags)
	{
		if (GEditor)
		{
			if (UWorld* World = GEditor->GetEditorWorldContext().World())
			{
				if (ULandscapeSubsystem* LandscapeSubsystem = World->GetSubsystem<ULandscapeSubsystem>())
				{
					LandscapeSubsystem->BuildAll(InBuildFlags);
				}
			}
		}
	}

	static FAutoConsoleCommand CmdLandscapeRebuildPhysicalMaterial(
		TEXT("landscape.RebuildPhysicalMaterial"),
		TEXT("Force a rebuild of the physical material data for all landscapes."),
		FConsoleCommandDelegate::CreateStatic(BuildPhysicalMaterial, EBuildFlags::ForceRebuild)
	);

#endif //WITH_EDITOR
}

ULandscapeSubsystem::ULandscapeSubsystem()
{
}

ULandscapeSubsystem::~ULandscapeSubsystem()
{
}

void ULandscapeSubsystem::RegisterActor(ALandscapeProxy* Proxy)
{
	check(Proxy != nullptr);
	SUBSYSTEM_DEBUG_LOG_REGISTER(TEXT("ULandscapeSubsystem::RegisterActor %p %s (%s) to subsystem %p in world %p (%s)"),
		Proxy, *Proxy->GetFullName(), *Proxy->GetClass()->GetName(), this, GetWorld(), *GetWorld()->GetName());

 	TObjectPtr<ALandscapeProxy> ProxyPtr(Proxy);

	// in editor we can get multiple registration calls, so ensure we don't register more than once
	if (Proxy->bIsRegisteredWithSubsystem)
	{
		check(Proxy->RegisteredToSubsystem == this);
		check(Proxies.Contains(ProxyPtr));
		return;
	}
	check(Proxy->RegisteredToSubsystem == nullptr);
	check(!Proxies.Contains(ProxyPtr));
	Proxies.Add(ProxyPtr);
	
	if (ALandscape* LandscapeActor = Cast<ALandscape>(Proxy))
	{
		TObjectPtr<ALandscape> LandscapeActorPtr(LandscapeActor);
		LandscapeActors.AddUnique(LandscapeActorPtr);
	}
	else if (!IsRunningCookCommandlet())
	{
		if (ALandscapeStreamingProxy* StreamingProxy = Cast<ALandscapeStreamingProxy>(Proxy))
		{
			// We want to know when a streaming proxy is moved, so we can adjust the map
			if (USceneComponent* RootComponent = StreamingProxy->GetRootComponent())
			{
				RootComponent->TransformUpdated.AddUObject(this, &ULandscapeSubsystem::OnProxyMoved);
			}
		}
	}

	Proxy->bIsRegisteredWithSubsystem = true;
	Proxy->RegisteredToSubsystem = this;
}

void ULandscapeSubsystem::UnregisterActor(ALandscapeProxy* Proxy)
{
	check(Proxy != nullptr);
	SUBSYSTEM_DEBUG_LOG_REGISTER(TEXT("ULandscapeSubsystem::UnregisterActor %p %s (%s) to subsystem %p in world %p (%s)"),
		Proxy, *Proxy->GetFullName(), *Proxy->GetClass()->GetName(), this, GetWorld(), *GetWorld()->GetName());

	TObjectPtr<ALandscapeProxy> ProxyPtr(Proxy);

	// in editor we can get multiple unregistration calls, so ensure we don't register more than once
	if (!Proxy->bIsRegisteredWithSubsystem)
	{
		check(Proxy->RegisteredToSubsystem == nullptr);
		check(!Proxies.Contains(ProxyPtr));
		return;
	}
	if (Proxy->RegisteredToSubsystem != this)
	{
		UE_LOGF(LogLandscape, Warning, "Landscape Proxy %ls was registered to world '%ls' but is being unregistered from world '%ls', this may indicate that worlds were changed without re-registering actors, which may cause issues in the landscape system.  We will assume it should be unregistered from the original world.",
			*Proxy->GetFullName(),
			*Proxy->RegisteredToSubsystem->GetWorld()->GetName(),
			*GetWorld()->GetName());
		Proxy->RegisteredToSubsystem->UnregisterActor(Proxy);
		return;
	}
	int32 RemovedCount = Proxies.Remove(ProxyPtr);
	check(RemovedCount == 1);

	if (ALandscape* LandscapeActor = Cast<ALandscape>(Proxy))
	{
		TObjectPtr<ALandscape> LandscapeActorPtr(LandscapeActor);
		LandscapeActors.Remove(LandscapeActorPtr);
	}
	else if (!IsRunningCookCommandlet())
	{
		if (ALandscapeStreamingProxy* StreamingProxy = Cast<ALandscapeStreamingProxy>(Proxy))
		{
			StreamingProxiesNeedingReregister.Remove(StreamingProxy);

			// We want to know when a streaming proxy is moved, so we can adjust the map
			if (USceneComponent* RootComponent = StreamingProxy->GetRootComponent())
			{
				RootComponent->TransformUpdated.RemoveAll(this);
			}
		}
	}

	Proxy->bIsRegisteredWithSubsystem = false;
	Proxy->RegisteredToSubsystem = nullptr;
}

void ULandscapeSubsystem::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);
	ULandscapeSubsystem* const TypedThis = Cast<ULandscapeSubsystem>(InThis);

	for (auto Pair : TypedThis->Groups)
	{
		FLandscapeGroup::AddReferencedObjects(Pair.Value, Collector);
	}
}

void ULandscapeSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	if (UWorld* World = GetWorld())
	{
		if (AWorldSettings* WorldSettings = World->GetWorldSettings())
		{
			OnNaniteWorldSettingsChangedHandle = WorldSettings->OnNaniteSettingsChanged.AddUObject(this, &ULandscapeSubsystem::OnNaniteWorldSettingsChanged);
		}
	}

	FWorldDelegates::LevelRemovedFromWorld.AddUObject(this, &ULandscapeSubsystem::OnLevelRemovedFromWorld);

	static IConsoleVariable* NaniteEnabledCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Nanite"));
	if (NaniteEnabledCVar && !NaniteEnabledCVar->OnChangedDelegate().IsBoundToObject(this))
	{
		NaniteEnabledCVar->OnChangedDelegate().AddUObject(this, &ULandscapeSubsystem::OnNaniteEnabledChanged);
	}

	static IConsoleVariable* LandscapeNaniteEnabledCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("landscape.RenderNanite"));
	if (LandscapeNaniteEnabledCVar && !LandscapeNaniteEnabledCVar->OnChangedDelegate().IsBoundToObject(this))
	{
		LandscapeNaniteEnabledCVar->OnChangedDelegate().AddUObject(this, &ULandscapeSubsystem::OnNaniteEnabledChanged);
	}

	TextureStreamingManager = new FLandscapeTextureStreamingManager();
	check(TextureStreamingManager);

	GrassMapsBuilder = new FLandscapeGrassMapsBuilder(GetWorld(), *TextureStreamingManager);

#if WITH_EDITOR
	PhysicalMaterialBuilder = new FLandscapePhysicalMaterialBuilder(GetWorld());

	if (!IsRunningCommandlet())
	{
		NotificationManager = new FLandscapeNotificationManager();
	}

#endif
	
	OnScalabilityChangedHandle = Scalability::OnScalabilitySettingsChanged.AddLambda([](const Scalability::FQualityLevels& QualityLevels)
    {
    	for (auto* LandscapeComponent : TObjectRange<ULandscapeComponent>(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage))
    	{
    		LandscapeComponent->MarkRenderStateDirty();
    	}
    });
    	
	FCoreUObjectDelegates::GetPostGarbageCollect().AddUObject(this, &ULandscapeSubsystem::HandlePostGarbageCollect);
}

void ULandscapeSubsystem::Deinitialize()
{
	FCoreUObjectDelegates::GetPostGarbageCollect().RemoveAll(this);

	if (OnNaniteWorldSettingsChangedHandle.IsValid())
	{
		UWorld* World = GetWorld();
		check(World != nullptr);

		AWorldSettings* WorldSettings = World->GetWorldSettings();
		check(WorldSettings != nullptr);

		WorldSettings->OnNaniteSettingsChanged.Remove(OnNaniteWorldSettingsChangedHandle);
		OnNaniteWorldSettingsChangedHandle.Reset();
	}

	FWorldDelegates::LevelRemovedFromWorld.RemoveAll(this);

	static IConsoleVariable* NaniteEnabledCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Nanite"));
	if (NaniteEnabledCVar)
	{
		NaniteEnabledCVar->OnChangedDelegate().RemoveAll(this);
	}

	static IConsoleVariable* LandscapeNaniteEnabledCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("landscape.RenderNanite"));
	if (LandscapeNaniteEnabledCVar)
	{
		LandscapeNaniteEnabledCVar->OnChangedDelegate().RemoveAll(this);
	}

	Scalability::OnScalabilitySettingsChanged.Remove(OnScalabilityChangedHandle);
	
#if WITH_EDITOR
	
	const bool bAllNaniteBuildsDone = FinishAllNaniteBuildsInFlightNow(ULandscapeSubsystem::EFinishAllNaniteBuildsInFlightFlags::Default);
	// Not passing ULandscapeSubsystem::EFinishAllNaniteBuildsInFlightFlags::AllowCancel, so there should be no way that FinishAllNaniteBuildsInFlightNow returns false :
	check(bAllNaniteBuildsDone && (NaniteBuildsInFlight == 0));

	delete PhysicalMaterialBuilder;
	delete NotificationManager;
#endif

	delete GrassMapsBuilder;
	delete TextureStreamingManager;

	// cleanup landscape groups
	for (auto Pair : Groups)
	{
		FLandscapeGroup* Group = Pair.Value;
		delete Group;
	}
	Groups.Empty();

	StreamingProxiesNeedingReregister.Empty();

	Proxies.Empty();
	LandscapeActors.Empty();

	Super::Deinitialize();
}

void ULandscapeSubsystem::HandlePostGarbageCollect()
{
	ALandscapeProxy::RemoveInvalidExclusionBoxes();
	GetTextureStreamingManager()->CleanupPostGarbageCollect();
}

void ULandscapeSubsystem::OnProxyMoved(USceneComponent* MovedComponent, EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport)
{
	AActor* Owner = MovedComponent->GetOwner();
	if (ALandscapeStreamingProxy* StreamingProxy = Cast<ALandscapeStreamingProxy>(Owner))
	{
		StreamingProxiesNeedingReregister.Add(StreamingProxy);
	}
}

TStatId ULandscapeSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(ULandscapeSubsystem, STATGROUP_Tickables);
}

FLandscapeGroup* ULandscapeSubsystem::GetLandscapeGroupForProxy(TObjectPtr<ALandscapeProxy> Proxy)
{
	check(IsValid(Proxy));

	// use LODGroupKey instead of LandscapeGUID when LODGroupKey is non-zero
	uint32 LandscapeGroupKey = (Proxy->LODGroupKey != 0) ? Proxy->LODGroupKey : GetTypeHash(Proxy->GetLandscapeGuid());

	FLandscapeGroup*& GroupPtr = Groups.FindOrAdd(LandscapeGroupKey);
	if (GroupPtr == nullptr)
	{
		GroupPtr = new FLandscapeGroup(LandscapeGroupKey);
	}
	return GroupPtr;
}

FLandscapeGroup* ULandscapeSubsystem::GetLandscapeGroupForComponent(ULandscapeComponent* Component)
{
	return GetLandscapeGroupForProxy(Component->GetLandscapeProxy());
}

void ULandscapeSubsystem::RegisterComponent(ULandscapeComponent* Component)
{
 	if (FLandscapeGroup* Group = GetLandscapeGroupForComponent(Component))
 	{
 		Group->RegisterComponent(Component);
 	}

	GetGrassMapBuilder()->RegisterComponent(Component);
}

void ULandscapeSubsystem::UnregisterComponent(ULandscapeComponent* Component)
{
	GetGrassMapBuilder()->UnregisterComponent(Component);

	if (FLandscapeGroup* Group = Component->RegisteredLandscapeGroup)
	{
		Group->UnregisterComponent(Component);
	}
}

void ULandscapeSubsystem::OnLevelRemovedFromWorld(ULevel* InLevel, UWorld* InWorld)
{
	// Clean up info for any landscape actor in the removed level
	// ALandscapeProxy::PostLoad expects no ULandscapeInfo to exist until registration during initial level loading
	for (const ALandscape* Landscape : LandscapeActors)
	{
		if (InLevel == Landscape->GetLevel())
		{
			int32 NumRemovedPairs = ULandscapeInfo::RemoveLandscapeInfo(InWorld, Landscape->GetLandscapeGuid());
			// Landscape actors in the level should still be registered and removable here
			check(NumRemovedPairs != 0);
		}
	}
}

void ULandscapeSubsystem::RemoveGrassInstances(const TSet<ULandscapeComponent*>* ComponentsToRemoveGrassInstances)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeSubsystem::RemoveGrassInstances);
	for (TObjectPtr<ALandscapeProxy> ProxyPtr : Proxies)
		{
			ALandscapeProxy* Proxy = ProxyPtr.Get();
			check(IsValid(Proxy)); // Validate our expectation that proxies will have Unregister() called before a Proxy is flagged as garbage.
			Proxy->FlushGrassComponents(ComponentsToRemoveGrassInstances, /*bFlushGrassMaps = */false);
		}
	}

void ULandscapeSubsystem::RegenerateGrass(bool bInFlushGrass, bool bInForceSync, TOptional<TArrayView<FVector>> InOptionalCameraLocations)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeSubsystem::RegenerateGrass);

	if (Proxies.IsEmpty())
	{
		return;
	}

	UWorld* World = GetWorld();

	if (bInFlushGrass)
	{
		RemoveGrassInstances();
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UpdateGrass);

		TArray<FVector> CameraLocations;
		if (InOptionalCameraLocations.IsSet())
		{
			CameraLocations = *InOptionalCameraLocations;
		}
		else
		{
			if (GUseStreamingManagerForCameras == 0)
			{
				CameraLocations = World->ViewLocationsRenderedLastFrame;
			}
			else if (int32 Num = IStreamingManager::Get().GetNumViews())
			{
				CameraLocations.Reserve(Num);
				for (int32 Index = 0; Index < Num; Index++)
				{
					const FStreamingViewInfo& ViewInfo = IStreamingManager::Get().GetViewInformation(Index);
					CameraLocations.Add(ViewInfo.ViewOrigin);
				}
			}
		}

		UE_IFVLOG(
			{
				if (CVarGrassVisualLogCameraLocations.GetValueOnGameThread())
				{
					for (const FVector& CameraLocation : CameraLocations)
					{
						UE_VLOG_LOCATION(this, LogLandscape, Log, CameraLocation, 10.0f, FColor::Green, TEXT("Grass Camera Location"));
					}
				}
			});

		// Update the grass near the specified location(s) : 
		for (TObjectPtr<ALandscapeProxy> ProxyPtr : Proxies)
		{
			ALandscapeProxy* Proxy = ProxyPtr.Get();
			Proxy->UpdateGrass(CameraLocations, bInForceSync);
		}
	}
}

ETickableTickType ULandscapeSubsystem::GetTickableTickType() const
{
	return HasAnyFlags(RF_ClassDefaultObject) || !GetWorld() || GetWorld()->IsNetMode(NM_DedicatedServer) ? ETickableTickType::Never : ETickableTickType::Always;
}

bool ULandscapeSubsystem::DoesSupportWorldType(const EWorldType::Type WorldType) const
{
	// we also support inactive worlds -- they are used when the world is already saved, but SaveAs renames it:
	// then it duplicates the world (producing an inactive world), which we then need to update Landscapes in during OnPreSave()
	return Super::DoesSupportWorldType(WorldType) || WorldType == EWorldType::Inactive;
}

void ULandscapeSubsystem::Tick(float DeltaTime)
{
	SCOPE_CYCLE_COUNTER(STAT_LandscapeSubsystemTick);
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeSubsystem::Tick);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(Landscape);
	LLM_SCOPE(ELLMTag::Landscape);

	Super::Tick(DeltaTime);

	UWorld* World = GetWorld();
	bool bIsGameWorld = World->IsGameWorld();

#if WITH_EDITOR
	AppCurrentDateTime = FDateTime::Now();
#endif // WITH_EDITOR

	uint32 VisualFrameNumber = (World->Scene != nullptr) ? World->Scene->GetFrameNumber() : LastVisualTickFrameNumber;
	const bool bIsVisualTimeOnlyTick = (VisualFrameNumber == LastVisualTickFrameNumber);

#if UE_ENABLE_DEBUG_DRAWING
	ILandscapeModule& LandscapeModule = FModuleManager::GetModuleChecked<ILandscapeModule>("Landscape");
	// Check if we need to start or stop creating Collision SceneProxies. Don't do this on time-only ticks as the viewport (therefore the scenes) are not drawn in that case, which would lead to wrongly
	//  assume that no view needed collision this frame
	if (!bIsVisualTimeOnlyTick)
	{
		int32 NumViewsWithShowCollision = LandscapeModule.GetLandscapeSceneViewExtension()->GetNumViewsWithShowCollision();
		const bool bNewShowCollisions = NumViewsWithShowCollision > 0;
		const bool bShowCollisionChanged = (bNewShowCollisions != bAnyViewShowCollisions);
		bAnyViewShowCollisions = bNewShowCollisions;
        
		if (bShowCollisionChanged)
		{
			for (ULandscapeHeightfieldCollisionComponent* LandscapeHeightfieldCollisionComponent : TObjectRange<ULandscapeHeightfieldCollisionComponent>(RF_ClassDefaultObject | RF_ArchetypeObject, true, EInternalObjectFlags::Garbage))
			{
				LandscapeHeightfieldCollisionComponent->MarkRenderStateDirty();
			}
		}
	}
#endif // UE_ENABLE_DEBUG_DRAWING
	
	// Double check requested textures are in the right state
	TextureStreamingManager->CheckRequestedTextures();

	static TArray<FVector> OldCameras;
	TArray<FVector>* Cameras = nullptr;
	if (GUseStreamingManagerForCameras == 0)
	{
		if (OldCameras.Num() || World->ViewLocationsRenderedLastFrame.Num())
		{
			// there is a bug here, which often leaves us with no cameras in the editor -- try to fall back to previous camera position(s)
			if (World->ViewLocationsRenderedLastFrame.Num())
			{
				check(IsInGameThread());
				OldCameras = World->ViewLocationsRenderedLastFrame;
			}
			Cameras = &OldCameras;
		}
	}
	else
	{
		int32 Num = IStreamingManager::Get().GetNumViews();
		if (Num)
		{
			OldCameras.Reset(Num);
			for (int32 Index = 0; Index < Num; Index++)
			{
				auto& ViewInfo = IStreamingManager::Get().GetViewInformation(Index);
				OldCameras.Add(ViewInfo.ViewOrigin);
			}
			Cameras = &OldCameras;
		}
	}

	// run early update on Proxies, and determine if all of the proxies are ready for grass generation to start
	bool bAllProxiesReadyForGrassMapGeneration = true;
	bool bAllProxiesRuntimeGrassMapsDisabled = true;
	bool bAnyLandscapeActorHasGrassEnabled = false;

#if WITH_EDITOR
	TSet<ALandscape*> DisallowedGrassTickLandscapes;
#endif // WITH_EDITOR

	for (TObjectPtr<ALandscape> ActorPtr : LandscapeActors)
	{
		ALandscape* Landscape = ActorPtr.Get();

		bAnyLandscapeActorHasGrassEnabled |= Landscape->GetGrassEnabled();

#if WITH_EDITOR
		if (ULandscapeInfo* LandscapeInfo = Landscape->GetLandscapeInfo())
		{
			const bool bLandscapeIsUpToDate = Landscape->IsUpToDate();
			const bool bLandscapeSupportsEditing = LandscapeInfo->SupportsLandscapeEditing();
			const bool bLandscapeUpdateAllowed = bLandscapeSupportsEditing && (Landscape->GetWorld()->GetFeatureLevel() >= ERHIFeatureLevel::SM5);

			// if either of these things are true, then we wait for them to complete before running ANY grass map updates..
			bool bLandscapeToolIsModifyingLandscape = !Landscape->bGrassUpdateEnabled;
			// Don't allow grass to tick if landscape is not up to date -- unless landscape update is not possible (preview or level instanced modes)
			bool bAllowGrassTick = bLandscapeIsUpToDate || !bLandscapeUpdateAllowed;
			if (bLandscapeToolIsModifyingLandscape || !bAllowGrassTick)
			{
				bAllProxiesReadyForGrassMapGeneration = false;
				DisallowedGrassTickLandscapes.Add(Landscape);
			}
		}
#endif // WITH_EDITOR
	}

	bAnyLandscapeActorHasGrassEnabled &= !!GGrassEnable;

	static TArray<ALandscapeProxy*> ActiveProxies;
	{
	ActiveProxies.Reset(Proxies.Num());

		for (TObjectPtr<ALandscapeProxy> ProxyPtr : Proxies)
		{
			ALandscapeProxy* Proxy = ProxyPtr.Get();
			ActiveProxies.Add(Proxy);
			
			// Update the proxies proxy
			{
				if (!Proxy->GetDisableRuntimeGrassMapGeneration())
				{
					bAllProxiesRuntimeGrassMapsDisabled = false;
				}

#if WITH_EDITOR
				if (!bIsGameWorld)
				{
					// in editor, automatically update component grass types if the material changes
					for (ULandscapeComponent* Component : Proxy->LandscapeComponents)
					{
						Component->UpdateGrassTypes();
					}
				}
#endif // WITH_EDITOR

				// Update the grass type summary if necessary
				if (!Proxy->IsGrassTypeSummaryValid())
				{
					Proxy->UpdateGrassTypeSummary();
				}
			}
		}

		// Move the parent ALandscapes to the front of the list.  Better to be consistent with TickLayers running on those first before doing other work on the proxies.
		Algo::Partition(ActiveProxies, [](ALandscapeProxy* Proxy) { return Cast<ALandscape>(Proxy) != nullptr; });
	}

	bool bGrassMapGenerationDisabled = bAllProxiesRuntimeGrassMapsDisabled;
#if WITH_EDITOR
	if (GIsEditor && !bIsGameWorld)
	{
		bGrassMapGenerationDisabled = false;
	}
#endif // WITH_EDITOR

	bool bAllowStartGrassMapGeneration = bAllProxiesReadyForGrassMapGeneration && !bGrassMapGenerationDisabled && (!bIsGameWorld || GGrassMapUseRuntimeGeneration);

	GrassMapsBuilder->AmortizedUpdateGrassMaps(Cameras ? *Cameras : TArray<FVector>(), bIsGrassCreationPrioritized, bAllowStartGrassMapGeneration, bAnyLandscapeActorHasGrassEnabled);

	// run edge fixup for streaming proxies, which only exist in partitioned worlds
	// TODO [chris.tchou] : remove this check once we support edge fixup on non-partitioned worlds (requires removing heightmap sharing between components)
	bool bRunEdgeFixup = UWorld::IsPartitionedWorld(GetWorld());
	if (bRunEdgeFixup)
	{
		TickEdgeFixup();
	}

#if WITH_EDITOR
	int32 NumProxiesUpdated = 0;
	int32 NumMeshesToUpdate = 0;
	NumNaniteMeshUpdatesAvailable += CVarMaxAsyncNaniteProxiesPerSecond->GetFloat() * DeltaTime;
	if (NumNaniteMeshUpdatesAvailable > 1.0f)
	{
		NumMeshesToUpdate = NumNaniteMeshUpdatesAvailable;
		NumNaniteMeshUpdatesAvailable -= NumMeshesToUpdate;
	}
#endif // WITH_EDITOR

	for (ALandscapeProxy* Proxy : ActiveProxies)
	{
#if WITH_EDITOR
		if (GIsEditor && !World->IsPlayInEditor() && GEditor->PlayWorld == nullptr)
		{
			if (ALandscape* Landscape = Cast<ALandscape>(Proxy))
			{
				Landscape->TickLayers(DeltaTime);
			}

			Proxy->UpdatePhysicalMaterialTasks();
		}

		Proxy->GetAsyncWorkMonitor().Tick(DeltaTime);

		if (IsLiveNaniteRebuildEnabled())
		{
			if (NumProxiesUpdated < NumMeshesToUpdate && Proxy->GetAsyncWorkMonitor().CheckIfUpdateTriggeredAndClear(FAsyncWorkMonitor::EAsyncWorkType::BuildNaniteMeshes))
			{
				NumProxiesUpdated++;
				Proxy->UpdateNaniteRepresentation(/* const ITargetPlatform* = */nullptr);
			}
		}
#endif //WITH_EDITOR

		// TODO [chris.tchou] : this stops all async task processing if cameras go away, which might leave tasks dangling
		bool bShouldTickGrass = Proxy->ShouldTickGrass();
#if WITH_EDITOR
		bShouldTickGrass &= !DisallowedGrassTickLandscapes.Contains(Proxy->GetLandscapeActor());
#endif // WITH_EDITOR

		if (bShouldTickGrass && (Cameras != nullptr))
		{
			int32 InOutNumComponentsCreated = 0;
			Proxy->UpdateGrass(*Cameras, InOutNumComponentsCreated);
		}
	}

	ActiveProxies.Reset();
	
	ALandscapeProxy::DebugDrawExclusionBoxes(World);

	LastVisualTickFrameNumber = VisualFrameNumber;

#if WITH_EDITOR
	if (GIsEditor && !World->IsPlayInEditor())
	{
		LandscapePhysicalMaterial::GarbageCollectTasks();

		if (NotificationManager)
		{
			NotificationManager->Tick();
		}
	}

	for (auto It = NaniteMeshBuildStates.CreateIterator(); It; ++It)
	{
		UE::Landscape::Nanite::FAsyncBuildData& BuildState = It->Get();
		if (BuildState.BuildCompleteEvent->IsComplete())
		{
			It.RemoveCurrentSwap();
		}
		else if (BuildState.CheckForStallAndWarn())
		{
			// build may be stalled (bug).  User has been warned, no need to do anything else (it may complete eventually...)
		}
	}


	UActionableMessageSubsystem* ActionableMessageSubsystem = World->GetSubsystem<UActionableMessageSubsystem>();
	
	if ((ActionableMessageSubsystem != nullptr) && GIsEditor)
	{
		FActionableMessage ActionableMessage;
		const FName LandscapeMessageProvider = "Landscape";

		if (!(GEditor->bIsSimulatingInEditor || (GEditor->PlayWorld != nullptr)) && GetActionableMessage(ActionableMessage))
		{
			ActionableMessageSubsystem->SetActionableMessage(LandscapeMessageProvider, ActionableMessage);
		}
		else
		{
			ActionableMessageSubsystem->ClearActionableMessage(LandscapeMessageProvider);
		}
	}

	TickNaniteFinalizeBuildEvents();
#endif // WITH_EDITOR
}

void ULandscapeSubsystem::ForEachLandscapeInfo(TFunctionRef<bool(ULandscapeInfo*)> ForEachLandscapeInfoFunc) const
{
	if (ULandscapeInfoMap* LandscapeInfoMap = ULandscapeInfoMap::FindLandscapeInfoMap(GetWorld()))
	{
		for (const auto& Pair : LandscapeInfoMap->Map)
		{
			if (ULandscapeInfo* LandscapeInfo = Pair.Value)
			{
				if (!ForEachLandscapeInfoFunc(LandscapeInfo))
				{
					return;
				}
			}
		}
	}
}

void ULandscapeSubsystem::OnNaniteEnabledChanged(IConsoleVariable*)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_Landscape_OnNaniteEnabledChanged);

	for (TObjectPtr<ALandscapeProxy>& ProxyPtr : Proxies)
		{
		ALandscapeProxy* Proxy = ProxyPtr.Get();
			Proxy->UpdateRenderingMethod();
		}
	}

void ULandscapeSubsystem::TickEdgeFixup()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeSubsystem::TickEdgeFixup);

	// apply any requested re-registrations first
	if (StreamingProxiesNeedingReregister.Num() > 0)
	{
		check(IsInGameThread());

		// Create a stable copy of the proxies in case the original set is modified while iterating
		TSet<TObjectPtr<ALandscapeStreamingProxy>> StreamingProxiesNeedingReregisterCopy;
		Swap(StreamingProxiesNeedingReregisterCopy, StreamingProxiesNeedingReregister);

		for (TObjectPtr<ALandscapeProxy> StreamingProxy : StreamingProxiesNeedingReregisterCopy)
		{
			check(IsValid(StreamingProxy));
			for (ULandscapeComponent* Component : StreamingProxy->LandscapeComponents)
			{
				if (FLandscapeGroup* Group = Component->RegisteredLandscapeGroup)
				{
					Group->UnregisterComponent(Component);
				}
			}
		}

		// then re-register them to the group they should be in
		for (TObjectPtr<ALandscapeProxy> StreamingProxy : StreamingProxiesNeedingReregisterCopy)
		{
			FLandscapeGroup* NewGroup = GetLandscapeGroupForProxy(StreamingProxy);
			for (ULandscapeComponent* Component : StreamingProxy->LandscapeComponents)
			{
				NewGroup->RegisterComponent(Component);
			}
		}
	}

	// tick each group
	const bool bForcePatchAll = UE::Landscape::ShouldPatchAllLandscapeComponentEdges(/* bResetForNext = */ true);
	for (auto Pair : Groups)
	{
		FLandscapeGroup* Group = Pair.Value;
		Group->TickEdgeFixup(this, bForcePatchAll);
	}
}

#if WITH_EDITOR

void ULandscapeSubsystem::BuildAll(UE::Landscape::EBuildFlags InBuildFlags)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeSubsystem::BuildAll);

	// This is a deliberate action, make sure to flush all packages that are 'pending dirty' :
	MarkModifiedLandscapesAsDirty(InBuildFlags);

	BuildGrassMaps(InBuildFlags);
	BuildPhysicalMaterial(InBuildFlags);
	BuildNanite(InBuildFlags);

	if (EnumHasAnyFlags(InBuildFlags, UE::Landscape::EBuildFlags::WriteFinalLog))
	{
		UE_LOGFMT_LOC(LogLandscape, Log, "BuildAllFinalLog", "Build All operation done.");
	}
}

void ULandscapeSubsystem::BuildGrassMaps(UE::Landscape::EBuildFlags InBuildFlags)
{
	GrassMapsBuilder->Build(InBuildFlags);
}

void ULandscapeSubsystem::BuildPhysicalMaterial(UE::Landscape::EBuildFlags InBuildFlags)
{
	PhysicalMaterialBuilder->Build(InBuildFlags);
}

TSet<UPackage*> ULandscapeSubsystem::GetDirtyLandscapeProxyPackages() const
{
	TSet<UPackage*> DirtyPackages;
	ForEachLandscapeInfo([&](ULandscapeInfo* LandscapeInfo)
		{
			LandscapeInfo->ForEachLandscapeProxy([&](const ALandscapeProxy* LandscapeProxy)
				{
					UPackage* Package = LandscapeProxy->GetPackage();
					if (Package->IsDirty())
					{
						DirtyPackages.Add(Package);
					}
					return true;
				});
			return true;
		});

	return DirtyPackages;
}

TArray<TTuple<ALandscapeProxy*, UE::Landscape::EOutdatedDataFlags>> ULandscapeSubsystem::GetOutdatedProxyDetails(UE::Landscape::EOutdatedDataFlags InMatchingOutdatedDataFlags, bool bInMustMatchAllFlags) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeSubsystem::GetOutdatedProxyDetails);

	UWorld* World = GetWorld();
	if (!World || World->IsGameWorld())
	{
		return {};
	}

	using ProxyAndFlagsArray = TArray<TTuple<ALandscapeProxy*, UE::Landscape::EOutdatedDataFlags>>;

	// Parallelize the retrieval of the outdated proxies and their flags by "fork and join". Each task in the parallel-for handles a certain number of proxies so task contexts simply consists in an array of proxy+flags : 
	TArray<ProxyAndFlagsArray> TaskContexts;
	ParallelForWithTaskContext(TaskContexts, Proxies.Num(), [this, InMatchingOutdatedDataFlags, bInMustMatchAllFlags](ProxyAndFlagsArray& InTaskContext, int32 InIndex)
		{
			FTaskTagScope Scope(ETaskTag::EParallelGameThread);
			ALandscapeProxy* ValidProxy = Proxies[InIndex].Get();
			const UE::Landscape::EOutdatedDataFlags ProxyOutdatedDataFlags = ValidProxy->GetOutdatedDataFlags();

			if ((bInMustMatchAllFlags && EnumHasAllFlags(ProxyOutdatedDataFlags, InMatchingOutdatedDataFlags))
				|| (!bInMustMatchAllFlags && EnumHasAnyFlags(ProxyOutdatedDataFlags, InMatchingOutdatedDataFlags)))
			{
				InTaskContext.Add({ ValidProxy, ProxyOutdatedDataFlags });
			}
		});

	// Join all outdated proxies that have been found by the different tasks :
	ProxyAndFlagsArray OutdatedProxies;
	OutdatedProxies.Reserve(Proxies.Num());
	Algo::ForEach(TaskContexts, [&OutdatedProxies](ProxyAndFlagsArray& InTaskContext) { OutdatedProxies.Append(InTaskContext); });
	return OutdatedProxies;
}

void ULandscapeSubsystem::BuildNanite(UE::Landscape::EBuildFlags InBuildFlags, TArrayView<ALandscapeProxy*> InProxiesToBuild)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeSubsystem::BuildNanite);

	const bool bForceRebuild = EnumHasAnyFlags(InBuildFlags, UE::Landscape::EBuildFlags::ForceRebuild);

	UWorld* World = GetWorld();
	if (!World || World->IsGameWorld() || !GIsEditor)
	{
		return;
	}

	if (InProxiesToBuild.IsEmpty() && Proxies.IsEmpty())
	{
		return;
	}

	TArray<ALandscapeProxy*> FinalProxiesToBuild;
	if (InProxiesToBuild.IsEmpty())
	{
		Algo::Transform(Proxies, FinalProxiesToBuild, [](const TObjectPtr<ALandscapeProxy>& InProxyPtr) { return InProxyPtr.Get(); });
	}
	else 
	{
		for (ALandscapeProxy* ProxyToBuild : InProxiesToBuild)
		{
			FinalProxiesToBuild.Add(ProxyToBuild);
			// Build all streaming proxies in the case of a ALandscape :
			if (ALandscape* Landscape = Cast<ALandscape>(ProxyToBuild))
			{
				ULandscapeInfo* LandscapeInfo = Landscape->GetLandscapeInfo();
				if (LandscapeInfo != nullptr)
				{
					Algo::Transform(LandscapeInfo->GetSortedStreamingProxies(), FinalProxiesToBuild, [](const TWeakObjectPtr<ALandscapeStreamingProxy>& InStreamingProxy) { return InStreamingProxy.Get(); });
				}
			}
		}
	}

	// Only keep unique copies : 
	FinalProxiesToBuild.Sort();
	FinalProxiesToBuild.SetNum(Algo::Unique(FinalProxiesToBuild));

	// Don't keep those that are null or already up to date :
	FinalProxiesToBuild.SetNum(Algo::RemoveIf(FinalProxiesToBuild, [bForceRebuild](ALandscapeProxy* InProxy) { return (InProxy == nullptr) || (!bForceRebuild && InProxy->IsNaniteMeshUpToDate()); }));

	bool bDoFinishAllNaniteBuildsInFlightNow = false;
	for (ALandscapeProxy* LandscapeProxy : FinalProxiesToBuild)
	{
		// reset the nanite content guid so we force rebuild nanite
		if (ULandscapeNaniteComponent* NaniteComponent = LandscapeProxy->GetComponentByClass<ULandscapeNaniteComponent>(); NaniteComponent != nullptr && bForceRebuild)
		{
			UE_LOGF(LogLandscape, Log, "Reset proxy: '%ls'", *LandscapeProxy->GetActorNameOrLabel());
			NaniteComponent->SetProxyContentId(FGuid());
		}

		if (LandscapeProxy->IsNaniteMeshUpToDate())
		{
			continue;
		}

		FGraphEventRef GraphEvent = LandscapeProxy->UpdateNaniteRepresentationAsync(nullptr);
		bDoFinishAllNaniteBuildsInFlightNow |= GraphEvent.IsValid();
	}

	if (bDoFinishAllNaniteBuildsInFlightNow)
	{
		const bool bAllNaniteBuildsDone = FinishAllNaniteBuildsInFlightNow(ULandscapeSubsystem::EFinishAllNaniteBuildsInFlightFlags::DisplaySlowTaskDialog);
		// Not passing ULandscapeSubsystem::EFinishAllNaniteBuildsInFlightFlags::AllowCancel, so there should be no way that FinishAllNaniteBuildsInFlightNow returns false :
		check(bAllNaniteBuildsDone);
	}
	check(NaniteBuildsInFlight == 0);

	if (EnumHasAnyFlags(InBuildFlags, UE::Landscape::EBuildFlags::WriteFinalLog))
	{
		UE_LOGFMT_LOC(LogLandscape, Log, "BuildNaniteFinalLog", "Build Nanite: {NumProxies} landscape {NumProxies}|plural(one=proxy,other=proxies) built.", ("NumProxies", FinalProxiesToBuild.Num()));
	}
}

bool ULandscapeSubsystem::GetDirtyOnlyInMode() const
{
	const ULandscapeSettings* Settings = GetDefault<ULandscapeSettings>();
	return (Settings->LandscapeDirtyingMode == ELandscapeDirtyingMode::InLandscapeModeOnly) 
		|| (Settings->LandscapeDirtyingMode == ELandscapeDirtyingMode::InLandscapeModeAndUserTriggeredChanges);
}

void ULandscapeSubsystem::SaveModifiedLandscapes(UE::Landscape::EBuildFlags InBuildFlags)
{	
	TSet<UPackage*> SetDirtyPackages;
	TSet<FName> PackagesToSave;

	const bool bSkipDirty = false;

	// Gather list of packages to save and make them dirty so they are considered by FEditorFileUtils::SaveDirtyPackages.
	ForEachLandscapeInfo([&](ULandscapeInfo* LandscapeInfo)
	{
		for(UPackage* ModifiedPackage : LandscapeInfo->GetModifiedPackages())
		{
			PackagesToSave.Add(ModifiedPackage->GetFName());
			if (!ModifiedPackage->IsDirty())
			{
				SetDirtyPackages.Add(ModifiedPackage);
				ModifiedPackage->SetDirtyFlag(true);
			}
		}
		return true;
	});

	const bool bPromptUserToSave = true;
	const bool bSaveMapPackages = true;
	const bool bSaveContentPackages = true;
	const bool bFastSave = false;
	const bool bNotifyNoPackagesSaved = false;
	const bool bCanBeDeclined = true;

	FEditorFileUtils::SaveDirtyPackages(bPromptUserToSave, bSaveMapPackages, bSaveContentPackages, bFastSave, bNotifyNoPackagesSaved, bCanBeDeclined, nullptr,
		[PackagesToSave](UPackage* DirtyPackage)
		{
			if (PackagesToSave.Contains(DirtyPackage->GetFName()))
			{
				return false;
			}
			return true;
		});

	// If Package wasn't saved it is still in the LandscapeInfo ModifiedPackage list, set its dirty flag back to false.
	int32 NumPackagesNotProcessed = 0;
	ForEachLandscapeInfo([&](ULandscapeInfo* LandscapeInfo)
	{
		for (UPackage* ModifiedPackage : LandscapeInfo->GetModifiedPackages())
		{
			if (SetDirtyPackages.Contains(ModifiedPackage))
			{
				ModifiedPackage->SetDirtyFlag(false);
				++NumPackagesNotProcessed;
			}
		}
		return true;
	});

	if (EnumHasAnyFlags(InBuildFlags, UE::Landscape::EBuildFlags::WriteFinalLog))
	{
		check(NumPackagesNotProcessed <= PackagesToSave.Num());
		UE_LOGFMT_LOC(LogLandscape, Log, "SaveModifiedLandscapesFinalLog", "Save Modified Landscapes : {NumPackagesToSave} landscape {NumPackagesToSave}|plural(one=proxy,other=proxies) considered : {NumPackagesSaved} saved ({NumPackagesNotProcessed} skipped).", 
			("NumPackagesToSave", PackagesToSave.Num()), ("NumPackagesSaved", (PackagesToSave.Num() - NumPackagesNotProcessed)), ("NumPackagesNotProcessed", NumPackagesNotProcessed));
	}
}

void ULandscapeSubsystem::MarkModifiedLandscapesAsDirty(UE::Landscape::EBuildFlags InBuildFlags)
{
	int32 NumDirtied = 0;
	// Flush all packages that are pending mark for dirty : 
	ForEachLandscapeInfo([InBuildFlags, &NumDirtied](ULandscapeInfo* LandscapeInfo)
	{
		NumDirtied += LandscapeInfo->MarkModifiedPackagesAsDirty();
		return true;
	});

	if (EnumHasAnyFlags(InBuildFlags, UE::Landscape::EBuildFlags::WriteFinalLog))
	{
		UE_LOGFMT_LOC(LogLandscape, Log, "MarkModifiedLandscapesAsDirtyFinalLog", "Mark Modified Landscapes Dirty : {NumProxies} {NumProxies}|plural(one=proxy,other=proxies) made dirty.", ("NumProxies", NumDirtied));
	}
}

bool ULandscapeSubsystem::HasModifiedLandscapes() const
{
	bool bHasModifiedLandscapes = false;
	ForEachLandscapeInfo([&](ULandscapeInfo* LandscapeInfo)
	{
		if (LandscapeInfo->GetModifiedPackageCount() > 0)
		{
			bHasModifiedLandscapes = true;
			return false;
		}
		return true;
	});
	
	return bHasModifiedLandscapes;
}

bool ULandscapeSubsystem::IsGridBased() const
{
	return UWorld::IsPartitionedWorld(GetWorld());
}

void ULandscapeSubsystem::ChangeGridSize(ULandscapeInfo* LandscapeInfo, uint32 GridSizeInComponents)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeSubsystem::ChangeGridSize);
	
	if (!IsGridBased())
	{
		return;
	}

	TSet<AActor*> ActorsToDelete;
	FLandscapeConfigHelper::ChangeGridSize(LandscapeInfo, GridSizeInComponents, ActorsToDelete);
	// This code path is used for converting a non grid based Landscape to a gridbased so it shouldn't delete any actors
	check(!ActorsToDelete.Num());
}

ALandscapeProxy* ULandscapeSubsystem::FindOrAddLandscapeProxy(ULandscapeInfo* LandscapeInfo, const FIntPoint& SectionBase)
{
	if (!IsGridBased())
	{
		return LandscapeInfo->GetCurrentLevelLandscapeProxy(true);
	}

	return FLandscapeConfigHelper::FindOrAddLandscapeStreamingProxy(LandscapeInfo, SectionBase);
}

bool ULandscapeSubsystem::GetActionableMessage(FActionableMessage& OutActionableMessage)
{
	using namespace UE::Landscape;

	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeSubsystem::GetActionableMessage);

	static const FText DefaultTooltip = LOCTEXT("DefaultLandscapeModified.ToolTip", "Assets that affect the Landscape may have changed. Rebuild the Landscape to see the results.");
	static const FText DefaultActionMessage = LOCTEXT("DefaultLandscapeModified.Action", "Rebuild");

	const TArray<TTuple<ALandscapeProxy*, EOutdatedDataFlags>> OutdatedProxies = GetOutdatedProxyDetails(EOutdatedDataFlags::All, /*bInMustMatchAllFlags = */false);
	TArray<int32> NumOutdatedProxyPerFlag;
	NumOutdatedProxyPerFlag.AddDefaulted(GetOutdatedDataFlagIndex(EOutdatedDataFlags::Last) + 1);
	int32 NumTotalOutdatedProxy = 0;
	EOutdatedDataFlags OutdatedFlagsUnion = EOutdatedDataFlags::None;
	Algo::ForEach(OutdatedProxies, [&OutdatedProxies, &NumOutdatedProxyPerFlag, &NumTotalOutdatedProxy, &OutdatedFlagsUnion](const TTuple<ALandscapeProxy*, EOutdatedDataFlags>& ProxyAndFlag)
	{
		ALandscape* ParentLandscape = ProxyAndFlag.Key->GetLandscapeActor();
		// Don't display any message for this landscape when it's being edited : we consider the landscape to be in "WIP state" while editing. 
		//  This avoids flickering of the message while the async stuff (grass, Nanite, ...) gets updated in the background
		if ((ParentLandscape == nullptr) || !ParentLandscape->HasLandscapeEdMode())
		{ 
			++NumTotalOutdatedProxy;
			OutdatedFlagsUnion |= ProxyAndFlag.Value;

			uint32 RemainingFlags = static_cast<uint32>(ProxyAndFlag.Value);
			while (RemainingFlags != 0)
	{
				uint32 FlagIndex = FBitSet::GetAndClearNextBit(RemainingFlags);
				++NumOutdatedProxyPerFlag[FlagIndex];
			}
	}
	});
	
	auto LogFinalNumberOfDirtyProxies = [this]() 
		{
			const int32 NumDirtyProxies = GetDirtyLandscapeProxyPackages().Num();
			UE_LOGFMT_LOC(LogLandscape, Log, "BuildFinalNumberOfDirtyProxies", "{NumProxies} landscape {NumProxies}|plural(one=proxy,other=proxies) now need to be saved.", ("NumProxies", NumDirtyProxies));
		};

	// If more than 1 action is required, go with a BuildAll action
	if (FMath::CountBits64(static_cast<uint64>(OutdatedFlagsUnion)) > 1)
	{
		OutActionableMessage.Message = FText::Format(LOCTEXT("SeveralLandscapeDataOutdated.Message", "{0} Landscape {0}|plural(one=actor,other=actors) {0}|plural(one=is,other=are) out of date and {0}|plural(one=needs,other=need) to be rebuilt"), NumTotalOutdatedProxy);;
		OutActionableMessage.Tooltip = DefaultTooltip;
		OutActionableMessage.ActionMessage = DefaultActionMessage;
		OutActionableMessage.ActionCallback = [LogFinalNumberOfDirtyProxies]()
			{
				UE::Landscape::BuildAll(UE::Landscape::EBuildFlags::WriteFinalLog);
				LogFinalNumberOfDirtyProxies();
			};

		return true;
	}
	
	if (int32 OutdatedProxiesCount = NumOutdatedProxyPerFlag[GetOutdatedDataFlagIndex(EOutdatedDataFlags::GrassMaps)])
	{
		OutActionableMessage.Message = FText::Format(LOCTEXT("GRASS_MAPS_NEED_TO_BE_REBUILT_FMT", "{0} Landscape {0}|plural(one=actor,other=actors) with grass maps {0}|plural(one=needs,other=need) to be rebuilt"), OutdatedProxiesCount);
		OutActionableMessage.Tooltip = DefaultTooltip;
		OutActionableMessage.ActionMessage = DefaultActionMessage;
		OutActionableMessage.ActionCallback = [LogFinalNumberOfDirtyProxies]()
			{
				UE::Landscape::BuildGrassMaps(UE::Landscape::EBuildFlags::WriteFinalLog);
				LogFinalNumberOfDirtyProxies();
			};

		return true;
	}
	
	if (int32 OutdatedProxiesCount = NumOutdatedProxyPerFlag[GetOutdatedDataFlagIndex(EOutdatedDataFlags::PhysicalMaterials)])
	{
		OutActionableMessage.Message = FText::Format(LOCTEXT("LANDSCAPE_PHYSICALMATERIAL_NEED_TO_BE_REBUILT_FMT", "{0} Landscape {0}|plural(one=actor,other=actors) with physical materials {0}|plural(one=needs,other=need) to be rebuilt"), OutdatedProxiesCount);
		OutActionableMessage.Tooltip = DefaultTooltip;
		OutActionableMessage.ActionMessage = DefaultActionMessage;
		OutActionableMessage.ActionCallback = [LogFinalNumberOfDirtyProxies]()
			{
				UE::Landscape::BuildPhysicalMaterial(UE::Landscape::EBuildFlags::WriteFinalLog);
				LogFinalNumberOfDirtyProxies();
			};

		return true;
	}

	if (int32 OutdatedProxiesCount = NumOutdatedProxyPerFlag[GetOutdatedDataFlagIndex(EOutdatedDataFlags::NaniteMeshes)])
	{
		OutActionableMessage.Message = FText::Format(LOCTEXT("LANDSCAPE_NANITE_MESHES_NEED_TO_BE_REBUILT_FMT", "{0} Landscape {0}|plural(one=actor,other=actors) with Nanite meshes {0}|plural(one=needs,other=need) to be rebuilt"), OutdatedProxiesCount);
		OutActionableMessage.Tooltip = DefaultTooltip;
		OutActionableMessage.ActionMessage = DefaultActionMessage;
		OutActionableMessage.ActionCallback = [LogFinalNumberOfDirtyProxies]()
			{ 
				UE::Landscape::BuildNanite(UE::Landscape::EBuildFlags::WriteFinalLog); 
				LogFinalNumberOfDirtyProxies();
			};

		return true;
	}

	if (int32 OutdatedProxiesCount = NumOutdatedProxyPerFlag[GetOutdatedDataFlagIndex(EOutdatedDataFlags::PackageModified)])
	{
		OutActionableMessage.Message = FText::Format(LOCTEXT("LandscapeModified.Message", "{0} Landscape {0}|plural(one=actor,other=actors) {0}|plural(one=is,other=are) out of date and {0}|plural(one=needs,other=need) to be rebuilt"), OutdatedProxiesCount);
		OutActionableMessage.Tooltip = LOCTEXT("LandscapeModified.Tooltip", "The Landscape actors visible in your level have been modified as a result of changes to other assets.\nThese changes need to be applied to the Landscape assets.");
		OutActionableMessage.ActionMessage = LOCTEXT("LandscapeModified.Action", "Update");
		OutActionableMessage.ActionCallback = [LogFinalNumberOfDirtyProxies]()
			{ 
				UE::Landscape::MarkModifiedLandscapesAsDirty(UE::Landscape::EBuildFlags::WriteFinalLog); 
				LogFinalNumberOfDirtyProxies();
			};

		return true;
	}

	return false;
}

FDateTime ULandscapeSubsystem::GetAppCurrentDateTime()
{
	return AppCurrentDateTime;
}

TSharedRef<UE::Landscape::Nanite::FAsyncBuildData> ULandscapeSubsystem::CreateTrackedNaniteBuildState(
	ALandscapeProxy* LandscapeProxy, int32 InLODToExport, const TArray<ULandscapeComponent*>& InComponentsToExport)
{
	check(Proxies.Contains(LandscapeProxy)); // proxies should be registered before attempting to build nanite
	TSharedRef<UE::Landscape::Nanite::FAsyncBuildData> AsyncBuildData = LandscapeProxy->MakeAsyncNaniteBuildData(InLODToExport, InComponentsToExport);
	AsyncBuildData->BuildCompleteEvent = FGraphEvent::CreateGraphEvent();
	AsyncBuildData->TimeStamp_Requested = FPlatformTime::Seconds();
	NaniteMeshBuildStates.Add(AsyncBuildData);
	return AsyncBuildData;
}

void ULandscapeSubsystem::AddNaniteFinalizeBuildEvent(FGraphEventRef InNaniteFinalizeBuildEvent)
{
	NaniteFinalizeBuildEvents.Add(InNaniteFinalizeBuildEvent);
}

bool ULandscapeSubsystem::FinishAllNaniteBuildsInFlightNow(EFinishAllNaniteBuildsInFlightFlags FinishFlags)
{
	check(IsInGameThread());

	const int32 TotalMeshes = NaniteBuildsInFlight.load();

	const bool bAllowCancel = EnumHasAnyFlags(FinishFlags, EFinishAllNaniteBuildsInFlightFlags::AllowCancel);

	TOptional<FSlowTask> SlowTask;
	if (EnumHasAnyFlags(FinishFlags, EFinishAllNaniteBuildsInFlightFlags::DisplaySlowTaskDialog))
	{
		SlowTask.Emplace(TotalMeshes, LOCTEXT("Landscape_BuildNanite", "Building Nanite Landscape Meshes"));

		const bool bShowCancelButton = bAllowCancel;
		SlowTask.GetValue().Initialize();
		SlowTask.GetValue().MakeDialog(bShowCancelButton);
	}

	TArray<UObject*> IncompleteMeshes;

	// we have to drain the game thread tasks and static mesh builds
	bool bCancelled = false;
	int32 LastRemainingMeshes = TotalMeshes;
	while (AreNaniteBuildsInProgress())
	{
		int32 Remaining = NaniteBuildsInFlight.load();
		int32 MeshesProcessed = LastRemainingMeshes - Remaining;
		LastRemainingMeshes = Remaining;

		if (SlowTask.IsSet())
		{
			SlowTask.GetValue().EnterProgressFrame(MeshesProcessed, FText::Format(LOCTEXT("Landscape_BuildNaniteProgress", "Building Nanite Landscape Mesh ({0} of {1})"),
				FText::AsNumber(TotalMeshes - LastRemainingMeshes),
				FText::AsNumber(TotalMeshes)));
		}

		IncompleteMeshes.Reset();
		bool bAnyStalled = false;
		bool bAnyNotStalled = false;
		for (TSharedRef<UE::Landscape::Nanite::FAsyncBuildData> AsyncBuildData : NaniteMeshBuildStates)
		{
			if (!AsyncBuildData->bIsComplete) // event IsComplete?
			{
				if (AsyncBuildData->CheckForStallAndWarn())
				{
					bAnyStalled = true;
				}
				else
				{
					bAnyNotStalled = true;
				}
				IncompleteMeshes.Add(AsyncBuildData->NaniteStaticMesh.Get());
			}
		}

		if (IncompleteMeshes.Num() > 0)
		{
			FAssetCompilingManager::Get().FinishCompilationForObjects(IncompleteMeshes);
		}

		const bool bCancelWhenAllStalled = false; // this is disabled (stall detection is not perfect) - for now we remain in the loop hoping it will complete.
		if (bCancelWhenAllStalled && bAnyStalled && !bAnyNotStalled)
		{
			// all remaining meshes are stalled..
			UE_LOGF(LogLandscape, Warning, "Cancelling Nanite Build because it is taking too long, nanite meshes may not be up to date");
			bCancelled = true;
			break;
		}

		if (bAllowCancel && SlowTask.IsSet() && SlowTask.GetValue().ShouldCancel())
		{
			// TODO [chris.tchou] Currently this just closes the dialog, but lets the tasks continue -- we should set the cancel flag in the outstanding tasks
			// to early out, and test to make sure this doesn't leave anything hanging or in a bad state.
			bCancelled = true;
			break;
		}

		// Make sure we have also executed all deferred "finalize Nanite build tasks"
		// Nanite finalize build events run on the game thread so drain all game thread tasks here
		FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
		// Tick NaniteFinalizeBuildEvents in order to empty the list of those that have been processed, that will allow us to eventually leave that while 
		// statement by having AreNaniteBuildsInProgress return true :
		TickNaniteFinalizeBuildEvents();
	}

	if (SlowTask.IsSet())
	{
		SlowTask.GetValue().Destroy();
		SlowTask.Reset();	// should destroy and teardown the slow task
	}

	return !bCancelled && !AreNaniteBuildsInProgress();
}

void ULandscapeSubsystem::TickNaniteFinalizeBuildEvents()
{
	check(IsInGameThread());
	NaniteFinalizeBuildEvents.RemoveAllSwap([](const FGraphEventRef& Ref) -> bool { return Ref->IsComplete(); });
}

bool ULandscapeSubsystem::IsMultithreadedNaniteBuildEnabled()
{
	return LandscapeMultithreadNaniteBuild > 0;
}

bool ULandscapeSubsystem::IsLiveNaniteRebuildEnabled()
{
	return LiveRebuildNaniteOnModification > 0;
}

bool ULandscapeSubsystem::AreNaniteBuildsInProgress() const
{
	check(IsInGameThread());
	return (NaniteBuildsInFlight.load() > 0) || !NaniteFinalizeBuildEvents.IsEmpty();
}

void ULandscapeSubsystem::IncNaniteBuild()
{
	NaniteBuildsInFlight++;
}

void ULandscapeSubsystem::DecNaniteBuild()
{
	NaniteBuildsInFlight--;
	NaniteStaticMeshesInFlight--;
}

void ULandscapeSubsystem::WaitLaunchNaniteBuild()
{
	ON_SCOPE_EXIT
	{
		++NaniteStaticMeshesInFlight;
	};
	
	if (LandscapeMultithreadNaniteBuild != 0 && LandscapeMaxSimultaneousMultithreadNaniteBuilds == -1)
	{
		return;
	}
	
	const int32 MaxNaniteBuilds  = LandscapeMultithreadNaniteBuild ? LandscapeMaxSimultaneousMultithreadNaniteBuilds : 1;

	if (MaxNaniteBuilds < 0)
	{
		return;
	}

	while(NaniteStaticMeshesInFlight >= MaxNaniteBuilds)
	{
		FPlatformProcess::Sleep(0.05f);
	}

	check((LandscapeMultithreadNaniteBuild != 0) || (NaniteStaticMeshesInFlight <= 1));
}

ULandscapeSubsystem::FDelegateAccess::FDelegateAccess(FOnHeightmapStreamedDelegate& InOnHeightmapStreamed, FOnLandscapeProxyComponentDataChanged& InOnLandscapeProxyComponentDataChanged, FOnLandscapeProxyMaterialChanged& InOnLandscapeProxyMaterialChanged) 
	: OnHeightmapStreamedDelegate(InOnHeightmapStreamed)
	, OnLandscapeProxyComponentDataChangedDelegate(InOnLandscapeProxyComponentDataChanged)
	, OnLandscapeProxyMaterialChangedDelegate(InOnLandscapeProxyMaterialChanged)
{}

ULandscapeSubsystem::FDelegateAccess ULandscapeSubsystem::GetDelegateAccess() const
{
	return FDelegateAccess(OnHeightmapStreamedDelegate, OnLandscapeProxyComponentDataChangedDelegate, OnLandscapeProxyMaterialChangedDelegate);
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
