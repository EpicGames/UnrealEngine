// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
LandscapeTextureHash.cpp: Track a Custom Hash on each landscape texture.
This hash tries to be insensitive to changes that are less than the thresholds,
and also ignores normal data channels on the heightmaps.
=============================================================================*/

#include "LandscapeTextureHash.h"
#include "LandscapePrivate.h"
#include "Engine/Texture2D.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapeTextureHash)

#if WITH_EDITORONLY_DATA

extern TAutoConsoleVariable<int32> CVarLandscapeDirtyHeightmapHeightThreshold;
extern TAutoConsoleVariable<int32> CVarLandscapeDirtyWeightmapThreshold;
extern int32 GPatchEdges;
extern int32 GPatchStreamingMipEdges;

namespace UE::Landscape::Private
{
	// produces a Valid (non-zero) FGuid from a 64 bit hash
	static inline FGuid Hash64ToGUID(uint64 Hash)
	{
		uint32 LowBits = Hash & 0xffffffff;
		uint32 HighBits = Hash >> 32;
		return FGuid(HighBits, HighBits + LowBits + 0xbb4824dc, HighBits ^ LowBits, LowBits);
	}

	// converts a GUID into a 64 bit hash (inverse of Hash64ToGUID)
	static inline uint64 GUIDToHash64(const FGuid& GUID)
	{
		return (static_cast<uint64>(GUID.A) << 32) + GUID.D;
	}
};

void ULandscapeTextureHash::SetInitialStateOnPostLoad(UTexture2D* LandscapeTexture, ELandscapeTextureUsage TextureUsage, ELandscapeTextureType TextureType)
{
	ULandscapeTextureHash* TextureHash = LandscapeTexture->GetAssetUserData<ULandscapeTextureHash>();
	if (TextureHash == nullptr)
	{
		// if there is no texture hash recorded, create a new one (using default SourceID as the hash) and record it in the recent serialized hashes
		TextureHash = NewObject<ULandscapeTextureHash>(LandscapeTexture);
		LandscapeTexture->AddAssetUserData(TextureHash);

		FGuid LandscapeTextureSourceId = LandscapeTexture->Source.GetId();
		TextureHash->TextureHashGUID = LandscapeTextureSourceId;
		TextureHash->LastSourceID = LandscapeTextureSourceId;
		TextureHash->TextureType = TextureType;
		TextureHash->TextureUsage = TextureUsage;
		TextureHash->RecentlySerializedHashes.Add(LandscapeTextureSourceId, LandscapeTextureSourceId);
	}
}

void ULandscapeTextureHash::CheckHashIsUpToDate(UTexture2D* LandscapeTexture)
{
	ULandscapeTextureHash* TextureHash = LandscapeTexture->GetAssetUserData<ULandscapeTextureHash>();
	check(TextureHash != nullptr);
	check(LandscapeTexture->Source.GetId() == TextureHash->LastSourceID);
}

FGuid ULandscapeTextureHash::CalculateTextureHashGUID(UTexture2D* LandscapeTexture, ELandscapeTextureType TextureType, bool& bOutHashIsValid)
{
	uint64 Hash64 = CalculateTextureHash64(LandscapeTexture, TextureType, bOutHashIsValid);
	return UE::Landscape::Private::Hash64ToGUID(Hash64);
}

uint64 ULandscapeTextureHash::CalculateTextureHash64(UTexture2D* LandscapeTexture, ELandscapeTextureType TextureType, bool& bOutHashIsValid)
{
	const int32 MipIndex = 0;
	const int64 MipSizeInBytes = LandscapeTexture->Source.CalcMipSize(MipIndex);
	const int64 MipSizeInPixels = MipSizeInBytes / 4;
	uint64 Hash;
	const FColor* MipData = (const FColor*)LandscapeTexture->Source.LockMipReadOnly(MipIndex);
	if (MipData) // this can be null if the texture data is corrupted and fails to decompress
	{
		bOutHashIsValid = true;
		Hash = CalculateTextureHash64(MipData, MipSizeInPixels, TextureType);
		LandscapeTexture->Source.UnlockMip(MipIndex);
	}
	else
	{
		UE_LOGF(LogLandscape, Warning, "Could not calculate the texture hash for landscape texture '%ls' because the texture could not be locked (it may be corrupt or being modified incorrectly)", *LandscapeTexture->GetName());
		Hash = 0;
		bOutHashIsValid = false;
	}
	return Hash;
}

uint64 ULandscapeTextureHash::CalculateTextureHash64(const FColor* Mip0Data, int32 PixelCount, ELandscapeTextureType TextureType)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeTextureHash::CalculateTextureHash);

	uint64 NewHash = 0;

	switch (TextureType)
	{
		case ELandscapeTextureType::Unknown:
			check(TextureType != ELandscapeTextureType::Unknown);
			break;
		case ELandscapeTextureType::Heightmap:
			{
				uint32 CRCHashG = 0;
				uint32 CRCHashR = 1;

				for (int32 i = 0; i < PixelCount; i++)
				{
					// the height is the Red and Green channels (ignore the normal data in the other channels)
					CRCHashG = FCrc::TypeCrc32(Mip0Data->G, CRCHashG);
					CRCHashR = FCrc::TypeCrc32(Mip0Data->R, CRCHashR);
					Mip0Data++;
				}

				NewHash = (((uint64)CRCHashR) << 32) + CRCHashG;
			}
			break;
		case ELandscapeTextureType::Weightmap:
			{
				NewHash = CityHash64(reinterpret_cast<const char*>(Mip0Data), PixelCount * sizeof(FColor));
			}
			break;
		default:
			checkf(false, TEXT("TextureType is invalid: %d"), EnumToUnderlyingType(TextureType));
			break;
	}

	return NewHash;
}

bool ULandscapeTextureHash::DoesTextureDataChangeExceedThreshold(
	const FColor* Mip0Data, const FColor* OldMip0Data, int32 PixelCount, ELandscapeTextureType TextureType, uint64 OldHash, uint64 NewHash, TOptional<uint8>& OutChangedWeightmapChannelsMasks)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeTextureHash::DoesTextureDataChangeExceedThreshold);

	bool bExceedsThreshold = false;

	switch (TextureType)
	{
		case ELandscapeTextureType::Unknown:
			check(TextureType != ELandscapeTextureType::Unknown);
			break;
		case ELandscapeTextureType::Heightmap:
			{
				int32 DirtyHeightmapHeightThreshold = CVarLandscapeDirtyHeightmapHeightThreshold.GetValueOnGameThread();
				if (DirtyHeightmapHeightThreshold <= 0)
				{
					// at at threshold of zero, any change at all will exceed
					bExceedsThreshold = (OldHash != NewHash);
					break;
				}

				for (int32 i = 0; i < PixelCount; i++)
				{
					const FColor& OldColor = *OldMip0Data;
					const FColor& NewColor = *Mip0Data;

					if (OldColor != NewColor)
					{
						uint16 OldHeight = ((static_cast<uint16>(OldColor.R) << 8) | static_cast<uint16>(OldColor.G));
						uint16 NewHeight = ((static_cast<uint16>(NewColor.R) << 8) | static_cast<uint16>(NewColor.G));
						if (uint16 Diff = (NewHeight > OldHeight) ? (NewHeight - OldHeight) : (OldHeight - NewHeight); Diff > DirtyHeightmapHeightThreshold)
						{
							bExceedsThreshold = true;
							break;
						}
					}

					Mip0Data++;
					OldMip0Data++;
				}
			}
			break;
		case ELandscapeTextureType::Weightmap:
			{
				int32 DirtyWeightmapThreshold = CVarLandscapeDirtyWeightmapThreshold.GetValueOnGameThread();
				if ((DirtyWeightmapThreshold <= 0) && !OutChangedWeightmapChannelsMasks.IsSet())
				{
					// at at threshold of zero, any change at all will exceed
					bExceedsThreshold = (OldHash != NewHash);
					break;
				}

				for (int32 Index = 0; Index < PixelCount; ++Index)
				{
					const FColor& OldColor = *OldMip0Data;
					const FColor& NewColor = *Mip0Data;

					if (OldColor != NewColor)
					{
						auto DiffChannel = [DirtyWeightmapThreshold](uint8 InOldValue, uint8 InNewValue) -> bool
							{
								uint8 Diff = (InNewValue > InOldValue) ? (InNewValue - InOldValue) : (InOldValue - InNewValue);
								return (Diff > DirtyWeightmapThreshold);
							};

						uint8 DiffMask =
							((uint8)(DiffChannel(OldColor.R, NewColor.R) ? 1 : 0) << 0)
							| ((uint8)(DiffChannel(OldColor.G, NewColor.G) ? 1 : 0) << 1)
							| ((uint8)(DiffChannel(OldColor.B, NewColor.B) ? 1 : 0) << 2)
							| ((uint8)(DiffChannel(OldColor.A, NewColor.A) ? 1 : 0) << 3);

						if (DiffMask != 0)
						{
							bExceedsThreshold = true;
							if (OutChangedWeightmapChannelsMasks.IsSet())
							{
								*OutChangedWeightmapChannelsMasks |= DiffMask;
							}
							else
							{
								// no need to report which channel has been changed, early out :
								break;
							}
						}
					}

					Mip0Data++;
					OldMip0Data++;
				}
			}
			break;
		default:
			checkf(false, TEXT("TextureType is invalid: %d"), EnumToUnderlyingType(TextureType));
			break;
	}

	return bExceedsThreshold;
}

void ULandscapeTextureHash::SetHash64(UTexture2D* LandscapeTexture, uint64 NewHash64, ELandscapeTextureUsage TextureUsage, ELandscapeTextureType TextureType)
{
	check(TextureUsage != ELandscapeTextureUsage::Unknown);
	check(TextureType != ELandscapeTextureType::Unknown);

	ULandscapeTextureHash* TextureHash = LandscapeTexture->GetAssetUserData<ULandscapeTextureHash>();
	if (TextureHash == nullptr)
	{
		// create a new one (with LandscapeTexture as outer)
		TextureHash = NewObject<ULandscapeTextureHash>(LandscapeTexture);
		LandscapeTexture->AddAssetUserData(TextureHash);
	}
	else
	{
		// pre-existing -- should have the same type (or unknown)
		check(TextureHash->TextureUsage == TextureUsage || TextureHash->TextureUsage == ELandscapeTextureUsage::Unknown);
		check(TextureHash->TextureType == TextureType || TextureHash->TextureType == ELandscapeTextureType::Unknown);
	}
	FGuid LandscapeTextureSourceId = LandscapeTexture->Source.GetId();

	// cached hashes take precedence -- this ensure that if the texture is brought back to a recently serialized state, it will have exactly the same hash that it was serialized with
	FGuid NewHashGUID;
	if (FGuid* CachedHash = TextureHash->RecentlySerializedHashes.Find(LandscapeTextureSourceId))
	{
		NewHashGUID = *CachedHash;
	}
	else
	{
		NewHashGUID = UE::Landscape::Private::Hash64ToGUID(NewHash64);
	}
	TextureHash->TextureHashGUID = NewHashGUID;
	TextureHash->LastSourceID = LandscapeTextureSourceId;
	TextureHash->TextureType = TextureType;
	TextureHash->TextureUsage = TextureUsage;
}

void ULandscapeTextureHash::UpdateHash(UTexture2D* LandscapeTexture, ELandscapeTextureUsage TextureUsage, ELandscapeTextureType TextureType, bool bForceUpdate)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeTextureHash::UpdateHash);

	FGuid LandscapeTextureSourceId = LandscapeTexture->Source.GetId();

	bool bNewlyCreated = false;
	ULandscapeTextureHash* TextureHash = LandscapeTexture->GetAssetUserData<ULandscapeTextureHash>();
	if (TextureHash == nullptr)
	{
		// create a new one (with LandscapeTexture as outer)
		TextureHash = NewObject<ULandscapeTextureHash>(LandscapeTexture);
		LandscapeTexture->AddAssetUserData(TextureHash);
		bNewlyCreated = true;
	}
	else
	{
		if (!bForceUpdate && (LandscapeTextureSourceId == TextureHash->LastSourceID))
		{
			// no need to update, it's the same
			check(TextureHash->TextureUsage == TextureUsage || TextureUsage == ELandscapeTextureUsage::Unknown);
			check(TextureHash->TextureType == TextureType || TextureType == ELandscapeTextureType::Unknown);
			return;
		}
	}

	if (TextureUsage == ELandscapeTextureUsage::Unknown)
	{
		TextureUsage = TextureHash->TextureUsage;
	}
	if (TextureType == ELandscapeTextureType::Unknown)
	{
		TextureType = TextureHash->TextureType;
	}

	FGuid NewHash;
	bool bHashIsValid = true;
	if ((TextureUsage != ELandscapeTextureUsage::FinalData) ||
		(TextureType == ELandscapeTextureType::Unknown))
	{
		// non-final data and/or unknown types don't need to use a hash, as we just use the SourceID directly
		NewHash = LandscapeTextureSourceId;
	}
	else
	{
		// if this SourceId is familiar, use the corresponding hash
		if (FGuid* CachedHash = TextureHash->RecentlySerializedHashes.Find(LandscapeTextureSourceId))
		{
			NewHash = *CachedHash;
		}
		else
		{
			// otherwise compute a new one
			NewHash = CalculateTextureHashGUID(LandscapeTexture, TextureType, bHashIsValid);
		}
	}

	if (bHashIsValid)
	{
		TextureHash->TextureHashGUID = NewHash;
		TextureHash->LastSourceID = LandscapeTextureSourceId;
		TextureHash->TextureType = TextureType;
		TextureHash->TextureUsage = TextureUsage;

		if (bNewlyCreated)
		{
			TextureHash->RecentlySerializedHashes.Add(LandscapeTextureSourceId, NewHash);
		}
	}
}

FGuid ULandscapeTextureHash::GetHash(UTexture2D* LandscapeTexture)
{
	// if no texture hash exists, or it's not a final layer-merged texture, just use the source ID
	ULandscapeTextureHash* TextureHash = LandscapeTexture->GetAssetUserData<ULandscapeTextureHash>();
	if ((TextureHash == nullptr) || (TextureHash->TextureUsage != ELandscapeTextureUsage::FinalData) || (TextureHash->TextureType == ELandscapeTextureType::Unknown))
	{
		// fallback to using the Source ID (matches old behavior)
		check(LandscapeTexture->Source.IsValid());
		return LandscapeTexture->Source.GetId();
	}
	if (LandscapeTexture->Source.GetId() != TextureHash->LastSourceID)
	{
		// NOTE: this can happen in WP mode when a final data texture is transacted for undo/redo.
		// It can also happen when we are using non-WP mode when directly modifying the final texture source on the CPU (as we don't rehash on all CPU modifications)

		// in either case we can just force update the hash to get an ok hash to use.
		// This won't take change thresholds into account, but that's as good as we can do in these cases.
		UpdateHash(LandscapeTexture, TextureHash->TextureUsage, TextureHash->TextureType, /*bForceUpdate=*/ true);
	}
	return TextureHash->TextureHashGUID;
}

void ULandscapeTextureHash::Serialize(FArchive& Ar)
{
	bool bUpdateRecentlySerializedHashes = Ar.IsPersistent() && !Ar.IsObjectReferenceCollector() && !HasAnyFlags(RF_ArchetypeObject | RF_ClassDefaultObject | RF_DefaultSubObject);
	if (bUpdateRecentlySerializedHashes && Ar.IsSaving())
	{
		UTexture2D* ParentTexture = Cast<UTexture2D>(GetOuter());
		if (ParentTexture && (ParentTexture->Source.GetId() != LastSourceID))
		{
			// The stored hash is out of date (it was modified without explicitly updating the hash)  Update it before serializing!
			// This won't take change thresholds into account, but that's as good as we can do in these cases.
			UpdateHash(ParentTexture, TextureUsage, TextureType, /*bForceUpdate=*/ true);
		}

		// as we're about to save this, make it an official recent value
		// (this guarantees that if we get back to the current state, we will get the same texture hash, despite the threshold-change shenanigans that might go on in the meantime)
		if (!RecentlySerializedHashes.Contains(LastSourceID))
		{
			RecentlySerializedHashes.Add(LastSourceID, TextureHashGUID);
		}
	}

	Super::Serialize(Ar);

	if (bUpdateRecentlySerializedHashes && Ar.IsLoading())
	{
		// as we just loaded this, make it an official recent value
		// (this guarantees that if we get back to the current state, we will get the same texture hash, despite the threshold-change shenanigans that might go on in the meantime)
		if (!RecentlySerializedHashes.Contains(LastSourceID))
		{
			RecentlySerializedHashes.Add(LastSourceID, TextureHashGUID);
		}
	}
}
#endif // WITH_EDITORONLY_DATA
