// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
LandscapeTextureStorageProvider.cpp: Alternative Texture Storage for Landscape Textures
=============================================================================*/

#include "LandscapeTextureStorageProvider.h"
#include "LandscapeDataAccess.h"
#include "RHIGlobals.h"
#include "ContentStreaming.h"
#include "LandscapeGroup.h"
#include "LandscapePrivate.h"
#include "ProfilingDebugging/IoStoreTrace.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapeTextureStorageProvider)

// enables verbose debug spew
//#define ENABLE_LANDSCAPE_PROVIDER_DEBUG_SPEW 1
#ifdef ENABLE_LANDSCAPE_PROVIDER_DEBUG_SPEW
#define PROVIDER_DEBUG_LOG(...) UE_LOG(LogLandscape, Warning, __VA_ARGS__)
#define PROVIDER_DEBUG_LOG_DETAIL(...) UE_LOG(LogLandscape, Warning, __VA_ARGS__)
#else
#define PROVIDER_DEBUG_LOG(...) UE_LOG(LogLandscape, Verbose, __VA_ARGS__)
#define PROVIDER_DEBUG_LOG_DETAIL(...) do {} while(0)
#endif // ENABLE_LANDSCAPE_PROVIDER_DEBUG_SPEW

namespace UE::Landscape::Private
{
	// Returns true if pixel position exists at a boundary offset (subsection/multi-component texture boundary)
	static inline bool IsDuplicateCoord(int32 XY, int32 MipBoundaryOffset)
	{
		checkSlow(FMath::IsPowerOfTwo(MipBoundaryOffset));
		return (XY != 0) && ((XY & (MipBoundaryOffset - 1)) == 0);
	}
}

FLandscapeTextureMipEdgeOverrideProvider::FLandscapeTextureMipEdgeOverrideProvider(ULandscapeHeightmapTextureEdgeFixup* InEdgeFixup, UTexture2D* InTexture)
	: FTextureMipDataProvider(InTexture, ETickState::GetMips, ETickThread::Async)
{
	this->EdgeFixup = InEdgeFixup;
	TextureName = InTexture->GetFName();
}

FLandscapeTextureMipEdgeOverrideProvider::~FLandscapeTextureMipEdgeOverrideProvider()
{
}

void FLandscapeTextureMipEdgeOverrideProvider::Init(const FTextureUpdateContext& Context, const FTextureUpdateSyncOptions& SyncOptions)
{
	AdvanceTo(ETickState::GetMips, ETickThread::Async);
}

int32 FLandscapeTextureMipEdgeOverrideProvider::GetMips(const FTextureUpdateContext& Context, int32 StartingMipIndex, const FTextureMipInfoArray& MipInfos, const FTextureUpdateSyncOptions& SyncOptions)
{
	// make a copy of the dest mip infos, for reference in PollMips
	DestMipInfos = MipInfos;

	AdvanceTo(ETickState::PollMips, ETickThread::Async);
	return StartingMipIndex;	// we don't directly handle any mips -- return the same starting mip index
}

bool FLandscapeTextureMipEdgeOverrideProvider::PollMips(const FTextureUpdateSyncOptions& SyncOptions)
{
	using namespace UE::Landscape;

	// poll mips will run once all io requests are complete (or cancelled)
	// here we are relying on the behavior of the default providers, whose PollMips run _after_ PollMips on custom providers like this one.
	// We rely on the fact that they do not modify the MipData in PollMips.
	// THIS IS NOT TRUE OF ALL PROVIDERS -- for example the FLandscapeTextureStorageMipProvider will write to mip data in PollMips
	// however, we handle that case by merging the override functionality into FLandscapeTextureStorageMipProvider, so we don't need a separate override provider.
	bool bSuccess= true;

	if (!ShouldPatchStreamingMipEdges())
	{
		AdvanceTo(ETickState::Done, ETickThread::None);
		return bSuccess;
	}

	if ((EdgeFixup == nullptr) || !EdgeFixup->IsActive())
	{
		// this heightmap is not yet registered and active -- we can't patch yet.
		// not to worry though!  When it DOES register, it will fix all existing mips.
		// (so this mip will be handled at that point)
		PROVIDER_DEBUG_LOG_DETAIL(TEXT("---- PollMips Coord Mips (%d ... %d) -- NOT READY"), PendingFirstLODIdx, CurrentFirstLODIdx - 1);
		AdvanceTo(ETickState::Done, ETickThread::None);
		return bSuccess;
	}

	int32 PatchedEdges = 0;

	// ensure no one modifies neighbor mapping or snapshots while we are reading them
	FReadScopeLock ScopeReadLock(EdgeFixup->ActiveGroup->RWLock);

	// Grab neighbor snapshots (null if they don't exist) -- IN A THREAD SAFE MANNER
	FNeighborSnapshots NeighborSnapshots;
	EdgeFixup->GetNeighborSnapshots(NeighborSnapshots);

	// patch edges for ALL mips that are requested
	if (NeighborSnapshots.ExistingNeighbors != ENeighborFlags::None)
	{
		PatchedEdges += EdgeFixup->PatchTextureEdgesForStreamingMips(PendingFirstLODIdx, CurrentFirstLODIdx, DestMipInfos, NeighborSnapshots);
	}

	PROVIDER_DEBUG_LOG(TEXT("---- PollMips Coord (%d,%d) Mips (%d ... %d) -- PATCHED %d edges"), EdgeFixup->GetGroupCoord().X, EdgeFixup->GetGroupCoord().Y, PendingFirstLODIdx, CurrentFirstLODIdx - 1, PatchedEdges);

	AdvanceTo(ETickState::Done, ETickThread::None);
	return bSuccess;
}

void FLandscapeTextureMipEdgeOverrideProvider::CleanUp(const FTextureUpdateSyncOptions& SyncOptions)
{
	AdvanceTo(ETickState::Done, ETickThread::None);
}

void FLandscapeTextureMipEdgeOverrideProvider::Cancel(const FTextureUpdateSyncOptions& SyncOptions)
{
}

FTextureMipDataProvider::ETickThread FLandscapeTextureMipEdgeOverrideProvider::GetCancelThread() const
{
	return ETickThread::None;
}

ULandscapeTextureMipEdgeOverrideFactory::ULandscapeTextureMipEdgeOverrideFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

ULandscapeTextureMipEdgeOverrideFactory* ULandscapeTextureMipEdgeOverrideFactory::AddTo(UTexture2D* TargetTexture)
{
	check(TargetTexture);

	// try to get an existing factory
	ULandscapeTextureMipEdgeOverrideFactory* Factory = TargetTexture->GetAssetUserData<ULandscapeTextureMipEdgeOverrideFactory>();
	if (Factory == nullptr)
	{
		// create a new one (with TargetTexture as outer)
		Factory = NewObject<ULandscapeTextureMipEdgeOverrideFactory>(TargetTexture);
		Factory->Texture = TargetTexture;
		TargetTexture->AddAssetUserData(Factory);
	}

	check(Factory->Texture == TargetTexture);
	check(Factory->GetOuter() == TargetTexture);

	return Factory;
}

void ULandscapeTextureMipEdgeOverrideFactory::SetupEdgeFixup(ULandscapeHeightmapTextureEdgeFixup* InEdgeFixup)
{
	this->EdgeFixup = InEdgeFixup;
}

void ULandscapeTextureMipEdgeOverrideFactory::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << Texture;
}

void ULandscapeTextureMipEdgeOverrideFactory::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);
	ULandscapeTextureMipEdgeOverrideFactory* const TypedThis = Cast<ULandscapeTextureMipEdgeOverrideFactory>(InThis);
	Collector.AddReferencedObject(TypedThis->Texture);
	Collector.AddReferencedObject(TypedThis->EdgeFixup);
}

FLandscapeTextureStorageMipProvider::FLandscapeTextureStorageMipProvider(ULandscapeTextureStorageProviderFactory* InFactory)
	: FTextureMipDataProvider(InFactory->Texture, ETickState::Init, ETickThread::Async)
{
	this->Factory = InFactory;
	TextureName = InFactory->Texture->GetFName();
}

FLandscapeTextureStorageMipProvider::~FLandscapeTextureStorageMipProvider()
{
}

ULandscapeTextureStorageProviderFactory::ULandscapeTextureStorageProviderFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void FLandscapeTexture2DMipMap::Serialize(FArchive& Ar, UObject* Owner, uint32 SaveOverrideFlags)
{
	Ar << SizeX;
	Ar << SizeY;
	Ar << bCompressed;
	BulkData.SerializeWithFlags(Ar, Owner, SaveOverrideFlags);
}

template<typename T, typename F>
static bool SerializeArray(FArchive& Ar, TArray<T>& Array, F&& SerializeElementFn)
{
	int32 Num = Array.Num();
	Ar << Num;
	if (Ar.IsLoading())
	{
		if (Num < 0)
		{
			return false;
		}
		else
		{
			Array.SetNum(Num);
			for (int32 Index = 0; Index < Num; ++Index)
			{
				SerializeElementFn(Ar, Index, Array[Index]);
			}
		}
	}
	else
	{
		for (int32 Index = 0; Index < Num; ++Index)
		{
			SerializeElementFn(Ar, Index, Array[Index]);
		}
	}
	return true;
}

void ULandscapeTextureStorageProviderFactory::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	//  mip 0                                                      mip N
	// 	high rez <---------------------------------------------> low rez
	// 	[ Optional Mips ][            Non Optional Mips                ]
	// 	                 [ Streaming Mips ][ Non Streaming Inline Mips ]

	int32 OptionalMips = Mips.Num() - NumNonOptionalMips;
	check(OptionalMips >= 0);

	int32 FirstInlineMip = Mips.Num() - NumNonStreamingMips;
	check(FirstInlineMip >= 0);

	Ar << NumNonOptionalMips;
	Ar << NumNonStreamingMips;
	Ar << LandscapeGridScale;

	SerializeArray(Ar, Mips,
		[this, OptionalMips, FirstInlineMip](FArchive& Ar, int32 Index, FLandscapeTexture2DMipMap& Mip)
		{
			// select bulk data flags for optional/streaming/inline mips
			uint32 BulkDataFlags;
			if (Index < OptionalMips)
			{
				// optional mip
				BulkDataFlags = BULKDATA_Force_NOT_InlinePayload | BULKDATA_OptionalPayload;
			}
			else if (Index < FirstInlineMip)
			{
				// streaming mip
				bool bDuplicateNonOptionalMips = false; // TODO [chris.tchou] : if we add support for optional mips, we might need to calculate this.
				BulkDataFlags = BULKDATA_Force_NOT_InlinePayload | (bDuplicateNonOptionalMips ? BULKDATA_DuplicateNonOptionalPayload : 0);
			}
			else
			{
				// non streaming inline mip (can be single use as we only need to upload to GPU once, are never streamed out)
				BulkDataFlags = BULKDATA_ForceInlinePayload | BULKDATA_SingleUse;
			}
			Mip.Serialize(Ar, this, BulkDataFlags);
		});

	Ar << Texture;
}

void ULandscapeTextureStorageProviderFactory::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);
	ULandscapeTextureStorageProviderFactory* const TypedThis = Cast<ULandscapeTextureStorageProviderFactory>(InThis);
	Collector.AddReferencedObject(TypedThis->Texture);
	Collector.AddReferencedObject(TypedThis->EdgeFixup);
}

FStreamableRenderResourceState ULandscapeTextureStorageProviderFactory::GetResourcePostInitState(const UTexture* Owner, bool bAllowStreaming)
{
	// We are using the non-offline mode to upload these textures currently, so we don't need to consider mip tails.
	// (RHI will handle it during upload, just less optimal than having them pre-packed)
	// If we ever want to optimize the GPU upload by using the offline mode, we can add the logic here to take mip tails into account.
	const int32 PlatformNumMipsInTail = 1;

	const int32 TotalMips = Mips.Num();
	const int32 ExpectedAssetLODBias = FMath::Clamp<int32>(Owner->GetCachedLODBias() - Owner->NumCinematicMipLevels, 0, TotalMips - 1);
	const int32 MaxRuntimeMipCount = FMath::Min<int32>(GMaxTextureMipCount, FStreamableRenderResourceState::MAX_LOD_COUNT);

	const int32 NumMips = FMath::Min<int32>(TotalMips - ExpectedAssetLODBias, MaxRuntimeMipCount);

	bool bTextureIsStreamable = true;		// landscape texture storage is always streamable (we should not use it for platforms that are not)

	// clamp non-optional and non-streaming mips to reflect potentially reduced mip count because of bias
	const int32 BiasedNumNonOptionalMips = FMath::Min<int32>(NumMips, NumNonOptionalMips);
	const int32 NumOfNonStreamingMips = FMath::Min<int32>(NumMips, NumNonStreamingMips);

	// Optional mips must be streaming mips :
	check(BiasedNumNonOptionalMips >= NumOfNonStreamingMips);

	if (NumOfNonStreamingMips == NumMips)
	{
		bTextureIsStreamable = false;
	}

	const int32 AssetMipIdxForResourceFirstMip = FMath::Max<int32>(0, TotalMips - NumMips);

	const bool bMakeStreamable = bAllowStreaming;
	int32 NumRequestedMips = 0;
	if (!bTextureIsStreamable)
	{
		// in Editor , NumOfNonStreamingMips may not be all mips
		// but once we cook it will be
		// so check this early to make behavior consistent
		NumRequestedMips = NumMips;
	}
	else if (bMakeStreamable && IStreamingManager::Get().IsRenderAssetStreamingEnabled(EStreamableRenderAssetType::Texture))
	{
		NumRequestedMips = NumOfNonStreamingMips;
	}
	else
	{
		// we are not streaming (bMakeStreamable is false)
		// but this may select a mip below the top mip
		// (due to cinematic lod bias)
		// but only if the texture itself is streamable

		// Adjust CachedLODBias so that it takes into account FStreamableRenderResourceState::AssetLODBias.
		const int32 ResourceLODBias = FMath::Max<int32>(0, Owner->GetCachedLODBias() - AssetMipIdxForResourceFirstMip);

		// Ensure NumMipsInTail is within valid range to safeguard on the above expressions. 
		const int32 NumMipsInTail = FMath::Clamp<int32>(PlatformNumMipsInTail, 1, NumMips);

		// Bias is not allowed to shrink the mip count below NumMipsInTail.
		NumRequestedMips = FMath::Max<int32>(NumMips - ResourceLODBias, NumMipsInTail);

		// If trying to load optional mips, check if the first resource mip is available.
		if (NumRequestedMips > BiasedNumNonOptionalMips && !DoesMipDataExist(AssetMipIdxForResourceFirstMip))
		{
			NumRequestedMips = BiasedNumNonOptionalMips;
		}

		// Ensure we don't request a top mip in the NonStreamingMips
		NumRequestedMips = FMath::Max(NumRequestedMips, NumOfNonStreamingMips);
	}

	const int32 MinRequestMipCount = 0;
	if (NumRequestedMips < MinRequestMipCount && MinRequestMipCount < NumMips)
	{
		NumRequestedMips = MinRequestMipCount;
	}

	FStreamableRenderResourceState PostInitState;
	PostInitState.bSupportsStreaming = bMakeStreamable;
	PostInitState.NumNonStreamingLODs = IntCastChecked<uint8>(NumOfNonStreamingMips);
	PostInitState.NumNonOptionalLODs = IntCastChecked<uint8>(BiasedNumNonOptionalMips);
	PostInitState.MaxNumLODs = IntCastChecked<uint8>(NumMips);
	PostInitState.AssetLODBias = IntCastChecked<uint8>(AssetMipIdxForResourceFirstMip);
	PostInitState.NumResidentLODs = IntCastChecked<uint8>(NumRequestedMips);
	PostInitState.NumRequestedLODs = IntCastChecked<uint8>(NumRequestedMips);

	return PostInitState;
}

bool ULandscapeTextureStorageProviderFactory::GetInitialMipData(int32 FirstMipToLoad, TArrayView<void*> OutMipData, TArrayView<int64> OutMipSize, FStringView DebugContext)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeTextureStorageProviderFactory::GetInitialMipData);
	check(FirstMipToLoad >= 0);
	int32 NumberOfMipsToLoad = OutMipData.Num();
	check(NumberOfMipsToLoad > 0);
	check(OutMipData.GetData());

	const int32 LoadableMips = Mips.Num();
	check(NumberOfMipsToLoad == LoadableMips - FirstMipToLoad);

	const int32 MipLoadEnd = FirstMipToLoad + NumberOfMipsToLoad;
	check(MipLoadEnd <= LoadableMips);

	check(OutMipSize.Num() == NumberOfMipsToLoad || OutMipSize.Num() == 0);

	int32 NumMipsCached = 0;

	// Handle the case where we inlined more mips than we intend to upload immediately, by discarding the unneeded mips
	for (int32 MipIndex = 0; MipIndex < FirstMipToLoad && MipIndex < LoadableMips; ++MipIndex)
	{
		FLandscapeTexture2DMipMap& Mip = Mips[MipIndex];
		if (Mip.BulkData.IsBulkDataLoaded())
		{
			// we know inline mips are set up with the discard after first use flag, so simply locking then unlocking will cause them to be deleted
			Mip.BulkData.Lock(LOCK_READ_ONLY);
			Mip.BulkData.Unlock();
		}
	}

	// Get data for the remaining mips from bulk data.
	for (int32 MipIndex = FirstMipToLoad; MipIndex < MipLoadEnd; ++MipIndex)
	{
		FLandscapeTexture2DMipMap& Mip = Mips[MipIndex];
		const int64 DestBytes = Mip.SizeX * Mip.SizeY * 4;
		const int64 BulkDataSize = Mip.BulkData.GetBulkDataSize();
		if (BulkDataSize > 0)
		{
			void* SourceData = nullptr;
			const bool bDiscardInternalCopy = true;
			Mip.BulkData.GetCopy(&SourceData, bDiscardInternalCopy);
			check(SourceData);

			if (Mip.bCompressed)
			{
				// decompress the mip to a new buffer, then free the original buffer
				uint8* DestData = static_cast<uint8*>(FMemory::Malloc(DestBytes));
				DecompressMip((uint8*)SourceData, BulkDataSize, DestData, DestBytes, MipIndex);
				OutMipData[MipIndex - FirstMipToLoad] = DestData;
				FMemory::Free(SourceData);
			}
			else
			{
				// mip is uncompressed, it should already be the correct size, and we can just use the source data buffer directly
				check(BulkDataSize == DestBytes);
				OutMipData[MipIndex - FirstMipToLoad] = SourceData;
			}

			if (OutMipSize.Num() > 0)
			{
				OutMipSize[MipIndex - FirstMipToLoad] = DestBytes;
			}
			NumMipsCached++;
		}
	}

	if (NumMipsCached != (LoadableMips - FirstMipToLoad))
	{
		UE_LOGF(LogLandscape, Warning, "ULandscapeTextureStorageProviderFactory::TryLoadMips failed for %.*ls, NumMipsCached: %d, LoadableMips: %d, FirstMipToLoad: %d",
			DebugContext.Len(), DebugContext.GetData(),
			NumMipsCached,
			LoadableMips,
			FirstMipToLoad);

		// Unable to cache all mips. Release memory for those that were cached.
		for (int32 MipIndex = FirstMipToLoad; MipIndex < LoadableMips; ++MipIndex)
		{
			FLandscapeTexture2DMipMap& Mip = Mips[MipIndex];
			UE_LOG(LogLandscape, Verbose, TEXT("  Mip %d, BulkDataSize: %" INT64_FMT),
				MipIndex,
				Mip.BulkData.GetBulkDataSize());

			if (OutMipData[MipIndex - FirstMipToLoad])
			{
				FMemory::Free(OutMipData[MipIndex - FirstMipToLoad]);
				OutMipData[MipIndex - FirstMipToLoad] = nullptr;
			}
			if (OutMipSize.Num() > 0)
			{
				OutMipSize[MipIndex - FirstMipToLoad] = 0;
			}
		}
		return false;
	}
	return true;
}


#if WITH_EDITORONLY_DATA
ULandscapeTextureStorageProviderFactory* ULandscapeTextureStorageProviderFactory::ApplyTo(UTexture2D* InTargetTexture, const FVector& InLandsapeGridScale, int32 InHeightmapCompressionMipThreshold, int32 InNumSubsections, int32 InSubsectionSizeQuads)
{
	check(InTargetTexture);
	check(InTargetTexture->Source.IsValid())

	check(InTargetTexture->Source.GetFormat() == TSF_BGRA8);

	// try to get an existing factory
	ULandscapeTextureStorageProviderFactory* Factory= InTargetTexture->GetAssetUserData<ULandscapeTextureStorageProviderFactory>();
	if (Factory == nullptr)
	{
		// create a new one
		Factory = NewObject<ULandscapeTextureStorageProviderFactory>(InTargetTexture);
		Factory->Texture = InTargetTexture;
		InTargetTexture->AddAssetUserData(Factory);
	}

	Factory->UpdateCompressedDataFromSource(InTargetTexture, InLandsapeGridScale, InHeightmapCompressionMipThreshold, InNumSubsections, InSubsectionSizeQuads);

	return Factory;
}

void ULandscapeTextureStorageProviderFactory::UpdateCompressedDataFromSource(UTexture2D* InTargetTexture, const FVector& InLandsapeGridScale, int32 InHeightmapCompressionMipThreshold, int32 InNumSubsections, int32 InSubsectionSizeQuads)
{
	check(this->Texture == InTargetTexture);

	int32 Width = InTargetTexture->Source.GetSizeX();
	int32 Height = InTargetTexture->Source.GetSizeY();
	int32 MipCount = InTargetTexture->Source.GetNumMips();

	// need this to properly calculate normals
	this->LandscapeGridScale = InLandsapeGridScale;

	// check if this is a shared texture (multi-component texture)
	const int32 ComponentWidth = InNumSubsections * (InSubsectionSizeQuads + 1);
	check(ComponentWidth > 0);
	const int32 NumComponentsX = Width / ComponentWidth;
	const int32 NumComponentsY = Height / ComponentWidth;

	// if the texture is shared, or has subsections, there is a duplicate row/column at every BoundaryOffset 
	const bool bHasDuplicateData = NumComponentsX > 1 || NumComponentsY > 1 || InNumSubsections > 1;
	this->BoundaryOffset = InSubsectionSizeQuads + 1;
	this->BoundaryCountX = bHasDuplicateData ? (NumComponentsX * InNumSubsections - 1) : 0;
	this->BoundaryCountY = bHasDuplicateData ? (NumComponentsY * InNumSubsections - 1) : 0;

	// calculate number of non-streaming mips
	// TODO [chris.tchou] : we could make this calculation platform specific, like Texture2D does.
	// We would have to calculate it during serialization, when we know the target platform.
	{
		int32 NumberOfNonStreamingMips = 1;

		// TODO [chris.tchou] : we could ensure Mip Tails are not streamed, as it's more overhead to upload.
		// we would have to query TextureCompressorModule for platform specific info.
		// Ignoring the mip tail should still work, just less optimal as it does more work at runtime to blit into the mip tail.
		int32 NumMipsInTail = 0;

		NumberOfNonStreamingMips = FMath::Max(NumberOfNonStreamingMips, NumMipsInTail);
		NumberOfNonStreamingMips = FMath::Max(NumberOfNonStreamingMips, UTexture2D::GetStaticMinTextureResidentMipCount());
		NumberOfNonStreamingMips = FMath::Min(NumberOfNonStreamingMips, MipCount);
		this->NumNonStreamingMips = NumberOfNonStreamingMips;
	}

	// calculate number of non-optional mips
	{
		// for now, landscape texture storage does not have any optional mips
		this->NumNonOptionalMips = MipCount;
	}

	this->Mips.Empty();
	int32 MipWidth = Width;
	int32 MipHeight = Height;
	for (int32 MipIndex = 0; MipIndex < MipCount; MipIndex++)
	{
		FLandscapeTexture2DMipMap* Mip = new(this->Mips) FLandscapeTexture2DMipMap();
		Mip->SizeX = MipWidth;
		Mip->SizeY = MipHeight;

		TArray64<uint8> MipData;
		InTargetTexture->Source.GetMipData(MipData, MipIndex);

		// Store mips below the threshold size uncompressed
		if ((MipWidth < InHeightmapCompressionMipThreshold) || (MipHeight < InHeightmapCompressionMipThreshold))
		{
			Mip->bCompressed = false;
			CopyMipToBulkData(MipIndex, MipWidth, MipHeight, MipData.GetData(), MipData.Num(), Mip->BulkData);
		}
		else
		{
			Mip->bCompressed = true;
			CompressMipToBulkData(MipIndex, MipWidth, MipHeight, MipData.GetData(), MipData.Num(), Mip->BulkData, BoundaryOffset >> MipIndex, BoundaryCountX, BoundaryCountY);
		}

		MipWidth = FMath::Max(MipWidth / 2, 1);
		MipHeight = FMath::Max(MipHeight / 2, 1);
	}
}
#endif // WITH_EDITORONLY_DATA


void ULandscapeTextureStorageProviderFactory::SetupEdgeFixup(ULandscapeHeightmapTextureEdgeFixup* InEdgeFixup)
{
	this->EdgeFixup = InEdgeFixup;
}

// Helper to configure the AsyncFileCallBack.
void FLandscapeTextureStorageMipProvider::CreateAsyncFileCallback(const FTextureUpdateSyncOptions& SyncOptions)
{
	FThreadSafeCounter* Counter = SyncOptions.Counter;
	FTextureUpdateSyncOptions::FCallback RescheduleCallback = SyncOptions.RescheduleCallback;
	check(Counter && RescheduleCallback);

	AsyncFileCallBack = [this, Counter, RescheduleCallback](bool bWasCancelled, IBulkDataIORequest* Req)
	{
		// At this point task synchronization would hold the number of pending requests.
		Counter->Decrement();

		if (bWasCancelled)
		{
			bIORequestCancelled = true;
		}

		if (Counter->GetValue() == 0)
		{
			RescheduleCallback();
		}
	};
}

void FLandscapeTextureStorageMipProvider::ClearIORequests()
{
	for (FIORequest& IORequest : IORequests)
	{
		// If requests are not yet completed, cancel and wait.
		if (IORequest.BulkDataIORequest && !IORequest.BulkDataIORequest->PollCompletion())
		{
			IORequest.BulkDataIORequest->Cancel();
			IORequest.BulkDataIORequest->WaitCompletion();
		}
	}
	IORequests.Empty();
}

void FLandscapeTextureStorageMipProvider::Init(const FTextureUpdateContext& Context, const FTextureUpdateSyncOptions& SyncOptions)
{
	IORequests.AddDefaulted(CurrentFirstLODIdx);

	// If this resource has optional LODs and we are streaming one of them.
	if (ResourceState.NumNonOptionalLODs < ResourceState.MaxNumLODs && PendingFirstLODIdx < ResourceState.LODCountToFirstLODIdx(ResourceState.NumNonOptionalLODs))
	{
		// Generate the FilenameHash of each optional LOD before the first one requested, so that we can handle properly PAK unmount events.
		// Note that streamer only stores the hash for the first optional mip.
		for (int32 MipIdx = 0; MipIdx < PendingFirstLODIdx; ++MipIdx)
		{
			const FLandscapeTexture2DMipMap* SourceMip = Factory->GetMip(MipIdx);
			// const FTexture2DMipMap& OwnerMip = *Context.MipsView[MipIdx];
			IORequests[MipIdx].FilenameHash = SourceMip->BulkData.GetIoFilenameHash();
		}
	}

	// Otherwise validate each streamed in mip.
	for (int32 MipIdx = PendingFirstLODIdx; MipIdx < CurrentFirstLODIdx; ++MipIdx)
	{
		const FLandscapeTexture2DMipMap* SourceMip = Factory->GetMip(MipIdx);
		if (SourceMip->BulkData.IsStoredCompressedOnDisk())
		{
			// Compression at the package level is no longer supported
			continue;
		}
		else if (SourceMip->BulkData.GetBulkDataSize() <= 0)
		{
			// Invalid bulk data size.
			continue;
		}
		else
		{
			IORequests[MipIdx].FilenameHash = SourceMip->BulkData.GetIoFilenameHash();
		}
	}

	AdvanceTo(ETickState::GetMips, ETickThread::Async);
}

int32 FLandscapeTextureStorageMipProvider::GetMips(const FTextureUpdateContext& Context, int32 StartingMipIndex, const FTextureMipInfoArray& MipInfos, const FTextureUpdateSyncOptions& SyncOptions)
{
	CreateAsyncFileCallback(SyncOptions);	// this just creates it... callback has to be passed to the IO request completion to actually get called...
	check(SyncOptions.Counter != nullptr);

	DestMipInfos = MipInfos;

	FirstRequestedMipIndex = StartingMipIndex;
	while (StartingMipIndex < CurrentFirstLODIdx && MipInfos.IsValidIndex(StartingMipIndex))
	{
		const FTextureMipInfo& DestMip = MipInfos[StartingMipIndex];
		const FLandscapeTexture2DMipMap* SourceMip = Factory->GetMip(StartingMipIndex);
		if (SourceMip == nullptr || !DestMip.DestData)
		{
			break;
		}

		// Check the validity of the filename.
		if (IORequests[StartingMipIndex].FilenameHash == INVALID_IO_FILENAME_HASH)
		{
			break;
		}


		// Increment the sync counter.  This causes the system to not advance to the next tick, until RescheduleCallback() is called (by AsyncFileCallBack when counter reaches zero)
		// If a request completes immediately, then it will call the RescheduleCallback,
		// but that won't do anything because the tick would not try to acquire the lock since it is already locked.
		SyncOptions.Counter->Increment();

		int64 StreamDataSize = SourceMip->BulkData.GetBulkDataSize();

		if (SourceMip->bCompressed)
		{
			// allocate a buffer to receive the streamed data
			uint8* StreamData = static_cast<uint8*>(FMemory::Malloc(StreamDataSize));

			TRACE_IOSTORE_METADATA_SCOPE_TAG("Landscape");
			IORequests[StartingMipIndex].BulkDataIORequest.Reset(
				SourceMip->BulkData.CreateStreamingRequest(
					0,
					StreamDataSize,
					(EAsyncIOPriorityAndFlags)FMath::Clamp<int32>(AIOP_Low + (bPrioritizedIORequest ? 1 : 0), AIOP_Low, AIOP_High) | AIOP_FLAG_DONTCACHE,
					&AsyncFileCallBack,
					StreamData)
			);
		}
		else
		{
			// If DataSize is specified (optional, may be zero), then check that the size matches expectations
			if ((DestMip.DataSize != 0) && (DestMip.DataSize != StreamDataSize))
			{
				// LogPlayLevel: Error: UAT: [2024.11.08 - 17.09.47:903] [10] LogLandscape : Error : Unexpected data size for landscape mip 0 : expected 0 bytes(512 x 512), has 1048576 bytes(512 x 512 compressed: 0)
				UE_LOG(LogLandscape, Error, TEXT("Unexpected data size for landscape mip %d : expected %" UINT64_FMT " bytes (%d x %d), has %" INT64_FMT " bytes (%d x %d compressed: %d)"),
					StartingMipIndex,
					DestMip.DataSize, DestMip.SizeX, DestMip.SizeY,
					StreamDataSize, SourceMip->SizeX, SourceMip->SizeY, SourceMip->bCompressed
				);

				check(StreamDataSize == DestMip.DataSize);
			}
			TRACE_IOSTORE_METADATA_SCOPE_TAG("Landscape");
			IORequests[StartingMipIndex].BulkDataIORequest.Reset(
				SourceMip->BulkData.CreateStreamingRequest(
					0,
					StreamDataSize,
					(EAsyncIOPriorityAndFlags)FMath::Clamp<int32>(AIOP_Low + (bPrioritizedIORequest ? 1 : 0), AIOP_Low, AIOP_High) | AIOP_FLAG_DONTCACHE,
					&AsyncFileCallBack,
					(uint8*) DestMip.DestData)	// when not compressed, we can stream directly into the dest mip memory
			);
		}

		// remember the dest mip data buffer (we can't fill it out now, must wait until streaming is complete)
		IORequests[StartingMipIndex].DestMipData = static_cast<uint8*>(DestMip.DestData);

		StartingMipIndex++;
	}

	AdvanceTo(ETickState::PollMips, ETickThread::Async);
	return StartingMipIndex;	// return the mips we handled (if this is not CurrentFirstLODIdx, it will fall back to other providers)
}

bool FLandscapeTextureStorageMipProvider::PollMips(const FTextureUpdateSyncOptions& SyncOptions)
{
	using namespace UE::Landscape;

	// poll mips will run once all io requests are complete (or cancelled)

	// Notify that some files have possibly been unmounted / missing.
	if (bIORequestCancelled && !bIORequestAborted)
	{
		IRenderAssetStreamingManager& StreamingManager = IStreamingManager::Get().GetRenderAssetStreamingManager();
		for (FIORequest& IORequest : IORequests)
		{
			StreamingManager.MarkMountedStateDirty(IORequest.FilenameHash);
		}
		UE_LOGF(LogLandscape, Warning, "[%ls] FLandscapeTextureStorageMipProvider Texture stream in request failed due to IO error (Mip %d-%d).", *TextureName.ToString(), ResourceState.AssetLODBias + PendingFirstLODIdx, ResourceState.AssetLODBias + CurrentFirstLODIdx - 1);
	}

	if (!bIORequestCancelled && !bIORequestAborted)
	{
		// decompress the mips (note that this is using the dest mip data pointer we memorized during GetMips)
		for (int MipIndex = FirstRequestedMipIndex; MipIndex < CurrentFirstLODIdx; MipIndex++)
		{
			const FLandscapeTexture2DMipMap* SourceMip = Factory->GetMip(MipIndex);

			if (SourceMip->bCompressed)
			{
				uint8* SourceData = IORequests[MipIndex].BulkDataIORequest->GetReadResults();
				int64 DestDataBytes = SourceMip->SizeX * SourceMip->SizeY * 4;
				uint8* DestData = IORequests[MipIndex].DestMipData;
				Factory->DecompressMip(SourceData, SourceMip->BulkData.GetBulkDataSize(), DestData, DestDataBytes, MipIndex);
				FMemory::Free(SourceData);
			}
			else
			{
				// uncompressed streams directly into the dst mip data buffer, so nothing to do here (other than a sanity check)
				uint8* SourceData = IORequests[MipIndex].BulkDataIORequest->GetReadResults();
				check(IORequests[MipIndex].DestMipData == SourceData);
			}
		}
		
		if (ShouldPatchStreamingMipEdges())
		{
			// run mip patching if EdgeFixup is valid
			ULandscapeHeightmapTextureEdgeFixup* EdgeFixup = Factory->EdgeFixup;
			if ((EdgeFixup != nullptr) && EdgeFixup->IsActive())
			{
				int32 PatchedEdges = 0;
			
				// ensure no one modifies neighbor mapping or snapshots while we are reading them
				FReadScopeLock ScopeReadLock(EdgeFixup->ActiveGroup->RWLock);

				// Grab neighbor snapshots (null if they don't exist)
				FNeighborSnapshots NeighborSnapshots;
				EdgeFixup->GetNeighborSnapshots(NeighborSnapshots);

				// patch edges for ALL mips that are requested
				if (NeighborSnapshots.ExistingNeighbors != ENeighborFlags::None)
				{
					PatchedEdges += EdgeFixup->PatchTextureEdgesForStreamingMips(PendingFirstLODIdx, CurrentFirstLODIdx, DestMipInfos, NeighborSnapshots);
				}

				PROVIDER_DEBUG_LOG(TEXT("---- PollMips Coord (%d,%d) Mips (%d ... %d) -- PATCHED COMPRESSED %d edges"), EdgeFixup->GetGroupCoord().X, EdgeFixup->GetGroupCoord().Y, PendingFirstLODIdx, CurrentFirstLODIdx - 1, PatchedEdges);
			}
		}
	}

	ClearIORequests();

	AdvanceTo(ETickState::Done, ETickThread::None);

	return !bIORequestCancelled;	// return true if successful and it can upload the DestMip data to the GPU
}

void FLandscapeTextureStorageMipProvider::AbortPollMips()
{
	// ... cancel all streaming ops in progress ...
	for (FIORequest& IORequest : IORequests)
	{
		if (IORequest.BulkDataIORequest)
		{
			// Calling cancel() here will trigger the AsyncFileCallBack and precipitate the execution of Cancel().
			IORequest.BulkDataIORequest->Cancel();
			bIORequestAborted = true;
		}
	}
}

void FLandscapeTextureStorageMipProvider::CleanUp(const FTextureUpdateSyncOptions& SyncOptions)
{
	AdvanceTo(ETickState::Done, ETickThread::None);
}

void FLandscapeTextureStorageMipProvider::Cancel(const FTextureUpdateSyncOptions& SyncOptions)
{
	ClearIORequests();
}

FTextureMipDataProvider::ETickThread FLandscapeTextureStorageMipProvider::GetCancelThread() const
{
	return IORequests.Num() ? FTextureMipDataProvider::ETickThread::Async : FTextureMipDataProvider::ETickThread::None;
}

void ULandscapeTextureStorageProviderFactory::CopyMipToBulkData(int32 MipIndex, int32 MipSizeX, int32 MipSizeY, uint8* SourceData, int32 SourceDataBytes, FByteBulkData& DestBulkData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeTextureStorageProviderFactory::CopyMipToBulkData);
	DestBulkData.Lock(LOCK_READ_WRITE);

	int32 TotalPixels = MipSizeX * MipSizeY;
	checkf(SourceDataBytes == TotalPixels * 4, TEXT("SourceDataBytes: %d TotalPixels: %d MipIndex: %d MipSizeX: %d MipSizeY: %d"), SourceDataBytes, TotalPixels, MipIndex, MipSizeX, MipSizeY);

	int32 DestBytes = SourceDataBytes;
	uint8* DestData = DestBulkData.Realloc(DestBytes);

	memcpy(DestData, SourceData, DestBytes);

	DestBulkData.Unlock();
}

void ULandscapeTextureStorageProviderFactory::CompressMipToBulkData(int32 MipIndex, int32 MipSizeX, int32 MipSizeY, uint8* SourceData, int32 SourceDataBytes, FByteBulkData& DestBulkData, int32 InMipBoundaryOffset, int32 InBoundaryCountX, int32 InBoundaryCountY)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeTextureStorageProviderFactory::CompressMipToBulkData);

	DestBulkData.Lock(LOCK_READ_WRITE);

	int32 DestPixels = MipSizeX * MipSizeY;
	check(SourceDataBytes == DestPixels * 4);
	check(DestPixels >= 16);	// shouldn't be used on very small mips

	const int32 NumSourcePixels = SourceDataBytes / 4;

	// If the texture is shared or has subsections, there is a duplicate row/column at every BoundaryOffset
	check(InBoundaryCountX >= 0 && InBoundaryCountY >= 0);
	// High index mips drop the row/column duplication at low resolutions. Offset is only relevant when duplication occurs atleast after every other pixel (Offset >= 2)
	const bool bHasDuplicateData = (InBoundaryCountX > 0 || InBoundaryCountY > 0) && InMipBoundaryOffset > 1;
	const int32 MipBoundaryCountX = bHasDuplicateData ? InBoundaryCountX : 0;
	const int32 MipBoundaryCountY = bHasDuplicateData ? InBoundaryCountY : 0;

	// Remove the duplicated row/columns along subsection boundaries in the destination texture 
	const int32 DestWidth = MipSizeX - MipBoundaryCountX;
	const int32 DestHeight = MipSizeY - MipBoundaryCountY;
	check(DestWidth >= 0 && DestHeight >= 0);
	const int32 NumDestPixels = DestWidth * DestHeight;

	/* DestData consists of a 16 bit height per pixel, then an 8:8 normal per edge pixel, edge normals are handled separately and do not remove duplicates
	*  EdgePixels = (MipSizeX + MipSizeY) * 2 - 4 
	*  TotalBytes = 2 * NumDestPixels + 2 * EdgePixels */
	int32 DestBytes = (NumDestPixels + (MipSizeX + MipSizeY) * 2 - 4) * 2;
	uint8* DestData = DestBulkData.Realloc(DestBytes);

	// delta encode the heights -- this (usually) greatly reduces the variance in the data, which makes it compress much better on disk when package compression is applied.
	uint16 LastHeight = 32768;
	int32 DuplicateRowsSkipped = 0;

	for (int32 Y = 0; Y < MipSizeY; Y++)
	{
		int32 SrcLineOffsetInPixels = Y * MipSizeX;
		int32 DestLineOffsetInPixels = (Y - DuplicateRowsSkipped) * DestWidth;
		FColor* Src = (FColor*)&SourceData[SrcLineOffsetInPixels * 4] ;
		uint8* Dst = &DestData[DestLineOffsetInPixels * 2];

		// Ignore duplicate rows, they will be regenerated
		if (bHasDuplicateData && UE::Landscape::Private::IsDuplicateCoord(Y, InMipBoundaryOffset))
		{
			DuplicateRowsSkipped++;
			continue;
		}

		for (int32 X = 0; X < MipSizeX; X++)
		{
			// Ignore duplicate columns, they will be regenerated
			if (bHasDuplicateData && UE::Landscape::Private::IsDuplicateCoord(X, InMipBoundaryOffset))
			{
				Src += 1;
				continue;
			}

			// texture data is stored as BGRA, or [normal x, height low bits, height high bits, normal y]
			uint16 Height = Src->R * 256 + Src->G;
			uint16 DeltaHeight = Height - LastHeight;
			LastHeight = Height;

			// store delta height
			Dst[0] = DeltaHeight >> 8;
			Dst[1] = DeltaHeight & 0xff;
			Dst += 2;
			Src += 1;
		}
	}

	// edge normals are written after all the height data 
	int32 DestOffset = NumDestPixels * 2;

	// capture normals along the edge (delta encoded clockwise starting from top left)
	// for efficiency, edge normals do not remove duplicate row/columns
	uint8 LastNormalX = 128;
	uint8 LastNormalY = 128;

	auto EncodeNormal = [&LastNormalX, &LastNormalY, SourceData, DestData, MipSizeX, &DestOffset](int32 X, int32 Y)
	{
		int32 SourceOffset = (Y * MipSizeX + X) * 4;
		uint8 NormalX = SourceData[SourceOffset + 0];
		uint8 NormalY = SourceData[SourceOffset + 3];
		DestData[DestOffset + 0] = NormalX - LastNormalX;
		DestData[DestOffset + 1] = NormalY - LastNormalY;
		LastNormalX = NormalX;
		LastNormalY = NormalY;
		DestOffset += 2;
	};

	for (int32 X = 0; X < MipSizeX; X++)				// [0 ... MipSizeX-1], 0
	{
		EncodeNormal(X, 0);
	}

	for (int32 Y = 1; Y < MipSizeY; Y++)				// MipSizeX-1, [1 ... MipSizeY-1]
	{
		EncodeNormal(MipSizeX - 1, Y);
	}

	for (int32 X = MipSizeX-2; X >= 0; X--)				// [MipSizeX-2 ... 0], MipSizeY-1
	{
		EncodeNormal(X, MipSizeY - 1);
	}

	for (int32 Y = MipSizeY-2; Y >= 1; Y--)				// 0, [MipSizeY-2 ... 1]
	{
		EncodeNormal(0, Y);
	}

	check(DestOffset == DestBytes);

	DestBulkData.Unlock();
}

// Compute the normal of the triangle formed by the 3 points (in winding order).
inline FVector ComputeTriangleNormal(const FVector& InPoint0, const FVector& InPoint1, const FVector& InPoint2)
{
	FVector Normal = (InPoint0 - InPoint1).Cross(InPoint1 - InPoint2);
	Normal.Normalize();
	return Normal;
}

// This function is unused, but explains how we get from ComputeTriangleNormal above to the optimized version below.
// When computing normals on a height grid, you can simplify the math, and only care about delta height in the +X and +Y directions.
// Then we can take advantage of the zeros in DX and DY to simplify the cross product:
inline FVector3f ComputeGridNormalFromDeltaHeights(float DHDX, float DHDY, int32 MipScale, const FVector& LandscapeGridScale)
{
	// by placing the origin at the center vertex, and ensuring one vector is along +X and the other along +Y, a lot of math is removed:
	// FVector3f Center(0.0f, 0.0f, 0.0f);
	FVector3f DX(MipScale * LandscapeGridScale.X, 0.0, DHDX * (float)LandscapeGridScale.Z);
	FVector3f DY(0.0, MipScale * LandscapeGridScale.Y, DHDY * (float)LandscapeGridScale.Z);

	FVector3f Normal; // = (DX-Center).Cross(DY-Center);
	{
		Normal.X = /*DX.Y * DY.Z*/ - DX.Z * DY.Y;		// DHDX * (-LGS.Z * LGS.Y * MipScale)		<< note values in parens are constant in the inner loop
		Normal.Y = /*DX.Z * DY.X*/ - DX.X * DY.Z;		// DHDY * (-LGS.Z * LGS.X * MipScale)
		Normal.Z = DX.X * DY.Y /* - DX.Y * DY.X*/;		// (LGS.X * LGS.Y * MipScale * MipScale)	<< fully constant in the inner loop
	}
	Normal.Normalize();
	return Normal;
}

inline FVector2f CalculatePremultU16(int32 MipIndex, const FVector& LandscapeGridScale)
{
	// We optimize the cross product calculation in the inner loop by precalculating the DHDX and DHDY multipliers.
	int32 MipScale = 1 << MipIndex;

	// Note that we're also doing an optimization trick by scaling the resulting vector such that CrossProductResult.Z == 1.0.
	// Since we pass the result through Normalize(), that scale factor doesn't matter -- but it's faster to calculate that way.
	// LANDSCAPE_ZSCALE comes from the fact that we are operating on the integer height values, and this converts the integer heights to landscape space
	// (and then the LandscapeGridScale converts that to world space)
	float ScaleFactor = -LANDSCAPE_ZSCALE / (LandscapeGridScale.X * LandscapeGridScale.Y * MipScale);
	FVector2f PremultU16;
	PremultU16.X = LandscapeGridScale.Z * LandscapeGridScale.Y * ScaleFactor;
	PremultU16.Y = LandscapeGridScale.Z * LandscapeGridScale.X * ScaleFactor;
	return PremultU16;
}

// This takes it a few steps further : we've minimized the math here by premultiplying everything related to LGS, MipScale, and LandscapeScale into PremultLGS
// We've also scaled up the results so that Normal.Z == 1, which reduces the math used by the Normalize.
inline FVector3f ComputeGridNormalFromDeltaHeightsPremultU16(int32 DHDX, int32 DHDY, const FVector2f& PremultU16)
{
	FVector3f Normal; // = DX.Cross(DY);
	{
		// we've calculated PremultU16 to ensure Normal.Z is 1.0 (see CalculatePremultU16), which saves some math in Normalize()
		Normal.X = DHDX * PremultU16.X;
		Normal.Y = DHDY * PremultU16.Y;
		// Normal.Z = 1.0f;
	}
	// Normal.Normalize(); optimized below
	{
		const float SquareSum = Normal.X * Normal.X + Normal.Y * Normal.Y + 1.0f;
		if (SquareSum > UE_SMALL_NUMBER)
		{
			// sqrt estimate should be more than sufficient for 8 bit results.
			const float Scale = FMath::InvSqrtEst(SquareSum);
			Normal.X *= Scale;
			Normal.Y *= Scale;
			Normal.Z = Scale;	// take advantage of knowing Normal.Z == 1.0
		}
		else
		{
			Normal.X = 0.0f;
			Normal.Y = 0.0f;
			Normal.Z = 1.0f;
		}
	}
	return Normal;
}

inline void SampleWorldPositionAtOffset(FVector& OutPoint, const uint8* MipData, int32 X, int32 Y, int32 MipSizeX, const FVector& InLandscapeGridScale)
{
	int32 OffsetBytes = (Y * MipSizeX + X) * 4;
	uint16 HeightData = MipData[OffsetBytes + 2] * 256 + MipData[OffsetBytes + 1];

	// NOTE: since we are using deltas between points to calculate the normal, we don't care about constant offsets in the position, only relative scales
	OutPoint.Set(
		X * InLandscapeGridScale.X,
		Y * InLandscapeGridScale.Y,
		LandscapeDataAccess::GetLocalHeight(HeightData) * InLandscapeGridScale.Z);
}

inline uint16 DecodeHeightU16(const FColor* Pixel)
{
	uint16 HeightData = Pixel->R * 256 + Pixel->G;
	return HeightData;
}

inline void FastNormalize(FVector3f& V)
{
	const float SquareSum = V.X * V.X + V.Y * V.Y + V.Z * V.Z;
	if (SquareSum > UE_SMALL_NUMBER)
	{
		const float Scale = FMath::InvSqrtEst(SquareSum);
		V *= Scale;
	}
	else
	{
		V.X = 0.0f;
		V.Y = 0.0f;
		V.Z = 1.0f;
	}
}

// The triangle topology is the following (where C = center, T = top, B = bottom, L = left, R = right and Nx the normals we need to interpolate):
// .  ------ . --------.
// |         | \       |
//    \                |
// |         |   \     |
//   P0'\ P1'| N0'     |	<< normals calculated for the previous line
// |         |     \   |
//        \            |
// |         |       \ |
// . - - - - TL ------ TT
// |         | \       |
//    \      |  \      |
// |         |   \     |
//   P0 \ P1 | N0 \ N1 |
// |         |     \   |
//        \  |      \  |
// |         |       \ |
// . - - - - LL ------ CC   << current pixel being processed
//
// we calculate normals while we decompress, as a single pass gives better cache coherency.
// while iterating each interior pixel left to right, top to bottom, we:
// 1) Decode Height at CC (current pixel)
// 2) Write Height at CC
// 3) Compute N0/N1 using heights at CC/TT/TL/LL (all previously decoded)
// 4) Complete Normal calculation for TL == (P0' + P1' + N0') + P1 + N0 + N1
// 5) Write Normal for TL
// 6) Store Partial Normal for LL in PrevLine cache -- stores P0 + P1 + N0

void ULandscapeTextureStorageProviderFactory::DecompressMip(uint8* SourceData, int64 SourceDataBytes, uint8* DestData, int64 DestDataBytes, int32 MipIndex)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeTextureStorageProviderFactory::DecompressMip);
	
	check(SourceData && DestData);

	FLandscapeTexture2DMipMap& Mip = Mips[MipIndex];

	check(Mip.bCompressed);		// uncompressed should be handled outside of this function

	const int32 DestWidth = Mip.SizeX;
	const int32 DestHeight = Mip.SizeY;

	// If the texture is shared, or there are subsections, need to add back a duplicate row/column at every BoundaryOffset 
	const int32 MipBoundaryOffset = BoundaryOffset >> MipIndex;
	// High index mips drop the row/column duplication at low resolutions. Offset is only relevant when duplication occurs at least after every other pixel (Offset >= 2)
	const bool bHasDuplicateData = (BoundaryCountX > 0 || BoundaryCountY > 0) && MipBoundaryOffset > 1;
	const int32 MipBoundaryCountX = bHasDuplicateData ? BoundaryCountX : 0;
	const int32 MipBoundaryCountY = bHasDuplicateData ? BoundaryCountY : 0;
	check(!bHasDuplicateData || FMath::IsPowerOfTwo(MipBoundaryOffset));

	const int32 SrcWidth = DestWidth - MipBoundaryCountX;
	const int32 SrcHeight = DestHeight - MipBoundaryCountY;
	check(SrcHeight >= 0 && SrcWidth >= 0);

	int NumSrcPixels = SrcWidth * SrcHeight;

	// Verify the Src heightmap data has removed the duplicate row/columns
	// Edge normals do not strip the duplicated row/columns
	check(SourceDataBytes == (NumSrcPixels + (DestWidth + DestHeight) * 2 - 4) * 2); // 2 bytes (height) for each pixel, plus 2 bytes (normal x/y) for each border pixel
	check(DestDataBytes == DestWidth * DestHeight * 4);

	// save some multiplying by premultiplying the grid scales, mip scale and ZScale
	FVector2f PremultU16 = CalculatePremultU16(MipIndex, LandscapeGridScale);

	// current center pixel height
	// (also used to delta decode the heights - initial value must match the initial value used during encoding)
	uint16 CC = 32768;

	// partial normal results recorded for the previous line
	TArray<FVector3f, TInlineAllocator<512>> PrevLinePartialNormals;
	PrevLinePartialNormals.SetNumZeroed(DestWidth);

	int32 DuplicateRowsSkipped = 0;
	int32 PreviousRowOffset = 1;
	// iterate each line
	for (int32 Y = 0; Y < DestHeight; Y++)
	{
		// Offset the src since the data has removed all of the duplicate rows
		int32 SrcLineOffsetInPixels = (Y - DuplicateRowsSkipped) * SrcWidth;
		int32 DestLineOffsetInPixels = Y * DestWidth;
		uint8* Src = &SourceData[SrcLineOffsetInPixels * 2];
		FColor* Dst = (FColor*)&DestData[DestLineOffsetInPixels * 4];

		if (Y == 0)
		{
			// just decode heights for the first line (normals don't matter they will be stomped below)
			for (int32 X = 0; X < DestWidth; X++)
			{
				// Skip the duplicate column. Duplicate columns will be copied once at the end of function
				if (bHasDuplicateData && UE::Landscape::Private::IsDuplicateCoord(X, MipBoundaryOffset))
				{
					Dst++;
					continue;
				}

				uint16 DeltaHeight = Src[0] * 256 + Src[1];
				CC += DeltaHeight;
				*Dst = FColor(CC >> 8, CC & 0xff, 128, 128);
				Src += 2;
				Dst++;
			}
		}
		else
		{
			// Duplicate rows will be copied once at the end of function
			if (bHasDuplicateData && UE::Landscape::Private::IsDuplicateCoord(Y, MipBoundaryOffset))
			{
				DuplicateRowsSkipped++;
				PreviousRowOffset = 2;
				continue;
			}

			// Duplicate rows are not considered in the normal calculation
			// The top pixel is two rows above after skipping a duplicate row
			const int32 TTOffset = DestWidth * PreviousRowOffset;

			// compute initial values (first pixel)
			FVector3f P1(ForceInitToZero), P01(ForceInitToZero);	// previous quad N1 and (N0+N1) normals
			uint16 TT;												// previous quad TT height
			{
				uint16 DeltaHeight = Src[0] * 256 + Src[1];
				CC += DeltaHeight;
				*Dst = FColor(CC >> 8, CC & 0xff, 128, 128);

				// load TT for first pixel (becomes TL for second pixel)
				TT = DecodeHeightU16(Dst - TTOffset);

				Src += 2;
				Dst++;
			}

			int32 PrevColumnOffset = -1;
			// rest of the pixels in the line
			for (int32 X = 1; X < DestWidth; X++)
			{ 
				// Duplicate column data is copied once at the end of function
				if (bHasDuplicateData && UE::Landscape::Private::IsDuplicateCoord(X, MipBoundaryOffset))
				{
					Dst++;
					PrevColumnOffset = -2;
					continue;
				}

				// re-use previous pixel TT and CC as this pixel TL and LL
				uint16 TL = TT;
				uint16 LL = CC;

				// 1) Decode Height at CC
				uint16 DeltaHeight = Src[0] * 256 + Src[1];
				CC += DeltaHeight;

				// load TT
				TT = DecodeHeightU16(Dst - TTOffset);

				// 2) Write Height at CC (normals get written during processing of the next line)
				*Dst = FColor(CC >> 8, CC & 0xff, 128, 128);

				// 3) Compute local normals N0/N1 for the current quad (CC/TT/TL/LL)
				FVector3f N0 = ComputeGridNormalFromDeltaHeightsPremultU16(CC - LL, LL - TL, PremultU16);
				FVector3f N1 = ComputeGridNormalFromDeltaHeightsPremultU16(TT - TL, CC - TT, PremultU16);
				FVector3f N01 = N0 + N1;

				// 4) Complete Normal calculation for TL - this takes the partial result from the previous line and fills in the rest
				FVector3f TL_Normal = PrevLinePartialNormals[X + PrevColumnOffset] + P1 + N01;
				FastNormalize(TL_Normal);

				// 5) Write Normal for TL
				Dst[-TTOffset + PrevColumnOffset].B = static_cast<uint8>(FMath::Clamp((TL_Normal.X * 127.5f + 127.5f), 0.0f, 255.0f));
				Dst[-TTOffset + PrevColumnOffset].A = static_cast<uint8>(FMath::Clamp((TL_Normal.Y * 127.5f + 127.5f), 0.0f, 255.0f));

				// 6) Store Partial Normal for LL in PrevLinePartialNormals (P0 + P1 + N0) - the rest will be filled in when processing the next line
				FVector3f LL_PartialNormal = P01 + N0;
				PrevLinePartialNormals[X + PrevColumnOffset] = LL_PartialNormal;

				// pass normals to next pixel
				P1 = N1;
				P01 = N01;

				Src += 2;
				Dst++;
				PrevColumnOffset = -1;
			}
			PreviousRowOffset = 1;
		}
	}

	// write out normals along the edge (delta encoded clockwise starting from top left)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(EdgeNormalFixup);

		uint8* Src = &SourceData[NumSrcPixels * 2];
		uint8 LastNormalX = 128;
		uint8 LastNormalY = 128;

		auto DecodeNormal = [&LastNormalX, &LastNormalY, DestData, DestWidth, &Src](int32 X, int32 Y)
			{
				int32 DestOffset = (Y * DestWidth + X) * 4;
				LastNormalX += Src[0];
				LastNormalY += Src[1];
				DestData[DestOffset + 0] = LastNormalX;
				DestData[DestOffset + 3] = LastNormalY;
				Src += 2;
			};

		for (int32 X = 0; X < DestWidth; X++)		// [0 ... Width-1], 0
		{
			DecodeNormal(X, 0);
		}

		for (int32 Y = 1; Y < DestHeight; Y++)		// Width-1, [1 ... Height-1]
		{
			DecodeNormal(DestWidth - 1, Y);
		}

		for (int32 X = DestWidth - 2; X >= 0; X--)	// [Width-2 ... 0], Height-1
		{
			DecodeNormal(X, DestHeight - 1);
		}

		for (int32 Y = DestHeight - 2; Y >= 1; Y--)	// 0, [Height-2 ... 1]
		{
			DecodeNormal(0, Y);
		}
		check(Src == &SourceData[SourceDataBytes]);

		// Copy all duplicate row/column data once all height/normal data is set
		if (bHasDuplicateData)
		{
			// For each duplicate row, copy the pixels above 
			for (int32 Y = MipBoundaryOffset; Y < DestHeight; Y += MipBoundaryOffset)
			{
				check(UE::Landscape::Private::IsDuplicateCoord(Y, MipBoundaryOffset));
				FColor* Dst = (FColor*)&DestData[Y * DestWidth * 4];

				for (int32 X = 0; X < DestWidth; X++)
				{
					*Dst = Dst[-DestWidth];
					Dst++;
				}
			}

			// For each duplicate column, copy the pixels to the left
			for (int32 X = MipBoundaryOffset; X < DestWidth; X += MipBoundaryOffset)
			{
				check(UE::Landscape::Private::IsDuplicateCoord(X, MipBoundaryOffset));
				FColor* Dst = (FColor*)&DestData[X * 4];
				for (int32 Y = 0; Y < DestHeight; Y++)
				{
					*Dst = Dst[-1];
					Dst += DestWidth;
				}
			}
		}
	}
}

#undef PROVIDER_DEBUG_LOG
#undef PROVIDER_DEBUG_LOG_DETAIL

