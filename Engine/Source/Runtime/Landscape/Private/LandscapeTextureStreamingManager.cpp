// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeTextureStreamingManager.h"
#include "Engine/Texture2D.h"
#include "TextureCompiler.h"
#include "LandscapePrivate.h"

namespace UE::Landscape
{
	// double check that a texture is forced resident
	static inline void EnsureTextureForcedResident(UTexture *Texture)
	{
		// if other systems mess with this flag, then restore it to what it should be
		// Any code that is directly messing with the flag on one of our
		// landscape related textures should go through this streaming system instead
		if (!ensure(Texture->bForceMiplevelsToBeResident))
		{
			Texture->bForceMiplevelsToBeResident = true;
		}
	}

	static bool IsTextureFullyStreamedIn(UTexture* InTexture)
	{
		return InTexture &&
#if WITH_EDITOR
			!InTexture->IsDefaultTexture() &&
#endif // WITH_EDITOR
			!InTexture->HasPendingInitOrStreaming() && InTexture->IsFullyStreamedIn();
	}

	static inline bool EnforceTextureIsFullyStreamedInNow(UTexture* Texture)
	{
#if WITH_EDITOR
		// in editor, textures can be not compiled yet - we should complete that first
		Texture->BlockOnAnyAsyncBuild();
#endif // WITH_EDITOR
		Texture->WaitForStreaming();
		bool bIsFullyStreamedIn = IsTextureFullyStreamedIn(Texture);
#if WITH_EDITOR
		// the above should handle ensure textures are fully streamed in.. but just in case it isn't...
		if (!bIsFullyStreamedIn)
		{
			// this is a sledgehammer, but should always fix it, by rebuilding the entire texture resource with streaming disabled
			if (UTexture2D* Texture2D = Cast<UTexture2D>(Texture))
			{
				Texture2D->TemporarilyDisableStreaming();
			}
			// It is almost guaranteed there is a pending RHI Init because of the full texture rebuild in TemporarilyDisableStreaming.
			// This should be fine for rendering purposes -- the texture will complete the RHI init before we render with it, and will have full mips.
			// (to ensure the texture is init before returning, we would have to stall to drain the render thread here..)
			bIsFullyStreamedIn = !Texture->IsDefaultTexture() && Texture->IsFullyStreamedIn();
		}
#endif // WITH_EDITOR
		check(bIsFullyStreamedIn);
		return bIsFullyStreamedIn;
	}
}

TArray<FLandscapeTextureStreamingManager*> FLandscapeTextureStreamingManager::AllStreamingManagers;

bool FLandscapeTextureStreamingManager::RequestTextureFullyStreamedIn(UTexture* Texture, bool bWaitForStreaming)
{
	using namespace UE::Landscape;
	check(Texture != nullptr);
	TWeakObjectPtr<UTexture> TexturePtr = Texture;
	FTextureState& State = TextureStates.FindOrAdd(TexturePtr);

	if (State.RequestCount == 0)
	{
		Texture->bForceMiplevelsToBeResident = true;
	}
	else
	{
		EnsureTextureForcedResident(Texture);
	}
	State.RequestCount++;

	if (IsTextureFullyStreamedIn(Texture))
	{
		return true;
	}
	else if (bWaitForStreaming)
	{
		return EnforceTextureIsFullyStreamedInNow(Texture);
	}
	return false;
}

bool FLandscapeTextureStreamingManager::RequestTextureFullyStreamedInForever(UTexture* Texture, bool bWaitForStreaming)
{
	using namespace UE::Landscape;
	check(Texture != nullptr);
	TWeakObjectPtr<UTexture> TexturePtr = Texture;
	FTextureState& State = TextureStates.FindOrAdd(TexturePtr);
	State.bForever = true;
	Texture->bForceMiplevelsToBeResident = true;

	if (IsTextureFullyStreamedIn(Texture))
	{
		return true;
	}
	else if (bWaitForStreaming)
	{
		return EnforceTextureIsFullyStreamedInNow(Texture);
	}
	return false;
}

void FLandscapeTextureStreamingManager::UnrequestTextureFullyStreamedIn(UTexture* Texture)
{
	if (Texture == nullptr)
	{
		return;
	}

	TWeakObjectPtr<UTexture> TexturePtr = Texture;
	FTextureState* State = TextureStates.Find(TexturePtr);
	if (State)
	{
		if (State->RequestCount > 0)
		{
			State->RequestCount--;
			if (!State->WantsTextureStreamedIn())
			{
				// remove state tracking for this texture
				TextureStates.Remove(TexturePtr);
				if ((AllStreamingManagers.Num() == 1) || !AnyStreamingManagerWantsTextureStreamedIn(TexturePtr))
				{
					// allow stream out
					Texture->bForceMiplevelsToBeResident = false;
				}
				else
				{
					UE::Landscape::EnsureTextureForcedResident(Texture);
				}
			}
			else
			{
				UE::Landscape::EnsureTextureForcedResident(Texture);
			}
		}
		else
		{
			UE_LOGF(LogLandscape, Warning, "Texture Streaming Manager received more Unrequests than Requests to stream texture %ls", *Texture->GetName());
		}
	}
}

bool FLandscapeTextureStreamingManager::WaitForTextureStreaming()
{
	using namespace UE::Landscape;
	TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeTextureStreamingaAnager_WaitForTextureStreaming);
	bool bAllTexturesFullyStreamed = true;
	for (auto It = TextureStates.CreateIterator(); It; ++It)
	{
		UTexture* Texture = It.Key().Get();
		if (Texture)
		{
			UE::Landscape::EnsureTextureForcedResident(Texture);
			if (!IsTextureFullyStreamedIn(Texture))
			{
				const bool bTextureIsFullyStreamed = EnforceTextureIsFullyStreamedInNow(Texture);
				bAllTexturesFullyStreamed = bAllTexturesFullyStreamed && bTextureIsFullyStreamed;
			}		
		}
		else
		{
			// the texture was unloaded... we can remove this entry
			It.RemoveCurrent();
		}
	}
	return bAllTexturesFullyStreamed;
}

void FLandscapeTextureStreamingManager::CleanupPostGarbageCollect()
{
	for (auto It = TextureStates.CreateIterator(); It; ++It)
	{
		UTexture* Texture = It.Key().Get();
		if (Texture == nullptr)
		{
			It.RemoveCurrent();
		}
		else
		{
			// reset the texture force resident after garbage collection (which clears it sometimes)
			FTextureState& State = It.Value();
			if (State.WantsTextureStreamedIn())
			{
				Texture->bForceMiplevelsToBeResident = true;
			}
		}
	}
}

void FLandscapeTextureStreamingManager::CheckRequestedTextures()
{
#if WITH_EDITOR
	if (UndoDetector.bUndoRedoPerformed)
	{
		// the force mip levels resident flag sometimes gets cleared on an undo after landscape creation, but we can fix it
		// (otherwise we may wait forever for them to become resident)
		for (auto It = TextureStates.CreateIterator(); It; ++It)
		{
			if (UTexture* Texture = It.Key().Get())
			{
				FTextureState& State = It.Value();
				if (State.WantsTextureStreamedIn())
				{
					if (!Texture->bForceMiplevelsToBeResident)
					{
						Texture->bForceMiplevelsToBeResident = true;
					}
				}
			}
		}
		UndoDetector.bUndoRedoPerformed = false;
	}
#endif // WITH_EDITOR
}

bool FLandscapeTextureStreamingManager::IsTextureFullyStreamedIn(UTexture* InTexture)
{
	return UE::Landscape::IsTextureFullyStreamedIn(InTexture);
}

bool FLandscapeTextureStreamingManager::AnyStreamingManagerWantsTextureStreamedIn(TWeakObjectPtr<UTexture> TexturePtr)
{
	for (FLandscapeTextureStreamingManager* Manager : AllStreamingManagers)
	{
		if (FTextureState* State = Manager->TextureStates.Find(TexturePtr))
		{
			if (State->WantsTextureStreamedIn())
			{
				return true;
			}
		}
	}
	return false;
}

FLandscapeTextureStreamingManager::FLandscapeTextureStreamingManager()
{
	AllStreamingManagers.Add(this);
}

FLandscapeTextureStreamingManager::~FLandscapeTextureStreamingManager()
{
	AllStreamingManagers.RemoveSwap(this, EAllowShrinking::No);

	// there could be some textures still requested, if they were requested "forever".
	// since the world is going away, we can re-evaluate whether they should remain streamed or not.
	int32 RemainingRequests = 0;
	for (auto It = TextureStates.CreateIterator(); It; ++It)
	{
		FTextureState& State = It.Value();
		if (State.RequestCount > 0)
		{
			RemainingRequests++;
		}

		if (UTexture* Texture = It.Key().Get())
		{
			if (!AnyStreamingManagerWantsTextureStreamedIn(It.Key()))
			{
				// none of the remaining streaming managers request this texture, we can disable the mip requests
				Texture->bForceMiplevelsToBeResident = false;
			}
		}
	}

	if (RemainingRequests > 0)
	{
		UE_LOGF(LogLandscape, Display, "At destruction, the Landscape Texture Streaming Manager still has streaming requests for %d Textures, this may indicate failure to clean them up.", RemainingRequests);
	}
}
