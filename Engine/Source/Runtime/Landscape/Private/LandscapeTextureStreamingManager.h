// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"

#if WITH_EDITOR

#include "EditorUndoClient.h"

class FLandscapeTextureStreamingManagerUndoDetector : FSelfRegisteringEditorUndoClient
{
public:
	bool bUndoRedoPerformed = false;

private:
	virtual void PostUndo(bool bSuccess) override
	{
		bUndoRedoPerformed = true;
	};

	virtual void PostRedo(bool bSuccess) override
	{
		bUndoRedoPerformed = true;
	};
};

#endif // WITH_EDITOR

class UTexture;

class FLandscapeTextureStreamingManager
{
public:
	// Put in a request to have all streamable mips of a texture streamed in.
	// bWaitForStreaming true will cause it to block until the texture streaming operation completes.
	// bWaitForStreaming false will asynchronously stream in the textures, and you can check for completion with IsTextureFullyStreamedIn().
	// The texture will* remain streamed in until you call UnrequestTextureFullyStreamedIn()
	// Returns true if the texture is currently fully streamed in.
	bool RequestTextureFullyStreamedIn(UTexture* Texture, bool bWaitForStreaming);

	// Remove a request to have all streamable mips of a texture streamed in.
	// The texture may still remain streamed in if someone else has a request,
	// or if the texture manager doesn't see any need to evict it to reclaim texture memory.
	void UnrequestTextureFullyStreamedIn(UTexture* Texture);

	// Request that this texture be fully streamed in for as long as the texture is loaded. No backsies.
	// bWaitForStreaming will also block until the texture is fully streamed in.
	// Returns true if the texture is fully streamed.
	bool RequestTextureFullyStreamedInForever(UTexture* Texture, bool bWaitForStreaming);

	// TODO [chris.tchou] : a non-blocking tick to update streaming textures would be more efficient in some looping cases.
	// void TickStreaming();

	// Synchronously waits for all textures to complete their requested streaming. Returns true if successful.
	// This necessary if you are requesting streaming and waiting for the textures to be streamed within a local loop.
	// (normally they don't complete streaming until a certain point in the update tick)
	bool WaitForTextureStreaming();

	// Call this post garbage collect to clean up any old entries in tracked TextureStates for textures that have been unloaded without first being Unrequested, and make sure the streaming flags weren't cleared
	void CleanupPostGarbageCollect();

	// Check that all requested textures are still requested.
	void CheckRequestedTextures();

	static bool IsTextureFullyStreamedIn(UTexture* Texture);

	FLandscapeTextureStreamingManager();
	~FLandscapeTextureStreamingManager();

private:
	struct FTextureState
	{
		int32 RequestCount = 0;
		bool bForever = false;

		bool WantsTextureStreamedIn() const
		{
			return bForever || (RequestCount > 0);
		}
	};

	TMap<TWeakObjectPtr<UTexture>, FTextureState, FDefaultSetAllocator, TWeakObjectPtrMapKeyFuncs<TWeakObjectPtr<UTexture>, FTextureState>> TextureStates;

	static TArray<FLandscapeTextureStreamingManager*> AllStreamingManagers;
	static bool AnyStreamingManagerWantsTextureStreamedIn(TWeakObjectPtr<UTexture> TexturePtr);

#if WITH_EDITOR
	FLandscapeTextureStreamingManagerUndoDetector UndoDetector;
#endif // WITH_EDITOR

};
