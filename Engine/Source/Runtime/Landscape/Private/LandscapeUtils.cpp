// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeUtils.h"

#include "Engine/Level.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "LandscapeLayerInfoObject.h"
#include "LandscapeProxy.h"
#include "LandscapeEditTypes.h"
#include "LandscapeGrassType.h"
#include "Algo/Transform.h"
#include "RenderGraphBuilder.h"
#include "TextureResource.h"
#include "ProfilingDebugging/RealtimeGPUProfiler.h"
#include "ShaderCore.h"
#include "ShaderCompilerCore.h"

#if WITH_EDITOR
#include "Algo/AllOf.h"
#include "Algo/ForEach.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Editor/EditorEngine.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorDirectories.h"
#include "Engine/Texture2D.h"
#include "MaterialCachedData.h"
#include "MaterialDomain.h"
#include "LandscapeComponent.h"
#include "LandscapeMaterialInstanceConstant.h"
#include "LandscapeSettings.h"
#include "ObjectTools.h"
#include "Selection.h"
#include "UnrealEdGlobals.h"
#endif // WITH_EDITOR

#if WITH_EDITOR
// Channel remapping
extern const size_t ChannelOffsets[4];
extern UNREALED_API UEditorEngine* GEditor;
#endif // WITH_EDITOR

extern int32 GGrassMapUseRuntimeGeneration;
extern int32 GGrassMapAlwaysBuildRuntimeGenerationResources;

static TAutoConsoleVariable<bool> CVarLandscapeDebugShaders(
	TEXT("landscape.DebugShaders"),
	false,
	TEXT("De-optimizes landscape shaders for easier debugging (use in conjunction with r.Shaders.Symbols=1 to dump shader symbols)"));


namespace UE::Landscape
{

bool DoesPlatformSupportEditLayers(EShaderPlatform InShaderPlatform)
{
	// Edit layers work on the GPU and are only available on SM5+ and in the editor : 
	return IsFeatureLevelSupported(InShaderPlatform, ERHIFeatureLevel::SM5)
		&& !IsConsolePlatform(InShaderPlatform)
		&& !IsMobilePlatform(InShaderPlatform);
}

void ModifyShaderCompilerEnvironmentForDebug(FShaderCompilerEnvironment& OutEnvironment)
{
#if !UE_BUILD_SHIPPING 
	if (CVarLandscapeDebugShaders.GetValueOnAnyThread())
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_Debug);
	}
#endif // !UE_BUILD_SHIPPING 
}

ELandscapeToolTargetTypeFlags GetLandscapeToolTargetTypeAsFlags(ELandscapeToolTargetType InTargetType)
{
	if (InTargetType == ELandscapeToolTargetType::Invalid)
	{
		return ELandscapeToolTargetTypeFlags::None;
	}

	uint8 TargetTypeValue = static_cast<uint8>(InTargetType);
	check(TargetTypeValue < static_cast<uint8>(ELandscapeToolTargetType::Count));
	return static_cast<ELandscapeToolTargetTypeFlags>(1 << TargetTypeValue);
}

ELandscapeToolTargetType GetLandscapeToolTargetTypeSingleFlagAsType(ELandscapeToolTargetTypeFlags InSingleFlag)
{
	check(FMath::CountBits64(static_cast<uint64>(InSingleFlag)) == 1);
	uint32 Index = FMath::FloorLog2(static_cast<uint8>(InSingleFlag));
	check(Index < static_cast<uint32>(ELandscapeToolTargetType::Count));
	return static_cast<ELandscapeToolTargetType>(Index);
}

FString GetLandscapeToolTargetTypeFlagsAsString(ELandscapeToolTargetTypeFlags InTargetTypeFlags)
{
	TArray<FString> TargetTypeStrings;
	Algo::Transform(MakeFlagsRange(InTargetTypeFlags), TargetTypeStrings, [](ELandscapeToolTargetTypeFlags InTargetTypeFlag) 
		{ 
			return UEnum::GetDisplayValueAsText(GetLandscapeToolTargetTypeSingleFlagAsType(InTargetTypeFlag)).ToString(); 
		});
	return *FString::Join(TargetTypeStrings, TEXT(","));
}

bool SupportsRuntimeGrassMapGeneration()
{
	return GGrassMapUseRuntimeGeneration || GGrassMapAlwaysBuildRuntimeGenerationResources;
}


// ----------------------------------------------------------------------------------

FRDGBuilderRecorder::~FRDGBuilderRecorder()
{
	checkf((State == EState::Immediate) && IsEmpty(), 
	   TEXT("The command recorder has %d commands pending while being destroyed. These commands will not get executed unless they are appended to a render command : use Flush() (or Clear() to remove all commands if this is intended)."), RDGCommands.Num());
}

void FRDGBuilderRecorder::StartRecording()
{
	if (State == EState::Immediate)
	{
		State = EState::Recording;
	}
}

void FRDGBuilderRecorder::StopRecording()
{
	if (State == EState::Recording)
	{
		State = EState::Immediate;
	}
}

void FRDGBuilderRecorder::StopRecordingAndFlush(FRDGEventName&& EventName)
{
	if (State == EState::Recording)
	{
		State = EState::Immediate;
		Flush(MoveTemp(EventName));
	}
}

void FRDGBuilderRecorder::Flush(FRDGEventName&& EventName)
{
	if (!IsEmpty())
	{
		checkf(State == EState::Immediate, TEXT("StopRecording needs to be called before flushing the recorded commands"));

		ENQUEUE_RENDER_COMMAND(FRDGBuilderRecorder_Flush)(
			[RDGCommands = MoveTemp(RDGCommands), RDGExternalTextureAccessFinal = MoveTemp(RDGExternalTextureAccessFinal), EventName = MoveTemp(EventName)](FRHICommandListImmediate& InRHICmdList) mutable
			{
				SCOPED_DRAW_EVENTF(InRHICmdList, FRDGBuilderRecorder_Flush, TEXT("%s"), FString(EventName.GetTCHAR()));

				FRDGBuilder GraphBuilder(InRHICmdList, EventName);

				for (const FRDGRecorderRDGCommand& Command : RDGCommands)
				{
					Command(GraphBuilder);
				}
				
				for (auto ItPair : RDGExternalTextureAccessFinal)
				{
					FRDGTextureRef TextureRef = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(ItPair.Key->GetTextureRHI(), TEXT("ExternalTexture")));
					check(TextureRef != nullptr);
					GraphBuilder.SetTextureAccessFinal(TextureRef, ItPair.Value);
				}

				GraphBuilder.Execute();
			});
		check(IsEmpty());

		Clear();
	}
}

void FRDGBuilderRecorder::EnqueueRDGCommand(FRDGRecorderRDGCommand InRDGCommand, TConstArrayView<FRDGExternalTextureAccessFinal> InRDGExternalTextureAccessFinalList)
{
	if (State == EState::Recording)
	{
		RDGCommands.Add(InRDGCommand);
		for (const FRDGExternalTextureAccessFinal& TextureAccess : InRDGExternalTextureAccessFinalList)
		{
			// Replace the existing value if any : this specifies the state of the texture at the moment the FRDGBuilder executes : 
			RDGExternalTextureAccessFinal.FindOrAdd(TextureAccess.TextureResource) = TextureAccess.Access;
		}
	}
	else
	{
		ENQUEUE_RENDER_COMMAND(FRDGBuilderRecorder_RDGCommand)([InRDGCommand, LocalRDGExternalTextureAccessFinal = TArray<FRDGExternalTextureAccessFinal>(InRDGExternalTextureAccessFinalList)](FRHICommandListImmediate& InRHICmdList)
			{ 
				FRDGBuilder GraphBuilder(InRHICmdList, RDG_EVENT_NAME("RDGImmediateRDGCommand"));
				
				InRDGCommand(GraphBuilder);

				for (const FRDGExternalTextureAccessFinal& TextureAccess : LocalRDGExternalTextureAccessFinal)
				{
					FRDGTextureRef TextureRef = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(TextureAccess.TextureResource->GetTextureRHI(), TEXT("ExternalTexture")));
					check(TextureRef != nullptr);
					GraphBuilder.SetTextureAccessFinal(TextureRef, TextureAccess.Access);
				}

				GraphBuilder.Execute();
			});
	}
}

void FRDGBuilderRecorder::EnqueueRenderCommand(FRDGRecorderRenderCommand InRenderCommand)
{
	if (State == EState::Recording)
	{
		auto RDGCommand = [InRenderCommand](FRDGBuilder& GraphBuilder)
			{
				GraphBuilder.AddPass(RDG_EVENT_NAME("RDGRecordedRenderCommand"), ERDGPassFlags::NeverCull, [InRenderCommand](FRHICommandListImmediate& InRHICmdList) mutable { InRenderCommand(InRHICmdList); });
			}; 
		RDGCommands.Add(RDGCommand);
	}
	else
	{
		ENQUEUE_RENDER_COMMAND(RDGImmediateRenderCommand)(InRenderCommand);
	}
}

bool FRDGBuilderRecorder::IsEmpty() const
{
	return RDGCommands.IsEmpty();
}

void FRDGBuilderRecorder::Clear()
{
	RDGCommands.Reset();
	RDGExternalTextureAccessFinal.Reset();
}


#if WITH_EDITOR

// ----------------------------------------------------------------------------------

FString GetSharedAssetsPath(const FString& InPath)
{
	FString Path = InPath + TEXT("_sharedassets/");

	if (Path.StartsWith("/Temp/"))
	{
		Path = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::LEVEL) / Path.RightChop(FString("/Temp/").Len());
	}

	return Path;
}

FString GetSharedAssetsPath(const ULevel* InLevel)
{
	return GetSharedAssetsPath(InLevel->GetOutermost()->GetName());
}

FString GetLayerInfoObjectPackageName(const FName& InLayerName, const FString& InPackagePath, FName& OutLayerObjectName)
{
	FString PackageName;
	FString PackageFilename;
	int32 Suffix = 1;

	OutLayerObjectName = FName(*FString::Printf(TEXT("%s_LayerInfo"), *ObjectTools::SanitizeInvalidChars(*InLayerName.ToString(), INVALID_LONGPACKAGE_CHARACTERS)));
	FPackageName::TryConvertFilenameToLongPackageName(InPackagePath / OutLayerObjectName.ToString(), PackageName);

	while (FPackageName::DoesPackageExist(PackageName, &PackageFilename))
	{
		OutLayerObjectName = FName(*FString::Printf(TEXT("%s_LayerInfo_%d"), *ObjectTools::SanitizeInvalidChars(*InLayerName.ToString(), INVALID_LONGPACKAGE_CHARACTERS), Suffix));
		if (!FPackageName::TryConvertFilenameToLongPackageName(InPackagePath / OutLayerObjectName.ToString(), PackageName))
		{
			break;
		}

		Suffix++;
	}

	return PackageName;
}

ULandscapeLayerInfoObject* CreateTargetLayerInfo(const FName& InLayerName, const FString& InFilePath)
{
	// Appends %s_LayerInfo_%d to ensure the new asset has a valid filename
	FName FileName;
	const FString PackageName = GetLayerInfoObjectPackageName(InLayerName, InFilePath, FileName);

	return CreateTargetLayerInfo(InLayerName, InFilePath, FileName.ToString());
}

ULandscapeLayerInfoObject* CreateTargetLayerInfo(const FName& InLayerName, const FString& InFilePath, const FString& InFileName)
{
	// Get the default asset from the project settings
	const ULandscapeSettings* Settings = GetDefault<ULandscapeSettings>();
	TSoftObjectPtr<ULandscapeLayerInfoObject> DefaultLayerInfoObject = Settings->GetDefaultLayerInfoObject().LoadSynchronous();

	// Ensure the package path has a terminating "/"
	const FString PackagePath = InFilePath.EndsWith("/") ? InFilePath + InFileName : InFilePath + "/" + InFileName;

	UPackage* Package = CreatePackage(*PackagePath);
	ULandscapeLayerInfoObject* LayerInfo = nullptr;
	check(Package != nullptr);

	if (DefaultLayerInfoObject.Get() != nullptr)
	{
		LayerInfo = DuplicateObject<ULandscapeLayerInfoObject>(DefaultLayerInfoObject.Get(), Package, *InFileName);
		LayerInfo->SetFlags(RF_Public | RF_Standalone | RF_Transactional);
	}
	else
	{
		// Do not pass RF_Transactional to NewObject, or the asset will mark itself as garbage on Undo (which is not a well-supported path, potentially causing crashes)
		LayerInfo = NewObject<ULandscapeLayerInfoObject>(Package, *InFileName, RF_Public | RF_Standalone);
		LayerInfo->SetFlags(RF_Transactional);	// we add RF_Transactional after creation, so that future edits _are_ recorded in undo
	}

	check(LayerInfo != nullptr);
	LayerInfo->SetLayerName(InLayerName, /*bInModify =*/false);
	LayerInfo->SetLayerUsageDebugColor(LayerInfo->GenerateLayerUsageDebugColor(), /*bInModify =*/false, /*InChangeType =*/ EPropertyChangeType::ValueSet);

	// Notify the asset registry
	FAssetRegistryModule::AssetCreated(LayerInfo);
	Package->MarkPackageDirty();
	LayerInfo->MarkPackageDirty();

	return LayerInfo;
}

bool IsVisibilityLayer(const ULandscapeLayerInfoObject* InLayerInfoObject)
{
	return (ALandscapeProxy::VisibilityLayer != nullptr) && (ALandscapeProxy::VisibilityLayer == InLayerInfoObject);
}

uint32 GetTypeHash(const FTextureCopyRequest& InKey)
{
	uint32 Hash = ::GetTypeHash(InKey.Source);
	uint32 HashSlice = ::GetTypeHash(InKey.DestinationSlice);
	return HashCombine(Hash, ::GetTypeHash(InKey.Destination)) ^ (HashSlice << 4) ^ ((uint32) InKey.TextureUsage << 2) ^ ((uint32) InKey.TextureType);
}

bool operator==(const FTextureCopyRequest& InEntryA, const FTextureCopyRequest& InEntryB)
{
	return (InEntryA.Source == InEntryB.Source) &&
		(InEntryA.Destination == InEntryB.Destination) &&
		(InEntryA.DestinationSlice == InEntryB.DestinationSlice) &&
		(InEntryA.TextureUsage == InEntryB.TextureUsage) &&
		(InEntryA.TextureType == InEntryB.TextureType);
}

bool FBatchTextureCopy::AddWeightmapCopy(UTexture* InDestination, int8 InDestinationSlice, int8 InDestinationChannel, const ULandscapeComponent* InComponent, ULandscapeLayerInfoObject* InLayerInfo)
{
	FTextureCopyRequest CopyRequest;
	const TArray<UTexture2D*>& ComponentWeightmapTextures = InComponent->GetWeightmapTextures();
	const TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = InComponent->GetWeightmapLayerAllocations();
	int8 SourceChannel = INDEX_NONE;

	CopyRequest.Destination = InDestination;
	CopyRequest.DestinationSlice = InDestinationSlice;

	// Find the proper Source Texture and channel from Layer Allocations
	for (const FWeightmapLayerAllocationInfo& ComponentWeightmapLayerAllocation : ComponentWeightmapLayerAllocations)
	{
		if ((ComponentWeightmapLayerAllocation.LayerInfo == InLayerInfo) &&
			ComponentWeightmapLayerAllocation.IsAllocated() &&
			ComponentWeightmapTextures.IsValidIndex(ComponentWeightmapLayerAllocation.WeightmapTextureIndex))
		{
			CopyRequest.Source = ComponentWeightmapTextures[ComponentWeightmapLayerAllocation.WeightmapTextureIndex];
			SourceChannel = ComponentWeightmapLayerAllocation.WeightmapTextureChannel;
			break;
		}
	}

	// Check if we found a proper allocation for this LayerInfo
	if (SourceChannel != INDEX_NONE)
	{
		check((InDestinationChannel < 4) && (SourceChannel < 4));
		FTextureCopyChannelMapping& ChannelMapping = CopyRequests.FindOrAdd(MoveTemp(CopyRequest));
		ChannelMapping[ChannelOffsets[InDestinationChannel]] = ChannelOffsets[SourceChannel];
		return true;
	}

	return false;
}

struct FSourceDataMipNumber
{
	TOptional<FTextureSource::FMipData> MipData;
	int32 MipNumber = 0;
};

struct FDestinationDataMipNumber
{
	TArray<uint8*> DestinationDataPtr;
	int32 MipNumber = 0;
	ELandscapeTextureUsage TextureUsage = ELandscapeTextureUsage::Unknown;
	ELandscapeTextureType TextureType = ELandscapeTextureType::Unknown;
};

bool FBatchTextureCopy::ProcessTextureCopies()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FBatchTextureCopy::ProcessTextureCopyRequest);
	TMap<UTexture2D*, FSourceDataMipNumber> Sources;
	TMap<UTexture*, FDestinationDataMipNumber> Destinations;

	if (CopyRequests.Num() == 0)
	{
		return false;
	}

	// Populate source/destination maps to filter unique occurrences
	for (const TPair<FTextureCopyRequest, FTextureCopyChannelMapping>& CopyRequest : CopyRequests)
	{
		FSourceDataMipNumber& SourceData = Sources.Add(CopyRequest.Key.Source);
		SourceData.MipNumber = CopyRequest.Key.Source->Source.GetNumMips();

		FDestinationDataMipNumber& DestinationData = Destinations.Add(CopyRequest.Key.Destination);
		DestinationData.MipNumber = CopyRequest.Key.Destination->Source.GetNumMips();
		DestinationData.TextureUsage = CopyRequest.Key.TextureUsage;
		DestinationData.TextureType = CopyRequest.Key.TextureType;
	}

	// Decompress (if needed) and get the source textures ready for access
	for (TPair<UTexture2D*, FSourceDataMipNumber>& Source : Sources)
	{
		Source.Value.MipData = Source.Key->Source.GetMipData(nullptr);
	}

	// Lock all destinations mips
	for (TPair<UTexture*, FDestinationDataMipNumber>& Destination : Destinations)
	{
		int32 MipNumber = Destination.Value.MipNumber;
		TArray<uint8*>& DestinationDataPtr = Destination.Value.DestinationDataPtr;

		for (int32 MipLevel = 0; MipLevel < MipNumber; ++MipLevel)
		{
			DestinationDataPtr.Add(Destination.Key->Source.LockMip(MipLevel));
		}
	}

	for (const TPair<FTextureCopyRequest, FTextureCopyChannelMapping>& CopyRequest : CopyRequests)
	{
		const FSourceDataMipNumber* SourceDataMipNumber = Sources.Find(CopyRequest.Key.Source);
		const FDestinationDataMipNumber* DestinationDataMipNumber = Destinations.Find(CopyRequest.Key.Destination);

		check((SourceDataMipNumber != nullptr) && (DestinationDataMipNumber != nullptr));
		check(SourceDataMipNumber->MipNumber == DestinationDataMipNumber->MipNumber);

		const int32 MipNumber = SourceDataMipNumber->MipNumber;

		for (int32 MipLevel = 0; MipLevel < MipNumber; ++MipLevel)
		{
			const int64 MipSizeInBytes = CopyRequest.Key.Source->Source.CalcMipSize(MipLevel);
			
			const int32 MipSize = CopyRequest.Key.Destination->Source.GetSizeX() >> MipLevel;
			check(MipSize == (CopyRequest.Key.Destination->Source.GetSizeY() >> MipLevel));

			int32 MipSizeSquare = FMath::Square(MipSize);
			FSharedBuffer MipSrcData = SourceDataMipNumber->MipData->GetMipData(0, 0, MipLevel);
			const uint8* SourceTextureData = static_cast<const uint8*>(MipSrcData.GetData());
			uint8* DestTextureData = DestinationDataMipNumber->DestinationDataPtr[MipLevel] + CopyRequest.Key.DestinationSlice * MipSizeInBytes;

			check((SourceTextureData != nullptr) && (DestTextureData != nullptr));

			const FTextureCopyChannelMapping& ChannelMapping = CopyRequest.Value;

			// Perform the copy, redirecting channels using mappings
			for (int32 Index = 0; Index < MipSizeSquare; ++Index)
			{
				int32 Base = Index * 4;

				for (int32 Channel = 0; Channel < 4; ++Channel)
				{
					if (ChannelMapping[Channel] == INDEX_NONE)
					{
						continue;
					}

					DestTextureData[Base + Channel] = SourceTextureData[Base + ChannelMapping[Channel]];
				}
			}
		}
	}

	// Note that source textures do not need unlocking, data will be released once the FMipData go out of scope
	
	// Unlock all destination mips
	for (TPair<UTexture*, FDestinationDataMipNumber>& Destination : Destinations)
	{
		int32 MipNumber = Destination.Value.MipNumber;
		
		for (int32 MipLevel = 0; MipLevel < MipNumber; ++MipLevel)
		{
			Destination.Key->Source.UnlockMip(MipLevel);
		}

		ULandscapeTextureHash::UpdateHash(CastChecked<UTexture2D>(Destination.Key), Destination.Value.TextureUsage, Destination.Value.TextureType);
	}

	return true;
}

UMaterialInstance* CreateToolLandscapeMaterialInstanceConstant(UMaterialInterface* BaseMaterial)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CreateToolLandscapeMaterialInstanceConstant);

	UObject* Outer = GetTransientPackage();
	// Use the base material's name as the base of our MIC to help debug: 
	FString MICName(FString::Format(TEXT("LandscapeMaterialInstanceConstant_{0}"), { *BaseMaterial->GetName() }));
	ULandscapeMaterialInstanceConstant* MaterialInstance = NewObject<ULandscapeMaterialInstanceConstant>(Outer, MakeUniqueObjectName(Outer, ULandscapeMaterialInstanceConstant::StaticClass(), FName(MICName)));
	MaterialInstance->bEditorToolUsage = true;
	MaterialInstance->SetParentEditorOnly(BaseMaterial);
	MaterialInstance->PostEditChange();
	return MaterialInstance;
}

ULandscapeMaterialInstanceConstant* CreateLandscapeLayerThumbnailMIC(FMaterialUpdateContext& MaterialUpdateContext, UMaterialInterface* LandscapeMaterial, FName LayerName)
{
	if (!GetDefault<ULandscapeSettings>()->ShouldDisplayTargetLayerThumbnails())
	{
		return nullptr;
	}

	if (LandscapeMaterial == nullptr)
	{
		LandscapeMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
	}

	FlushRenderingCommands();

	ULandscapeMaterialInstanceConstant* MaterialInstance = NewObject<ULandscapeMaterialInstanceConstant>(GetTransientPackage());
	MaterialInstance->bIsLayerThumbnail = true;
	MaterialInstance->bMobile = false;
	MaterialInstance->SetParentEditorOnly(LandscapeMaterial, false);

	FStaticParameterSet StaticParameters;
	MaterialInstance->GetStaticParameterValues(StaticParameters);

	// Customize that material instance to only enable our terrain layer's weightmap : 
	StaticParameters.EditorOnly.TerrainLayerWeightParameters.Add(FStaticTerrainLayerWeightParameter(LayerName, /*InWeightmapIndex = */0));

	MaterialInstance->UpdateStaticPermutation(StaticParameters, &MaterialUpdateContext);

	static UTexture2D* ThumbnailWeightmap = LoadObject<UTexture2D>(nullptr, TEXT("/Engine/EditorLandscapeResources/LandscapeThumbnailWeightmap.LandscapeThumbnailWeightmap"), nullptr, LOAD_None, nullptr);
	static UTexture2D* ThumbnailHeightmap = LoadObject<UTexture2D>(nullptr, TEXT("/Engine/EditorLandscapeResources/LandscapeThumbnailHeightmap.LandscapeThumbnailHeightmap"), nullptr, LOAD_None, nullptr);

	FLinearColor Mask(1.0f, 0.0f, 0.0f, 0.0f);
	MaterialInstance->SetVectorParameterValueEditorOnly(FName(*FString::Printf(TEXT("LayerMask_%s"), *LayerName.ToString())), Mask);
	MaterialInstance->SetTextureParameterValueEditorOnly(FName("Weightmap0"), ThumbnailWeightmap);
	MaterialInstance->SetTextureParameterValueEditorOnly(FName("Heightmap"), ThumbnailHeightmap);

	MaterialInstance->PostEditChange();

	return MaterialInstance;
}

FString ConvertTargetLayerNamesToString(const TArrayView<const FName>& InTargetLayerNames)
{
	TArray<FString> TargetLayerStrings;
	Algo::Transform(InTargetLayerNames, TargetLayerStrings, [](FName InTargetLayerName) { return InTargetLayerName.ToString(); });
	return *FString::Join(TargetLayerStrings, TEXT(","));
}

bool DeleteActors(const TArray<AActor*>& InActorsToDelete, UWorld* InWorld, bool bInAllowUI)
{
	bool bSuccess = true;
	check(Algo::AllOf(InActorsToDelete, [InWorld](AActor* Actor) { return (Actor != nullptr) && (Actor->GetWorld() == InWorld); }));
	// If UI is allowed, prefer UUnrealEdEngine::DeleteActors, which handles references to the actor being deleted and asks the user what to do about it :
	if (bInAllowUI && GUnrealEd && GUnrealEd->GetSelectedActors() && GUnrealEd->GetSelectedActors()->GetElementSelectionSet())
	{
		bSuccess = GUnrealEd->DeleteActors(InActorsToDelete, InWorld, GUnrealEd->GetSelectedActors()->GetElementSelectionSet());
	}
	else
	{
		Algo::ForEach(InActorsToDelete, [InWorld, &bSuccess](AActor* Actor)
			{
				bSuccess &= InWorld->DestroyActor(Actor);
			});
	}

	return bSuccess;
}

TArray<FName> RetrieveTargetLayerNamesFromMaterial(UMaterialInterface* InMaterial, bool bIncludeVisibilityLayer)
{
	TArray<FName> LayerNames;
	if (InMaterial)
	{
		const FMaterialCachedExpressionData& CachedExpressionData = InMaterial->GetCachedExpressionData();
		if (CachedExpressionData.EditorOnlyData)
		{
			LayerNames = CachedExpressionData.EditorOnlyData->LandscapeLayerNames;

			const FName VisibilityLayerName = (ALandscapeProxy::VisibilityLayer != nullptr) ? ALandscapeProxy::VisibilityLayer->GetLayerName() : FName();
			LayerNames.RemoveAll([bIncludeVisibilityLayer, &VisibilityLayerName](const FName& Name)
				{
					return Name.IsNone() || (!bIncludeVisibilityLayer && Name == VisibilityLayerName);
				});
		}
	}
	return LayerNames;
}

#endif // WITH_EDITOR

bool LandscapeProxySortPredicate(const TWeakObjectPtr<ALandscapeProxy> APtr, const TWeakObjectPtr<ALandscapeProxy> BPtr)
{
	ALandscapeProxy* A = APtr.Get();
	ALandscapeProxy* B = BPtr.Get();

	// sort nulls, assuming null < !null
	if (!A || !B)
	{
		return !!B;
	}

	FIntPoint SectionBaseA = A->GetSectionBase();
	FIntPoint SectionBaseB = B->GetSectionBase();

	if (SectionBaseA.X != SectionBaseB.X)
	{
		return SectionBaseA.X < SectionBaseB.X;
	}

	return SectionBaseA.Y < SectionBaseB.Y;
}

} // end namespace UE::Landscape


// ----------------------------------------------------------------------------------

namespace UE::Landscape::Grass
{

TArray<TPair<FName, TObjectPtr<ULandscapeGrassType>>> ConvertToNamedGrassTypesList(const TMap<FName, TObjectPtr<ULandscapeGrassType>>& InNamedGrassTypes)
{
	TArray<TPair<FName, TObjectPtr<ULandscapeGrassType>>> LocalNamedGrassTypes;
	LocalNamedGrassTypes.Reserve(InNamedGrassTypes.Num());
	Algo::Transform(InNamedGrassTypes, LocalNamedGrassTypes, [](const TPair<FName, TObjectPtr<ULandscapeGrassType>>& ItPair) { return ItPair; });
	return LocalNamedGrassTypes;
}

TMap<FName, TObjectPtr<ULandscapeGrassType>> FilterValidNamedGrassTypes(const TMap<FName, TObjectPtr<ULandscapeGrassType>>& InNamedGrassTypes)
{
	TMap<FName, TObjectPtr<ULandscapeGrassType>> ValidNamedGrassTypes;
	ValidNamedGrassTypes.Reserve(InNamedGrassTypes.Num());
	for (const TPair<FName, TObjectPtr<ULandscapeGrassType>>& ItPair : InNamedGrassTypes)
	{
		if (!ItPair.Key.IsNone() && (ItPair.Value != nullptr))
		{
			ValidNamedGrassTypes.Add(ItPair);
		}
	}
	return ValidNamedGrassTypes;
}

} // namespace UE::Landscape::Grass