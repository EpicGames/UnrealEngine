// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeUtilsPrivate.h"

#include "DataDrivenShaderPlatformInfo.h"
#include "LandscapeComponent.h"
#include "LandscapeGrassType.h"
#include "LandscapeProxy.h"
#include "LandscapeInfo.h"
#include "LandscapeUtils.h"
#include "RenderingThread.h"

#if WITH_EDITOR
#include "EditorDirectories.h"
#include "ObjectTools.h"
#include "AssetRegistry/AssetRegistryModule.h"
#endif

// Channel remapping
extern const size_t ChannelOffsets[4];

namespace UE::Landscape::Private
{

bool DoesPlatformSupportEditLayers(EShaderPlatform InShaderPlatform)
{
	// Edit layers work on the GPU and are only available on SM5+ and in the editor : 
	return IsFeatureLevelSupported(InShaderPlatform, ERHIFeatureLevel::SM5)
		&& !IsConsolePlatform(InShaderPlatform)
		&& !IsMobilePlatform(InShaderPlatform);
}

int32 ComputeMaxDeltasOffsetForMip(int32 InMipIndex, int32 InNumRelevantMips)
{
	int32 Offset = 0;
	for (int32 X = 0; X < InMipIndex; ++X)
	{
		Offset += InNumRelevantMips - 1 - X;
	}
	return Offset;
}

int32 ComputeMaxDeltasCountForMip(int32 InMipIndex, int32 InNumRelevantMips)
{
	return InNumRelevantMips - 1 - InMipIndex;
}

int32 ComputeMipToMipMaxDeltasIndex(int32 InSourceMipIndex, int32 InDestinationMipIndex, int32 InNumRelevantMips)
{
	check((InSourceMipIndex >= 0) && (InSourceMipIndex < InNumRelevantMips));
	check((InDestinationMipIndex > InSourceMipIndex) && (InDestinationMipIndex < InNumRelevantMips));
	return ComputeMaxDeltasOffsetForMip(InSourceMipIndex, InNumRelevantMips) + InDestinationMipIndex - InSourceMipIndex - 1;
}

int32 ComputeMipToMipMaxDeltasCount(int32 InNumRelevantMips)
{
	int32 Count = 0;
	for (int32 MipIndex = 0; MipIndex < InNumRelevantMips - 1; ++MipIndex)
	{
		Count += InNumRelevantMips - 1 - MipIndex;
	}
	return Count;
}

#if WITH_EDITOR

int32 LandscapeMobileWeightTextureArray = 0;
static FAutoConsoleVariableRef CVarLandscapeMobileWeightTextureArray(
	TEXT("landscape.MobileWeightTextureArray"),
	LandscapeMobileWeightTextureArray,
	TEXT("Use Texture Arrays for weights on Mobile platforms"),
	ECVF_ReadOnly | ECVF_MobileShaderChange);

bool IsMobileWeightmapTextureArrayEnabled()
{
	return LandscapeMobileWeightTextureArray != 0;	
}
	
bool UseWeightmapTextureArray(EShaderPlatform InPlatform)
{
	return IsMobilePlatform(InPlatform) && (LandscapeMobileWeightTextureArray != 0);	
}

void CreateLandscapeComponentLayerDataDuplicate(const FLandscapeLayerComponentData& InSourceComponentData, FLandscapeLayerComponentData& OutDestComponentData)
{
	// Duplicate Heightmap Texture
	OutDestComponentData.HeightmapData.Texture = DuplicateObject(InSourceComponentData.HeightmapData.Texture, InSourceComponentData.HeightmapData.Texture->GetOuter());
	OutDestComponentData.HeightmapData.Texture->PostEditChange();

	// Duplicate Weightmap Textures - avoid duplicating WeightmapTextureUsage since it can be regenerated based on the textures + allocations
	OutDestComponentData.WeightmapData.Textures.Reset(InSourceComponentData.WeightmapData.Textures.Num());
	for (TObjectPtr<UTexture2D> Texture : InSourceComponentData.WeightmapData.Textures)
	{
		TObjectPtr<UTexture2D> WeightmapTextureCopy = DuplicateObject(Texture, Texture->GetOuter());
		WeightmapTextureCopy->PostEditChange();
		OutDestComponentData.WeightmapData.Textures.Add(WeightmapTextureCopy);
	}

	// Copy Layer Allocations
	OutDestComponentData.WeightmapData.LayerAllocations = InSourceComponentData.WeightmapData.LayerAllocations;
}

#endif //!WITH_EDITOR

FIntPoint FLandscapeComponent2DIndexerKeyFuncs::GetKey(ULandscapeComponent* InComponent)
{
	return InComponent->GetComponentKey();
}

FLandscapeComponent2DIndexer CreateLandscapeComponent2DIndexer(const ULandscapeInfo* InInfo)
{
	TArray<ULandscapeComponent*> AllValidComponents;
	InInfo->XYtoComponentMap.GenerateValueArray(AllValidComponents);
	return FLandscapeComponent2DIndexer(AllValidComponents);
}

template<typename DestType, typename ConvertFromColorType>
static int32 BlitRGChannelsToSingleTmpl(DestType* DestBuffer, const FBlitBuffer2DDesc& Dest, uint32* SrcBuffer, const FBlitBuffer2DDesc& Src, FIntRect ClipRect, ConvertFromColorType ConvertFromColor)
{
	check(Dest.Rect.Height() * Dest.Stride <= Dest.BufferSize);
	check(Src.Rect.Height() * Src.Stride <= Src.BufferSize);
	FIntRect BlitRect = Dest.Rect;
	BlitRect.Clip(Src.Rect);
	BlitRect.Clip(ClipRect);
	if (BlitRect.Area() <= 0)
	{
		return 0;
	}

	FIntPoint ReadOffset = BlitRect.Min - Src.Rect.Min;
	FIntPoint WriteOffset = BlitRect.Min - Dest.Rect.Min;
	FIntPoint BlitSize = BlitRect.Size();

	for (int32 Y = BlitRect.Min.Y; Y < BlitRect.Max.Y; ++Y)
	{
		// Set up the pointers that describe the start and end of each scanline to copy.
		int32 SrcY = Y - Src.Rect.Min.Y;
		uint32* Read = SrcBuffer + (SrcY * Src.Stride) + ReadOffset.X;
		uint32* ReadEnd = Read + BlitSize.X;
		int32 DestY = Y - Dest.Rect.Min.Y;
		DestType* Write = DestBuffer + (DestY * Dest.Stride) + WriteOffset.X;
		DestType* WriteEnd = Write + BlitSize.X;

		check(Read >= SrcBuffer);
		check(ReadEnd <= SrcBuffer + Src.BufferSize);
		check(Read <= ReadEnd);
		check(Write >= DestBuffer);
		check(WriteEnd <= DestBuffer + Dest.BufferSize);
		check(Write <= WriteEnd);

		for (; Read < ReadEnd && Write < WriteEnd; ++Read, ++Write)
		{
			FColor Color(*Read);
			DestType Value = ConvertFromColor(Color);
			*Write = Value;
		}
		check(Read == ReadEnd && Write == WriteEnd);
	}
	return BlitRect.Area();
}


int32 BlitHeightChannelsToUint16(uint16* DestBuffer, const FBlitBuffer2DDesc& Dest, uint32* SrcBuffer, const FBlitBuffer2DDesc& Src, FIntRect ClipRect)
{
	// Get 16 bit height value from R and G channels
	auto GetHeight = [](const FColor& HeightAndNormal) { return (static_cast<uint16>(HeightAndNormal.R) << 8) | static_cast<uint16>(HeightAndNormal.G); };

	return BlitRGChannelsToSingleTmpl(DestBuffer, Dest, SrcBuffer, Src, ClipRect, GetHeight);
};

int32 BlitWeightChannelsToUint8(uint8* DestBuffer, const FBlitBuffer2DDesc& Dest, uint32* SrcBuffer, const FBlitBuffer2DDesc& Src, FIntRect ClipRect)
{
	// 16 bit weight value in R and G channels represents 0..1.  65535 maps to 255.
	auto GetWeight = [](const FColor& HeightAndNormal)
		{
			uint32 Weight = (static_cast<uint16>(HeightAndNormal.R) << 8) | static_cast<uint16>(HeightAndNormal.G);
			Weight = Weight * 255 / 65535;
			check(Weight <= 255);
			return static_cast<uint8>(Weight);
		};

	return BlitRGChannelsToSingleTmpl(DestBuffer, Dest, SrcBuffer, Src, ClipRect, GetWeight);
};

void SubmitGPUCommands(bool bBlockUntilRTComplete, bool bBlockRTUntilGPUComplete)
{
	FEvent* ResultsReadyEvent = nullptr;
	if (bBlockUntilRTComplete)
	{
		ResultsReadyEvent = FPlatformProcess::GetSynchEventFromPool(true);
	}

	ENQUEUE_RENDER_COMMAND(FFlushResourcesCommand)(
		[ResultsReadyEvent, bBlockRTUntilGPUComplete](FRHICommandListImmediate& RHICmdList)
		{
			constexpr bool bFlushResources = true;
			RHICmdList.SubmitAndBlockUntilGPUIdle(bFlushResources);

			if (ResultsReadyEvent)
			{
				if (bBlockRTUntilGPUComplete)
				{
					// Block render thread waiting for GPU to complete.  Note this can be very expensive on some platforms.
					RHICmdList.SubmitAndBlockUntilGPUIdle();
				}

				ResultsReadyEvent->Trigger();
			}
		});

	if (ResultsReadyEvent)
	{
		// block game thread waiting for render thread to tell us the GPU is complete
		ResultsReadyEvent->Wait();
		ResultsReadyEvent->Reset();
		FPlatformProcess::ReturnSynchEventToPool(ResultsReadyEvent);
	}

	return;
}

} // end namespace UE::Landscape::Private


// ----------------------------------------------------------------------------------

namespace UE::Landscape::Grass::Private
{

uint32 ComputeNamedGrassTypesHash(const TMap<FName, TObjectPtr<ULandscapeGrassType>>& InNamedGrassTypes)
{
	uint32 Hash = 0;

	// Only keep the valid entries
	TMap<FName, TObjectPtr<ULandscapeGrassType>> ValidNamedGrassTypes = FilterValidNamedGrassTypes(InNamedGrassTypes);
	// Turn into an array  
	const TArray<TPair<FName, TObjectPtr<ULandscapeGrassType>>> SortedNamedGrassTypes = ConvertToNamedGrassTypesList(ValidNamedGrassTypes);
	// Then sort by grass name for hash stability
	Algo::SortBy(SortedNamedGrassTypes, &TPair<FName, TObjectPtr<ULandscapeGrassType>>::Key, FNameLexicalLess());

	// If anything changes in the grass types, that affects the grass instances
	for (const TPair<FName, TObjectPtr<ULandscapeGrassType>>& ItPair : SortedNamedGrassTypes)
	{
		check(!ItPair.Key.IsNone() && (ItPair.Value != nullptr));

		Hash = FCrc::TypeCrc32(GetTypeHash(ItPair.Key.ToString().ToLower()), Hash);
		Hash = FCrc::TypeCrc32(ItPair.Value->StateHash, Hash);
	}

	return Hash;
}

} // namespace UE::Landscape::Grass::Private
