// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"

enum EShaderPlatform : uint16;
class ULandscapeComponent;
class ULandscapeInfo;
class ULandscapeGrassType;
struct FLandscapeLayerComponentData;

namespace UE::Landscape::Private
{

int32 ComputeMaxDeltasOffsetForMip(int32 InMipIndex, int32 InNumRelevantMips);
int32 ComputeMaxDeltasCountForMip(int32 InMipIndex, int32 InNumRelevantMips);
int32 ComputeMipToMipMaxDeltasIndex(int32 InSourceMipIndex, int32 InDestinationMipIndex, int32 InNumRelevantMips);
int32 ComputeMipToMipMaxDeltasCount(int32 InNumRelevantMips);

#if WITH_EDITOR
/** Returns true if InPlatform is a mobile platform and the Landscape.MobileWeightTextureArray CVar set */
bool UseWeightmapTextureArray(EShaderPlatform InPlatform);

/** Check if Landscape.MobileWeightTextureArray CVar set and we should attempt to use texture arrays for weight maps on mobile platforms  */
bool IsMobileWeightmapTextureArrayEnabled();

void CreateLandscapeComponentLayerDataDuplicate(const FLandscapeLayerComponentData& InSourceComponentData, FLandscapeLayerComponentData& OutDestComponentData);
#endif //!WITH_EDITOR

/** 
 * Helper class to transform a list of values into a 2D array. This is a replacement for a TMap<FIntPoint, ValueType>, where FIntPoint is a 2D key and ValueType, the type of object to store. 
 * It uses bit arrays to note the presence of an object and the 2D key to index it. The advantage is that the key can trivially be turned into linear index (and vice-versa), derived from the bounds 
 * of all registered keys. It is meant to be used as a temporary helper, which is why there are currently no Add/Remove functions (all objects are registered in the constructor) but 
 * this could be done eventually, and even replace ULandscapeInfo's XYToComponentMap entirely */ 
template <typename ValueType, typename KeyFuncs>
class T2DIndexer
{
public:
	T2DIndexer(TConstArrayView<ValueType> InValues)
	{
		FIntRect MinMaxKeys(FIntPoint(MAX_int32, MAX_int32), FIntPoint(MIN_int32, MIN_int32));

		TArray<FIntPoint> AllKeys;
		const int32 NumValues = InValues.Num();
		AllKeys.AddDefaulted(NumValues);

		// First, we need the bound of all values so that we can properly size the linear array : 
		for (int32 Index = 0; Index < NumValues; ++Index)
		{
			const ValueType& Value = InValues[Index];
			FIntPoint Key = KeyFuncs::GetKey(Value);
			AllKeys[Index] = Key;
			MinMaxKeys.Include(Key);
		}
		if (!AllKeys.IsEmpty())
		{
			KeyExclusiveBounds = FIntRect(MinMaxKeys.Min, MinMaxKeys.Max + 1);
		}

		// Now we can properly size the array and fill it : 
		int32 NumEntries = KeyExclusiveBounds.Area();
		AllValues.AddDefaulted(NumEntries);
		ValidValueBitIndices.Init(false, NumEntries);
		for (int32 Index = 0; Index < NumValues; ++Index)
		{
			int32 ValueIndex = GetValueIndexForKey(AllKeys[Index]);
			AllValues[ValueIndex] = InValues[Index];
			ValidValueBitIndices[ValueIndex] = true;
		}
	}
	virtual ~T2DIndexer() = default;
	T2DIndexer(const T2DIndexer& Other) = default;
	T2DIndexer(T2DIndexer&& Other) = default;
	T2DIndexer& operator=(const T2DIndexer& Other) = default;
	T2DIndexer& operator=(T2DIndexer&& Other) = default;

	bool IsValidKey(const FIntPoint& InKey) const
	{
		return KeyExclusiveBounds.Contains(InKey);
	}

	bool IsValidValueIndex(int32 InIndex) const
	{
		return AllValues.IsValidIndex(InIndex);
	}

	ValueType& GetValueForKey(const FIntPoint& InKey) const
	{
		return AllValues[GetValueIndexForKey(InKey)];
	}

	ValueType& GetValueForKeySafe(const FIntPoint& InKey) const
	{
		return IsValidKey(InKey) ? AllValues[GetValueIndexForKey(InKey)] : nullptr;
	}

	ValueType& GetValueForKeyChecked(const FIntPoint& InKey) const
	{
		check(IsValidKey(InKey));
		return AllValues[GetValueIndexForKey(InKey)];
	}

	int32 GetValueIndexForKey(const FIntPoint& InKey) const
	{
		FIntPoint RelativeKey = InKey - KeyExclusiveBounds.Min;
		return RelativeKey.Y * KeyExclusiveBounds.Width() + RelativeKey.X;
	}

	int32 GetValueIndexForKeySafe(const FIntPoint& InKey) const
	{
		return IsValidKey(InKey) ? GetValueIndexForKey(InKey) : INDEX_NONE;
	}

	int32 GetValueIndexForKeyChecked(const FIntPoint& InKey) const
	{
		check(IsValidKey(InKey));
		return GetValueIndexForKey(InKey);
	}

	FIntPoint GetValueKeyForIndex(int32 InIndex) const
	{
		const int32 Stride = KeyExclusiveBounds.Width();
		return KeyExclusiveBounds.Min + FIntPoint(InIndex % Stride, InIndex / Stride);
	}

	FIntPoint GetValueKeyForIndexSafe(int32 InIndex) const
	{
		return IsValidValueIndex(InIndex) ? GetValueKeyForIndex(InIndex) : FIntPoint::NoneValue;
	}

	FIntPoint GetValueKeyForIndexChecked(int32 InIndex) const
	{
		check(IsValidValueIndex(InIndex));
		return GetValueKeyForIndex(InIndex);
	}

	int32 GetValueIndex(const ValueType& InValue) const
	{
		return GetValueIndexForKey(KeyFuncs::GetKey(InValue));
	}

	int32 GetValueIndexSafe(const ValueType& InValue) const
	{
		return GetValueIndexForKeySafe(KeyFuncs::GetKey(InValue));
	}

	int32 GetValueIndexChecked(const ValueType& InValue) const
	{
		return GetValueIndexForKeyChecked(KeyFuncs::GetKey(InValue));
	}

	bool IsValidValue(const FIntPoint& InKey) const
	{
		return ValidValueBitIndices[GetValueIndexForKey(InKey)];
	}

	bool IsValidValueSafe(const FIntPoint& InKey) const
	{
		return IsValidKey(InKey) ? ValidValueBitIndices[GetValueIndexForKey(InKey)] : false;
	}

	bool IsValidValueChecked(const FIntPoint& InKey) const
	{
		check(IsValidKey(InKey));
		return ValidValueBitIndices[GetValueIndexForKey(InKey)];
	}

	bool HasValidValueInBounds(const FIntRect& InBounds, bool bInInclusiveBounds)
	{
		FIntRect LocalBounds = InBounds;
		// Convert to exclusive bounds if necessary :
		if (bInInclusiveBounds)
		{
			LocalBounds.Max += FIntPoint(1, 1);
		}
		// Intersect with the current key bounds in order to limit the test to only the tracked area :
		LocalBounds.Clip(KeyExclusiveBounds);
		if (LocalBounds.Area() <= 0)
		{
			return false;
		}

		for (int32 Y = LocalBounds.Min.Y; (Y < LocalBounds.Max.Y); ++Y)
		{
			for (int32 X = LocalBounds.Min.X; (X < LocalBounds.Max.X); ++X)
			{
				if (IsValidValue(FIntPoint(X, Y)))
				{
					return true;
				}
			}
		}

		return false;
	}

	TBitArray<> GetValidValueBitIndicesInBounds(const FIntRect& InBounds, bool bInInclusiveBounds)
	{
		TBitArray<> Result(false, ValidValueBitIndices.Num());

		FIntRect LocalBounds = InBounds;
		// Convert to inclusive bounds if necessary :
		if (bInInclusiveBounds)
		{
			LocalBounds.Max += FIntPoint(1, 1);
		}
		// Intersect with the current key bounds in order to limit the test to only the tracked area :
		LocalBounds.Clip(KeyExclusiveBounds);
		if (LocalBounds.Area() <= 0)
		{
			return Result;
		}

		for (int32 Y = LocalBounds.Min.Y; (Y < LocalBounds.Max.Y); ++Y)
		{
			for (int32 X = LocalBounds.Min.X; (X < LocalBounds.Max.X); ++X)
			{
				int32 ValueIndex = GetValueIndexForKey(FIntPoint(X, Y));
				Result[ValueIndex] = ValidValueBitIndices[ValueIndex];
			}
		}

		return Result;
	}

	TArray<ValueType> GetValidValuesForBitIndices(const TBitArray<>& InBitIndices)
	{
		// Intersect the list and the list of valid elements : 
		check(InBitIndices.Num() == ValidValueBitIndices.Num());
		TBitArray<> LocalBitIndices = TBitArray<>::BitwiseAND(ValidValueBitIndices, InBitIndices, EBitwiseOperatorFlags::MinSize);

		TArray<ValueType> Result;
		Result.Reserve(InBitIndices.CountSetBits());
		for (TConstSetBitIterator It(LocalBitIndices); It; ++It)
		{
			Result.Add(AllValues[It.GetIndex()]);
		}
		return Result;
	}

	FIntRect GetValidValuesBoundsForBitIndices(const TBitArray<>& InBitIndices, bool bInInclusiveBounds)
	{
		// Intersect the list and the list of valid elements : 
		check(InBitIndices.Num() == ValidValueBitIndices.Num());
		TBitArray<> LocalBitIndices = TBitArray<>::BitwiseAND(ValidValueBitIndices, InBitIndices, EBitwiseOperatorFlags::MinSize);
		if (LocalBitIndices.IsEmpty())
		{
			return FIntRect();
		}

		const int32 FirstSetBitIndex = InBitIndices.Find(true);
		const int32 LastSetBitIndex = InBitIndices.FindLast(true);
		if (FirstSetBitIndex == INDEX_NONE)
		{
			check(LastSetBitIndex == INDEX_NONE);
			return FIntRect();
		}

		bool bIsValid = false;
		const int32 Stride = KeyExclusiveBounds.Width();
		FIntRect Bounds(FIntPoint(MAX_int32, MAX_int32), FIntPoint(MIN_int32, MIN_int32));
		const int32 YMax = FMath::DivideAndRoundDown(LastSetBitIndex, Stride) + 1;
		for (int32 Y = FirstSetBitIndex / Stride; Y < YMax; ++Y)
		{
			const int32 LineFirstSetBitIndex = LocalBitIndices.FindFrom(true, Y * Stride);
			if (LineFirstSetBitIndex != INDEX_NONE)
			{
				const int32 LineLastSetBitIndex = LocalBitIndices.FindLastFrom(true, (Y + 1) * Stride - 1);
				check(LineLastSetBitIndex != INDEX_NONE);
				FIntPoint LineMinKey = GetValueKeyForIndex(LineFirstSetBitIndex);
				FIntPoint LineMaxKey = GetValueKeyForIndex(LineLastSetBitIndex);
				Bounds.Min = Bounds.Min.ComponentMin(LineMinKey);
				Bounds.Max = Bounds.Max.ComponentMax(LineMaxKey);
				bIsValid = true;
			}
		}

		if (!bInInclusiveBounds)
		{
			Bounds.Max += FIntPoint(1, 1);
		}

		check(bIsValid);
		return Bounds;
	}

	TArray<ValueType> GetValidValues()
	{
		TArray<ValueType> Result;
		Result.Reserve(ValidValueBitIndices.CountSetBits());
		for (TConstSetBitIterator It(ValidValueBitIndices); It; ++It)
		{
			Result.Add(AllValues[It.GetIndex()]);
		}
		return Result;
	}

	const TArray<ValueType>& GetAllValues() const { return AllValues; }
	const TBitArray<>& GetValidValueBitIndices() const { return ValidValueBitIndices; }

private:
	FIntRect KeyExclusiveBounds;
	TArray<ValueType> AllValues;
	TBitArray<> ValidValueBitIndices;
};

struct FLandscapeComponent2DIndexerKeyFuncs
{
	static FIntPoint GetKey(ULandscapeComponent* InComponent);
};

using FLandscapeComponent2DIndexer = T2DIndexer<ULandscapeComponent*, FLandscapeComponent2DIndexerKeyFuncs>;
FLandscapeComponent2DIndexer CreateLandscapeComponent2DIndexer(const ULandscapeInfo* InInfo);


struct FBlitBuffer2DDesc
{
	int32 BufferSize = 0;  // Buffer size in elements.
	int32 Stride = 0;      // Buffer stride in elements.  i.e. the distance between the start of each row.
	FIntRect Rect;         // The 2D region represented by this buffer.  Rect.Area() should be <= BufferSize.
};

/* Copy height values from the R+G channels of the source buffer into a plain uint16 dest buffer.  The source and destination buffers
 * represent sub-regions of a shared 2D coordinate system expressed in the Rect parameters.  ClipRect is the desired area of that
 * shared coordinate space to limit the operation to.  Copies only values in the the intersection of Src.Rect, Dest.Rect, and ClipRect.
 * All FIntRect params are treated as half-open.  .Min is included.  .Max is excluded.
 * @param DestBuffer destination buffer
 * @param Dest description of the destination buffer
 * @param SrcBuffer source buffer
 * @param Src description of the source buffer
 * @param ClipRect The operation will be limited to this region, in the same shared coordinate space of the rects from the buffer descriptions.
 * @return number of elements copied.
 */
int32 BlitHeightChannelsToUint16(uint16* DestBuffer, const FBlitBuffer2DDesc& Dest, uint32* SrcBuffer, const FBlitBuffer2DDesc& Src, FIntRect ClipRect);


/* Copy 16-bit weight values from the R+G channels of the source buffer, scaled down into a plain uint8 dest buffer.  The source and destination buffers
 * represent sub-regions of a shared 2D coordinate system expressed in the Rect parameters.  ClipRect is the desired area of that
 * shared coordinate space to limit the operation to.  Copies only values in the the intersection of Src.Rect, Dest.Rect, and ClipRect.
 * All FIntRect params are treated as half-open.  .Min is included.  .Max is excluded.
 * @param DestBuffer destination buffer
 * @param Dest description of the destination buffer
 * @param SrcBuffer source buffer
 * @param Src description of the source buffer
 * @param ClipRect The operation will be limited to this region, in the same shared coordinate space of the rects from the buffer descriptions.
 * @return number of elements copied.
 */
int32 BlitWeightChannelsToUint8(uint8* DestBuffer, const FBlitBuffer2DDesc& Dest, uint32* SrcBuffer, const FBlitBuffer2DDesc& Src, FIntRect ClipRect);

void SubmitGPUCommands(bool bBlockUntilRTComplete, bool bBlockRTUntilGPUComplete);

} // end namespace UE::Landscape::Private


// ----------------------------------------------------------------------------------

namespace UE::Landscape::Grass::Private
{

uint32 ComputeNamedGrassTypesHash(const TMap<FName, TObjectPtr<ULandscapeGrassType>>& InNamedGrassTypes);

} // namespace UE::Landscape::Grass::Private