// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"

// Custom version for landscape serialization
namespace FLandscapeCustomVersion
{
	enum Type
	{
		// Before any version changes were made in the plugin
		BeforeCustomVersionWasAdded = 0,
		// Changed to TMap properties instead of manual serialization and fixed landscape spline control point cross-level mesh components not being serialized
		NewSplineCrossLevelMeshSerialization,
		// Support material world-position-offset in the heightmap used for grass placement
		GrassMaterialWPO,
		// Support material world-position-offset in landscape simple collision
		CollisionMaterialWPO,
		// Support material world-position-offset in landscape lighting mesh
		LightmassMaterialWPO,
		// Fix for landscape grass not updating when using a material instance as the landscape material and changing parameters
		GrassMaterialInstanceFix,
		// Fix for spline Foreign data that used TMap<TLazyObjectPtr, ...> to now use TArray<...> to prevent key creation every time we do a search thus putting the package dirty
		SplineForeignDataLazyObjectPtrFix,
		// Migration of deprecated property from old landscape rendering to new landscape Rendering 
		MigrateOldPropertiesToNewRenderingProperties,
		// Migration of old EnabledCollision for spline elements to the full body instance exposed
		AddingBodyInstanceToSplinesElements,
		// Spline Layer Falloff
		AddSplineLayerFalloff,
		// Spline Layer Width
		AddSplineLayerWidth,
		// New LOD distribution and tessellation calculations are introduced, needs parameter conversion
		NewLandscapeContinuousLOD,
		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version
	const static FGuid GUID = FGuid(0x23AFE18E, 0x4CE14E58, 0x8D61C252, 0xB953BEB7);
};
