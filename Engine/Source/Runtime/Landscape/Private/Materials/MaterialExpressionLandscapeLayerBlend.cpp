// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/MaterialExpressionLandscapeLayerBlend.h"
#include "Engine/Engine.h"
#include "Engine/Texture.h"
#include "EngineGlobals.h"
#include "MaterialCompiler.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "LandscapeUtilsPrivate.h"
#include "Materials/MaterialAttributeDefinitionMap.h"
#include "Materials/MaterialExpressionShadingModel.h"

#if WITH_EDITOR
#include "MaterialGraph/MaterialGraphNode.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionLandscapeLayerBlend)

#define LOCTEXT_NAMESPACE "Landscape"


///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionLandscapeLayerBlend
///////////////////////////////////////////////////////////////////////////////
UMaterialExpressionLandscapeLayerBlend::UMaterialExpressionLandscapeLayerBlend(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_Landscape;
		FConstructorStatics()
			: NAME_Landscape(LOCTEXT("Landscape", "Landscape"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Landscape);
#endif
}

void UMaterialExpressionLandscapeLayerBlend::Serialize(FStructuredArchive::FRecord Record)
{
	Super::Serialize(Record);
	FArchive& UnderlyingArchive = Record.GetUnderlyingArchive();

	if (UnderlyingArchive.IsLoading() && UnderlyingArchive.UEVer() < VER_UE4_ADD_LB_WEIGHTBLEND)
	{
		// convert any LB_AlphaBlend entries to LB_WeightBlend
		for (FLayerBlendInput& LayerInput : Layers)
		{
			if (LayerInput.BlendType == LB_AlphaBlend)
			{
				LayerInput.BlendType = LB_WeightBlend;
			}
		}
	}
}

#if WITH_EDITOR

TArrayView<FExpressionInput*> UMaterialExpressionLandscapeLayerBlend::GetInputsView()
{
	CachedInputs.Empty();
	CachedInputs.Reserve(Layers.Num() * 2);
	for (int32 LayerIdx = 0; LayerIdx<Layers.Num(); LayerIdx++)
	{
		CachedInputs.Add(&Layers[LayerIdx].LayerInput);
		if (Layers[LayerIdx].BlendType == LB_HeightBlend)
		{
			CachedInputs.Add(&Layers[LayerIdx].HeightInput);
		}
	}
	return CachedInputs;
}


FExpressionInput* UMaterialExpressionLandscapeLayerBlend::GetInput(int32 InputIndex)
{
	int32 Idx = 0;
	for (int32 LayerIdx = 0; LayerIdx<Layers.Num(); LayerIdx++)
	{
		if (InputIndex == Idx++)
		{
			return &Layers[LayerIdx].LayerInput;
		}
		if (Layers[LayerIdx].BlendType == LB_HeightBlend)
		{
			if (InputIndex == Idx++)
			{
				return &Layers[LayerIdx].HeightInput;
			}
		}
	}
	return nullptr;
}


FName UMaterialExpressionLandscapeLayerBlend::GetInputName(int32 InputIndex) const
{
	int32 Idx = 0;
	for (int32 LayerIdx = 0; LayerIdx<Layers.Num(); LayerIdx++)
	{
		if (InputIndex == Idx++)
		{
			return *FString::Printf(TEXT("Layer %s"), *Layers[LayerIdx].LayerName.ToString());
		}
		if (Layers[LayerIdx].BlendType == LB_HeightBlend)
		{
			if (InputIndex == Idx++)
			{
				return *FString::Printf(TEXT("Height %s"), *Layers[LayerIdx].LayerName.ToString());
			}
		}
	}
	return NAME_None;
}

EMaterialValueType UMaterialExpressionLandscapeLayerBlend::GetInputValueType(int32 InputIndex)
{
	int32 Idx = 0;
	for (int32 LayerIdx = 0; LayerIdx<Layers.Num(); LayerIdx++)
	{
		if (InputIndex == Idx++)
		{
			return static_cast<EMaterialValueType>(MCT_Float | MCT_MaterialAttributes); // can accept pretty much anything including MaterialAttributes
		}
		if (Layers[LayerIdx].BlendType == LB_HeightBlend)
		{
			if (InputIndex == Idx++)
			{
				return MCT_Float1; // the height input must be float1
			}
		}
	}

	return MCT_Unknown;
}

bool UMaterialExpressionLandscapeLayerBlend::IsResultMaterialAttributes(int32 OutputIndex)
{
	for (int32 LayerIdx = 0; LayerIdx < Layers.Num(); LayerIdx++)
	{
		if (Layers[LayerIdx].LayerInput.Expression && Layers[LayerIdx].LayerInput.Expression->IsResultMaterialAttributes(Layers[LayerIdx].LayerInput.OutputIndex))
		{
			return true;
		}
	}
	return false;
}

int32 UMaterialExpressionLandscapeLayerBlend::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// For renormalization
	bool bNeedsRenormalize = false;
	int32 WeightSumCode = Compiler->Constant(0);
	// Temporary store for each layer's weight
	TArray<int32> WeightCodes;
	WeightCodes.Empty(Layers.Num());

	const bool bTextureArrayEnabled = UE::Landscape::Private::UseWeightmapTextureArray(Compiler->GetShaderPlatform());
	
	for (int32 LayerIdx = 0; LayerIdx<Layers.Num(); LayerIdx++)
	{
		WeightCodes.Add(INDEX_NONE);

		FLayerBlendInput& Layer = Layers[LayerIdx];

		// LB_AlphaBlend layers are blended last
		if (Layer.BlendType != LB_AlphaBlend)
		{
			// Height input
			const int32 HeightCode = Layer.HeightInput.Expression ? Layer.HeightInput.Compile(Compiler) : Compiler->Constant(Layer.ConstHeightInput);

			const int32 DefaultWeightCode = Layer.PreviewWeight > 0.0f ? Compiler->Constant(Layer.PreviewWeight) : INDEX_NONE;
			const int32 WeightCode = Compiler->StaticTerrainLayerWeight(Layer.LayerName, DefaultWeightCode, bTextureArrayEnabled);
			if (WeightCode != INDEX_NONE)
			{
				switch (Layer.BlendType)
				{
				case LB_WeightBlend:
				{
					// Store the weight plus accumulate the sum of all weights so far
					WeightCodes[LayerIdx] = WeightCode;
					WeightSumCode = Compiler->Add(WeightSumCode, WeightCode);
				}
					break;
				case LB_HeightBlend:
				{
					bNeedsRenormalize = true;

					// Modify weight with height
					int32 ModifiedWeightCode = Compiler->Clamp(
						Compiler->Add(Compiler->Lerp(Compiler->Constant(-1.f), Compiler->Constant(1.f), WeightCode), HeightCode),
						Compiler->Constant(0.0001f), Compiler->Constant(1.f));

					// Store the final weight plus accumulate the sum of all weights so far
					WeightCodes[LayerIdx] = ModifiedWeightCode;
					WeightSumCode = Compiler->Add(WeightSumCode, ModifiedWeightCode);
				}
					break;
				}
			}
		}
	}

	int32 InvWeightSumCode = Compiler->Div(Compiler->Constant(1.f), WeightSumCode);

	// If the output has a ShadingModel type, regular arithmetic nodes (weight/blend) needs to be handled differently
	int32 OutputCode = INDEX_NONE;
	int32 DefaultCode = INDEX_NONE;
	bool bIsShadingModel = false;
	const EMaterialProperty Property = FMaterialAttributeDefinitionMap::GetProperty(Compiler->GetMaterialAttribute());
	if (Property == MP_ShadingModel)
	{
		OutputCode = Compiler->ShadingModel(EMaterialShadingModel::MSM_DefaultLit);
		bIsShadingModel = true;
	}
	else
	{
		// Ensure all MA properties have proper default values compiled
		if (IsResultMaterialAttributes(OutputIndex) && Property != MP_MAX)
		{
			OutputCode = FMaterialAttributeDefinitionMap::CompileDefaultExpression(Compiler, Compiler->GetMaterialAttribute());
			// When blending MA, accumulate deltas from the default (LayerCode - DefaultCode) so that the formula Default + Sum(delta * W * InvSum) gives the correct weighted average.
			DefaultCode = OutputCode;
		}
		else
		{
			OutputCode = Compiler->Constant(0);
		}
	}

	for (int32 LayerIdx = 0; LayerIdx<Layers.Num(); LayerIdx++)
	{
		FLayerBlendInput& Layer = Layers[LayerIdx];

		if (WeightCodes[LayerIdx] != INDEX_NONE)
		{
			// Layer input
			const int32 LayerCode = Layer.LayerInput.Expression
				                        ? Layer.LayerInput.Compile(Compiler)
				                        : Compiler->Constant3(static_cast<float>(Layer.ConstLayerInput.X),
															  static_cast<float>(Layer.ConstLayerInput.Y),
				                                              static_cast<float>(Layer.ConstLayerInput.Z));

			const int32 ContributionCode = (DefaultCode != INDEX_NONE) ? Compiler->Sub(LayerCode, DefaultCode) : LayerCode;
			if (bIsShadingModel && Compiler->GetType(LayerCode) == EMaterialValueType::MCT_ShadingModel)
			{
				OutputCode = CompileShadingModelBlendFunction(Compiler, OutputCode, LayerCode, WeightCodes[LayerIdx]);
			}
			else if (bNeedsRenormalize)
			{
				// Renormalize the weights as our height modification has made them non-uniform
				OutputCode = Compiler->Add(OutputCode, Compiler->Mul(ContributionCode, Compiler->Mul(InvWeightSumCode, WeightCodes[LayerIdx])));
			}
			else
			{
				// No renormalization is necessary, so just add the weights
				OutputCode = Compiler->Add(OutputCode, Compiler->Mul(ContributionCode, WeightCodes[LayerIdx]));
			}
		}
	}

	// Blend in LB_AlphaBlend layers
	for (FLayerBlendInput& Layer : Layers)
	{
		if (Layer.BlendType == LB_AlphaBlend)
		{
			const int32 DefaultWeightCode = Layer.PreviewWeight > 0.0f ? Compiler->Constant(Layer.PreviewWeight) : INDEX_NONE;
			const int32 WeightCode = Compiler->StaticTerrainLayerWeight(Layer.LayerName, DefaultWeightCode, bTextureArrayEnabled);
			if (WeightCode != INDEX_NONE)
			{
				const int32 LayerCode = Layer.LayerInput.Expression
					                        ? Layer.LayerInput.Compile(Compiler)
					                        : Compiler->Constant3(static_cast<float>(Layer.ConstLayerInput.X),
					                                              static_cast<float>(Layer.ConstLayerInput.Y),
					                                              static_cast<float>(Layer.ConstLayerInput.Z));
				
				// Blend in the layer using the alpha value
				if (bIsShadingModel && Compiler->GetType(LayerCode) == EMaterialValueType::MCT_ShadingModel)
				{
					OutputCode = CompileShadingModelBlendFunction(Compiler, OutputCode, LayerCode, WeightCode);
				}
				else
				{
					OutputCode = Compiler->Lerp(OutputCode, LayerCode, WeightCode);
				}
			}
		}
	}

	if (OutputCode != INDEX_NONE)
	{
		// We've definitely passed the reentrant check here so we're good to call IsResultMaterialAttributes().
		bool bFoundExpression = false;
		bool bIsResultMaterialAttributes = false;
		for (int32 LayerIdx = 0; LayerIdx < Layers.Num(); LayerIdx++)
		{
			if (Layers[LayerIdx].HeightInput.Expression)
			{
				bool bHeightIsMaterialAttributes = Layers[LayerIdx].HeightInput.Expression->IsResultMaterialAttributes(Layers[LayerIdx].HeightInput.OutputIndex);
				if (bHeightIsMaterialAttributes)
				{
					Compiler->Errorf(TEXT("Height input (%s) does not accept MaterialAttributes"), *(Layers[LayerIdx].LayerName.ToString()));
				}
			}
			if (Layers[LayerIdx].LayerInput.Expression)
			{
				bool bLayerIsMaterialAttributes = Layers[LayerIdx].LayerInput.Expression->IsResultMaterialAttributes(Layers[LayerIdx].LayerInput.OutputIndex);
				if (!bFoundExpression)
				{
					bFoundExpression = true;
					bIsResultMaterialAttributes = bLayerIsMaterialAttributes;
				}
				else if (bIsResultMaterialAttributes != bLayerIsMaterialAttributes)
				{
					Compiler->Error(TEXT("Cannot mix MaterialAttributes and non MaterialAttributes nodes"));
					break;
				}
			}
		}
	}

	return OutputCode;
}

#endif // WITH_EDITOR

UObject* UMaterialExpressionLandscapeLayerBlend::GetReferencedTexture() const
{
	return GEngine->WeightMapPlaceholderTexture;
}

UMaterialExpression::ReferencedTextureArray UMaterialExpressionLandscapeLayerBlend::GetReferencedTextures() const
{
	return { GEngine->WeightMapPlaceholderTexture, GEngine->WeightMapArrayPlaceholderTexture };
}

#if WITH_EDITOR
void UMaterialExpressionLandscapeLayerBlend::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("Landscape Layer Blend"));
}

void UMaterialExpressionLandscapeLayerBlend::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Clear out any height expressions for layers not using height blending
	for (int32 LayerIdx = 0; LayerIdx<Layers.Num(); LayerIdx++)
	{
		if (Layers[LayerIdx].BlendType != LB_HeightBlend)
		{
			Layers[LayerIdx].HeightInput.Expression = nullptr;
		}
	}

	if (PropertyChangedEvent.MemberProperty)
	{
		const FName PropertyName = PropertyChangedEvent.MemberProperty->GetFName();
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UMaterialExpressionLandscapeLayerBlend, Layers))
		{
			if (UMaterialGraphNode* MatGraphNode = Cast<UMaterialGraphNode>(GraphNode))
			{
				MatGraphNode->RecreateAndLinkNode();
			}
		}
	}
}

void UMaterialExpressionLandscapeLayerBlend::GetLandscapeLayerNames(TArray<FName>& OutLayers) const
{
	for (const FLayerBlendInput& Layer : Layers)
	{
		OutLayers.AddUnique(Layer.LayerName);
	}
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
