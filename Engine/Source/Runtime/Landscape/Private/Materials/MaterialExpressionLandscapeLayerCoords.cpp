// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/MaterialExpressionLandscapeLayerCoords.h"
#include "LandscapePrivate.h"
#include "MaterialCompiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionLandscapeLayerCoords)

#define LOCTEXT_NAMESPACE "Landscape"


///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionLandscapeLayerCoords
///////////////////////////////////////////////////////////////////////////////

UMaterialExpressionLandscapeLayerCoords::UMaterialExpressionLandscapeLayerCoords(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_Landscape;
		FConstructorStatics()
			: NAME_Landscape(LOCTEXT("Landscape", "Landscape"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Landscape);

	bCollapsed = false;
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionLandscapeLayerCoords::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	switch (CustomUVType)
	{
	case LCCT_CustomUV0:
		return Compiler->TextureCoordinate(0, false, false);
	case LCCT_CustomUV1:
		return Compiler->TextureCoordinate(1, false, false);
	case LCCT_CustomUV2:
		return Compiler->TextureCoordinate(2, false, false);
	case LCCT_WeightMapUV:
		return Compiler->TextureCoordinate(3, false, false);
	default:
		break;
	}

	int32 BaseUV;

	switch (MappingType)
	{
	case TCMT_Auto:
	case TCMT_XY: BaseUV = Compiler->TextureCoordinate(0, false, false); break;
	case TCMT_XZ: BaseUV = Compiler->TextureCoordinate(1, false, false); break;
	case TCMT_YZ:
		{
			int Y = Compiler->ComponentMask(Compiler->TextureCoordinate(0, false, false), 0, 1, 0, 0);
			int Z = Compiler->ComponentMask(Compiler->TextureCoordinate(1, false, false), 0, 1, 0, 0);

			BaseUV = Compiler->AppendVector(Y, Z);
		}
		break;
	default:
		UE_LOGF(LogLandscape, Fatal, "Invalid mapping type %u", (uint8)MappingType);
		return INDEX_NONE;
	};

	float Scale = (MappingScale == 0.0f) ? 1.0f : 1.0f / MappingScale;
	const float RotX = FMath::Cos(MappingRotation * PI / 180.0f) * Scale;
	const float RotY = FMath::Sin(MappingRotation * PI / 180.0f) * Scale;

	int32 TransformedUV = Compiler->Add(
		Compiler->AppendVector(
			Compiler->Dot(BaseUV, Compiler->Constant2(+RotX, +RotY)),
			Compiler->Dot(BaseUV, Compiler->Constant2(-RotY, +RotX))),
		Compiler->Constant2(MappingPanU, MappingPanV)
	);
	
	return TransformedUV;
}


void UMaterialExpressionLandscapeLayerCoords::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("Landscape Coords"));
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
