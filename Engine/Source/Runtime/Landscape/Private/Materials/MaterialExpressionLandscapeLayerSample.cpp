// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/MaterialExpressionLandscapeLayerSample.h"
#include "Engine/Engine.h"
#include "Engine/Texture.h"
#include "EngineGlobals.h"
#include "MaterialCompiler.h"
#include "Materials/Material.h"
#include "LandscapeUtilsPrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionLandscapeLayerSample)

#define LOCTEXT_NAMESPACE "Landscape"


///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionLandscapeLayerSample
///////////////////////////////////////////////////////////////////////////////

UMaterialExpressionLandscapeLayerSample::UMaterialExpressionLandscapeLayerSample(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_Landscape;
		FConstructorStatics()
			: NAME_Landscape(LOCTEXT("Landscape", "Landscape"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;
#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Landscape);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionLandscapeLayerSample::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	const bool bTextureArrayEnabled = UE::Landscape::Private::UseWeightmapTextureArray(Compiler->GetShaderPlatform());
	const int32 WeightCode = Compiler->StaticTerrainLayerWeight(ParameterName, Compiler->Constant(PreviewWeight), bTextureArrayEnabled);
	if (WeightCode == INDEX_NONE)
	{
		// layer is not used in this component, sample value is 0.
		return Compiler->Constant(0.f);
	}
	else
	{
		return WeightCode;
	}
}

#endif // WITH_EDITOR

UObject* UMaterialExpressionLandscapeLayerSample::GetReferencedTexture() const
{
	return GEngine->WeightMapPlaceholderTexture;
}

UMaterialExpression::ReferencedTextureArray UMaterialExpressionLandscapeLayerSample::GetReferencedTextures() const
{
	return { GEngine->WeightMapPlaceholderTexture, GEngine->WeightMapArrayPlaceholderTexture };
}

#if WITH_EDITOR
FString UMaterialExpressionLandscapeLayerSample::GetEditableName() const
{
	return ParameterName.ToString();
}

void UMaterialExpressionLandscapeLayerSample::SetEditableName(const FString& NewName)
{
	ParameterName = *NewName;
}

void UMaterialExpressionLandscapeLayerSample::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Landscape Layer Sample"));
	OutCaptions.Add(FString::Printf(TEXT("'%s'"), *ParameterName.ToString()));
}

bool UMaterialExpressionLandscapeLayerSample::MatchesSearchQuery(const TCHAR* SearchQuery)
{
	TArray<FString> Captions;
	GetCaption(Captions);
	for (const FString& Caption : Captions)
	{
		if (Caption.Contains(SearchQuery))
		{
			return true;
		}
	}

	return Super::MatchesSearchQuery(SearchQuery);
}

void UMaterialExpressionLandscapeLayerSample::GetLandscapeLayerNames(TArray<FName>& OutLayers) const
{
	OutLayers.AddUnique(ParameterName);
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE

