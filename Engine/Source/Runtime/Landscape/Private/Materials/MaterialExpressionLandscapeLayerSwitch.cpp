// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/MaterialExpressionLandscapeLayerSwitch.h"
#include "Engine/Engine.h"
#include "Engine/Texture.h"
#include "EngineGlobals.h"
#include "MaterialCompiler.h"
#include "Materials/Material.h"
#include "LandscapeUtilsPrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionLandscapeLayerSwitch)

#define LOCTEXT_NAMESPACE "Landscape"


///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionLandscapeLayerSwitch
///////////////////////////////////////////////////////////////////////////////

UMaterialExpressionLandscapeLayerSwitch::UMaterialExpressionLandscapeLayerSwitch(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_Landscape;
		FConstructorStatics()
			: NAME_Landscape(LOCTEXT("Landscape", "Landscape"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Landscape);

	bCollapsed = false;
#endif

	PreviewUsed = true;
}

#if WITH_EDITOR
bool UMaterialExpressionLandscapeLayerSwitch::IsResultMaterialAttributes(int32 OutputIndex)
{
	bool bLayerUsedIsMaterialAttributes = LayerUsed.Expression != nullptr && LayerUsed.Expression->IsResultMaterialAttributes(LayerUsed.OutputIndex);
	bool bLayerNotUsedIsMaterialAttributes = LayerNotUsed.Expression != nullptr && LayerNotUsed.Expression->IsResultMaterialAttributes(LayerNotUsed.OutputIndex);
	return bLayerUsedIsMaterialAttributes || bLayerNotUsedIsMaterialAttributes;
}

int32 UMaterialExpressionLandscapeLayerSwitch::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	const bool bTextureArrayEnabled = UE::Landscape::Private::UseWeightmapTextureArray(Compiler->GetShaderPlatform());
	const int32 WeightCode = Compiler->StaticTerrainLayerWeight(
		ParameterName,
		PreviewUsed ? Compiler->Constant(1.0f) : INDEX_NONE,
		bTextureArrayEnabled
		);

	int32 ReturnCode = INDEX_NONE;
	if (WeightCode != INDEX_NONE)
	{
		ReturnCode = LayerUsed.Compile(Compiler);
	}
	else
	{
		ReturnCode = LayerNotUsed.Compile(Compiler);
	}

	if (ReturnCode != INDEX_NONE && //If we've already failed for some other reason don't bother with this check. It could have been the reentrant check causing this to loop infinitely!
		LayerUsed.Expression != nullptr && LayerNotUsed.Expression != nullptr &&
		LayerUsed.Expression->IsResultMaterialAttributes(LayerUsed.OutputIndex) != LayerNotUsed.Expression->IsResultMaterialAttributes(LayerNotUsed.OutputIndex))
	{
		Compiler->Error(TEXT("Cannot mix MaterialAttributes and non MaterialAttributes nodes"));
	}

	return ReturnCode;
}

#endif // WITH_EDITOR

UObject* UMaterialExpressionLandscapeLayerSwitch::GetReferencedTexture() const
{
	return GEngine->WeightMapPlaceholderTexture;
}

UMaterialExpression::ReferencedTextureArray UMaterialExpressionLandscapeLayerSwitch::GetReferencedTextures() const
{
	return { GEngine->WeightMapPlaceholderTexture, GEngine->WeightMapArrayPlaceholderTexture };
}

#if WITH_EDITOR
FString UMaterialExpressionLandscapeLayerSwitch::GetEditableName() const
{
	return ParameterName.ToString();
}

void UMaterialExpressionLandscapeLayerSwitch::SetEditableName(const FString& NewName)
{
	ParameterName = *NewName;
}

void UMaterialExpressionLandscapeLayerSwitch::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Landscape Layer Switch"));
	OutCaptions.Add(FString::Printf(TEXT("'%s'"), *ParameterName.ToString()));
}

bool UMaterialExpressionLandscapeLayerSwitch::MatchesSearchQuery(const TCHAR* SearchQuery)
{
	TArray<FString> Captions;
	GetCaption(Captions);
	for (const FString& Caption : Captions)
	{
		if (Caption.Contains(SearchQuery))
		{
			return true;
		}
	}

	return Super::MatchesSearchQuery(SearchQuery);
}

#endif // WITH_EDITOR

void UMaterialExpressionLandscapeLayerSwitch::Serialize(FStructuredArchive::FRecord Record)
{
	Super::Serialize(Record);

	if (Record.GetUnderlyingArchive().UEVer() < VER_UE4_FIX_TERRAIN_LAYER_SWITCH_ORDER)
	{
		Swap(LayerUsed, LayerNotUsed);
	}
}


void UMaterialExpressionLandscapeLayerSwitch::PostLoad()
{
	Super::PostLoad();

	if (GetLinkerUEVersion() < VER_UE4_FIXUP_TERRAIN_LAYER_NODES)
	{
		UpdateParameterGuid(true, true);
	}
}

#if WITH_EDITOR
void UMaterialExpressionLandscapeLayerSwitch::GetLandscapeLayerNames(TArray<FName>& OutLayers) const
{
	OutLayers.AddUnique(ParameterName);
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE

