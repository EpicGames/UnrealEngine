// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/MaterialExpressionLandscapeLayerWeight.h"
#include "Engine/Engine.h"
#include "Engine/Texture.h"
#include "EngineGlobals.h"
#include "MaterialCompiler.h"
#include "Materials/Material.h"
#include "LandscapeUtilsPrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionLandscapeLayerWeight)

#define LOCTEXT_NAMESPACE "Landscape"


///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionLandscapeLayerWeight
///////////////////////////////////////////////////////////////////////////////

UMaterialExpressionLandscapeLayerWeight::UMaterialExpressionLandscapeLayerWeight(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_Landscape;
		FConstructorStatics()
			: NAME_Landscape(LOCTEXT("Landscape", "Landscape"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Landscape);
#endif

	PreviewWeight = 0.0f;
	ConstBase = FVector(0.f, 0.f, 0.f);
}

void UMaterialExpressionLandscapeLayerWeight::PostLoad()
{
	Super::PostLoad();

	if (GetLinkerUEVersion() < VER_UE4_FIXUP_TERRAIN_LAYER_NODES)
	{
		UpdateParameterGuid(true, true);
	}
}

#if WITH_EDITOR
bool UMaterialExpressionLandscapeLayerWeight::IsResultMaterialAttributes(int32 OutputIndex)
{
	bool bLayerIsMaterialAttributes = Layer.Expression != nullptr && Layer.Expression->IsResultMaterialAttributes(Layer.OutputIndex);
	bool bBaseIsMaterialAttributes = Base.Expression != nullptr && Base.Expression->IsResultMaterialAttributes(Base.OutputIndex);
	return bLayerIsMaterialAttributes || bBaseIsMaterialAttributes;
}

int32 UMaterialExpressionLandscapeLayerWeight::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	const bool bTextureArrayEnabled = UE::Landscape::Private::UseWeightmapTextureArray(Compiler->GetShaderPlatform());
	const int32 BaseCode = Base.Expression
		                       ? Base.Compile(Compiler)
		                       : Compiler->Constant3(static_cast<float>(ConstBase.X), static_cast<float>(ConstBase.Y), static_cast<float>(ConstBase.Z));
	const int32 WeightCode = Compiler->StaticTerrainLayerWeight(ParameterName, Compiler->Constant(PreviewWeight), bTextureArrayEnabled);

	int32 ReturnCode = INDEX_NONE;
	if (WeightCode == INDEX_NONE)
	{
		ReturnCode = BaseCode;
	}
	else
	{
		const int32 LayerCode = Layer.Compile(Compiler);
		ReturnCode = Compiler->Add(BaseCode, Compiler->Mul(LayerCode, WeightCode));
	}

	if (ReturnCode != INDEX_NONE && //If we've already failed for some other reason don't bother with this check. It could have been the reentrant check causing this to loop infinitely!
		Layer.Expression != nullptr && Base.Expression != nullptr &&
		Layer.Expression->IsResultMaterialAttributes(Layer.OutputIndex) != Base.Expression->IsResultMaterialAttributes(Base.OutputIndex))
	{
		Compiler->Error(TEXT("Cannot mix MaterialAttributes and non MaterialAttributes nodes"));
	}

	return ReturnCode;
}
#endif // WITH_EDITOR

UObject* UMaterialExpressionLandscapeLayerWeight::GetReferencedTexture() const
{
	return GEngine->WeightMapPlaceholderTexture;
}

UMaterialExpression::ReferencedTextureArray UMaterialExpressionLandscapeLayerWeight::GetReferencedTextures() const
{
	return { GEngine->WeightMapPlaceholderTexture, GEngine->WeightMapArrayPlaceholderTexture };
}

#if WITH_EDITOR
FString UMaterialExpressionLandscapeLayerWeight::GetEditableName() const
{
	return ParameterName.ToString();
}

void UMaterialExpressionLandscapeLayerWeight::SetEditableName(const FString& NewName)
{
	ParameterName = *NewName;
}

void UMaterialExpressionLandscapeLayerWeight::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("Landscape Layer Weight"));
	OutCaptions.Add(FString::Printf(TEXT("'%s'"), *ParameterName.ToString()));
}

bool UMaterialExpressionLandscapeLayerWeight::MatchesSearchQuery(const TCHAR* SearchQuery)
{
	TArray<FString> Captions;
	GetCaption(Captions);
	for (const FString& Caption : Captions)
	{
		if (Caption.Contains(SearchQuery))
		{
			return true;
		}
	}

	return Super::MatchesSearchQuery(SearchQuery);
}

void UMaterialExpressionLandscapeLayerWeight::GetLandscapeLayerNames(TArray<FName>& OutLayers) const
{
	OutLayers.AddUnique(ParameterName);
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE

