// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/MaterialExpressionLandscapePhysicalMaterialOutput.h"

#include "EdGraph/EdGraphNode.h"
#include "Engine/Engine.h"
#include "EngineGlobals.h"
#include "MaterialCompiler.h"
#include "PhysicalMaterials/PhysicalMaterial.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionLandscapePhysicalMaterialOutput)

#define LOCTEXT_NAMESPACE "Landscape"

UMaterialExpressionLandscapePhysicalMaterialOutput::UMaterialExpressionLandscapePhysicalMaterialOutput(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_Landscape;
		FConstructorStatics()
			: NAME_Landscape(LOCTEXT("Landscape", "Landscape"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Landscape);

	// No outputs
	Outputs.Reset();
#endif
}

#if WITH_EDITOR

void UMaterialExpressionLandscapePhysicalMaterialOutput::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("Landscape Physical Material Output"));
}

TArrayView<FExpressionInput*> UMaterialExpressionLandscapePhysicalMaterialOutput::GetInputsView()
{
	CachedInputs.Empty();
	CachedInputs.Reserve(Inputs.Num());
	for (auto& Input : Inputs)
	{
		CachedInputs.Add(&Input.Input);
	}
	return CachedInputs;
}

FExpressionInput* UMaterialExpressionLandscapePhysicalMaterialOutput::GetInput(int32 InputIndex)
{
	return Inputs.IsValidIndex(InputIndex) ? &Inputs[InputIndex].Input : nullptr;
}


FName UMaterialExpressionLandscapePhysicalMaterialOutput::GetInputName(int32 InputIndex) const
{
	UPhysicalMaterial* PhysicalMaterial = Inputs[InputIndex].PhysicalMaterial;
	if (PhysicalMaterial != nullptr)
	{
		return PhysicalMaterial->GetFName();
	}

	return NAME_None;
}

int32 UMaterialExpressionLandscapePhysicalMaterialOutput::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	if (Inputs.IsValidIndex(OutputIndex))
	{
		if (Inputs[OutputIndex].PhysicalMaterial == nullptr)
		{
			return CompilerError(Compiler, TEXT("Physical material not set."));
		}
		
		if (Inputs[OutputIndex].Input.Expression == nullptr)
		{
			return CompilerError(Compiler, TEXT("Input missing."));
		}
		
		return Compiler->CustomOutput(this, OutputIndex, Inputs[OutputIndex].Input.Compile(Compiler));
	}

	return INDEX_NONE;
}

void UMaterialExpressionLandscapePhysicalMaterialOutput::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.MemberProperty)
	{
		const FName PropertyName = PropertyChangedEvent.MemberProperty->GetFName();
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UMaterialExpressionLandscapePhysicalMaterialOutput, Inputs))
		{
			if (GraphNode)
			{
				GraphNode->ReconstructNode();
			}
		}
	}
}

#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE

