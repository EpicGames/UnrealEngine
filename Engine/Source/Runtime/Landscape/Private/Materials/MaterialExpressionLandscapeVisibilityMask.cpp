// Copyright Epic Games, Inc. All Rights Reserved.

#include "Materials/MaterialExpressionLandscapeVisibilityMask.h"
#include "Engine/Engine.h"
#include "Engine/Texture.h"
#include "EngineGlobals.h"
#include "MaterialCompiler.h"
#include "Materials/Material.h"
#include "LandscapeUtilsPrivate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MaterialExpressionLandscapeVisibilityMask)

#define LOCTEXT_NAMESPACE "Landscape"


///////////////////////////////////////////////////////////////////////////////
// UMaterialExpressionLandscapeVisibilityMask
///////////////////////////////////////////////////////////////////////////////

FName UMaterialExpressionLandscapeVisibilityMask::ParameterName = FName("__LANDSCAPE_VISIBILITY__");

UMaterialExpressionLandscapeVisibilityMask::UMaterialExpressionLandscapeVisibilityMask(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		FText NAME_Landscape;
		FConstructorStatics()
			: NAME_Landscape(LOCTEXT("Landscape", "Landscape"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

#if WITH_EDITORONLY_DATA
	MenuCategories.Add(ConstructorStatics.NAME_Landscape);
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionLandscapeVisibilityMask::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	const bool bTextureArrayEnabled = UE::Landscape::Private::UseWeightmapTextureArray(Compiler->GetShaderPlatform());
	int32 MaskLayerCode = Compiler->StaticTerrainLayerWeight(ParameterName, Compiler->Constant(0.f), bTextureArrayEnabled);
	return MaskLayerCode == INDEX_NONE ? Compiler->Constant(1.f) : Compiler->Sub(Compiler->Constant(1.f), MaskLayerCode);
}

#endif // WITH_EDITOR

UObject* UMaterialExpressionLandscapeVisibilityMask::GetReferencedTexture() const
{
	return GEngine->WeightMapPlaceholderTexture;
}

UMaterialExpression::ReferencedTextureArray UMaterialExpressionLandscapeVisibilityMask::GetReferencedTextures() const
{
	return { GEngine->WeightMapPlaceholderTexture, GEngine->WeightMapArrayPlaceholderTexture };
}

#if WITH_EDITOR
void UMaterialExpressionLandscapeVisibilityMask::GetLandscapeLayerNames(TArray<FName>& OutLayers) const
{
	OutLayers.AddUnique(ParameterName);
}

void UMaterialExpressionLandscapeVisibilityMask::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(FString("Landscape Visibility Mask"));
}
#endif // WITH_EDITOR


#undef LOCTEXT_NAMESPACE

