// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h" // FTransform, FIntPoint
#include "Misc/CoreMiscDefines.h" // PURE_VIRTUAL
#include "UObject/Interface.h"

#include "GlobalMergeLegacySupportUtil.generated.h"

// Note: this file is likely to be removed once global merge is fully phased out.

class UTextureRenderTarget2D;
struct FLandscapeBrushParameters;

class UE_DEPRECATED(5.7, "Global merge has been deprecated so this is no longer used. There's no good way of deprecating UInterfaces ATM, hence this forward-declaration") ULandscapeBrushRenderCallAdapter_GlobalMergeLegacySupport {};
class UE_DEPRECATED(5.7, "Global merge has been deprecated so this is no longer used. There's no good way of deprecating UInterfaces ATM, hence this forward-declaration") ILandscapeBrushRenderCallAdapter_GlobalMergeLegacySupport {};

// UObject for the UInterface
UINTERFACE(MinimalAPI)
class ULandscapeBrushRenderCallAdapter_GlobalMergeLegacySupport_DEPRECATED : public UInterface
{
	GENERATED_BODY()
};

/**
 * Interface that allows an edit layer UObject to receive a render call the way that a blueprint
 * brush would in global merge mode, so that an edit layer UObject that implements custom batched
 * merge functions can still perform its work if global merge mode is used instead.
 * 
 * This interface is likely to be removed once global merge is no longer used.
 */
 
class ILandscapeBrushRenderCallAdapter_GlobalMergeLegacySupport_DEPRECATED
{
	GENERATED_BODY()

#if WITH_EDITOR
public:
	/**
	 * In global merge mode, this function is called in the same places that FLandscapeLayerBrush::RenderLayer(BrushParameters)
	 * is called. This function calls the virtual InitializeAsBlueprintBrush and RenderLayerAsBlueprintBrush
	 * functions, which are equivalent to a blueprint brush actor's Initialize_Native and RenderLayer_Native.
	 */
	UTextureRenderTarget2D* RenderAsBlueprintBrush(const FLandscapeBrushParameters& InParameters, const FTransform& LandscapeTransform) { return nullptr; }

	// Called in the same places as the equivalent methods on FLandscapeLayerBrush.
	virtual bool AffectsHeightmapAsBlueprintBrush() const { return false; }
	virtual bool AffectsWeightmapLayerAsBlueprintBrush(const FName& InLayerName) const { return false; }
	virtual bool AffectsVisibilityLayerAsBlueprintBrush() const { return false; }

protected:
	
	// Overridable, equivalent to Initialize_Native on a blueprint brush actor
	virtual void InitializeAsBlueprintBrush(const FTransform& InLandscapeTransform,
		const FIntPoint& InLandscapeSize,
		const FIntPoint& InLandscapeRenderTargetSize) {}

	// Overridable, equivalent to RenderLayer_Native on a blueprint brush actor.
	virtual UTextureRenderTarget2D* RenderLayerAsBlueprintBrush(const FLandscapeBrushParameters& InParameters)
		PURE_VIRTUAL(ILandscapeBrushRenderCallAdapter_GlobalMergeLegacySupport_DEPRECATED::InitializeAsBlueprintBrush, return nullptr; );

	// Used in RenderAsBlueprintBrush to cache things the same way that blueprint brushes do
	FTransform CurrentRenderAreaWorldTransform;
	FIntPoint CurrentRenderAreaSize = FIntPoint(ForceInitToZero);
	FIntPoint CurrentRenderTargetSize = FIntPoint(ForceInitToZero);
#endif
};
