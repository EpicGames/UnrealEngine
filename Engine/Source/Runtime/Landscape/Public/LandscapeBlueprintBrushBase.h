// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "LandscapeEditTypes.h"
#include "LandscapeEditLayerRenderer.h"

#include "LandscapeBlueprintBrushBase.generated.h"

class UTextureRenderTarget2D;

USTRUCT(BlueprintType)
struct FLandscapeBrushParameters
{
	GENERATED_BODY()

	FLandscapeBrushParameters() = default;
	FLandscapeBrushParameters(bool bInIsHeightmapMerge, const FTransform& InRenderAreaWorldTransform, const FIntPoint& InRenderAreaSize, UTextureRenderTarget2D* InCombinedResult, const FName& InWeightmapLayerName = FName());

	UPROPERTY(Category = "Settings", EditAnywhere, BlueprintReadWrite)
	FTransform RenderAreaWorldTransform;

	UPROPERTY(Category = "Settings", EditAnywhere, BlueprintReadWrite)
	FIntPoint RenderAreaSize = FIntPoint(MAX_int32, MAX_int32);

	UPROPERTY(Category = "Settings", EditAnywhere, BlueprintReadWrite)
	TObjectPtr<UTextureRenderTarget2D> CombinedResult;

	UPROPERTY(Category = "Settings", EditAnywhere, BlueprintReadWrite)
	ELandscapeToolTargetType LayerType = ELandscapeToolTargetType::Invalid;

	UPROPERTY(Category = "Settings", EditAnywhere, BlueprintReadWrite)
	FName WeightmapLayerName;
};


UCLASS(Abstract, NotPlaceable, NotBlueprintable, MinimalAPI)
class ALandscapeBlueprintBrushBase : public AActor
#if CPP && WITH_EDITOR // UHT doesn't support inheriting from namespaced class
	, public ILandscapeEditLayerRenderer
	, public UE::Landscape::EditLayers::IEditLayerRendererProvider
#endif // CPP && WITH_EDITOR
{
	GENERATED_UCLASS_BODY()

protected:
	UPROPERTY(Category = "Settings", EditAnywhere, BlueprintReadWrite)
	bool UpdateOnPropertyChange;

	UPROPERTY(Category = "Settings", EditAnywhere, BlueprintReadWrite, Setter = "SetCanAffectHeightmap")
	bool AffectHeightmap;

	UPROPERTY(Category = "Settings", EditAnywhere, BlueprintReadWrite, Setter = "SetCanAffectWeightmap")
	bool AffectWeightmap;

	UPROPERTY(Category = "Settings", EditAnywhere, BlueprintReadWrite, Setter="SetCanAffectVisibilityLayer")
	bool AffectVisibilityLayer;

	UPROPERTY(Category = "Settings", EditAnywhere, BlueprintReadWrite)
	TArray<FName> AffectedWeightmapLayers;

	/** 
	 * It is possible to define whether this BP brush works exclusively with power-of-2 render targets. This used to be mandatory so for full backwards-compatibility, 
	 * this is still the case by default but it can be turned off. 
	 */
	UPROPERTY(Category = "Settings", EditAnywhere, BlueprintReadWrite, Setter="SetCanAffectVisibilityLayer")
	bool bUsePowerOfTwoRenderTarget = true;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	TObjectPtr<class ALandscape> OwningLandscape;

	UPROPERTY(Transient)
	bool bIsVisible;

	uint32 LastRequestLayersContentUpdateFrameNumber;

	FTransform CurrentRenderAreaWorldTransform;
	FIntPoint CurrentRenderAreaSize = FIntPoint(ForceInitToZero);
	FIntPoint CurrentRenderTargetSize = FIntPoint(ForceInitToZero);
#endif // WITH_EDITORONLY_DATA

public:
	UFUNCTION(BlueprintNativeEvent, meta = (DeprecatedFunction, DeprecationMessage = "Please use RenderLayer instead."))
	LANDSCAPE_API UTextureRenderTarget2D* Render(bool InIsHeightmap, UTextureRenderTarget2D* InCombinedResult, const FName& InWeightmapLayerName);

	UFUNCTION(BlueprintNativeEvent)
	LANDSCAPE_API UTextureRenderTarget2D* RenderLayer(const FLandscapeBrushParameters& InParameters);
	LANDSCAPE_API virtual UTextureRenderTarget2D* RenderLayer_Native(const FLandscapeBrushParameters& InParameters);

	UFUNCTION(BlueprintNativeEvent)
	LANDSCAPE_API void Initialize(const FTransform& InLandscapeTransform, const FIntPoint& InLandscapeSize, const FIntPoint& InLandscapeRenderTargetSize);
	virtual void Initialize_Native(const FTransform& InLandscapeTransform, const FIntPoint& InLandscapeSize, const FIntPoint& InLandscapeRenderTargetSize) {}

	UFUNCTION(BlueprintCallable, Category = "Landscape")
	LANDSCAPE_API void RequestLandscapeUpdate(bool bInUserTriggered = false);

	UFUNCTION(BlueprintImplementableEvent, CallInEditor)
	LANDSCAPE_API void GetBlueprintRenderDependencies(TArray<UObject*>& OutStreamableAssets);

	LANDSCAPE_API void SetCanAffectHeightmap(bool bInCanAffectHeightmap);
	LANDSCAPE_API void SetCanAffectWeightmap(bool bInCanAffectWeightmap);
	LANDSCAPE_API void SetCanAffectVisibilityLayer(bool bInCanAffectVisibilityLayer);
	LANDSCAPE_API void SetUsePowerOfTwoRenderTarget(bool bInUsePowerOfTwoRenderTarget);

#if WITH_EDITOR
	//~ Begin ILandscapeEditLayerRenderer implementation
	LANDSCAPE_API virtual void GetRendererStateInfo(const UE::Landscape::EditLayers::FMergeContext* InMergeContext,
		UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutSupportedTargetTypeState, UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutEnabledTargetTypeState, 
		TArray<UE::Landscape::EditLayers::FTargetLayerGroup>& OutTargetLayerGroups) const override;
	virtual UE::Landscape::EditLayers::ERenderFlags GetRenderFlags(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const override { return UE::Landscape::EditLayers::ERenderFlags::RenderMode_Immediate; }
	LANDSCAPE_API virtual TArray<UE::Landscape::EditLayers::FEditLayerRenderItem> GetRenderItems(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const override;
	LANDSCAPE_API virtual bool RenderLayer(UE::Landscape::EditLayers::FRenderParams& RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder) override;
	LANDSCAPE_API virtual FString GetEditLayerRendererDebugName() const override;
	//~ End ILandscapeEditLayerRenderer implementation

	//~ Begin UE::Landscape::EditLayers::IEditLayerRendererProvider implementation
	LANDSCAPE_API TArray<UE::Landscape::EditLayers::FEditLayerRendererState> GetEditLayerRendererStates(const UE::Landscape::EditLayers::FMergeContext* InMergeContext);
	//~ End UE::Landscape::EditLayers::IEditLayerRendererProvider implementation

	UTextureRenderTarget2D* Execute(const FLandscapeBrushParameters& InParameters);

	LANDSCAPE_API virtual void CheckForErrors() override;

	LANDSCAPE_API virtual void GetRenderDependencies(TSet<UObject*>& OutDependencies);

	LANDSCAPE_API virtual void SetOwningLandscape(class ALandscape* InOwningLandscape);
	LANDSCAPE_API class ALandscape* GetOwningLandscape() const;

	/** CanAffect... methods indicate the brush has the _capacity_ to affect this or that aspect of the landscape. 
	*  Note: it doesn't mean the brush currently affects it : the Affects... methods are used for that. 
	*/
	bool CanAffectHeightmap() const { return AffectHeightmap; }
	bool CanAffectWeightmap() const { return AffectWeightmap; }
	bool CanAffectVisibilityLayer() const { return AffectVisibilityLayer; }
	LANDSCAPE_API virtual bool CanAffectWeightmapLayer(const FName& InLayerName) const;
	LANDSCAPE_API virtual bool AffectsHeightmap() const;
	LANDSCAPE_API virtual bool AffectsWeightmap() const;
	LANDSCAPE_API virtual bool AffectsWeightmapLayer(const FName& InLayerName) const;
	LANDSCAPE_API virtual bool AffectsVisibilityLayer() const;
	virtual bool SupportsBlueprintBrushTool() const { return true; }

	bool IsVisible() const { return bIsVisible; }
	LANDSCAPE_API bool IsLayerUpdatePending() const;

	LANDSCAPE_API void SetIsVisible(bool bInIsVisible);

	LANDSCAPE_API virtual bool ShouldTickIfViewportsOnly() const override;
	LANDSCAPE_API virtual void Tick(float DeltaSeconds) override;
	LANDSCAPE_API virtual void PostEditMove(bool bFinished) override;
	LANDSCAPE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	LANDSCAPE_API virtual void PostEditUndo() override;
	LANDSCAPE_API virtual void Destroyed() override;

	LANDSCAPE_API virtual void PushDeferredLayersContentUpdate();

	virtual bool CanChangeIsSpatiallyLoadedFlag() const override { return false; }
#endif // WITH_EDITOR
};
