// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ActorPartition/ActorPartitionSubsystem.h"
#include "LandscapeConfigHelper.generated.h"

UENUM()
enum class ELandscapeResizeMode : uint8
{
	Resample = 0,
	Clip = 1,
	Expand = 2
};

#if WITH_EDITOR

class ULandscapeInfo;
class ULandscapeComponent;
class ALandscapeProxy;
struct FLandscapeImportLayerInfo;

/**
 * 
 */
struct FLandscapeConfig
{
	FLandscapeConfig(int32 InComponentNumSubSections, int32 InSubsectionSizeQuads, int32 InGridSizeInComponents)
		: ComponentNumSubsections(InComponentNumSubSections)
		, SubsectionSizeQuads(InSubsectionSizeQuads)
		, GridSizeInComponents(InGridSizeInComponents)
	{
	}

	LANDSCAPE_API FLandscapeConfig(ULandscapeInfo* InLandscapeInfo);

	int32 GetComponentSizeQuads() const { return SubsectionSizeQuads * ComponentNumSubsections; }
	int32 GetComponentSizeVerts() const { return (SubsectionSizeQuads + 1) * ComponentNumSubsections; }
	int32 GetGridSizeQuads() const { return GetComponentSizeQuads() * GridSizeInComponents; }

	int32 ComponentNumSubsections;
	int32 SubsectionSizeQuads;
	int32 GridSizeInComponents;

	static LANDSCAPE_API int32 NumSectionValues[2];
	static LANDSCAPE_API int32 SubsectionSizeQuadsValues[6];
};

struct FLandscapeConfigChange : public FLandscapeConfig
{
	FLandscapeConfigChange(int32 InComponentNumSubSections, int32 InSubsectionSizeQuads, int32 InGridSize, ELandscapeResizeMode InResizeMode, bool bInZeroBased)
		: FLandscapeConfig(InComponentNumSubSections, InSubsectionSizeQuads, InGridSize)
		, ResizeMode(InResizeMode)
		, bZeroBased(bInZeroBased)
	{
	}

	LANDSCAPE_API bool Validate() const;

	ELandscapeResizeMode ResizeMode;
	bool bZeroBased;
};

class FLandscapeConfigHelper
{
public:
	LANDSCAPE_API static ALandscapeProxy* FindOrAddLandscapeStreamingProxy(ULandscapeInfo* InLandscapeInfo, const FIntPoint& InSectionBase);
	LANDSCAPE_API static ULandscapeInfo* ChangeConfiguration(ULandscapeInfo* InLandscapeInfo, const FLandscapeConfigChange& InNewConfig, TSet<AActor*>& OutActorsToDelete, TSet<AActor*>& OutModifiedActors);
	LANDSCAPE_API static bool ChangeGridSize(ULandscapeInfo* InLandscapeInfo, uint32 InNewGridSizeInComponents, TSet<AActor*>& OutActorsToDelete);
	LANDSCAPE_API static bool PartitionLandscape(UWorld* InWorld, ULandscapeInfo* InLandscapeInfo, uint32 InGridSizeInComponents);

private:
	static ALandscapeProxy* FindOrAddLandscapeStreamingProxy(UActorPartitionSubsystem* ActorPartitionSubsystem, ULandscapeInfo* LandscapeInfo, const UActorPartitionSubsystem::FCellCoord& CellCoord);
	static void CopyRegionToComponent(ULandscapeInfo* LandscapeInfo, const FIntRect& Region, bool bResample, ULandscapeComponent* Component);
	static void ExtractLandscapeData(ULandscapeInfo* LandscapeInfo, const FIntRect& Region, const FGuid& LayerGuid, TArray<uint16>& OutHeightData, TArray<FLandscapeImportLayerInfo>& OutImportMaterialLayerInfos);
	static void MoveSplinesToLandscape(ULandscapeInfo* InLandscapeInfo, ALandscapeProxy* InLandscape, float InScaleFactor);
	static void MoveFoliageToLandscape(ULandscapeInfo* InLandscapeInfo, ULandscapeInfo* InNewLandscapeInfo);

public:
	template<typename T>
	static void CopyData(const TArray<T>& InData, TArray<T>& OutData, const FIntRect& InSrcRegion, const FIntRect& InDestRegion, bool bInResample)
	{
		if (bInResample)
		{
			ResampleData(InData, OutData, InSrcRegion, InDestRegion);
		}
		else
		{
			ExpandData(InData, OutData, InSrcRegion, InDestRegion, true);
		}
	}

	template<typename T>
	static void ExpandData(const TArray<T>& InData, TArray<T>& OutData, const FIntRect& InSrcRegion, const FIntRect& InDestRegion, bool bOffset)
	{
		// Regions are in ComponentQuads and we want Vertices (+1)
		const int32 SrcWidth = InSrcRegion.Width() + 1;
		const int32 SrcHeight = InSrcRegion.Height() + 1;
		const int32 DstWidth = InDestRegion.Width() + 1;
		const int32 DstHeight = InDestRegion.Height() + 1;
		const int32 OffsetX = bOffset ? InDestRegion.Min.X - InSrcRegion.Min.X : 0;
		const int32 OffsetY = bOffset ? InDestRegion.Min.Y - InSrcRegion.Min.Y : 0;
		
		OutData.Empty(DstWidth * DstHeight);
		OutData.AddUninitialized(DstWidth * DstHeight);

		for (int32 DstY = 0; DstY < DstHeight; ++DstY)
		{
			const int32 SrcY = FMath::Clamp<int32>(DstY + OffsetY, 0, SrcHeight - 1);

			// Pad anything to the left
			const T PadLeft = InData[SrcY * SrcWidth];
			int32 EndPadLeft = FMath::Min<int32>(-OffsetX, DstWidth);
			for (int32 DstX = 0; DstX < EndPadLeft; ++DstX)
			{
				OutData[DstY * DstWidth + DstX] = PadLeft;
			}

			{
				const int32 DstX = FMath::Max(0, -OffsetX);
				const int32 SrcX = FMath::Clamp<int32>(DstX + OffsetX, 0, SrcWidth - 1);
				// Limit to DstWidth-DstX to avoid writing past the end of the destination scanline.  This will also make CopySize negative if the target
				// starts past the end of the scanline.
				const int32 CopySize = FMath::Min<int32>(SrcWidth - SrcX, DstWidth - DstX) * sizeof(T);
				if (CopySize > 0)
				{
					FMemory::Memcpy(&OutData[DstY * DstWidth + DstX], &InData[SrcY * SrcWidth + SrcX], CopySize);
				}
			}

			const T PadRight = InData[SrcY * SrcWidth + SrcWidth - 1];
			int32 StartPadRight = FMath::Max<int32>(-OffsetX + SrcWidth, 0);
			for (int32 DstX = StartPadRight; DstX < DstWidth; ++DstX)
			{
				OutData[DstY * DstWidth + DstX] = PadRight;
			}
		}
	}

	template<typename T>
	static void CopySubregion(const TArrayView<T>& InData, TArray<T>& OutData, const FIntRect& InSrcRegion, uint32 InSrcDataPitch)
	{
		check(InData.Size() >= InSrcRegion.Area());

		const int32 SrcWidth = InSrcRegion.Width();
		const int32 SrcHeight = InSrcRegion.Height();
		
		OutData.Empty(SrcWidth * SrcHeight);
		OutData.AddUninitialized(SrcWidth * SrcHeight);

		for (int32 Y = 0; Y < SrcHeight; ++Y)
		{
			for (int32 X = 0; X < SrcWidth; ++X)
			{
				const int32 SrcX = X + InSrcRegion.Min.X;
				const int32 SrcY = Y + InSrcRegion.Min.Y;

				const int32 SrcIndex = SrcX + SrcY * InSrcDataPitch;
				const int32 DstIndex = X + Y * InSrcRegion.Width();

				OutData[DstIndex] = InData[SrcIndex];
			}
		}
	}
		
	template<typename T>
	static void ResampleData(const TArray<T>& InData, TArray<T>& OutData, const FIntRect& InSrcRegion, const FIntRect& InDestRegion)
	{
		// Regions are in ComponentQuads and we want Vertices (+1)
		const int32 SrcWidth = InSrcRegion.Width() + 1;
		const int32 SrcHeight = InSrcRegion.Height() + 1;
		const int32 DestWidth = InDestRegion.Width() + 1;
		const int32 DestHeight = InDestRegion.Height() + 1;

		OutData.Empty(DestWidth * DestHeight);
		OutData.AddUninitialized(DestWidth * DestHeight);

		const float XScale = (float)(SrcWidth - 1) / (DestWidth - 1);
		const float YScale = (float)(SrcHeight - 1) / (DestHeight - 1);
		for (int32 Y = 0; Y < DestHeight; ++Y)
		{
			for (int32 X = 0; X < DestWidth; ++X)
			{
				const float OldY = Y * YScale;
				const float OldX = X * XScale;
				const int32 X0 = FMath::FloorToInt(OldX);
				const int32 X1 = FMath::Min(FMath::FloorToInt(OldX) + 1, SrcWidth - 1);
				const int32 Y0 = FMath::FloorToInt(OldY);
				const int32 Y1 = FMath::Min(FMath::FloorToInt(OldY) + 1, SrcHeight - 1);
				const T& Original00 = InData[Y0 * SrcWidth + X0];
				const T& Original10 = InData[Y0 * SrcWidth + X1];
				const T& Original01 = InData[Y1 * SrcWidth + X0];
				const T& Original11 = InData[Y1 * SrcWidth + X1];
				int32 Index = Y * DestWidth + X;
				check(Index < OutData.Num());
				OutData[Y * DestWidth + X] = FMath::BiLerp(Original00, Original10, Original01, Original11, FMath::Fractional(OldX), FMath::Fractional(OldY));
			}
		}
	}
};

#endif
