// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
LandscapeDataAccess.h: Classes for the editor to access to Landscape data
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Engine/Texture2D.h"

#define LANDSCAPE_VALIDATE_DATA_ACCESS 1
#define LANDSCAPE_ZSCALE		(1.0f/128.0f)
#define LANDSCAPE_INV_ZSCALE	128.0f

#define LANDSCAPE_XYOFFSET_SCALE UE_DEPRECATED_MACRO(5.7, "RetopologizeTool/XYOffset deprecated with the removal of non-edit layer landscapes") (1.0f/256.f)
#define LANDSCAPE_INV_XYOFFSET_SCALE UE_DEPRECATED_MACRO(5.7, "RetopologizeTool/XYOffset deprecated with the removal of non-edit layer landscapes") 256.f

#define LANDSCAPE_VISIBILITY_THRESHOLD (2.0f/3.0f)

class ULandscapeComponent;
class ULandscapeLayerInfoObject;

namespace LandscapeDataAccess
{
	inline constexpr int32 MaxValue = 65535;
	inline constexpr float MidValue = 32768.f;
	inline constexpr float MidNormalValue = 127.5f;

	FORCEINLINE float GetLocalHeight(uint16 Height)
	{
		return (static_cast<float>(Height) - MidValue) * LANDSCAPE_ZSCALE;
	}

	FORCEINLINE uint16 GetTexHeight(float Height)
	{
		return static_cast<uint16>(FMath::RoundToInt(FMath::Clamp<float>(Height * LANDSCAPE_INV_ZSCALE + MidValue, 0.f, MaxValue)));		
	}

	FORCEINLINE FColor PackHeight(uint16 Height)
	{
		FColor Color(ForceInit);
		Color.R = Height >> 8;
		Color.G = Height & 255;
		return MoveTemp(Color);
	}

	FORCEINLINE float UnpackHeight(const FColor& InHeightmapSample)
	{
		uint16 Height = (uint16)(InHeightmapSample.R << 8) | (uint16)InHeightmapSample.G;
		return GetLocalHeight(Height);
	}

	FORCEINLINE FColor PackNormal(const FVector& InNormal)
	{
		const FVector Normal = InNormal.GetSafeNormal();

		FColor Color(ForceInit);
		Color.B = static_cast<uint8>(FMath::Clamp(Normal.X * 127.5 + 127.5, 0.0f, 255.0f));
		Color.A = static_cast<uint8>(FMath::Clamp(Normal.Y * 127.5 + 127.5, 0.0f, 255.0f));
		return MoveTemp(Color);
	}

	FORCEINLINE FVector UnpackNormal(const FColor& InHeightmapSample)
	{
		FVector Normal;
		Normal.X = 2.f * static_cast<float>(InHeightmapSample.B) / 255.f - 1.f;
		Normal.Y = 2.f * static_cast<float>(InHeightmapSample.A) / 255.f - 1.f;
		Normal.Z = FMath::Sqrt(FMath::Max(1.0f - (FMath::Square(Normal.X) + FMath::Square(Normal.Y)), 0.0f));
		return Normal;
	}

	FORCEINLINE FColor GetDefaultPackedHeightColor()
	{
		const uint16 DefaultHeight = static_cast<uint16>(MidValue);
		const uint8 DefaultNormal = static_cast<uint8>(MidNormalValue);
		return FColor(DefaultHeight >> 8, DefaultHeight & 255, DefaultNormal, DefaultNormal);
	}
};

#if WITH_EDITOR

class ULandscapeComponent;
class ULandscapeLayerInfoObject;

//
// FLandscapeDataInterface
//
struct FLandscapeDataInterface
{
private:

	struct FLockedMipDataInfo
	{
		FLockedMipDataInfo()
		:	LockCount(0)
		{}

		TArray64<uint8> MipData;
		int32 LockCount;
	};

public:
	// Constructor
	// @param bInAutoDestroy - shall we automatically clean up when the last 
	FLandscapeDataInterface()
	{}

	void* LockMip(UTexture2D* Texture, int32 MipLevel)
	{
		check(Texture->Source.IsValid());
		check(MipLevel < Texture->Source.GetNumMips());

		TArray<FLockedMipDataInfo>* MipInfo = LockedMipInfoMap.Find(Texture);
		if( MipInfo == nullptr )
		{
			MipInfo = &LockedMipInfoMap.Add(Texture, TArray<FLockedMipDataInfo>() );
			for (int32 i = 0; i < Texture->Source.GetNumMips(); ++i)
			{
				MipInfo->Add(FLockedMipDataInfo());
			}
		}

		if( (*MipInfo)[MipLevel].MipData.Num() == 0 )
		{
			verify( Texture->Source.GetMipData((*MipInfo)[MipLevel].MipData, MipLevel) );
		}
		(*MipInfo)[MipLevel].LockCount++;

		return (*MipInfo)[MipLevel].MipData.GetData();
	}

	void UnlockMip(UTexture2D* Texture, int32 MipLevel)
	{
		TArray<FLockedMipDataInfo>* MipInfo = LockedMipInfoMap.Find(Texture);
		check(MipInfo);

		if ((*MipInfo)[MipLevel].LockCount <= 0)
			return;
		(*MipInfo)[MipLevel].LockCount--;
		if( (*MipInfo)[MipLevel].LockCount == 0 )
		{
			check( (*MipInfo)[MipLevel].MipData.Num() != 0 );
			(*MipInfo)[MipLevel].MipData.Empty();
		}		
	}

private:
	TMap<UTexture2D*, TArray<FLockedMipDataInfo> > LockedMipInfoMap;
};
	

struct FLandscapeComponentDataInterfaceBase
{
	FLandscapeComponentDataInterfaceBase() {}
	LANDSCAPE_API FLandscapeComponentDataInterfaceBase(ULandscapeComponent* InComponent, int32 InMipLevel, bool InWorkOnEditingLayer = true);

	// Accessors
	void VertexIndexToXY(int32 VertexIndex, int32& OutX, int32& OutY) const
	{
		OutX = VertexIndex % ComponentSizeVerts;
		OutY = VertexIndex / ComponentSizeVerts;
	}

	// Accessors
	void QuadIndexToXY(int32 QuadIndex, int32& OutX, int32& OutY) const
	{
		OutX = QuadIndex % (ComponentSizeVerts-1);
		OutY = QuadIndex / (ComponentSizeVerts-1);
	}

	int32 VertexXYToIndex(int32 VertX, int32 VertY) const
	{
		return VertY * ComponentSizeVerts + VertX;
	}

	void ComponentXYToSubsectionXY(int32 CompX, int32 CompY, int32& SubNumX, int32& SubNumY, int32& SubX, int32& SubY ) const
	{
		// We do the calculation as if we're looking for the previous vertex.
		// This allows us to pick up the last shared vertex of every subsection correctly.
		SubNumX = (CompX-1) / (SubsectionSizeVerts - 1);
		SubNumY = (CompY-1) / (SubsectionSizeVerts - 1);
		SubX = (CompX-1) % (SubsectionSizeVerts - 1) + 1;
		SubY = (CompY-1) % (SubsectionSizeVerts - 1) + 1;

		// If we're asking for the first vertex, the calculation above will lead
		// to a negative SubNumX/Y, so we need to fix that case up.
		if( SubNumX < 0 )
		{
			SubNumX = 0;
			SubX = 0;
		}

		if( SubNumY < 0 )
		{
			SubNumY = 0;
			SubY = 0;
		}
	}

	void VertexXYToTexelXY(int32 VertX, int32 VertY, int32& OutX, int32& OutY) const
	{
		int32 SubNumX, SubNumY, SubX, SubY;
		ComponentXYToSubsectionXY(VertX, VertY, SubNumX, SubNumY, SubX, SubY);

		OutX = SubNumX * SubsectionSizeVerts + SubX;
		OutY = SubNumY * SubsectionSizeVerts + SubY;
	}
	
	int32 VertexIndexToTexel(int32 VertexIndex) const
	{
		int32 VertX, VertY;
		VertexIndexToXY(VertexIndex, VertX, VertY);
		int32 TexelX, TexelY;
		VertexXYToTexelXY(VertX, VertY, TexelX, TexelY);
		return TexelXYToIndex(TexelX, TexelY);
	}

	int32 TexelXYToIndex(int32 TexelX, int32 TexelY) const
	{
		return TexelY * ComponentNumSubsections * SubsectionSizeVerts + TexelX;
	}

	uint16 GetHeight(int32 LocalX, int32 LocalY, const TArray<FColor>& HeightAndNormals) const
	{
		const FColor* Texel = GetHeightData(LocalX, LocalY, HeightAndNormals);
		return (uint16)(Texel->R << 8) | (uint16)Texel->G;
	}

	float GetScaleFactor() const
	{
		return (float)ComponentSizeQuads / (float)(ComponentSizeVerts - 1);
	}

	FVector GetLocalVertex(int32 LocalX, int32 LocalY, const TArray<FColor>& HeightAndNormals) const
	{
		const float ScaleFactor = GetScaleFactor();
		
		return FVector(ScaleFactor * (float)LocalX, ScaleFactor * (float)LocalY, LandscapeDataAccess::GetLocalHeight(GetHeight(LocalX, LocalY, HeightAndNormals)));
	}

	float GetLocalHeight(int32 LocalX, int32 LocalY, const TArray<FColor>& HeightAndNormals) const
	{
		return LandscapeDataAccess::GetLocalHeight(GetHeight(LocalX, LocalY, HeightAndNormals));
	}

	const FColor* GetHeightData(int32 LocalX, int32 LocalY, const TArray<FColor>& HeightAndNormals) const
	{
#if LANDSCAPE_VALIDATE_DATA_ACCESS
		check(LocalX >= 0 && LocalY >= 0 && LocalX < ComponentSizeVerts&& LocalY < HeightmapStride);
#endif

		int32 TexelX, TexelY;
		VertexXYToTexelXY(LocalX, LocalY, TexelX, TexelY);

		return &HeightAndNormals[TexelX + HeightmapComponentOffsetX + (TexelY + HeightmapComponentOffsetY) * HeightmapStride];
	}

	void GetLocalTangentVectors(int32 LocalX, int32 LocalY, FVector& LocalTangentX, FVector& LocalTangentY, FVector& LocalTangentZ, const TArray<FColor>& HeightAndNormals) const
	{
		// Note: these are still pre-scaled, just not rotated
		const FColor* Data = GetHeightData(LocalX, LocalY, HeightAndNormals);
		LocalTangentZ = LandscapeDataAccess::UnpackNormal(*Data);
		LocalTangentX = FVector(-LocalTangentZ.Z, 0.f, LocalTangentZ.X);
		LocalTangentY = FVector(0.f, LocalTangentZ.Z, -LocalTangentZ.Y);
	}

	int32 GetComponentSizeVerts() const { return ComponentSizeVerts; }

public:
	// offset of this component's data into heightmap texture
	int32 HeightmapStride = 0;
	int32 HeightmapComponentOffsetX = 0;
	int32 HeightmapComponentOffsetY = 0;
	int32 HeightmapSubsectionOffset = 0;
	const int32 MipLevel = 0;

protected:
	int32 ComponentSizeQuads = 0;
	int32 ComponentSizeVerts = 0;
	int32 SubsectionSizeVerts = 0;
	int32 ComponentNumSubsections = 0;
};

//
// FLandscapeComponentDataInterface
//
struct FLandscapeComponentDataInterface : public FLandscapeComponentDataInterfaceBase
{
	friend struct FLandscapeDataInterface;

	// tors
	LANDSCAPE_API FLandscapeComponentDataInterface(ULandscapeComponent* InComponent, int32 InMipLevel = 0, bool InWorkOnEditingLayer = true);
	LANDSCAPE_API ~FLandscapeComponentDataInterface();

	FColor* GetRawHeightData() const
	{
		return HeightMipData;
	}

	void SetRawHeightData(FColor* NewHeightData)
	{
		HeightMipData = NewHeightData;
	}

	LANDSCAPE_API int32 GetHeightmapSizeX(int32 MipIndex) const;
	LANDSCAPE_API int32 GetHeightmapSizeY(int32 MipIndex) const;

	/* Return the raw heightmap data exactly same size for Heightmap texture which belong to only this component */
	LANDSCAPE_API void GetHeightmapTextureData(TArray<FColor>& OutData, bool bOkToFail = false);

	LANDSCAPE_API bool GetWeightmapTextureData(ULandscapeLayerInfoObject* InLayerInfo, TArray<uint8>& OutData, bool bInUseEditingWeightmap = false, bool bInRemoveSubsectionDuplicates = false);

	FColor* GetHeightData(int32 LocalX, int32 LocalY) const
	{
#if LANDSCAPE_VALIDATE_DATA_ACCESS
		checkSlow(IsInParallelGameThread() || IsInGameThread());
		check(Component);
		check(HeightMipData);
		check(LocalX >=0 && LocalY >=0 && LocalX < ComponentSizeVerts && LocalY < ComponentSizeVerts);
#endif

		int32 TexelX, TexelY;
		VertexXYToTexelXY(LocalX, LocalY, TexelX, TexelY);
		
		return &HeightMipData[TexelX + HeightmapComponentOffsetX + (TexelY + HeightmapComponentOffsetY) * HeightmapStride];
	}

	UE_DEPRECATED(5.7, "RetopologizeTool/XYOffset deprecated with the removal of non-edit layer landscapes")
	LANDSCAPE_API FColor* GetXYOffsetData(int32 LocalX, int32 LocalY) const;

	uint16 GetHeight( int32 LocalX, int32 LocalY ) const
	{
		FColor* Texel = GetHeightData(LocalX, LocalY);
		return (uint16)(Texel->R << 8) | (uint16)Texel->G;
	}

	uint16 GetHeight( int32 VertexIndex ) const
	{
		int32 X, Y;
		VertexIndexToXY( VertexIndex, X, Y );
		return GetHeight( X, Y );
	}

	float GetLocalHeight(int32 LocalX, int32 LocalY) const
	{
		return LandscapeDataAccess::GetLocalHeight(GetHeight(LocalX, LocalY));
	}

	float GetLocalHeight(int32 VertexIndex) const
	{
		return LandscapeDataAccess::GetLocalHeight(GetHeight(VertexIndex));
	}

	LANDSCAPE_API FVector GetLocalVertex(int32 LocalX, int32 LocalY) const;

	void GetLocalTangentVectors( int32 LocalX, int32 LocalY, FVector& LocalTangentX, FVector& LocalTangentY, FVector& LocalTangentZ ) const
	{
		// Note: these are still pre-scaled, just not rotated

		FColor* Data = GetHeightData( LocalX, LocalY );
		LocalTangentZ = LandscapeDataAccess::UnpackNormal(*Data);
		LocalTangentX = FVector(-LocalTangentZ.Z, 0.f, LocalTangentZ.X);
		LocalTangentY = FVector(0.f, LocalTangentZ.Z, -LocalTangentZ.Y);
	}

	FVector GetLocalVertex( int32 VertexIndex ) const
	{
		int32 X, Y;
		VertexIndexToXY( VertexIndex, X, Y );
		return GetLocalVertex( X, Y );
	}

	void GetLocalTangentVectors( int32 VertexIndex, FVector& LocalTangentX, FVector& LocalTangentY, FVector& LocalTangentZ ) const
	{
		int32 X, Y;
		VertexIndexToXY( VertexIndex, X, Y );
		GetLocalTangentVectors( X, Y, LocalTangentX, LocalTangentY, LocalTangentZ );
	}

	LANDSCAPE_API FVector GetWorldVertex(int32 LocalX, int32 LocalY) const;

	LANDSCAPE_API void GetWorldTangentVectors(int32 LocalX, int32 LocalY, FVector& WorldTangentX, FVector& WorldTangentY, FVector& WorldTangentZ) const;

	LANDSCAPE_API void GetWorldPositionTangents(int32 LocalX, int32 LocalY, FVector& WorldPos, FVector& WorldTangentX, FVector& WorldTangentY, FVector& WorldTangentZ) const;

	FVector GetWorldVertex( int32 VertexIndex ) const
	{
		int32 X, Y;
		VertexIndexToXY( VertexIndex, X, Y );
		return GetWorldVertex( X, Y );
	}

	void GetWorldTangentVectors( int32 VertexIndex, FVector& WorldTangentX, FVector& WorldTangentY, FVector& WorldTangentZ ) const
	{
		int32 X, Y;
		VertexIndexToXY( VertexIndex, X, Y );
		GetWorldTangentVectors( X, Y, WorldTangentX, WorldTangentY, WorldTangentZ );
	}

	void GetWorldPositionTangents( int32 VertexIndex, FVector& WorldPos, FVector& WorldTangentX, FVector& WorldTangentY, FVector& WorldTangentZ ) const
	{
		int32 X, Y;
		VertexIndexToXY( VertexIndex, X, Y );
		GetWorldPositionTangents( X, Y, WorldPos, WorldTangentX, WorldTangentY, WorldTangentZ );
	}

private:
	FLandscapeDataInterface DataInterface;
	ULandscapeComponent* Component = nullptr;
	bool bWorkOnEditingLayer = false;
	FColor* HeightMipData = nullptr;
};

// Helper functions
template<typename T>
void FillCornerValues(uint8& CornerSet, T* CornerValues)
{
	uint8 OriginalSet = CornerSet;

	if (CornerSet)
	{
		// Fill unset values
		while (CornerSet != 15)
		{
			if (CornerSet != 15 && (OriginalSet & 1))
			{
				if (!(CornerSet & 1 << 1))
				{
					CornerValues[1] = CornerValues[0];
					CornerSet |= 1 << 1;
				}
				if (!(CornerSet & 1 << 2))
				{
					CornerValues[2] = CornerValues[0];
					CornerSet |= 1 << 2;
				}
			}
			if (CornerSet != 15 && (OriginalSet & 1 << 1))
			{
				if (!(CornerSet & 1))
				{
					CornerValues[0] = CornerValues[1];
					CornerSet |= 1;
				}
				if (!(CornerSet & 1 << 3))
				{
					CornerValues[3] = CornerValues[1];
					CornerSet |= 1 << 3;
				}
			}
			if (CornerSet != 15 && (OriginalSet & 1 << 2))
			{
				if (!(CornerSet & 1))
				{
					CornerValues[0] = CornerValues[2];
					CornerSet |= 1;
				}
				if (!(CornerSet & 1 << 3))
				{
					CornerValues[3] = CornerValues[2];
					CornerSet |= 1 << 3;
				}
			}
			if (CornerSet != 15 && (OriginalSet & 1 << 3))
			{
				if (!(CornerSet & 1 << 1))
				{
					CornerValues[1] = CornerValues[3];
					CornerSet |= 1 << 1;
				}
				if (!(CornerSet & 1 << 2))
				{
					CornerValues[2] = CornerValues[3];
					CornerSet |= 1 << 2;
				}
			}

			OriginalSet = CornerSet;
		}
	}
}

#endif // WITH_EDITOR
