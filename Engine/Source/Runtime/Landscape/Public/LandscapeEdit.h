// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
LandscapeEdit.h: Classes for the editor to access to Landscape data
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "RHI.h"
#include "LandscapeProxy.h"
#include "Engine/Texture2D.h"
#include "AI/NavigationSystemBase.h"
#include "Components/ActorComponent.h"
#include "LandscapeInfo.h"
#include "LandscapeComponent.h"
#include "LandscapeEditTypes.h"
#include "LandscapeHeightfieldCollisionComponent.h"
#include "InstancedFoliageActor.h"
#include "LandscapeLayerInfoObject.h"
#include "LandscapeSettings.h"

#if WITH_EDITOR
#include "Containers/ArrayView.h"
#include "LandscapeTextureHash.h"
#endif

#include "Landscape.h"

class ULandscapeComponent;
class ULandscapeInfo;
class ULandscapeLayerInfoObject;

#if WITH_EDITOR

/** Landscape EdMode Interface (used by ALandscape in Editor mode to access EdMode properties) */
class ILandscapeEdModeInterface
{
public:
	virtual void PostUpdateLayerContent() PURE_VIRTUAL(ILandscapeEdModeInterface::PostUpdateLayerContent(), return;);
	virtual ELandscapeToolTargetType GetLandscapeToolTargetType() const PURE_VIRTUAL(ILandscapeEdModeInterface::GetLandscapeToolTargetType(), return ELandscapeToolTargetType::Invalid;);
	virtual const ULandscapeEditLayerBase* GetLandscapeSelectedLayer() const PURE_VIRTUAL(ILandscapeEdModeInterface::GetLandscapeSelectedLayer(), return nullptr;);
	virtual ULandscapeLayerInfoObject* GetSelectedLandscapeLayerInfo() const PURE_VIRTUAL(ILandscapeEdModeInterface::GetSelectedLandscapeLayerInfo(), return nullptr;);
	UE_DEPRECATED(5.7, "Non-edit layer landscapes are deprecated, all landscapes use the edit layer system now.")
	virtual void OnCanHaveLayersContentChanged() PURE_VIRTUAL(ILandscapeEdModeInterface::OnCanHaveLayersContentChanged(), return;);
};

struct FLandscapeTextureDataInfo
{
	struct FMipInfo
	{
		void* MipData = nullptr;
		TArray<FUpdateTextureRegion2D> MipUpdateRegions;
		bool bFull = false;
	};

	FLandscapeTextureDataInfo(UTexture2D* InTexture, bool bShouldDirtyPackage, ELandscapeTextureUsage TextureUsage = ELandscapeTextureUsage::Unknown, ELandscapeTextureType TextureType = ELandscapeTextureType::Unknown);
	virtual ~FLandscapeTextureDataInfo();

	// returns true if we need to block on the render thread before unlocking the mip data
	bool UpdateTextureData();

	int32 NumMips() { return MipInfo.Num(); }

	void AddMipUpdateRegion(int32 InMipIndex, int32 InX1, int32 InY1, int32 InX2, int32 InY2);
	
	void* GetMipData(int32 InMipIndex);

	int32 GetMipSizeX(int32 InMipIndex) const
	{
		return FMath::Max(Texture->Source.GetSizeX() >> InMipIndex, 1);
	}

	int32 GetMipSizeY(int32 InMipIndex) const
	{
		return FMath::Max(Texture->Source.GetSizeY() >> InMipIndex, 1);
	}

private:
	UTexture2D* Texture = nullptr;
	TArray<FMipInfo> MipInfo;
	ELandscapeTextureUsage TextureUsage = ELandscapeTextureUsage::Unknown;
	ELandscapeTextureType TextureType = ELandscapeTextureType::Unknown;
};

struct FLandscapeTextureDataInterface
{
	LANDSCAPE_API FLandscapeTextureDataInterface(bool bInUploadTextureChangesToGPU = true);
	LANDSCAPE_API virtual ~FLandscapeTextureDataInterface();
		
	void SetShouldDirtyPackage(bool bValue) { bShouldDirtyPackage = bValue; }
	bool GetShouldDirtyPackage() const { return bShouldDirtyPackage; }

	// Texture data access
	LANDSCAPE_API FLandscapeTextureDataInfo* GetTextureDataInfo(UTexture2D* Texture);

	// Flush texture updates
	LANDSCAPE_API void Flush();

	// Texture bulk operations for weightmap reallocation
	LANDSCAPE_API void CopyTextureChannel(UTexture2D* Dest, int32 DestChannel, UTexture2D* Src, int32 SrcChannel);
	LANDSCAPE_API void ZeroTextureChannel(UTexture2D* Dest, int32 DestChannel);
	LANDSCAPE_API void CopyTextureFromHeightmap(UTexture2D* Dest, ULandscapeComponent* Comp, int32 MipIndex);
	LANDSCAPE_API void CopyTextureFromHeightmap(UTexture2D* Dest, int32 DestChannel, ULandscapeComponent* Comp, int32 SrcChannel);
	LANDSCAPE_API void CopyTextureFromWeightmap(UTexture2D* Dest, int32 DestChannel, ULandscapeComponent* Comp, ULandscapeLayerInfoObject* LayerInfo);
	LANDSCAPE_API void CopyTextureFromWeightmap(UTexture2D* Dest, int32 DestChannel, ULandscapeComponent* Comp, ULandscapeLayerInfoObject* LayerInfo, int32 MipIndex);

	template<typename TData>
	void SetTextureValueTempl(UTexture2D* Dest, TData Value);
	LANDSCAPE_API void ZeroTexture(UTexture2D* Dest);
	LANDSCAPE_API void SetTextureValue(UTexture2D* Dest, FColor Value);

	template<typename TData>
	bool EqualTextureValueTempl(UTexture2D* Src, TData Value);
	LANDSCAPE_API bool EqualTextureValue(UTexture2D* Src, FColor Value);

private:
	LANDSCAPE_API void CopyTextureFromWeightmap(FLandscapeTextureDataInfo* DestDataInfo, int32 DestChannel, ULandscapeComponent* Comp, ULandscapeLayerInfoObject* LayerInfo, int32 MipIndex);

	TMap<UTexture2D*, FLandscapeTextureDataInfo*> TextureDataMap;
	bool bUploadTextureChangesToGPU;
	bool bShouldDirtyPackage;
};

struct FLandscapeEditDataInterface : public FLandscapeTextureDataInterface
{
	// this constructor will build an interface that works in the current edit layer (uses the ALandscape PrivateEditingLayer state)
	LANDSCAPE_API FLandscapeEditDataInterface(ULandscapeInfo* InLandscape, bool bInUploadTextureChangesToGPU = true);

	// this constructor will build an interface that works in the specified edit layer : InEditLayerGUID
	LANDSCAPE_API FLandscapeEditDataInterface(ULandscapeInfo* InLandscape, const FGuid& InEditLayerGUID, bool bInUploadTextureChangesToGPU = true);

	// switch the layer this interface works in
	LANDSCAPE_API void SetEditLayer(const FGuid& InEditLayerGUID);

	// returns the current layer this interface is working in
	LANDSCAPE_API FGuid GetEditLayer() const;

	// Misc
	LANDSCAPE_API bool GetComponentsInRegion(int32 X1, int32 Y1, int32 X2, int32 Y2, TSet<ULandscapeComponent*>* OutComponents = NULL);

	//
	// Heightmap access
	//
	LANDSCAPE_API void SetHeightData(int32 X1, int32 Y1, int32 X2, int32 Y2, const uint16* InData, int32 InStride, bool InCalcNormals, const uint16* InNormalData = nullptr, const uint16* InHeightAlphaBlendData = nullptr, const uint8* InHeightRaiseLowerData = nullptr, bool InCreateComponents = false, UTexture2D* InHeightmap = nullptr, UTexture2D* InXYOffsetmapTexture = nullptr,
					   bool InUpdateBounds = true, bool InUpdateCollision = true, bool InGenerateMips = true);

	// Helper accessor
	LANDSCAPE_API FORCEINLINE uint16 GetHeightMapData(const ULandscapeComponent* Component, int32 TexU, int32 TexV, FColor* TextureData = NULL);
	// Helper accessor
	LANDSCAPE_API FORCEINLINE uint16 GetHeightMapAlphaBlendData(const ULandscapeComponent* Component, int32 TexU, int32 TexV, FColor* TextureData = NULL);
	// Helper accessor
	LANDSCAPE_API FORCEINLINE uint8 GetHeightMapFlagsData(const ULandscapeComponent* Component, int32 TexU, int32 TexV, FColor* TextureData = NULL);
	// Generic
	template<typename TStoreData>
	void GetHeightDataTempl(int32& X1, int32& Y1, int32& X2, int32& Y2, TStoreData& StoreData);
	template<typename TStoreData>
	void GetHeightAlphaBlendDataTempl(int32& X1, int32& Y1, int32& X2, int32& Y2, TStoreData& StoreData);
	template<typename TStoreData>
	void GetHeightFlagsDataTempl(int32& X1, int32& Y1, int32& X2, int32& Y2, TStoreData& StoreData);
	// Without data interpolation, able to get normal data
	template<typename TStoreData>
	void GetHeightDataTemplFast(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TStoreData& StoreData, UTexture2D* InHeightmap = nullptr, TStoreData* NormalData = NULL);
	// Implementation for fixed array
	LANDSCAPE_API void GetHeightData(int32& X1, int32& Y1, int32& X2, int32& Y2, uint16* Data, int32 Stride);
	LANDSCAPE_API void GetHeightDataFast(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, uint16* Data, int32 Stride, uint16* NormalData = NULL, UTexture2D* InHeightmap = nullptr);
	LANDSCAPE_API void GetHeightAlphaBlendData(int32& X1, int32& Y1, int32& X2, int32& Y2, uint16* Data, int32 Stride);
	LANDSCAPE_API void GetHeightFlagsData(int32& X1, int32& Y1, int32& X2, int32& Y2, uint8* Data, int32 Stride);
	// Implementation for sparse array
	LANDSCAPE_API void GetHeightData(int32& X1, int32& Y1, int32& X2, int32& Y2, TMap<FIntPoint, uint16>& SparseData);
	LANDSCAPE_API void GetHeightDataFast(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TMap<FIntPoint, uint16>& SparseData, TMap<FIntPoint, uint16>* NormalData = NULL, UTexture2D* InHeightmap = nullptr);

	UE_DEPRECATED(5.7, "XYOffset deprecated with the removal of non-edit layer landscapes")
	LANDSCAPE_API void RecalculateNormals();

	//
	// Weightmap access
	//
	// Helper accessor
	LANDSCAPE_API FORCEINLINE uint8 GetWeightMapData(const ULandscapeComponent* Component, ULandscapeLayerInfoObject* LayerInfo, int32 TexU, int32 TexV, uint8 Offset = 0, UTexture2D* Texture = NULL, uint8* TextureData = NULL);
	template<typename TStoreData>
	void GetWeightDataTempl(ULandscapeLayerInfoObject* LayerInfo, int32& X1, int32& Y1, int32& X2, int32& Y2, TStoreData& StoreData);
	// Without data interpolation
	template<typename TStoreData>
	void GetWeightDataTemplFast(ULandscapeLayerInfoObject* LayerInfo, const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TStoreData& StoreData);
	// Implementation for fixed array
	LANDSCAPE_API void GetWeightData(ULandscapeLayerInfoObject* LayerInfo, int32& X1, int32& Y1, int32& X2, int32& Y2, uint8* Data, int32 Stride);
	//void GetWeightData(FName LayerName, int32& X1, int32& Y1, int32& X2, int32& Y2, TArray<uint8>* Data, int32 Stride);
	LANDSCAPE_API void GetWeightDataFast(ULandscapeLayerInfoObject* LayerInfo, const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, uint8* Data, int32 Stride);
	LANDSCAPE_API void GetWeightDataFast(ULandscapeLayerInfoObject* LayerInfo, const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TArray<uint8>* Data, int32 Stride);
	// Implementation for sparse array
	LANDSCAPE_API void GetWeightData(ULandscapeLayerInfoObject* LayerInfo, int32& X1, int32& Y1, int32& X2, int32& Y2, TMap<FIntPoint, uint8>& SparseData);
	//void GetWeightData(FName LayerName, int32& X1, int32& Y1, int32& X2, int32& Y2, TMap<uint64, TArray<uint8>>& SparseData);
	LANDSCAPE_API void GetWeightDataFast(ULandscapeLayerInfoObject* LayerInfo, const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TMap<FIntPoint, uint8>& SparseData);
	LANDSCAPE_API void GetWeightDataFast(ULandscapeLayerInfoObject* LayerInfo, const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TMap<FIntPoint, TArray<uint8>>& SparseData);
	// Updates weightmap for LayerInfo
	LANDSCAPE_API void SetAlphaData(ULandscapeLayerInfoObject* LayerInfo, const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, const uint8* Data, int32 Stride, ELandscapeLayerPaintingRestriction PaintingRestriction = ELandscapeLayerPaintingRestriction::None);
	UE_DEPRECATED(5.7, "Support for bWeightAdjust and bTotalWeightAdjust have been removed, this function shall be used anymore")
	LANDSCAPE_API void SetAlphaData(ULandscapeLayerInfoObject* LayerInfo, const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, const uint8* Data, int32 Stride, ELandscapeLayerPaintingRestriction PaintingRestriction, bool bWeightAdjust, bool bTotalWeightAdjust);
	// Updates weightmaps for all layers. Data points to packed data for all layers in the landscape info
	LANDSCAPE_API void SetAlphaData(const TSet<ULandscapeLayerInfoObject*>& DirtyLayerInfos, const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, const uint8* Data, int32 Stride, ELandscapeLayerPaintingRestriction PaintingRestriction = ELandscapeLayerPaintingRestriction::None);
	// Delete a layer and re-normalize other layers
	LANDSCAPE_API void DeleteLayer(ULandscapeLayerInfoObject* LayerInfo);
	// Fill a layer and re-normalize other layers
	LANDSCAPE_API void FillLayer(ULandscapeLayerInfoObject* LayerInfo);
	// Fill all empty layers and re-normalize layers
	LANDSCAPE_API void FillEmptyLayers(ULandscapeLayerInfoObject* LayerInfo);
	
	// Replace/merge a layer (across all edit layers and the main runtime layer)
	LANDSCAPE_API void ReplaceLayer(ULandscapeLayerInfoObject* FromLayerInfo, ULandscapeLayerInfoObject* ToLayerInfo);

	template<typename TStoreData>
	void GetEditToolTextureData(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TStoreData& StoreData, TFunctionRef<UTexture2D*(ULandscapeComponent*)> GetComponentTexture);
	LANDSCAPE_API void SetEditToolTextureData(int32 X1, int32 Y1, int32 X2, int32 Y2, const uint8* Data, int32 Stride, TFunctionRef<UTexture2D*&(ULandscapeComponent*)> GetComponentTexture, TextureGroup InTextureGroup = TEXTUREGROUP_Terrain_Weightmap);

	// Without data interpolation, Select Data 
	template<typename TStoreData>
	void GetSelectDataTempl(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TStoreData& StoreData);
	LANDSCAPE_API void GetSelectData(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, uint8* Data, int32 Stride);
	LANDSCAPE_API void GetSelectData(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TMap<FIntPoint, uint8>& SparseData);
	LANDSCAPE_API void SetSelectData(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, const uint8* Data, int32 Stride);
	
	template<typename TStoreData>
	void GetLayerContributionDataTempl(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TStoreData& StoreData);
	LANDSCAPE_API void GetLayerContributionData(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, uint8* Data, int32 Stride);
	LANDSCAPE_API void GetLayerContributionData(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TMap<FIntPoint, uint8>& SparseData);
	LANDSCAPE_API void SetLayerContributionData(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, const uint8* Data, int32 Stride);

	template<typename TStoreData>
	void GetDirtyDataTempl(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TStoreData& StoreData);
	LANDSCAPE_API void GetDirtyData(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, uint8* Data, int32 Stride);
	LANDSCAPE_API void GetDirtyData(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TMap<FIntPoint, uint8>& SparseData);
	LANDSCAPE_API void SetDirtyData(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, const uint8* Data, int32 Stride);

	UE_DEPRECATED(5.7, "XYOffset deprecated with the removal of non-edit layer landscapes")
	LANDSCAPE_API void SetXYOffsetData(int32 X1, int32 Y1, int32 X2, int32 Y2, const FVector2D* Data, int32 Stride);
	UE_DEPRECATED(5.7, "XYOffset deprecated with the removal of non-edit layer landscapes")
	LANDSCAPE_API void SetXYOffsetData(int32 X1, int32 Y1, int32 X2, int32 Y2, const FVector* Data, int32 Stride);
	// Helper accessor
	UE_DEPRECATED(5.7, "XYOffset deprecated with the removal of non-edit layer landscapes")
	LANDSCAPE_API FORCEINLINE FVector2D GetXYOffsetmapData(const ULandscapeComponent* Component, int32 TexU, int32 TexV, FColor* TextureData = NULL);

	UE_DEPRECATED(5.7, "XYOffset deprecated with the removal of non-edit layer landscapes")
	LANDSCAPE_API void GetXYOffsetData(int32& X1, int32& Y1, int32& X2, int32& Y2, FVector2D* Data, int32 Stride);
	UE_DEPRECATED(5.7, "XYOffset deprecated with the removal of non-edit layer landscapes")
	LANDSCAPE_API void GetXYOffsetData(int32& X1, int32& Y1, int32& X2, int32& Y2, TMap<FIntPoint, FVector2D>& SparseData);
	UE_DEPRECATED(5.7, "XYOffset deprecated with the removal of non-edit layer landscapes")
	LANDSCAPE_API void GetXYOffsetData(int32& X1, int32& Y1, int32& X2, int32& Y2, FVector* Data, int32 Stride);
	UE_DEPRECATED(5.7, "XYOffset deprecated with the removal of non-edit layer landscapes")
	LANDSCAPE_API void GetXYOffsetData(int32& X1, int32& Y1, int32& X2, int32& Y2, TMap<FIntPoint, FVector>& SparseData);

	// Without data interpolation, able to get normal data
	UE_DEPRECATED(5.7, "XYOffset deprecated with the removal of non-edit layer landscapes")
	LANDSCAPE_API void GetXYOffsetDataFast(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, FVector2D* Data, int32 Stride);
	UE_DEPRECATED(5.7, "XYOffset deprecated with the removal of non-edit layer landscapes")
	LANDSCAPE_API void GetXYOffsetDataFast(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TMap<FIntPoint, FVector2D>& SparseData);
	UE_DEPRECATED(5.7, "XYOffset deprecated with the removal of non-edit layer landscapes")
	LANDSCAPE_API void GetXYOffsetDataFast(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, FVector* Data, int32 Stride);
	UE_DEPRECATED(5.7, "XYOffset deprecated with the removal of non-edit layer landscapes")
	LANDSCAPE_API void GetXYOffsetDataFast(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TMap<FIntPoint, FVector>& SparseData);

	template<typename T>
	static void ShrinkData(TArray<T>& Data, int32 OldMinX, int32 OldMinY, int32 OldMaxX, int32 OldMaxY, int32 NewMinX, int32 NewMinY, int32 NewMaxX, int32 NewMaxY);

	LANDSCAPE_API const ALandscape* GetTargetLandscape() const;

	UE_DEPRECATED(5.7, "Non-edit layer landscapes are deprecated, all landscapes use the edit layer system now.")
	LANDSCAPE_API bool CanHaveLandscapeLayersContent() const;
	UE_DEPRECATED(5.7, "Non-edit layer landscapes are deprecated, all landscapes use the edit layer system now.")
	LANDSCAPE_API bool HasLandscapeLayersContent() const;

private:
	int32 ComponentSizeQuads;
	int32 SubsectionSizeQuads;
	int32 ComponentNumSubsections;
	FVector DrawScale;

	ULandscapeInfo* LandscapeInfo;
	
	// if true, we use the LandscapeActor->PrivateEditingLayer, otherwise we use the LocalEditLayerGUID specified below
	bool bUseSharedLandscapeEditLayer;
	FGuid LocalEditLayerGUID;

	void FillLayer(ULandscapeLayerInfoObject* LayerInfo, bool bEmptyLayersOnly);

	// Only for Missing Data interpolation... only internal usage
	template<typename TData, typename TStoreData, typename FType>
	void CalcMissingValues(const int32 X1, const int32 X2, const int32 Y1, const int32 Y2,
		const int32 ComponentIndexX1, const int32 ComponentIndexX2, const int32 ComponentIndexY1, const int32 ComponentIndexY2,
		const int32 ComponentSizeX, const int32 ComponentSizeY, TData* CornerValues,
		TArray<bool>& NoBorderY1, TArray<bool>& NoBorderY2, TArray<bool>& ComponentDataExist, TStoreData& StoreData);

	// Generic Height Data access
	template<typename TStoreData, typename TGetHeightMapDataFunction>
	void GetHeightDataInternal(int32& ValidX1, int32& ValidY1, int32& ValidX2, int32& ValidY2, TStoreData& StoreData, TGetHeightMapDataFunction GetHeightMapDataFunction);

	FORCEINLINE const FColor& GetHeightMapColor(const ULandscapeComponent* Component, int32 TexU, int32 TexV, FColor* TextureData);

	// test if layer is allowed for a given texel
	inline bool IsLayerAllowed(const ULandscapeLayerInfoObject* LayerInfo,
	                          int32 ComponentIndexX, int32 SubIndexX, int32 SubX,
	                          int32 ComponentIndexY, int32 SubIndexY, int32 SubY);
};

struct FLandscapeDoNotDirtyScope
{
	FLandscapeDoNotDirtyScope(FLandscapeEditDataInterface& InEditInterface, bool bInScopeEnabled = true)
		: EditInterface(InEditInterface), bScopeEnabled(bInScopeEnabled)
	{
		if (bScopeEnabled)
		{
			bPreviousValue = EditInterface.GetShouldDirtyPackage();
			EditInterface.SetShouldDirtyPackage(false);
		}
	}

	~FLandscapeDoNotDirtyScope()
	{
		if (bScopeEnabled)
		{
			EditInterface.SetShouldDirtyPackage(bPreviousValue);
		}
	}

	FLandscapeEditDataInterface& EditInterface;
	bool bPreviousValue;
	bool bScopeEnabled;
};

template<typename T>
void FLandscapeEditDataInterface::ShrinkData(TArray<T>& Data, int32 OldMinX, int32 OldMinY, int32 OldMaxX, int32 OldMaxY, int32 NewMinX, int32 NewMinY, int32 NewMaxX, int32 NewMaxY)
{
	check(OldMinX <= OldMaxX && OldMinY <= OldMaxY);
	check(NewMinX >= OldMinX && NewMaxX <= OldMaxX);
	check(NewMinY >= OldMinY && NewMaxY <= OldMaxY);

	if (NewMinX != OldMinX || NewMinY != OldMinY ||
		NewMaxX != OldMaxX || NewMaxY != OldMaxY)
	{
		// if only the MaxY changes we don't need to do the moving, only the truncate
		if (NewMinX != OldMinX || NewMinY != OldMinY || NewMaxX != OldMaxX)
		{
			for (int32 DestY = 0, SrcY = NewMinY - OldMinY; DestY <= NewMaxY - NewMinY; DestY++, SrcY++)
			{
//				UE_LOGF(LogLandscape, Warning, "Dest: %d, %d = %d Src: %d, %d = %d Width = %d", 0, DestY, DestY * (1 + NewMaxX - NewMinX), NewMinX - OldMinX, SrcY, SrcY * (1 + OldMaxX - OldMinX) + NewMinX - OldMinX, (1 + NewMaxX - NewMinX));
				T* DestData = &Data[DestY * (1 + NewMaxX - NewMinX)];
				const T* SrcData = &Data[SrcY * (1 + OldMaxX - OldMinX) + NewMinX - OldMinX];
				FMemory::Memmove(DestData, SrcData, (1 + NewMaxX - NewMinX) * sizeof(T));
			}
		}

		const int32 NewSize = (1 + NewMaxY - NewMinY) * (1 + NewMaxX - NewMinX);
		Data.RemoveAt(NewSize, Data.Num() - NewSize);
	}
}

//
// FHeightmapAccessor
//
template<bool bInUseInterp>
struct FHeightmapAccessor
{
	enum { bUseInterp = bInUseInterp };
	FHeightmapAccessor(ULandscapeInfo* InLandscapeInfo, ULandscapeLayerInfoObject* InLayerInfo = nullptr)
		: LandscapeInfo(InLandscapeInfo)
		, LandscapeEdit(InLandscapeInfo)
	{
		// We don't care about InLayerInfo since heightmaps are not a target layer per se
		check(InLayerInfo == nullptr);
	}

	FHeightmapAccessor(FHeightmapAccessor&& InOther) = default;
	FHeightmapAccessor& operator = (FHeightmapAccessor&& InOther) = default;
	FHeightmapAccessor(const FHeightmapAccessor& InOther) = delete;
	FHeightmapAccessor& operator = (const FHeightmapAccessor& InOther) = delete;

	void SetEditLayer(const FGuid& InEditLayerGUID)
	{
		LandscapeEdit.SetEditLayer(InEditLayerGUID);
	}

	FGuid GetEditLayer() const
	{
		return LandscapeEdit.GetEditLayer();
	}

	// accessors
	void GetData(int32& X1, int32& Y1, int32& X2, int32& Y2, TMap<FIntPoint, uint16>& Data)
	{
		LandscapeEdit.GetHeightData(X1, Y1, X2, Y2, Data);
	}

	void GetData(int32& X1, int32& Y1, int32& X2, int32& Y2, uint16* Data)
	{
		LandscapeEdit.GetHeightData(X1, Y1, X2, Y2, Data, 0);
	}

	void GetDataFast(int32 X1, int32 Y1, int32 X2, int32 Y2, TMap<FIntPoint, uint16>& Data)
	{
		LandscapeEdit.GetHeightDataFast(X1, Y1, X2, Y2, Data);
	}

	void GetDataFast(int32 X1, int32 Y1, int32 X2, int32 Y2, uint16* Data)
	{
		LandscapeEdit.GetHeightDataFast(X1, Y1, X2, Y2, Data, 0);
	}

	// Render the heightmap with overrides on edit layer visibility.  Does not modify persistent landscape data like FLandscapeEditDataInterface does.
	// This function exists mainly as a template customization point to handle uint16 for heights and uint8 for weights.
	// EvalRect is half-open.
	static bool GetSelectiveLayerData(ALandscape* Landscape, FIntRect EvalRect, TBitArray<> ActiveEditLayers, TArrayView<uint16> OutData)
	{
		FLandscapeEditLayerRenderHeightParams RenderParams;
		RenderParams.Bounds = EvalRect;
		RenderParams.ActiveEditLayers = ActiveEditLayers;
		RenderParams.CpuResult = OutData;

		return Landscape->SelectiveRenderEditLayersHeightmaps(RenderParams);
	}

	void SetData(int32 X1, int32 Y1, int32 X2, int32 Y2, const uint16* Data, ELandscapeLayerPaintingRestriction PaintingRestriction = ELandscapeLayerPaintingRestriction::None)
	{
		TSet<ULandscapeComponent*> Components;
		if (LandscapeInfo && LandscapeEdit.GetComponentsInRegion(X1, Y1, X2, Y2, &Components))
		{
			// Update data
			ChangedComponents.Append(Components);

			for (ULandscapeComponent* Component : Components)
			{
				Component->RequestHeightmapUpdate();
			}

			if (GetDefault<ULandscapeSettings>()->GetFoliageLandscapeSnappingMode() == EFoliageLandscapeSnappingMode::SnapUnderBrushOnly)
			{
				bool bHasAttachedFoliage = false;
				for (ULandscapeComponent* Component : Components)
				{
					if (ULandscapeHeightfieldCollisionComponent* CollisionComponent = Component->GetCollisionComponent();
						CollisionComponent && AInstancedFoliageActor::HasFoliageAttached(CollisionComponent))
					{
						bHasAttachedFoliage = true;
						break;
					}
				}

				if (bHasAttachedFoliage)
				{
					TArray<ULandscapeHeightfieldCollisionComponent*> CollisionComponents;
					TArray<FBox> PreUpdateLocalBoxes;
					CollisionComponents.Reserve(Components.Num());
					PreUpdateLocalBoxes.Reserve(Components.Num());

					for (ULandscapeComponent* Component : Components)
					{
						if (ULandscapeHeightfieldCollisionComponent* CollisionComponent = Component->GetCollisionComponent())
						{
							CollisionComponents.Add(CollisionComponent);
							PreUpdateLocalBoxes.Add(FBox(
								FVector(static_cast<float>(X1), static_cast<float>(Y1), Component->CachedLocalBox.Min.Z),
								FVector(static_cast<float>(X2), static_cast<float>(Y2), Component->CachedLocalBox.Max.Z)));
						}
					}

					const FMatrix44d LandscapeToWorld = LandscapeInfo->GetLandscapeProxy()->LandscapeActorToWorld().ToMatrixWithScale();

					for (int32 Index = 0; Index < CollisionComponents.Num(); ++Index)
					{
						CollisionComponents[Index]->MarkAreaForSnapFoliageInstances(
							PreUpdateLocalBoxes[Index].TransformBy(LandscapeToWorld).ExpandBy(1.0f));
					}
				}
			}
			
			LandscapeEdit.SetHeightData(X1, Y1, X2, Y2, Data, 0, /*InCalcNormals=*/ false);
		}
	}

	void Flush()
	{
		LandscapeEdit.Flush();
	}

	virtual ~FHeightmapAccessor()
	{
		// Flush here manually so it will release the lock of the textures, as we will re lock for other things afterward
		Flush();
	}

private:
	ULandscapeInfo* LandscapeInfo;
	FLandscapeEditDataInterface LandscapeEdit;
	TSet<ULandscapeComponent*> ChangedComponents;
};

//
// TAlphamapAccessor
//
template<bool bInUseInterp>
struct TAlphamapAccessor
{
	enum { bUseInterp = bInUseInterp };
	TAlphamapAccessor(ULandscapeInfo* InLandscapeInfo, ULandscapeLayerInfoObject* InLayerInfo)
		: LandscapeInfo(InLandscapeInfo)
		, LandscapeEdit(InLandscapeInfo)
		, LayerInfo(InLayerInfo)
	{
		check(InLayerInfo != nullptr);
	}

	TAlphamapAccessor(TAlphamapAccessor&& InOther) = default;
	TAlphamapAccessor& operator = (TAlphamapAccessor&& InOther) = default;
	TAlphamapAccessor(const TAlphamapAccessor& InOther) = delete;
	TAlphamapAccessor& operator = (const TAlphamapAccessor& InOther) = delete;

	void GetData(int32& X1, int32& Y1, int32& X2, int32& Y2, TMap<FIntPoint, uint8>& Data)
	{
		LandscapeEdit.GetWeightData(LayerInfo, X1, Y1, X2, Y2, Data);
	}

	void GetData(int32& X1, int32& Y1, int32& X2, int32& Y2, uint8* Data)
	{
		LandscapeEdit.GetWeightData(LayerInfo, X1, Y1, X2, Y2, Data, 0);
	}

	void GetDataFast(int32 X1, int32 Y1, int32 X2, int32 Y2, TMap<FIntPoint, uint8>& Data)
	{
		LandscapeEdit.GetWeightDataFast(LayerInfo, X1, Y1, X2, Y2, Data);
	}

	void GetDataFast(int32 X1, int32 Y1, int32 X2, int32 Y2, uint8* Data)
	{
		LandscapeEdit.GetWeightDataFast(LayerInfo, X1, Y1, X2, Y2, Data, 0);
	}

	static bool GetSelectiveLayerData(ALandscape* Landscape, FIntRect EvalRect, TBitArray<> ActiveEditLayers, TArrayView<uint8> OutData)
	{
		unimplemented();
		return false;
	}

	void SetData(int32 X1, int32 Y1, int32 X2, int32 Y2, const uint8* Data, ELandscapeLayerPaintingRestriction PaintingRestriction)
	{
		TSet<ULandscapeComponent*> Components;
		if (LandscapeEdit.GetComponentsInRegion(X1, Y1, X2, Y2, &Components))
		{
			for (ULandscapeComponent* LandscapeComponent : Components)
			{
				// Flag both modes depending on client calling SetData
				LandscapeComponent->RequestWeightmapUpdate();
			}
			
			LandscapeEdit.SetAlphaData(LayerInfo, X1, Y1, X2, Y2, Data, /*Stride = */0, PaintingRestriction);
			//LayerInfo->IsReferencedFromLoadedData = true;
		}
	}

	void Flush()
	{
		LandscapeEdit.Flush();
	}

	void SetEditLayer(const FGuid& InEditLayerGUID)
	{
		LandscapeEdit.SetEditLayer(InEditLayerGUID);
	}
	
	FGuid GetEditLayer() const
	{
		return LandscapeEdit.GetEditLayer();
	}

private:
	ULandscapeInfo* LandscapeInfo;
	FLandscapeEditDataInterface LandscapeEdit;
	TSet<ULandscapeComponent*> ModifiedComponents;
	ULandscapeLayerInfoObject* LayerInfo;
	bool bPerformLegacyWeightBalancing;
};

template<bool bInUseInterp, bool bInUseTotalNormalize>
struct UE_DEPRECATED(5.7, "Use TAlphamapAccessor instead") FAlphamapAccessor : public TAlphamapAccessor<bInUseInterp>
{
};

#endif

