// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StaticArray.h"
#include "CoreMinimal.h"
#include "LandscapeEditTypes.h"


// ----------------------------------------------------------------------------------
// Forward declarations

class ALandscape;
class ULandscapeInfo;
class ULandscapeLayerInfoObject;

namespace UE::Landscape::EditLayers
{
	class FTargetLayerGroup;
} // namespace UE::Landscape::EditLayers

// ----------------------------------------------------------------------------------

namespace UE::Landscape::EditLayers
{

#if WITH_EDITOR

/**
 * Global info about the landscape being merged that can be used throughout the merge. 
 */
class FMergeContext
{
public:
	friend ALandscape;

	using FTargetLayerGroupsPerBlendingMethod = TStaticArray<TArray<FTargetLayerGroup>, static_cast<uint8>(ELandscapeTargetLayerBlendMethod::Count)>;

	FMergeContext(ALandscape* InLandscape, bool bInIsHeightmapMerge, bool bInSkipProceduralRenderers);
	virtual ~FMergeContext() = default;
	FMergeContext(const FMergeContext& Other) = default;
	FMergeContext(FMergeContext&& Other) = default;
	FMergeContext& operator=(const FMergeContext& Other) = default;
	FMergeContext& operator=(FMergeContext&& Other) = default;

	inline ALandscape* GetLandscape() const { return Landscape; }

	inline ULandscapeInfo* GetLandscapeInfo() const { return LandscapeInfo; }
	
	/**
	 * @return all target layer names declared in this context, including ones that are potentially invalid (i.e. no layer info associated)
	 */
	inline const TArray<FName>& GetAllTargetLayerNames() const { return AllTargetLayerNames; }

	/**
	 * @return all target layer names that have a valid layer info object. Will return the height layer in the case of a heightmap merge, even though there's no layer info for those : 
	 */
	TArray<FName> GetValidTargetLayerNames() const;

	/**
	 * @return all layer info objects declared in this context (can contain null entries for invalid layers or in the case of a heightmap merge)
	 */
	inline const TArray<ULandscapeLayerInfoObject*>& GetAllWeightmapLayerInfos() const { return AllWeightmapLayerInfos; }

	/**
	 * @return true if this merge context is used for a heightmap merge, false otherwise (visibility / weightmap)
	 */
	inline bool IsHeightmapMerge() const { return bIsHeightmapMerge; }

	/**
	 * @return true if this merge context is used for a heightmap merge, false otherwise (visibility / weightmap)
	 */
	inline bool ShouldSkipProceduralRenderers() const { return bSkipProceduralRenderers; }

	/**
	 * @return true if the target layer name is registered to this merge context and has an associated layer info object
	 */
	LANDSCAPE_API int32 IsValidTargetLayerName(const FName& InName) const;

	/**
	 * @return true if the target layer name has an associated layer info object. Asserts if the layer name isn't known to the merge context
	 */
	LANDSCAPE_API int32 IsValidTargetLayerNameChecked(const FName& InName) const;

	/**
	 * @return true if the target layer index is valid in the list of declared target layers on the merge context. 
	 *  Important note : it doesn't mean that it's associated with a valid layer info object, use IsValidTargetLayerName for this!
	 */
	LANDSCAPE_API int32 IsTargetLayerIndexValid(int32 InIndex) const;

	/**
	 * @return the index of this target layer name in this context, INDEX_NONE if not found
	 */
	LANDSCAPE_API int32 GetTargetLayerIndexForName(const FName& InName) const;

	/**
	 * @return the index of this target layer name in this context. Asserts if the layer name isn't known to the merge context
	 */
	LANDSCAPE_API int32 GetTargetLayerIndexForNameChecked(const FName& InName) const;

	/**
	 * @return the target layer name corresponding to the index in this context, NAME_None if not found
	 */
	LANDSCAPE_API FName GetTargetLayerNameForIndex(int32 InIndex) const;

	/**
	 * @return the target layer name corresponding to the index in this context, NAME_None if not found. Asserts if the layer index isn't known to the merge context
	 */
	LANDSCAPE_API FName GetTargetLayerNameForIndexChecked(int32 InIndex) const;

	/**
	 * @return the index of this landscape layer info in this context, INDEX_NONE if not found
	 */
	LANDSCAPE_API int32 GetTargetLayerIndexForLayerInfo(ULandscapeLayerInfoObject* InLayerInfo) const;

	/**
	 * @return the index of this landscape layer info in this context. Asserts if the layer info isn't known to the merge context
	 */
	LANDSCAPE_API int32 GetTargetLayerIndexForLayerInfoChecked(ULandscapeLayerInfoObject* InLayerInfo) const;

	/**
	 * @return the landscape layer object corresponding to the target layer name in this context, nullptr if not found
	 */
	LANDSCAPE_API ULandscapeLayerInfoObject* GetTargetLayerInfoForName(const FName& InName) const;

	/**
	 * @return the landscape layer object corresponding to the target layer name in this context. Asserts if the layer name isn't known to the merge context
	 */
	LANDSCAPE_API ULandscapeLayerInfoObject* GetTargetLayerInfoForNameChecked(const FName& InName) const;

	/**
	 * @return the landscape layer object corresponding to the index in this context, nullptr if not found. Asserts if the layer index isn't known to the merge context
	 */
	LANDSCAPE_API ULandscapeLayerInfoObject* GetTargetLayerInfoForIndex(int32 InIndex) const;

	/**
	 * @return a TBitArray<> where, for each layer from InTargetLayerNames that is found in this context, the corresponding bit is set
	 */
	LANDSCAPE_API TBitArray<> ConvertTargetLayerNamesToBitIndices(TConstArrayView<FName> InTargetLayerNames) const;

	/**
	 * @return a TBitArray<> where, for each layer from InTargetLayerNames that is found in this context, the corresponding bit is set. Asserts if one of the layer names isn't known to the merge context
	 */
	LANDSCAPE_API TBitArray<> ConvertTargetLayerNamesToBitIndicesChecked(TConstArrayView<FName> InTargetLayerNames) const;

	/**
	 * @return a list of target layer names for each set bit in InTargetLayerBitIndices. Asserts if InTargetLayerBitIndices isn't the same size as AllTargetLayerNames
	 */
	LANDSCAPE_API TArray<FName> ConvertTargetLayerBitIndicesToNames(const TBitArray<>& InTargetLayerBitIndices) const;

	/**
	 * @return a list of landscape layer info objects for each set bit in InTargetLayerBitIndices. Asserts if InTargetLayerBitIndices isn't the same size as AllTargetLayerNames
	 */
	LANDSCAPE_API TArray<ULandscapeLayerInfoObject*> ConvertTargetLayerBitIndicesToLayerInfos(const TBitArray<>& InTargetLayerBitIndices) const;

	/**
	 * Runs the given function for each all valid target layer in the bit indices in parameters, with the possibility of early exit
	 * Most easily used with a lambda as follows:
	 * ForEachTargetLayer([](int32 InTargetLayerIndex, FName InTargetLayerName) -> bool
	 * {
	 *     return continueLoop ? true : false;
	 * });
	 */
	LANDSCAPE_API void ForEachTargetLayer(const TBitArray<>& InTargetLayerBitIndices, TFunctionRef<bool(int32 /*InTargetLayerIndex*/, const FName& /*InTargetLayerName*/, ULandscapeLayerInfoObject* /*InWeightmapLayerInfo*/)> Fn) const;

	/** Same as ForEachTargetLayer but skips over invalid target layers */
	LANDSCAPE_API void ForEachValidTargetLayer(TFunctionRef<bool(int32 /*InTargetLayerIndex*/, const FName& /*InTargetLayerName*/, ULandscapeLayerInfoObject* /*InWeightmapLayerInfo*/)> Fn) const;

	/**
	 * @return the index corresponding to the visibility target layer in AllTargetLayerNames, INDEX_NONE if it's not in the list of layers
	 */
	inline int32 GetVisibilityTargetLayerIndex() const { return VisibilityTargetLayerIndex; }

	/**
	 * @return a bit mask the size of AllTargetLayerNames, where all bits are set to zero except the one corresponding to the visibility target layer
	 */
	inline const TBitArray<>& GetVisibilityTargetLayerMask() const { return VisibilityTargetLayerMask; }

	/** Same as GetVisibilityTargetLayerMask() but negated */
	inline const TBitArray<>& GetNegatedVisibilityTargetLayerMask() const { return NegatedVisibilityTargetLayerMask; }

	/**
	 * @return a bit mask for all target layers from AllTargetLayerNames that have a valid landscape layer info object
	 */
	inline const TBitArray<>& GetValidTargetLayerBitIndices() const { return ValidTargetLayerBitIndices; }

	/**
	 * @return a bit mask the size of AllTargetLayerNames, where all bits are set to bBitValue
	 */
	LANDSCAPE_API TBitArray<> BuildTargetLayerBitIndices(bool bInBitValue) const;

	const FTargetLayerGroupsPerBlendingMethod& GetTargetLayerGroupsPerBlendingMethod() const { return TargetLayerGroupsPerBlendingMethod; }

	/** 
	 * Return the target layer groups for the renderers performing the generic blend. This concerns only the blend methods that are applied per renderer
	 * @param InTargetLayerBitIndices bit mask of the target layers that are concerned by this blend
	 */
	LANDSCAPE_API TArray<FTargetLayerGroup> BuildGenericBlendTargetLayerGroups(const TBitArray<>& InTargetLayerBitIndices) const;

private:
	void FinalizeTargetLayerGroupsPerBlendingMethod();
	TArray<FTargetLayerGroup> GatherTargetLayerGroupsForBlendMethod(ELandscapeTargetLayerBlendMethod InBlendMethod) const;

protected:
	// COMMENT [jonathan.bard] : purposefully don't use ELandscapeToolTargetType here as ELandscapeToolTargetType::Weightmap and ELandscapeToolTargetType::Visibility have to be processed together (because of weightmap packing, 
	//  which means a visibility weightmap could be another channel of a texture which contains weightmap up to 3 other weightmaps, so we have to resolve the 4 channels altogether). 
	//  Note: this could change when SUPPORTS_LANDSCAPE_EDITORONLY_UBER_MATERIAL is done...
	/** Type of merge being requested */
	bool bIsHeightmapMerge = false;

	/** Allows to skip all non-default (i.e. persistent edit layer) layer renderers for this merge. Mostly there for backwards compatibility and debugging purposes. */
	bool bSkipProceduralRenderers = false;

	/** Landscape being merged */
	ALandscape* Landscape = nullptr;

	/** Landscape info associated with the landscape being merged */
	ULandscapeInfo* LandscapeInfo = nullptr;

	/** List of all target layer names that can possibly be rendered on this landscape (even invalid ones). The index of the target layer in that list is important :
	  it allows to use a TBitArray instead of a TSet to designate a list of target layers all throughout the merge, which greatly accelerates all the intersection tests being made.
	  If merging heightmaps, it contains a single name (that is only meaningful for debug display). */
	TArray<FName> AllTargetLayerNames;

	/** Bit mask that corresponds to only the visibility layer */
	TBitArray<> VisibilityTargetLayerMask;

	/** Bit mask that corresponds to all target layers except the visibility layer */
	TBitArray<> NegatedVisibilityTargetLayerMask;

	/** Per-target layer info. Same size as AllTargetLayerNames (only meaningful when rendering weightmaps) */
	TArray<ULandscapeLayerInfoObject*> AllWeightmapLayerInfos;

	/** List of valid target layers. If a target layer name is present here, it's because it has a valid landscape layer info object. Each bit in that bit array corresponds to an entry in AllTargetLayerNames */
	TBitArray<> ValidTargetLayerBitIndices;

	/** Contains the list of target layer groups for each weight-blending method as defined by the landscape layer info objects */
	FTargetLayerGroupsPerBlendingMethod TargetLayerGroupsPerBlendingMethod;

	// Index of the visibility layer in AllTargetLayerNames (always valid for weightmap merges)
	int32 VisibilityTargetLayerIndex = INDEX_NONE;
};

#endif // WITH_EDITOR

} // namespace UE::Landscape::EditLayers
