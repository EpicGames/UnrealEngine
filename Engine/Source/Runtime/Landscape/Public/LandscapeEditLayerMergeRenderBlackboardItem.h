// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "LandscapeEditLayerMergeRenderBlackboardItem.generated.h"


// ----------------------------------------------------------------------------------

/**
 * Base class for various blackboard items that can be attached to FMergeRenderContext in order to transfer temporary information
 *  between renderers. It's a USTRUCT only to benefit from the RTTI services provided by TInstancedStruct
 */
USTRUCT(meta = (Abstract))
struct FLandscapeEditLayerMergeRenderBlackboardItem
{
	GENERATED_BODY()
	
	virtual ~FLandscapeEditLayerMergeRenderBlackboardItem() = default;
};
