// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LandscapeEditTypes.h"
#include "LandscapeEditLayerMergeContext.h"
#include "LandscapeEditLayerRendererState.h"
#include "LandscapeEditLayerTargetTypeState.h"
#include "LandscapeEditLayerTypes.h"
#include "LandscapeUtils.h"
#include "EngineDefines.h"
#include "StructUtils/InstancedStruct.h"


// ----------------------------------------------------------------------------------
// Forward declarations

class ALandscape;
class ULandscapeComponent;
class ULandscapeLayerInfoObject;
class ULandscapeScratchRenderTarget;
class ILandscapeEditLayerRenderer;
struct FLandscapeEditLayerMergeRenderBlackboardItem;

namespace UE::Landscape
{
	class FRDGBuilderRecorder;
} // namespace UE::Landscape


namespace UE::Landscape::EditLayers
{
	class FMergeRenderContext;
} //namespace UE::Landscape::EditLayers


// ----------------------------------------------------------------------------------

namespace UE::Landscape::EditLayers
{

#if WITH_EDITOR

// ----------------------------------------------------------------------------------

/** 
 * Params struct passed to the merge function. It contains everything needed for requesting a given set of target layers (for weightmaps) on a given number of components and for a certain configuration of edit layers
 *  Note that this is what is requested by the caller, but in practice, there might be more renderers (e.g. some might get added e.g. legacy weight-blending, some removed because they turn out to be disabled...)
 *  and more weightmaps being rendered (e.g. a requested weightmap might depend on another one that has not been requested), or less (e.g. a requested weightmap is actually invalid) :
 */
struct FMergeRenderParams
{
	FMergeRenderParams(TArray<ULandscapeComponent*> InComponentsToMerge, const TArrayView<FEditLayerRendererState>& InEditLayerRendererStates, const TSet<FName>& InWeightmapLayerNames = {}, bool bInRequestAllLayers = false);

	/** List of components that need merging */
	TArray<ULandscapeComponent*> ComponentsToMerge;

	/** Requested states for every edit layer renderer participating to the merge */
	TArray<FEditLayerRendererState> EditLayerRendererStates;
	
	/** List of weightmap layers being requested. */
	TSet<FName> WeightmapLayerNames;

	/** Ignore the computed list of requested layers and instead request all valid layers */
	bool bRequestAllLayers = false;
};


// ----------------------------------------------------------------------------------

/**
 * Defines an individual render step of the batch merge
 */
struct FMergeRenderStep
{
	/** Describes what kind of operation this render step executes. Despite some of the names in there, those steps all run on the game thread but the _RenderThread ones will defer operations to the render thread. */
	enum class EType
	{
		Invalid,

		BeginRenderCommandRecorder, // Initiates a render command recorder that will batch one or several RenderLayer_Recorded operations, runs on the game thread
		EndRenderCommandRecorder, // Ends the render command recorder initiated by the last BeginRenderCommandRecorder, runs on the game thread but this is where the render command executing the recorded operations from (several) RenderLayer_Recorded step(s) will be pushed (on the render thread, this is where the FRDBuilder is created and executed)

		BeginRenderLayerGroup, // Initiates a series of RenderLayer steps for renderers that can be rendered one after another without an intermediate BlendLayer. Triggers a call to BeginRenderLayerGroup() on the first renderer in the group 
		EndRenderLayerGroup, // Ends a series of RenderLayer steps for renderers that can be rendered one after another without an intermediate BlendLayer. Triggers a call to EndRenderLayerGroup() on the last renderer in the group 

		RenderLayer, // Performs the rendering of a target layer group on an edit layer on a given world region (i.e. in a batch), runs on the game thread. Use the command recorder to perform the render operations. In RenderMode_Recorded, these will be delayed until EndRenderCommandRecorder
		BlendLayer, // Performs the rendering of a target layer group on an edit layer on a given world region (i.e. in a batch), runs on the game thread. Use the command recorder to perform the render operations. In RenderMode_Recorded, these will be delayed until EndRenderCommandRecorder

		SignalBatchMergeGroupDone, // Final step when rendering a target layer group on a given world region (i.e. in a batch) : runs on the game thread and allows to retrieve the result of the merge and do something with it (e.g. resolve the final textures)
	};

	FMergeRenderStep(EType InType)
		: Type(InType)
	{}


	FMergeRenderStep(EType InType, const TBitArray<>& InTargetLayerGroupBitIndices, const TArrayView<ULandscapeComponent*>& InComponentsToRender)
		: Type(InType)
		, TargetLayerGroupBitIndices(InTargetLayerGroupBitIndices)
		, ComponentsToRender(InComponentsToRender)
	{}

	FMergeRenderStep(EType InType, ERenderFlags InRenderFlags, const FEditLayerRendererState& InRendererState, const TBitArray<>& InTargetLayerGroupBitIndices, const TArrayView<ULandscapeComponent*>& InComponentsToRender)
		: Type(InType)
		, RenderFlags(InRenderFlags)
		, RendererState(InRendererState)
		, TargetLayerGroupBitIndices(InTargetLayerGroupBitIndices)
		, ComponentsToRender(InComponentsToRender)
	{}

	/** Type of operation for this step */
	EType Type = EType::Invalid;
	
	/** The render flags corresponding to this state */
	ERenderFlags RenderFlags = ERenderFlags::None;

PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	/** Renderer state to be used this step. This includes the renderer as well as its precise step (e.g. which weightmap are supported? which are enabled?) */
	FEditLayerRendererState RendererState = FEditLayerRendererState::GetDummyRendererState();
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

	/** List of target layers being involved in this step. Each bit in that bit array corresponds to an entry in FMergeContext's AllTargetLayerNames */
	TBitArray<> TargetLayerGroupBitIndices;

	/** List of components involved in this step */
	TArray<ULandscapeComponent*> ComponentsToRender;
};


// ----------------------------------------------------------------------------------

/** 
 * Defines an individual render batch when merging the landscape. A batch corresponds to a target layer group on a world's region, i.e. a set of weightmaps (or the heightmap) to render on a 
 *  portion of the world. Each batch is composed of a series of render steps. 
 */
struct FMergeRenderBatch
{
	bool operator < (const FMergeRenderBatch& InOther) const;

	ALandscape* Landscape = nullptr;

	/** Entire section of the the landscape currently loaded (in landscape vertex coordinates, inclusive bounds)  */
	FIntRect LandscapeExtent;

	/** Section of the landscape being covered by this batch (in landscape vertex coordinates, inclusive bounds) */
	FIntRect SectionRect;

	/** Resolution of the render target needed for this batch (including duplicate borders) */
	// TODO [jonathan.bard] : rename EffectiveResolution and make private maybe?
	FIntPoint Resolution = FIntPoint(ForceInitToZero);
	
	FIntPoint MinComponentKey = FIntPoint(MAX_int32, MAX_int32);
	FIntPoint MaxComponentKey = FIntPoint(MIN_int32, MIN_int32);
	
	/** Sequential list of rendering operations that need to be performed to fully render this batch*/
	TArray<FMergeRenderStep> RenderSteps;
	
	/** List of all components involved in this batch */
	TSet<ULandscapeComponent*> ComponentsToRender;

	/** List of all target layers being rendered in this batch (i.e. bitwise OR of all of the render steps' TargetLayerGroupBitIndices). Each bit in that bit array corresponds to an entry in FMergeRenderContext's AllTargetLayerNames  */
	TBitArray<> TargetLayerBitIndices;

	/** List of components involved in this batch and the target layers they're writing to (redundant with ComponentsToRender but we keep the latter for convenience)
	 (each bit corresponds to a target layer name in FMergeContext's AllTargetLayerNames) */
	TMap<ULandscapeComponent*, TBitArray<>> ComponentToTargetLayerBitIndices;

	/** Reverse lookup of ComponentToTargetLayerBitIndices : one entry per element in ComponentToTargetLayerBitIndices, each entry containing all of the components involved in this merge for this target layer */
	TArray<TSet<ULandscapeComponent*>> TargetLayersToComponents;

public:
	FIntPoint GetRenderTargetResolution(bool bInWithDuplicateBorders) const;
	
	/**
	 * Find the area in the render batch render target corresponding to each of the subsections of this component 
	 * @param InComponent Which component to compute the subsection rects for 
	 * @param OutSubsectionRects List of (up to 4) subsection rects when *not* taking into account duplicate borders (inclusive bounds)
	 * @param OutSubsectionRectsWithDuplicateBorders List of (up to 4) subsection rects when taking into account duplicate borders (inclusive bounds)
	 * @param bInWithDuplicateBorders indicates whether the rect coordinates should include the duplicated column/row at the end of the subsection or not
	 * 
	 * @return number of subsections
	 */
	int32 ComputeSubsectionRects(ULandscapeComponent* InComponent, TArray<FIntRect, TInlineAllocator<4>>& OutSubsectionRects, TArray<FIntRect, TInlineAllocator<4>>& OutSubsectionRectsWithDuplicateBorders) const;

	/**
	 * Find the area in the render batch render target corresponding to this component 
	 * @param InComponent Which component to compute the section rect for 
	 * @param bInWithDuplicateBorders indicates whether the rect coordinates should include the duplicated columns/rows at the end of each subsection or not
	 * 
	 * @Return the section rect (inclusive bounds)
	 */
	FIntRect ComputeSectionRect(ULandscapeComponent* InComponent, bool bInWithDuplicateBorders) const;

	/**
	 * Returns all subregions of the render batch representing valid data.
	 * @param bInWithDuplicateBorders indicates whether the rect coordinates should include the duplicated columns/rows at the end of each subsection or not
	 */
	TArray<FIntRect> ComputeAllComponentSectionRects(bool bInWithDuplicateBorders) const;

	/**
	 * Compute the rects corresponding to the sub-sections that need to be read from and written to when expanding the render target (inclusive bounds)
	 * @param OutSubsectionRects List of all subsection rects when *not* taking into account duplicate borders (inclusive bounds)
	 * @param OutSubsectionRectsWithDuplicateBorders List of all subsection rects when taking into account duplicate borders (inclusive bounds)
	 */
	void ComputeAllSubsectionRects(TArray<FIntRect>& OutSubsectionRects, TArray<FIntRect>& OutSubsectionRectsWithDuplicateBorders) const;

	/** Compute the component's linear index within that batch starting from the upper-left corner. This ensures a deterministic order within batch */
	uint32 ComputeComponentLinearIndex(const FIntPoint& InComponentKey) const;
};


// ----------------------------------------------------------------------------------

/**
 * Utility struct for attaching some information that pertains to a given landscape component in the context of a batch render
 */
struct FComponentMergeRenderInfo
{
	/** Component to render */
	ULandscapeComponent* Component = nullptr;

	/** Texture region that corresponds to this component in the render area's render target */
	FIntRect ComponentRegionInRenderArea;

	/** Index of the component in the render area's render target */
	FIntPoint ComponentKeyInRenderArea = FIntPoint(ForceInitToZero);

	bool operator < (const FComponentMergeRenderInfo& InOther) const;
};

/** 
 * Struct passed to ILandscapeEditLayerRenderer's render functions. This would normally be an inner struct of ILandscapeEditLayerRenderer but 
 *  this prevents forward-declaration so we leave in the UE::Landscape::EditLayers namespace instead. 
 */
struct FRenderParams
{
	FRenderParams(FMergeRenderContext* InMergeRenderContext,
		const TArrayView<FName>& InTargetLayerGroupLayerNames,
		const TArrayView<ULandscapeLayerInfoObject*>& InTargetLayerGroupLayerInfos,
		const FEditLayerRendererState& InRendererState,
		const TArrayView<FComponentMergeRenderInfo>& InSortedComponentMergeRenderInfos,
		const FTransform& InRenderAreaWorldTransform,
		const FIntRect& InRenderAreaSectionRect,
		int32 InNumSuccessfulRenderLayerStepsUntilBlendLayerStep);

	/** Merge context */
	FMergeRenderContext* MergeRenderContext = nullptr;

	/** List of target layers being involved in this step */
	TArray<FName> TargetLayerGroupLayerNames;

	/** List of target layer info objects being involved in this step (same size as TargetLayerGroupLayerNames) */
	TArray<ULandscapeLayerInfoObject*> TargetLayerGroupLayerInfos;

	/** Full state for the renderer involved in this step. This allows to retrieve the exact state of this renderer (e.g. enabled weightmaps, which can be different than the target layer group, in that
	 target layers A, B and C might belong to the same group but this renderer actually only has A enabled). This is therefore the renderer's responsibility to check that a given target layer from the
	 target layer group is effectively enabled. */
	FEditLayerRendererState RendererState;

	/** List of components (with additional info) to render */
	TArray<FComponentMergeRenderInfo> SortedComponentMergeRenderInfos;

	// TODO [jonathan.bard] Verify that scale is correct
	/** World transform that corresponds to the origin (bottom left corner) of the render area. The scale corresponds to the size of each quad in the landscape. */
	FTransform RenderAreaWorldTransform;

	/** SectionRect (i.e. landscape vertex coordinates, in landscape space) that corresponds to this render area */
	FIntRect RenderAreaSectionRect;

	/** When separate blend is enabled, tracks how many RenderLayer calls have succeeded yet (valid until the BlendLayer step occurs) */
	int32 NumSuccessfulRenderLayerStepsUntilBlendLayerStep = 0;
};


// ----------------------------------------------------------------------------------

/** 
 * Utility class that contains everything necessary to perform the batched merge : scratch render targets, list of batches, etc.
 */
class FMergeRenderContext : public FMergeContext
{
public:
	friend class ::ALandscape;

	FMergeRenderContext(const FMergeContext& InMergeContext);
	virtual ~FMergeRenderContext();
	FMergeRenderContext(const FMergeRenderContext& Other) = default;
	FMergeRenderContext(FMergeRenderContext&& Other) = default;
	FMergeRenderContext& operator=(const FMergeRenderContext& Other) = default;
	FMergeRenderContext& operator=(FMergeRenderContext&& Other) = default;

	bool IsValid() const;

	/**
	 * Cycle between the 3 render targets used for blending:
	 *  Write becomes Read -> Read becomes ReadPrevious -> ReadPrevious becomes Write
	 *	The new Write RT will be transitioned to state == InDesiredWriteAccess (if != ERHIAccess::None)
	 *  The new Read RT will be transitioned to state ERHIAccess::SRVMask
	 *  The new ReadPrevious RT will stay in state ERHIAccess::SRVMask
	 * 
	 * @param RDGBuilderRecorder recorder to append operations to a single FRDGBuilder, if the the recorder in "recording" mode. In "immediate" mode, a render command will be enqueued immediately
	 */
	LANDSCAPE_API void CycleBlendRenderTargets(UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder);

	LANDSCAPE_API ULandscapeScratchRenderTarget* GetBlendRenderTargetWrite() const;
	LANDSCAPE_API ULandscapeScratchRenderTarget* GetBlendRenderTargetRead() const;
	LANDSCAPE_API ULandscapeScratchRenderTarget* GetBlendRenderTargetReadPrevious() const;
	LANDSCAPE_API ULandscapeScratchRenderTarget* GetComponentIdRenderTarget() const;
	
	struct FOnRenderBatchTargetGroupDoneParams
	{
		FOnRenderBatchTargetGroupDoneParams(FMergeRenderContext* InMergeRenderContext, 
			const TArrayView<FName>& InTargetLayerGroupLayerNames,
			const TArrayView<ULandscapeLayerInfoObject*>& InTargetLayerGroupLayerInfos,
			const TArrayView<FComponentMergeRenderInfo>& InSortedComponentMergeRenderInfos)
			: MergeRenderContext(InMergeRenderContext)
			, TargetLayerGroupLayerNames(InTargetLayerGroupLayerNames)
			, TargetLayerGroupLayerInfos(InTargetLayerGroupLayerInfos)
			, SortedComponentMergeRenderInfos(InSortedComponentMergeRenderInfos)
		{}

		/** Render context : this is still active in this step and can be used for doing additional renders in the blend render targets, etc. */
		FMergeRenderContext* MergeRenderContext = nullptr;

		/** List of target layers being involved in this step */
		TArray<FName> TargetLayerGroupLayerNames;

		/** List of target layer info objects being involved in this step (same size as TargetLayerGroupLayerNames) */
		TArray<ULandscapeLayerInfoObject*> TargetLayerGroupLayerInfos;

		/** Additional info about the components that have been processed in this batch render */
		const TArray<FComponentMergeRenderInfo> SortedComponentMergeRenderInfos;
	};
	void Render(TFunction<void(const FOnRenderBatchTargetGroupDoneParams& /*InParams*/, UE::Landscape::FRDGBuilderRecorder& /*RDGBuilderRecorder*/)> OnRenderBatchTargetGroupDone);

	static constexpr uint8 ComponentLinearIdBitCount = 32;
	static constexpr uint32 MaxComponentsPerBatch = static_cast<uint32>(1u << (ComponentLinearIdBitCount - 1)); 

	inline FIntPoint GetMaxNeededResolution() const { return MaxNeededResolution; }
	inline const TArray<FMergeRenderBatch>& GetRenderBatches() const { return RenderBatches; }
	LANDSCAPE_API const FMergeRenderBatch* GetCurrentRenderBatch() const;
	LANDSCAPE_API const int32 GetCurrentRenderBatchIdx() const;

	LANDSCAPE_API FTransform ComputeVisualLogTransform(const FTransform& InTransform) const;
	LANDSCAPE_API void IncrementVisualLogOffset();
	LANDSCAPE_API void ResetVisualLogOffset();

#if ENABLE_VISUAL_LOG
	LANDSCAPE_API static int32 GetVisualLogAlpha();
	LANDSCAPE_API bool IsVisualLogEnabled() const;
#endif // ENABLE_VISUAL_LOG

	/** 
	 * Render the component ids in this merge for this batch 
	 * 
	 * @param RDGBuilderRecorder recorder to append operations to a single FRDGBuilder, if the the recorder in "recording" mode. In "immediate" mode, a render command will be enqueued immediately
	 */
	void RenderComponentIds(UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder);

	/** 
	 * Duplicates the vertex data from the (sub-)sections of the batch, assuming GetBlendRenderTargetRead() is the RT that is read from and GetBlendRenderTargetWrite() the one that is written to 
	 * 
	 * @param RDGBuilderRecorder recorder to append operations to a single FRDGBuilder, if the the recorder in "recording" mode. In "immediate" mode, a render command will be enqueued immediately
	 */
	void RenderExpandedRenderTarget(UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder);

	/** 
	 * Performs a generic blend assuming GetBlendRenderTargetWrite() is the RT that contains the layer to blend and GetBlendRenderTargetRead() the one that contains the result of the merge up until this layer.
	 *  It will cycle the RTs such as after this call, GetBlendRenderTargetWrite() will contain the merge result
	 * 
	 * @param InBlendParams defines how heightmaps/weightmaps should be blended. In the case of weightmaps, there must be as may entries in WeightmapBlendParams as there are layers in the target layers group
	 * @param RDGBuilderRecorder recorder to append operations to a single FRDGBuilder, if the the recorder in "recording" mode. In "immediate" mode, a render command will be enqueued immediately
	 */
	LANDSCAPE_API void GenericBlendLayer(const FBlendParams& InBlendParams, FRenderParams& RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder);

	const TBitArray<>& GetFinalTargetLayerBitIndices() const 
	{ 
		return FinalTargetLayerBitIndices; 
	}

	/** 
	 * @return true if there is at least one blackboard item of type T in the context's list 
	 */
	template <typename T, typename = std::enable_if_t<std::is_base_of_v<FLandscapeEditLayerMergeRenderBlackboardItem, std::decay_t<T>>>>
	bool HasBlackboardItem() const;

	/** 
	 * Create a new blackboard item of type T and adds it to the context's list
	 * 
	 * @return the newly-created blackboard item
	 */
	template <typename T, typename... TArgs, typename = std::enable_if_t<std::is_base_of_v<FLandscapeEditLayerMergeRenderBlackboardItem, std::decay_t<T>>>>
	T& AddBlackboardItem(TArgs&&... InArgs);

	/**
	 * @return the first blackboard item of type T from the context's list or nullptr if there isn't one
	 */
	template <typename T, typename = std::enable_if_t<std::is_base_of_v<FLandscapeEditLayerMergeRenderBlackboardItem, std::decay_t<T>>>>
	T* TryGetBlackboardItem();

	/**
	 * @return the first blackboard item of type T from the context's list. Asserts if there isn't one
	 */
	template <typename T, typename = std::enable_if_t<std::is_base_of_v<FLandscapeEditLayerMergeRenderBlackboardItem, std::decay_t<T>>>>
	T& GetBlackboardItem();

	/**
	 * @return the first blackboard item of type T from the context's list or create a new one if there isn't one
	 */
	template <typename T, typename... TArgs, typename = std::enable_if_t<std::is_base_of_v<FLandscapeEditLayerMergeRenderBlackboardItem, std::decay_t<T>>>>
	T& GetOrCreateBlackboardItem(TArgs&&... InArgs);

	/**
	 * @return all blackboard items of type T from the context's list
	 */
	template <typename T, typename = std::enable_if_t<std::is_base_of_v<FLandscapeEditLayerMergeRenderBlackboardItem, std::decay_t<T>>>>
	TArray<T*> GetBlackboardItems();

	/**
	 * @return all blackboard items currently stored in the context
	 */
	const TArray<TInstancedStruct<FLandscapeEditLayerMergeRenderBlackboardItem>>& GetAllBlackboardItems() const { return BlackboardItems; }

private:
	/** Allocate all needed render targets for this merge */
	void AllocateResources();
	/** Free all render targets used in this merge */
	void FreeResources();
	/** Allocate all needed render targets for this batch */
	void AllocateBatchResources(const FMergeRenderBatch& InRenderBatch);
	/** Free all render targets used in this batch */
	void FreeBatchResources(const FMergeRenderBatch& InRenderBatch);

private:
	// Blending is pretty much what we only do during the merge. It requires 3 render targets : 1 that we write to and therefore use as RTV (Write) and 2 that we read from and therefore use 
	//  as SRV (one that contains the layer to merge, the other the accumulated result so far) : Previous(SRV) + Current(SRV) --> Write(RTV)
	static constexpr int32 NumBlendRenderTargets = 3; 

	/** Render targets that are used throughout the blending operations (they could be texture arrays in the case of multiple weightmaps) */
	TStaticArray<ULandscapeScratchRenderTarget*, NumBlendRenderTargets> BlendRenderTargets;
	int32 CurrentBlendRenderTargetWriteIndex = -1; 

	/** Final list of target layer names being involved in this merge context. If a target layer name is present here, it's because it's a valid target layer and it needs to be rendered
	  because it has been requested or one of the target layers that have been requested needs it to be present (e.g. weight-blending). Each bit in that bit array corresponds to an entry in AllTargetLayerNames */
	TBitArray<> FinalTargetLayerBitIndices;

	/** Render targets storing the id (uint32, stored as a RTF_R32f) of the component within the batch that each pixel belongs to. It can be used as a stencil-like buffer, which can be useful when sampling neighbors,
	  to know whether the data there corresponds to a valid neighbor, but also to resolve ambiguities when rendering overlapping quads (because the borders then belong to 2 components but this render target
	  has the authority as to which one of those a given pixel "belongs to") */
	ULandscapeScratchRenderTarget* ComponentIdRenderTarget = nullptr;

	/** Maximum resolution needed by a given batch in this context (means we won't ever need more than this size of a render target during the whole merge) */
	FIntPoint MaxNeededResolution = FIntPoint(ForceInit);

	/** Maximum number of slices needed by a given batch / target layer group in this context (means we won't ever need more than this number of slices in a given render target texture array during the whole merge) */
	int32 MaxNeededNumSlices = 0;

	/** Maximum number of components needed by a given batch in this context (means we won't ever render more than this amount of components during the whole merge) */
	FIntPoint MaxNumComponents = FIntPoint(ForceInit);

	/** Successive batches of components being processed by this context. Each batch should be self-contained so that we won't ever need to keep more than one in memory (VRAM)
	  in order to compute the info we need (heightmap/weightmaps), i.e. nothing should ever depend on 2 different batches. */
	TArray<FMergeRenderBatch> RenderBatches;

	/** Current batch being rendered */
	int32 CurrentRenderBatchIndex = INDEX_NONE;

	/** Offset for visual debugging */
	FVector CurrentVisualLogOffset = FVector(ForceInitToZero);

	/** Maximum height of all components to render in local space. */
	double MaxLocalHeight = DBL_MIN;

	/** List of components involved in this merge and the target layers they're writing to (each bit corresponds to the target layer name in AllTargetLayerNames) */
	TMap<ULandscapeComponent*, TBitArray<>> ComponentToTargetLayerBitIndices;

	/** Reverse lookup of ComponentToTargetLayerBitIndices : one entry per element in ComponentToTargetLayerBitIndices, each entry containing all of the components involved in this merge for this target layer */
	TArray<TSet<ULandscapeComponent*>> TargetLayersToComponents;

	/** Generic data struct to store some data to pass around throughout the context's lifetime. Basically, this is a mini-RTTI system that e.g. allows to store/pass around specific data between renderers in a generic way */
	TArray<TInstancedStruct<FLandscapeEditLayerMergeRenderBlackboardItem>> BlackboardItems;
};

#endif // WITH_EDITOR

} //namespace UE::Landscape::EditLayers

#include "LandscapeEditLayerMergeRenderContext.inl"
