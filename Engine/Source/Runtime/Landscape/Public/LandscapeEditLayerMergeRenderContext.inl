// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "StructUtils/InstancedStruct.h"
#include "LandscapeEditLayerMergeRenderBlackboardItem.h"

namespace UE::Landscape::EditLayers
{

#if WITH_EDITOR

template <typename T, typename>
bool UE::Landscape::EditLayers::FMergeRenderContext::HasBlackboardItem() const
{
	for (const TInstancedStruct<FLandscapeEditLayerMergeRenderBlackboardItem>& Item : BlackboardItems)
	{
		if (const T* TypedItem = Item.template GetPtr<T>())
		{
			return true;
		}
	}
	return false;
}

template <typename T, typename... TArgs, typename>
T& UE::Landscape::EditLayers::FMergeRenderContext::AddBlackboardItem(TArgs&&... InArgs)
{
	T* Item = BlackboardItems.Add_GetRef(TInstancedStruct<T>::Make(Forward<TArgs>(InArgs)...)).template GetMutablePtr<T>();
	return *Item;
}

template <typename T, typename>
T* UE::Landscape::EditLayers::FMergeRenderContext::TryGetBlackboardItem()
{
	for (TInstancedStruct<FLandscapeEditLayerMergeRenderBlackboardItem>& Item : BlackboardItems)
	{
		if (T* TypedItem = Item.template GetMutablePtr<T>())
		{
			return TypedItem;
		}
	}
	return nullptr;
}

template <typename T, typename>
T& UE::Landscape::EditLayers::FMergeRenderContext::GetBlackboardItem()
{
	for (TInstancedStruct<FLandscapeEditLayerMergeRenderBlackboardItem>& Item : BlackboardItems)
	{
		if (T* TypedItem = Item.template GetMutablePtr<T>())
		{
			return *TypedItem;
		}
	}
	check(false);

	// this is absolutely wrong, but we shouldn't ever reach that line and we need to return something anyway : 
	static FLandscapeEditLayerMergeRenderBlackboardItem DummyItem;
	return *reinterpret_cast<T*>(&DummyItem);
}

template <typename T, typename... TArgs, typename>
T& UE::Landscape::EditLayers::FMergeRenderContext::GetOrCreateBlackboardItem(TArgs&&... InArgs)
{
	T* Item = TryGetBlackboardItem<T>();
	if (Item == nullptr)
	{
		Item = BlackboardItems.Add_GetRef(TInstancedStruct<T>::Make(Forward<TArgs>(InArgs)...)).template GetMutablePtr<T>();
	}
	check(Item != nullptr);
	return *Item;
}

template <typename T, typename>
TArray<T*> UE::Landscape::EditLayers::FMergeRenderContext::GetBlackboardItems()
{
	TArray<T*> Items;
	for (TInstancedStruct<FLandscapeEditLayerMergeRenderBlackboardItem>& Item : BlackboardItems)
	{
		if (T* TypedItem = Item.template GetMutablePtr<T>())
		{
			Items.Add(TypedItem);
		}
	}
	return Items;
}

#endif // WITH_EDITOR

} //namespace UE::Landscape::EditLayers