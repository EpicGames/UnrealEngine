// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LandscapeEditTypes.h"
#include "LandscapeUtils.h"
#include "LandscapeEditLayerRendererState.h"
#include "LandscapeEditLayerTargetTypeState.h"
#include "LandscapeEditLayerTypes.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "LandscapeEditLayerMergeRenderContext.h"
#include "LandscapeEditLayerMergeRenderBlackboardItem.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6

#include "LandscapeEditLayerRenderer.generated.h"

// ----------------------------------------------------------------------------------
// Forward declarations

class ULandscapeLayerInfoObject;

namespace UE::Landscape
{
	class FRDGBuilderRecorder;
} // namespace UE::Landscape

namespace UE::Landscape::EditLayers
{
	struct FComponentMergeRenderInfo;
	struct FRenderParams;
	class FMergeRenderContext;
	class FTargetLayerGroup;
} // namespace UE::Landscape::EditLayers


// ----------------------------------------------------------------------------------

namespace UE::Landscape::EditLayers
{

#if WITH_EDITOR

// ----------------------------------------------------------------------------------

/** A simple world space Object-Oriented Bounding Box */
// TODO [jonathan.bard] : use FOrientedBox2d instead?
struct FOOBox2D
{
	FOOBox2D() = default;
	FOOBox2D(const FTransform& InTransform, const FVector2D& InExtents)
		: Transform(InTransform)
		, Extents(InExtents)
	{}

	FTransform Transform;
	FVector2D Extents = FVector2D(ForceInit);

	FBox BuildAABB() const;
};


// ----------------------------------------------------------------------------------

/** 
 * Describes the input area needed for a given edit layer renderer's render item: this allows to infer the dependency between each component being rendered and the components it depends on
 */
class FInputWorldArea
{
public:
	enum class EType
	{
		LocalComponent, // Designates any landscape component (i.e. the input area corresponds to the component being requested), with an optional number of neighboring components around it
		SpecificComponent, // Designates a specific landscape component (based on its component key), with an optional number of neighboring components around it
		OOBox, // Designates a fixed world area (an object-oriented box)
		Infinite, // Designates the entire loaded landscape area
	};

	static FInputWorldArea CreateInfinite() { return FInputWorldArea(EType::Infinite); }
	static FInputWorldArea CreateLocalComponent(const FIntRect& InLocalArea = FIntRect()) { return FInputWorldArea(EType::LocalComponent, FIntPoint(ForceInit), InLocalArea); }
	static FInputWorldArea CreateSpecificComponent(const FIntPoint& InComponentKey, const FIntRect& InLocalArea = FIntRect()) { return FInputWorldArea(EType::SpecificComponent, InComponentKey, InLocalArea); }
	static FInputWorldArea CreateOOBox(const FOOBox2D& InOOBox) { return FInputWorldArea(EType::OOBox, FIntPoint(ForceInit), FIntRect(), InOOBox); }

	EType GetType() const { return Type; }
	/** In the EType::LocalComponent case, returns the component's coordinates and the local area around it (inclusive bounds) */
	FIntRect GetLocalComponentKeys(const FIntPoint& InComponentKey) const;
	/** In the EType::SpecificComponent case, returns the component's coordinates and the local area around it (inclusive bounds) */
	FIntRect GetSpecificComponentKeys() const;
	/** In the EType::OOBox case, returns the OOBox */
	const FOOBox2D& GetOOBox() const { check(Type == EType::OOBox); return OOBox2D; }
	const FOOBox2D* TryGetOOBox() const { return (Type == EType::OOBox) ? &OOBox2D : nullptr; }

	FBox ComputeWorldAreaAABB(const FTransform& InLandscapeTransform, const FBox& InLandscapeLocalBounds, const FTransform& InComponentTransform, const FBox& InComponentLocalBounds) const;
	FOOBox2D ComputeWorldAreaOOBB(const FTransform& InLandscapeTransform, const FBox& InLandscapeLocalBounds, const FTransform& InComponentTransform, const FBox& InComponentLocalBounds) const;

private: 
	FInputWorldArea(EType InType, const FIntPoint& InComponentKey = FIntPoint(ForceInit), const FIntRect& InLocalArea = FIntRect(), const FOOBox2D& InOOBox2D = FOOBox2D())
		: Type(InType)
		, SpecificComponentKey(InComponentKey)
		, LocalArea(InLocalArea)
		, OOBox2D(InOOBox2D)
	{}

private:
	EType Type = EType::LocalComponent;
	/** Coordinates of the component (see ULandscapeComponent::GetComponentKey()) in the EType::SpecificComponent case */
	FIntPoint SpecificComponentKey = FIntPoint(ForceInit);
	/** Area around the component that is needed in the EType::LocalComponent / EType::SpecificComponent case (in component coordinates (see ULandscapeComponent::GetComponentKey()), 
	 e.g. use (-1, -1, 1, 1) for the component and its immediate neighbors all around) */
	FIntRect LocalArea; 

	/** World space object-oriented box in the EType::OOBBox case */
	FOOBox2D OOBox2D;
};


// ----------------------------------------------------------------------------------

/**
 * Describes the output area needed where a given edit layer renderer's render item writes: this allows to define the components of landscape that need to be processed and allows to divide
 *  the work into batches
 */
class FOutputWorldArea
{
public:
	enum class EType
	{
		LocalComponent, // Designates any landscape component (i.e. the input area corresponds to the component being requested)
		SpecificComponent, // Designates a specific landscape component (based on its component key)
		OOBox, // Designates a fixed world area (an Object-oriented box)
	};

	static FOutputWorldArea CreateLocalComponent() { return FOutputWorldArea(EType::LocalComponent); }
	static FOutputWorldArea CreateSpecificComponent(const FIntPoint& InComponentKey) { return FOutputWorldArea(EType::SpecificComponent, InComponentKey); }
	static FOutputWorldArea CreateOOBox(const FOOBox2D& InOOBox) { return FOutputWorldArea(EType::OOBox, FIntPoint(ForceInit), InOOBox); }

	EType GetType() const { return Type; }
	/** In the EType::SpecificComponent case, returns the component's coordinates */
	const FIntPoint& GetSpecificComponentKey() const { check(Type == EType::SpecificComponent); return SpecificComponentKey; }
	/** In the EType::OOBox case, returns the OOBox */
	const FOOBox2D& GetOOBox() const { check(Type == EType::OOBox); return OOBox2D; }
	const FOOBox2D* TryGetOOBox() const { return (Type == EType::OOBox) ? &OOBox2D : nullptr; }

	FBox ComputeWorldAreaAABB(const FTransform& InComponentTransform, const FBox& InComponentLocalBounds) const;
	FOOBox2D ComputeWorldAreaOOBB(const FTransform& InComponentTransform, const FBox& InComponentLocalBounds) const;

private:
	FOutputWorldArea(EType InType, const FIntPoint& InComponentKey = FIntPoint(ForceInit), const FOOBox2D& InOOBox = FOOBox2D())
		: Type(InType)
		, SpecificComponentKey(InComponentKey)
		, OOBox2D(InOOBox)
	{}

private:
	EType Type = EType::LocalComponent;
	/** Coordinates of the component (see ULandscapeComponent::GetComponentKey()) in the EType::SpecificComponent case */
	FIntPoint SpecificComponentKey = FIntPoint(ForceInit);
	/** World space Object-oriented box in the EType::OOBBox case */
	FOOBox2D OOBox2D;
};


// ----------------------------------------------------------------------------------

/** 
 * Each edit layer render item represents the capabilities of what a given edit layer can render in terms of landscape data : a renderer can provide one or many render items, which contain
 *  the "locality" (what area do I affect?) as well as the "capability" (what target tool type do I affect? what weightmap(s)?) information related to what this layer item can do. 
 *  See ILandscapeEditLayerRenderer::GetRenderItems
 */
class FEditLayerRenderItem
{
public:
	FEditLayerRenderItem() = delete;
	FEditLayerRenderItem(const FEditLayerTargetTypeState& InTargetTypeState, const FInputWorldArea& InInputWorldArea, const FOutputWorldArea& InOutputWorldArea, bool bInModifyExistingWeightmapsOnly)
		: TargetTypeState(InTargetTypeState)
		, InputWorldArea(InInputWorldArea)
		, OutputWorldArea(InOutputWorldArea)
		, bModifyExistingWeightmapsOnly(bInModifyExistingWeightmapsOnly)
	{}

	const FEditLayerTargetTypeState& GetTargetTypeState() const { return TargetTypeState; }

	const FInputWorldArea& GetInputWorldArea() const { return InputWorldArea; }
	void SetInputWorldArea(const FInputWorldArea& InInputWorldArea) { InputWorldArea = InInputWorldArea; }

	const FOutputWorldArea& GetOutputWorldArea() const { return OutputWorldArea; }
	void SetOutputWorldArea(const FOutputWorldArea& InOutputWorldArea) { OutputWorldArea = InOutputWorldArea; }

	bool GetModifyExistingWeightmapsOnly() const { return bModifyExistingWeightmapsOnly; }

private:
	
	/** Target types / weightmaps that this render item writes to */
	FEditLayerTargetTypeState TargetTypeState;

	/**
	 * Area that this render item needs in order to render properly. 
	 *  - If Infinite, it is assumed the render item needs the entire loaded landscape to render properly (i.e. it's dependent on all loaded landscape components)
	 *  - If Local, it requires a particular component and optionally its immediate neighbors
	 *  - If OOBox, then only the landscape components covered by this area will be considered as inputs
	 */ 
	FInputWorldArea InputWorldArea;

	/**
	 * Area that this render item writes to.
	 *  - If Infinite, the render item writes everywhere (use only if necessary, as it will
	 *  - If Local, it requires a particular component and optionally its immediate neighbors
	 *  - If OOBox, then only the landscape components covered by this area will be considered as inputs
	 */
	FOutputWorldArea OutputWorldArea;

	/** Indicates whether this render item actually outputs weightmaps (if false) or only modifies existing ones underneath (i.e. blending-only)*/
	bool bModifyExistingWeightmapsOnly = false;
};


// ----------------------------------------------------------------------------------

/**
 * Interface to implement to be able to provide an ordered list of renderers to the landscape.
 */
class IEditLayerRendererProvider
{
public:
	virtual ~IEditLayerRendererProvider() = default;

	/**
	 * @param InMergeContext context containing all sorts of information related to this merge operation
	 *
	 * @return a list of renderer states (i.e. a ILandscapeEditLayerRenderer and its current state) to be processed in that order by the merge operation
	 */
	virtual TArray<FEditLayerRendererState> GetEditLayerRendererStates(const FMergeContext* InMergeContext) PURE_VIRTUAL(IEditLayerRendererProvider::GetEditLayerRendererStates, return {}; );
};

#endif // WITH_EDITOR

} // namespace UE::Landscape::EditLayers


// ----------------------------------------------------------------------------------

/** 
 * UInterface for a landscape edit layer renderer 
 */
UINTERFACE()
class ULandscapeEditLayerRenderer :
	public UInterface
{
	GENERATED_BODY()

};

/** 
 * Interface that needs to be implemented for anything that can render heightmap/weightmap/visibility when merging landscape edit layers. 
 *  Ideally it would have been defined in the UE::Landscape::EditLayers namespace but UHT prevents us from doing so.
 *  The renderers are provided to the landscape by a IEditLayerRendererProvider.
 */
class ILandscapeEditLayerRenderer
{
	GENERATED_BODY()

#if WITH_EDITOR
public:
	/**
	 * GetRendererStateInfo retrieves the current state of this renderer (what it can and does render, as well as how to group target layers together), and part of this will then be mutable for the duration of the merge. 
	 *  The idea is that FEditLayerRendererState's SupportedTargetTypeState tells the capabilities of this renderer, while EnabledTargetTypeState tells what it currently does render.
	 *  A target type must be both supported and enabled in order to have this renderer affect it and the "enabled" state can be changed at will by the user (e.g. to temporarily disable 
	 *  a given edit layer just for the duration of the merge) : see FEditLayerRendererState
	 * 
	 * @param InMergeContext context containing all sorts of information related to this merge operation
	 * 
	 * @param OutSupportedTargetTypeState List of all target types / weightmaps that this renderer supports (e.g. if, say ELandscapeToolTargetType::Weightmap, is not included, the renderer
	 *  will *not* be used at all when rendering any kind of weightmap)
	 * 
	 * @param OutEnabledTargetTypeMask List of all target types / weightmaps that this renderer is currently enabled for (i.e. default state of this renderer wrt this target type).
	 *  A target type must be both supported and enabled in order to have this renderer affect it.
	 *  The "enabled" state can be changed at will by the user (e.g. to temporarily disable a given edit layer) : see FEditLayerRendererState
	 * 
	 * @param OutTargetLayerGroups List of groups of target layers that this renderer requires to be rendered together.
	 *  This allows it to perform horizontal blending (i.e.adjust the weights of the targeted weightmaps wrt one another).
	 *  Depending on the other renderer's needs, the final target layer groups might contain more layers than was requested by a given renderer. This only means that more layers will be processed together
	 *  and if this renderer doesn't act on one of these layers, it will simply do nothing with it in its RenderLayer function
	 */
	virtual void GetRendererStateInfo(const UE::Landscape::EditLayers::FMergeContext* InMergeContext,
		UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutSupportedTargetTypeState, UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutEnabledTargetTypeState, 
		TArray<UE::Landscape::EditLayers::FTargetLayerGroup>& OutTargetLayerGroups) const
		PURE_VIRTUAL(ILandscapeEditLayerRenderer::GetRendererStateInfo, );

	/**
	* @return the a debug name for this renderer
	*/
	virtual FString GetEditLayerRendererDebugName() const 
		PURE_VIRTUAL(ILandscapeEditLayerRenderer::GetEditLayerRendererDebugName, return TEXT(""); );

	/**
	 * GetRenderItems retrieves information about the areas this renderer renders to and specifically what respective input area they require to render properly
	 *
	 * @param InMergeContext context containing all sorts of information related to this merge operation
	 *
	 * @return list of all render items that affect this renderer
	*/
	virtual TArray<UE::Landscape::EditLayers::FEditLayerRenderItem> GetRenderItems(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const
		PURE_VIRTUAL(ILandscapeEditLayerRenderer::GetRenderItems, return { }; );

	/**
	 * @param InMergeContext context containing all sorts of information related to this merge operation
	 * 
	 * @return details about how this renderer's render method is implemented 
	*/
	virtual UE::Landscape::EditLayers::ERenderFlags GetRenderFlags(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const
		PURE_VIRTUAL(ILandscapeEditLayerRenderer::GetRenderFlags, return UE::Landscape::EditLayers::ERenderFlags::None; );

	/**
	 * RenderLayer is where the renderer has a chance to render its content and eventually blend it with the merged result of all preceding layers (if ERenderFlags::BlendMode_SeparateBlend is not returned)
	 *  It operates on a limited set of components (depending on the size of the render batches) and on a set of target layers (e.g. multiple weightmaps).
	 *  It guarantees access to the merged result from preceding layers of each target layer. 
	 * 
	 * @param RenderParams contains parameters necessary for this render (affected components, merge context, etc.)
	 * 
	 * @param RDGBuilderRecorder recorder to append operations to a single FRDGBuilder, if the the recorder in "recording" mode. In "immediate" mode, a render command will be enqueued immediately. 
	 *  The recorder can be in 2 states, depending on the result from GetRenderFlags
	 *  - ERenderFlags::RenderMode_Immediate : the recorder is in immediate mode renderer can enqueue render commands just like any other game thread-based renderer
	 *  - ERenderFlags::RenderMode_Recorded : the function runs on the game thread but is *not* meant to enqueue render commands directly. Instead, it registers
	 *     consecutive "render commands" via "RDG/render command" accumulated on a FRDGBuilderRecorder. This allows to coalesce several render commands onto the same FRDGBuilder (which is critical 
	 *     for performance) while still allowing to interleave game thread-based renders (flushing the command recorder, enqueuing the game thread-based render, and starting recording on the same (or a 
	 *     new) command recorder again...) 
	 *     TLDR: when using the ERenderFlags::RenderMode_Recorded method, the user should use the command recorder to enqueue lambdas instead if enqueuing render commands directly. 
	 *     Corollary: Any render command issued by RenderLayer will end up being pushed before the render operations recorded on the RDG command recorder (unless it is flushed), so there's no reason 
	 *     to actually do it, unless you want those commands to run before the recorded commands
	 * 
	 * @return true if anything was rendered
	*/
	virtual bool RenderLayer(UE::Landscape::EditLayers::FRenderParams& RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder)
		PURE_VIRTUAL(ILandscapeEditLayerRenderer::RenderLayer, return false; );

	/**
	 * BlendLayer is where the renderer has a chance to blend its content with the merged result of all preceding layers
	 * It operates on a limited set of components (depending on the size of the render batches) and on a set of target layers (e.g. multiple weightmaps).
	 * It guarantees access to the merged result from preceding layers of each target layer. 
	 * 
	 * @param RenderParams contains parameters necessary for this render (affected components, merge context, etc.)
	 * 
	 * @param RDGBuilderRecorder (see RenderLayer)
	*/
	virtual void BlendLayer(UE::Landscape::EditLayers::FRenderParams& RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder) {}

	/**
	 * Return whether this renderer's RenderLayer operation can be grouped with one of the previous renderers in the render layer group. 
	 *  Only called if ERenderFlags::RenderLayerGroup_SupportsGrouping is returned by GetRenderFlags() on both renderers
	 * 
	 * @param InOtherRenderer the other renderer we're trying to group this renderer's RenderLayer operation with
	 * 
	 * @return true if both RenderLayer can be done in the same render layer group
	*/
	virtual bool CanGroupRenderLayerWith(TScriptInterface<ILandscapeEditLayerRenderer> InOtherRenderer) const { return false; }

	/**
	 * Called before the first call to RenderLayer, on the first renderer of a render layer group : 
	 *  Only called if ERenderFlags::RenderLayerGroup_SupportsGrouping is returned by GetRenderFlags() 
	 *
	 * @param RenderParams contains parameters necessary for this render (affected components, merge context, etc.)
	 */
	virtual void BeginRenderLayerGroup(UE::Landscape::EditLayers::FRenderParams& RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder) {}

	/**
	 * Called after the last call to RenderLayer, on the last renderer of a render layer group. 
	 *  Only called if ERenderFlags::RenderLayerGroup_SupportsGrouping is returned by GetRenderFlags() 
	 *
	 * @param RenderParams contains parameters necessary for this render (affected components, merge context, etc.)
	 */
	virtual void EndRenderLayerGroup(UE::Landscape::EditLayers::FRenderParams& RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder) {}

#endif // WITH_EDITOR
};
