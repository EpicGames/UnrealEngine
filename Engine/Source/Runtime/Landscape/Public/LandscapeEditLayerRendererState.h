// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LandscapeEditTypes.h"
#include "LandscapeEditLayerTargetTypeState.h"
#include "LandscapeEditLayerTargetLayerGroup.h"
#include "UObject/ScriptInterface.h"


// ----------------------------------------------------------------------------------
// Forward declarations

class ILandscapeEditLayerRenderer;


namespace UE::Landscape::EditLayers
{

#if WITH_EDITOR

// ----------------------------------------------------------------------------------

/**
 * FEditLayerRendererState describes the entire state of an edit layer renderer : what it is capable of doing (SupportedTargetTypeState, immutable) and what it is currently doing (EnabledTargetTypeState, mutable)
 *  These states are provided by IEditLayerRendererProvider in order to describe both what the renderer can do and what it currently does by default. e.g. a disabled edit layer supports rendering heightmaps but
 *  its enabled state for the heightmap target type is false. This way, the user can selectively enable it at merge time without altering the entire landscape's state (i.e. just for the purpose of a specific merge render)
 *  A target type must be both supported and enabled on a given edit layer renderer in order for this renderer to render anything. 
 *  It also describes the target layer groups this renderer needs when rendering its weightmap (i.e. which weightmap needs to be rendered with which weightmaps : e.g. for weight blending)
 */
class FEditLayerRendererState
{
private:
	// Private constructor : either use the constructors taking a merge context in parameter or use GetDummyRendererState()
	FEditLayerRendererState() = default;

public:
	static const FEditLayerRendererState& GetDummyRendererState();

	LANDSCAPE_API explicit FEditLayerRendererState(const FMergeContext* InMergeContext);
	LANDSCAPE_API explicit FEditLayerRendererState(const FMergeContext* InMergeContext, TScriptInterface<ILandscapeEditLayerRenderer> InRenderer);

	/** Returns the edit layer renderer which this state relates to */
	TScriptInterface<ILandscapeEditLayerRenderer> GetRenderer() const { return Renderer; }

	/** Returns a mask of all target types / weightmaps supported by this renderer */
	const FEditLayerTargetTypeState& GetSupportedTargetTypeState() const { return SupportedTargetTypeState; }

	/** Returns a mask of all target types / weightmaps enabled on this renderer */
	const FEditLayerTargetTypeState& GetEnabledTargetTypeState() const { return EnabledTargetTypeState; }

	/** Returns a mask of all target types / weightmaps supported and enabled on this renderer */
	FEditLayerTargetTypeState GetActiveTargetTypeState() const { return ActiveTargetTypeState; }

	/** Mutates the EnabledTargetTypeState by adding the target type in parameter to the mask of active target types */
	LANDSCAPE_API void EnableTargetType(ELandscapeToolTargetType InTargetType);
	
	/** Mutates the EnabledTargetTypeState by adding the target type mask in parameter to the mask of active target types */
	LANDSCAPE_API void EnableTargetTypeMask(ELandscapeToolTargetTypeFlags InTargetTypeMask);

	/** Mutates the EnabledTargetTypeState by removing the target type in parameter from the mask of active target types */
	LANDSCAPE_API void DisableTargetType(ELandscapeToolTargetType InTargetType);

	/** Mutates the EnabledTargetTypeState by removing the target type mask in parameter from the mask of active target types */
	LANDSCAPE_API void DisableTargetTypeMask(ELandscapeToolTargetTypeFlags InTargetTypeMask);

	/**
	 * Mutates the EnabledTargetTypeState by adding the weightmap in parameter to the list of enabled weightmaps, if it is known to the merge context
	 */
	LANDSCAPE_API void EnableWeightmap(ELandscapeToolTargetType InTargetType, const FName& InWeightmapLayerName);

	/**
	 * Mutates the EnabledTargetTypeState by adding the weightmap in parameter to the list of enabled weightmaps. Asserts if the layer name isn't known to the merge context
	 */
	LANDSCAPE_API void EnableWeightmapChecked(ELandscapeToolTargetType InTargetType, const FName& InWeightmapLayerName);

	/** 
	 * Mutates the EnabledTargetTypeState by adding the weightmap in parameter to the list of enabled weightmaps. Asserts if the layer index isn't known to the merge context
	 */
	LANDSCAPE_API void EnableWeightmap(ELandscapeToolTargetType InTargetType, int32 InWeightmapLayerIndex);

	/**
	 * Mutates the EnabledTargetTypeState by removing the weightmap in parameter from the list of enabled weightmaps, if it is known to the merge context
	 */
	LANDSCAPE_API void DisableWeightmap(const FName& InWeightmapLayerName);

	/**
	 * Mutates the EnabledTargetTypeState by removing the weightmap in parameter from the list of enabled weightmaps. Asserts if the layer index isn't known to the merge context
	 */
	LANDSCAPE_API void DisableWeightmapChecked(const FName& InWeightmapLayerName);

	/** 
	 * Mutates the EnabledTargetTypeState by removing the weightmap in parameter from the list of enabled weightmaps. Asserts if the layer index isn't known to the merge context
	 */
	LANDSCAPE_API void DisableWeightmap(int32 InWeightmapLayerIndex);

	/** Returns the mask of target types that are both supported and enabled by this renderer */

	LANDSCAPE_API ELandscapeToolTargetTypeFlags GetActiveTargetTypeMask() const;

	/**
	 * Indicates whether a given target type and weightmap layer name is currently supported and enabled by this renderer. 
	 * @param InTargetType the requested target type (heightmap/weightmap/visibility)
	 * @param InWeightmapLayerName (optional) is the requested weightmap, only relevant for the ELandscapeToolTargetType::Weightmap case
	 */
	LANDSCAPE_API bool IsTargetActive(ELandscapeToolTargetType InTargetType, const FName& InWeightmapLayerName = NAME_None) const;

	/**
	 * Indicates whether a given target type and weightmap layer name is currently supported and enabled by this renderer. Asserts if the layer name isn't known to the merge context (except if NAME_None)
	 * @param InTargetType the requested target type (heightmap/weightmap/visibility)
	 * @param InWeightmapLayerName (optional) is the requested weightmap, only relevant for the ELandscapeToolTargetType::Weightmap case
	 */
	LANDSCAPE_API bool IsTargetActiveChecked(ELandscapeToolTargetType InTargetType, const FName& InWeightmapLayerName = NAME_None) const;

	/**
	 * Indicates whether a given target type and weightmap layer name is currently supported and enabled by this renderer. Asserts if the layer index isn't known to the merge context (except if INDEX_NONE)
	 * @param InTargetType the requested target type (heightmap/weightmap/visibility)
	 * @param InWeightmapLayerIndex (optional) is the requested weightmap's index in the merge index, only relevant for the ELandscapeToolTargetType::Weightmap case
	 */
	LANDSCAPE_API bool IsTargetActive(ELandscapeToolTargetType InTargetType, int32 InWeightmapLayerIndex = INDEX_NONE) const;

	/**
	 * Returns a list of all weightmaps supported and enabled by this renderer (only relevant for ELandscapeToolTargetType::Weightmap (and ELandscapeToolTargetType::Visibility))
	 */
	LANDSCAPE_API TArray<FName> GetActiveTargetWeightmaps() const;

	/** 
	 * Returns a list of all weightmaps supported and enabled by this renderer (only relevant for ELandscapeToolTargetType::Weightmap (and ELandscapeToolTargetType::Visibility))
	 */
	LANDSCAPE_API TBitArray<> GetActiveTargetWeightmapBitIndices() const;

	/** Returns the target layer groups associated with this renderer. A target layer group is a set of target layers (weightmaps) that depend on one another in order to produce the output target layers.
	 This allows to implement "horizontal blending", where weightmaps can be blended with one another at each step of the landscape edit layers merge algorithm */
	TArray<FTargetLayerGroup> GetTargetLayerGroups() const { return TargetLayerGroups; }

private:
	void UpdateActiveTargetTypeState();

private:
	/** Global context being used for this merge : contains generic information about the landscape, its available layer names, etc. */
	const FMergeContext* MergeContext = nullptr;

	/** Renderer associated with this state */
	TScriptInterface<ILandscapeEditLayerRenderer> Renderer;

	/** For debug purposes : this is the same as Renderer->GetEditLayerRendererDebugName() but having a member makes it easier to debug in the watch window */
	FString DebugName;
	
PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	/** Struct that contains the supported target types and the which weightmap are supported by this renderer. Immutable. */
	FEditLayerTargetTypeState SupportedTargetTypeState = FEditLayerTargetTypeState::GetDummyTargetTypeState();
	
	/** Struct that contains the enabled target types and the which weightmap are currently enabled by this renderer. Can be set by the user using EnableTargetType. */
	FEditLayerTargetTypeState EnabledTargetTypeState = FEditLayerTargetTypeState::GetDummyTargetTypeState();

	/** Intersection of SupportedTargetTypeState and EnabledTargetTypeState (defines what is both supported and enabled on this renderer) */
	FEditLayerTargetTypeState ActiveTargetTypeState = FEditLayerTargetTypeState::GetDummyTargetTypeState();
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

	/* List of groups of target layers that this renderer requires to be rendered together. All target layers listed in SupportedTargetTypeState must belong to 1 (and 1 only) target layer group.
	 *  Each target layer group is a bit array for which each bit corresponds to an entry in FMergeContext's AllTargetLayerNames
	 */
	TArray<FTargetLayerGroup> TargetLayerGroups;
};

#endif // WITH_EDITOR

} //namespace UE::Landscape::EditLayers
