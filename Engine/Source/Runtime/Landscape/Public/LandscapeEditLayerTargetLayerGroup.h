// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"


// ----------------------------------------------------------------------------------

namespace UE::Landscape::EditLayers
{

#if WITH_EDITOR

/** 
 * FTargetLayerGroup describes a group of target layers that need to be rendered together, typically for weight-blending reasons
 */
class FTargetLayerGroup
{
public:
	inline FTargetLayerGroup(const FName& InName, const TBitArray<>& InWeightmapTargetLayerBitIndices)
		: Name(InName)
		, WeightmapTargetLayerBitIndices(InWeightmapTargetLayerBitIndices)
	{}

	inline FName GetName() const 
	{ 
		return Name;
	}

	inline const TBitArray<>& GetWeightmapTargetLayerBitIndices() const 
	{
		return WeightmapTargetLayerBitIndices;
	}

private:
	/** 
	 * Identifier for this group of target layers
	 */
	FName Name;

	/**
	 * List of weightmaps that belong to this group
	 *  Each bit in that bit array corresponds to an entry in FMergeContext's AllTargetLayerNames
	 */
	TBitArray<> WeightmapTargetLayerBitIndices;
};

#endif // WITH_EDITOR

} //namespace UE::Landscape::EditLayers
