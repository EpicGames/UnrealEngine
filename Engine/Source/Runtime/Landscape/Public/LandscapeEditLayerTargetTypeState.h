// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LandscapeEditTypes.h"


// ----------------------------------------------------------------------------------
// Forward declarations

class ILandscapeEditLayerRenderer;

namespace UE::Landscape::EditLayers
{
	class FMergeContext;
	class FEditLayerRendererState;
}
// namespace UE::Landscape::EditLayers


// ----------------------------------------------------------------------------------

namespace UE::Landscape::EditLayers
{

#if WITH_EDITOR

/** 
 * FEditLayerTargetTypeState fully describes the state of an edit layer renderer wrt its target types. It's named after the enum "ELandscapeToolTargetType" in order to tell
 *  whether the renderer's heightmaps and/or visibility and/or weightmaps are enabled (and if so, which weightmap is enabled exactly)  
 *  It is meant to be provided by the edit layer renderer's GetRendererStateInfo implementation.
 */
class FEditLayerTargetTypeState
{
	friend ILandscapeEditLayerRenderer;
	friend FEditLayerRendererState;

private:
	// Private constructor : either use the constructors taking a merge context in parameter or use GetDummyTargetTypeState()
	FEditLayerTargetTypeState() = default;

public:
	static const FEditLayerTargetTypeState& GetDummyTargetTypeState();

	/**
	 * Constructs an empty FEditLayerTargetTypeState (all target types turned off, no active weightmaps)
	 * 
	 * @param InMergeContext context containing all sorts of information related to this merge operation
	 */
	LANDSCAPE_API FEditLayerTargetTypeState(const FMergeContext* InMergeContext);

	/**
	 * Initializes a FEditLayerTargetTypeState with the target types passed in parameter 
	 * 
	 * @param InMergeContext context containing all sorts of information related to this merge operation
	 * @param InTargetTypeMask target type mask supported by this target type state (no active weightmaps)
	 */
	LANDSCAPE_API FEditLayerTargetTypeState(const FMergeContext* InMergeContext, ELandscapeToolTargetTypeFlags InTargetTypeMask);

	/**
	 * Initializes a FEditLayerTargetTypeState with the target types and the weightmaps passed in parameter (by name)
	 * 
	 * @param InMergeContext context containing all sorts of information related to this merge operation
	 * @param InTargetTypeMask target type mask supported by this target type state
	 * @param InSupportedWeightmaps names of weightmaps supported by this target type state 
	 * @param bInChecked, if true, will validate that every weightmap requested is part of the merge context (if false, only the known weightmaps will be marked as supported)
	 */
	LANDSCAPE_API FEditLayerTargetTypeState(const FMergeContext* InMergeContext, ELandscapeToolTargetTypeFlags InTargetTypeMask, const TArrayView<const FName>& InSupportedWeightmaps, bool bInChecked);

	/**
	 * Initializes a FEditLayerTargetTypeState with the target types and (optionally) the weightmaps passed in parameter (by bit index)
	 * 
	 * @param InMergeContext context containing all sorts of information related to this merge operation
	 * @param InTargetTypeMask target type mask supported by this target type state
	 * @param InSupportedWeightmaps bit indices of the weightmaps supported by this target type state (corresponds to the FMergeContext's AllTargetLayerNames)
	 * Asserts if one of the layer indices isn't valid for the merge context (except when InSupportedWeightmapLayerIndices is == TBitArray<>)
	 */
	LANDSCAPE_API FEditLayerTargetTypeState(const FMergeContext* InMergeContext, ELandscapeToolTargetTypeFlags InTargetTypeMask, const TBitArray<>& InSupportedWeightmapLayerIndices);
	
	/**
	 * Indicates whether a given target type is currently active in this state
	 * @param InTargetType the requested target type (heightmap/weightmap/visibility)
	 * @param InWeightmapLayerName (optional) is the requested weightmap, only relevant for the ELandscapeToolTargetType::Weightmap case
	 */
	LANDSCAPE_API bool IsActive(ELandscapeToolTargetType InTargetType, const FName& InWeightmapLayerName = NAME_None) const;

		/**
	 * Indicates whether a given target type is currently active in this state
	 * @param InTargetType the requested target type (heightmap/weightmap/visibility)
	 * @param InWeightmapLayerName (optional) is the requested weightmap, only relevant for the ELandscapeToolTargetType::Weightmap case. Asserts if the layer name isn't valid for the merge context (except for NAME_None)
	 */
	LANDSCAPE_API bool IsActiveChecked(ELandscapeToolTargetType InTargetType, const FName& InWeightmapLayerName = NAME_None) const;

	/**
	 * Indicates whether a given target type is currently active in this state
	 * @param InTargetType is the requested target type (heightmap/weightmap/visibility)
	 * @param int32 (optional) is the requested weightmap index (in FMergeContex), only relevant for the ELandscapeToolTargetType::Weightmap case. Asserts if the layer index isn't valid for the merge context (except for INDEX_NONE)
	 */
	LANDSCAPE_API bool IsActive(ELandscapeToolTargetType InTargetType, int32 InWeightmapLayerIndex = INDEX_NONE) const;

	/** Returns the currently active weightmaps : 
	 *  - If Weightmap is amongst the target types it will return all the weightmaps
	 *  - Additionally, if Visibility is amongst the target types, it will also return the visibility weightmap
	 *  - If neither Weightmap nor Visibility is amongst the target types, it will return an empty array
	 */
	LANDSCAPE_API TArray<FName> GetActiveWeightmaps() const;

	/** Returns the currently active weightmaps :
	 *  - If Weightmap is amongst the target types it will return all the weightmaps
	 *  - Additionally, if Visibility is amongst the target types, it will also return the visibility weightmap
	 *  - If neither Weightmap nor Visibility is amongst the target types, it will return an empty array
	 */
	LANDSCAPE_API TBitArray<> GetActiveWeightmapBitIndices() const;

	/** Returns the target type mask (i.e. same as ELandscapeToolTargetType, but as bit flags) */
	ELandscapeToolTargetTypeFlags GetTargetTypeMask() const { return TargetTypeMask; }

	/** Sets the target type mask (i.e. same as ELandscapeToolTargetType, but as bit flags) */
	LANDSCAPE_API void SetTargetTypeMask(ELandscapeToolTargetTypeFlags InTargetTypeMask);

	/** Adds the target type in parameter to the mask of active target types */
	LANDSCAPE_API void AddTargetType(ELandscapeToolTargetType InTargetType);

	/** Appends the target type mask in parameter to the mask of active target types */
	LANDSCAPE_API void AddTargetTypeMask(ELandscapeToolTargetTypeFlags InTargetTypeMask);

	/** Removes a single target type from the mask of active target types */
	LANDSCAPE_API void RemoveTargetType(ELandscapeToolTargetType InTargetType);

	/** Removes the target type mask in parameter from the mask of active target types */
	LANDSCAPE_API void RemoveTargetTypeMask(ELandscapeToolTargetTypeFlags InTargetTypeMask);

	/** Adds a weightmap to the list of active weightmaps (make sure ELandscapeToolTargetType::Weightmap is amongst the supported target types) */
	LANDSCAPE_API void AddWeightmap(const FName& InWeightmapLayerName);

	/** Adds a weightmap to the list of active weightmaps (make sure ELandscapeToolTargetType::Weightmap is amongst the supported target types). Asserts if the layer name isn't known to the merge context.  */
	LANDSCAPE_API void AddWeightmapChecked(const FName& InWeightmapLayerName);

	/** Adds a weightmap to the list of active weightmaps (make sure ELandscapeToolTargetType::Weightmap is amongst the supported target types). Asserts if the layer index isn't valid for the merge context */
	LANDSCAPE_API void AddWeightmap(int32 InWeightmapLayerIndex);

	/** Removes a weightmap from the list of active weightmaps */
	LANDSCAPE_API void RemoveWeightmap(const FName& InWeightmapLayerName);

	/** Removes a weightmap from the list of active weightmaps. Asserts if the layer name isn't known to the merge context. */
	LANDSCAPE_API void RemoveWeightmapChecked(const FName& InWeightmapLayerName);

	/** Removes a weightmap from the list of active weightmaps. Asserts if the layer index isn't valid for the merge context. */
	LANDSCAPE_API void RemoveWeightmap(int32 InWeightmapLayerIndex);

	/**
	 * Returns the "intersection" (AND operation) between the target type state and the one in parameter. e.g. if state S0 is:
	 *  - (---------|Visibility|Weightmap) with active weightmaps (A|B|C|-), and state S1 is:
	 *  - (Heightmap|----------|Weightmap) with active weightmaps (-|-|C|D), then state S0.Intersect(S1) is:
	 *  - (---------|----------|Weightmap) with active weightmaps (-|-|C|-)
	 */
	FEditLayerTargetTypeState Intersect(const FEditLayerTargetTypeState& InOther) const;

	bool operator == (const FEditLayerTargetTypeState& InOther) const;

	FString ToString() const;

private:
	/** Global context being used for this merge : contains generic information about the landscape, its available layer names, etc. */
	const FMergeContext* MergeContext = nullptr;

	/** Bitmask of the target types that are supported  */
	ELandscapeToolTargetTypeFlags TargetTypeMask = ELandscapeToolTargetTypeFlags::None;
	
	/** 
	 * List of weightmaps that are supported for the ELandscapeToolTargetType::Weightmap/ELandscapeToolTargetType::Visibility type. 
	 *  Each bit in that bit array corresponds to an entry in FMergeContext's AllTargetLayerNames
	 */
	TBitArray<> WeightmapTargetLayerBitIndices;
};
#endif // WITH_EDITOR

} //namespace UE::Landscape::EditLayers
