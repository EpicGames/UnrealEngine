// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// ----------------------------------------------------------------------------------

namespace UE::Landscape::EditLayers
{

// ----------------------------------------------------------------------------------

// Must match EHeightmapBlendMode in LandscapeEditLayersHeightmaps.usf
enum class EHeightmapBlendMode : uint32
{
	Additive = 0,
	LegacyAlphaBlend, // In legacy alpha blend, the layer stores the height with premultiplied alpha (legacy landscape splines)
	AlphaBlend,

	Num,
};


// ----------------------------------------------------------------------------------

// Must match EHeightmapAlphaFlags in LandscapeCommon.ush
enum class EHeightmapAlphaFlags : uint8
{
	None = 0, // aka Additive
	Additive = 0, // The height is considered to be an offset (positive or negative)
	Min = (1 << 0), // Only lower the existing landscape values
	Max = (1 << 1), // Only raise the existing landscape values
	AlphaBlend = (Min | Max), // Raise or lower the existing landscape values
};
ENUM_CLASS_FLAGS(EHeightmapAlphaFlags);


// ----------------------------------------------------------------------------------

// Must match EWeightmapBlendMode in LandscapeEditLayersWeightmaps.usf
enum class EWeightmapBlendMode : uint32
{
	None = 0, // aka Additive
	Additive = 0,
	Subtractive,
	Passthrough,
	AlphaBlend, 

	Num,
};


// ----------------------------------------------------------------------------------

// Must match EWeightmapAlphaFlags in LandscapeCommon.ush
enum class EWeightmapAlphaFlags : uint8
{
	None = 0, // aka Additive
	Additive = 0, // The weight is considered to be an offset (positive)
	Min = (1 << 0), // Only retain the min between the weight and the existing landscape weight value
	Max = (1 << 1), // Only retain the max between the weight and the existing landscape weight value
	AlphaBlend = (Min | Max), // Full alpha blending of the weight against the existing landscape weight value
};
ENUM_CLASS_FLAGS(EWeightmapAlphaFlags);


// ----------------------------------------------------------------------------------

// Must match EWeightmapTargetLayerFlags in LandscapeEditLayersWeightmaps.usf
enum class EWeightmapTargetLayerFlags : uint32
{
	IsVisibilityLayer = (1 << 0), // This target layer is the visibility layer
	IsWeightBlended = (1 << 1), // Blend the target layer's value with all the other target layers weights
	Skip = (1 << 2), // This layer should be skipped from blending because it has not been rendered in this batch
	IsPremultipliedAlphaWeightBlended = (1 << 3), // Blend the target layer's weight with other target layers' weights of the same target layer blend group, using the premultiplied alpha blend formula

	None = 0
};
ENUM_CLASS_FLAGS(EWeightmapTargetLayerFlags);


// ----------------------------------------------------------------------------------

// Must match FFinalWeightBlendingTargetLayerInfo in LandscapeEditLayersWeightmaps.usf
struct FFinalWeightBlendingTargetLayerInfo
{
	EWeightmapTargetLayerFlags Flags = EWeightmapTargetLayerFlags::None; // Additional info about this target layer
};


// ----------------------------------------------------------------------------------

// Must match FMergeEditLayerTargetLayerInfo in LandscapeEditLayersWeightmaps.usf
struct FMergeEditLayerTargetLayerInfo
{
	EWeightmapTargetLayerFlags Flags = EWeightmapTargetLayerFlags::None; // Additional info about this target layer
	int32 BlendGroupIndex = INDEX_NONE; // Defines the target layer blend group that this target layer belongs to in this blend operation. -1 if not applicable
};


// ----------------------------------------------------------------------------------

// Defines how heightmaps should be blended (see GenericBlendLayer)
struct FHeightmapBlendParams
{
	FHeightmapBlendParams() = default;
	FHeightmapBlendParams(EHeightmapBlendMode InBlendMode)
		: BlendMode(InBlendMode)
	{}

	EHeightmapBlendMode BlendMode = EHeightmapBlendMode::Additive;
	float Alpha = 1.0f;
};


// ----------------------------------------------------------------------------------

// Defines how weightmaps should be blended (see GenericBlendLayer)
struct FWeightmapBlendParams
{
	FWeightmapBlendParams() = default;
	FWeightmapBlendParams(EWeightmapBlendMode InBlendMode)
		: BlendMode(InBlendMode)
	{}

	static const FWeightmapBlendParams& GetDefaultPassthroughBlendParams()
	{
		static const FWeightmapBlendParams BlendParams(EWeightmapBlendMode::Passthrough);
		return BlendParams;
	}

	static const FWeightmapBlendParams& GetDefaultAdditiveBlendParams()
	{
		static const FWeightmapBlendParams BlendParams(EWeightmapBlendMode::Additive);
		return BlendParams;
	}

	EWeightmapBlendMode BlendMode = EWeightmapBlendMode::Passthrough;
	float Alpha = 1.0f;
};


// ----------------------------------------------------------------------------------

// Defines heightmaps+weightmaps blending params (see GenericBlendLayer)
//  There should be as many blend params as there are target layers to be blended in the blend operation, the others are simply passthrough
struct FBlendParams
{
	FHeightmapBlendParams HeightmapBlendParams;
	TMap<FName, FWeightmapBlendParams> WeightmapBlendParams;
};


#if WITH_EDITOR

// ----------------------------------------------------------------------------------

/** Flags that allow a given ILandscapeEditLayerRenderer to customize the way it renders/blends */
enum class ERenderFlags : uint8
{
	None = 0,

	// Render mode flags
	RenderMode_Recorded = (1 << 0), // This renderer can record its render commands into a single FRDGBuilder on the render thread (prefer this if possible), via a FRDGBuilderRecorder in "recording" mode. Exclusive with RenderMode_Immediate
	RenderMode_Immediate = (1 << 1), // This renderer enqueues its render commands immediately either via the FRDGBuilderRecorder in "immediate" mode or just enqueuing render commands the usual way. Exclusive with RenderMode_Recorded
	RenderMode_Mask = RenderMode_Recorded | RenderMode_Immediate,

	// Blend mode flags
	BlendMode_SeparateBlend = (1 << 2), // This renderer has a separate render function for blending. When this flag is not set, only RenderLayer is called and is assumed to both render the layer and blend it. When it's set, RenderLayer will be followed by BlendLayer.

	// Render layer group flags
	RenderLayerGroup_SupportsGrouping = (1 << 3), // This renderer supports being rendered along with others in a series of RenderLayer steps before performing a single BlendLayer. Assumes BlendMode_SeparateBlend
};
ENUM_CLASS_FLAGS(ERenderFlags);

#endif // WITH_EDITOR

} //namespace UE::Landscape::EditLayers