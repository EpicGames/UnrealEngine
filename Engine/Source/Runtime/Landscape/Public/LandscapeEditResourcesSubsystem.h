// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/EngineSubsystem.h"
#include "Engine/TextureRenderTarget2D.h"
#include "LandscapeEditTypes.h"
#include "RHIAccess.h"
#include "RHITransition.h"

#if WITH_EDITOR
#include "Materials/Material.h"
#include "LandscapeUtils.h"
#endif // WITH_EDITOR

#include "LandscapeEditResourcesSubsystem.generated.h"

struct FResourceSizeEx;
class UTextureRenderTarget2D;
class UTextureRenderTarget2DArray;
class FRDGBuilder;
class FRHICommandListImmediate;

namespace UE::Landscape 
{
class FRDGBuilderRecorder;

struct FScratchRenderTargetParams
{
	FScratchRenderTargetParams() = default;
	explicit FScratchRenderTargetParams(const FString& InDebugName, bool bInExactDimensions, bool bInUseUAV, bool bInTargetArraySlicesIndependently, const FIntPoint& InResolution, int32 InNumSlices, ETextureRenderTargetFormat InFormat, const FLinearColor& InClearColor, ERHIAccess InInitialState)
		: DebugName(InDebugName)
		, bExactDimensions(bInExactDimensions)
		, bUseUAV(bInUseUAV)
		, bTargetArraySlicesIndependently(bInTargetArraySlicesIndependently)
		, Resolution(InResolution)
		, NumSlices(InNumSlices)
		, Format(InFormat)
		, ClearColor(InClearColor)
		, InitialState(InInitialState)
	{}

	/** Allows to specify a friendly name for this render target (for the duration that the scratch render target is in use : a new name will be used the next time it's recycled) */
	FString DebugName;
	/** Use this to make sure the returned render target has the exact dimension as was requested (if false, a larger RT might be recycled) */
	bool bExactDimensions = false;
	/** Requires the render target to have the ETextureCreateFlags::UAV flag */
	bool bUseUAV = false;
	/** Requires the render target to have the ETextureCreateFlags::TargetArraySlicesIndependently flag */
	bool bTargetArraySlicesIndependently = false;
	/** Requested (minimal) resolution for this render target */
	FIntPoint Resolution = FIntPoint(ForceInit);
	/** Number of slices requested for this render target. 0 means the render target will be a UTextureRenderTarget2D, > 0 means it will be a UTextureRenderTarget2DArray */
	int32 NumSlices = 0; 
	/** Format of the requested render target */
	ETextureRenderTargetFormat Format = ETextureRenderTargetFormat::RTF_RGBA8;
	/** Default clear color of the requested render target */
	FLinearColor ClearColor = FLinearColor(ForceInitToZero);
	/** (optional) State the render target should be in when it's being requested */
	ERHIAccess InitialState = ERHIAccess::None;
};

struct FScratchRenderTargetScope : public FNoncopyable
{
	FScratchRenderTargetScope() = delete;
	FScratchRenderTargetScope(const FScratchRenderTargetParams& InParams);
	~FScratchRenderTargetScope();

	ULandscapeEditResourcesSubsystem* EditResourcesSubsystem = nullptr;
	ULandscapeScratchRenderTarget* RenderTarget = nullptr;
};

} // namespace UE::Landscape

/** 
* ULandscapeScratchRenderTarget holds a UTextureRenderTarget2D. It can be used in the landscape tools as transient memory by requesting/releasing an instance via ULandscapeEditResourcesSubsystem
*  It contains information about the current state (ERHIAccess) of the resource in order to automate/minimize state transitions.
*  In order to minimize memory consumption, the internal render target can be larger than what was requested. It's therefore important to take that into account when setting up draw calls on this
*  render target and use GetEffectiveResolution() instead of the RT's resolution
*/
UCLASS(Transient)
class ULandscapeScratchRenderTarget : public UObject
{
	GENERATED_BODY()

	friend class ULandscapeEditResourcesSubsystem;
	friend class FTransitionBatcherScope;

public:
	ULandscapeScratchRenderTarget();

	/** @return a debug name while this render target is in use. This helps track things down, since the underlying resource can be recycled */
	LANDSCAPE_API const FString& GetDebugName() const;
	/** @return Getter for the internal UTextureRenderTarget */
	UTextureRenderTarget* GetRenderTarget() const { return RenderTarget; }
	/** @return Getter for the internal UTextureRenderTarget2D (only when CurrentRenderTargetParams.NumSlices == 0, asserts if CurrentRenderTargetParams.NumSlices > 0) */
	LANDSCAPE_API UTextureRenderTarget2D* GetRenderTarget2D() const;
	/** @return the internal UTextureRenderTarget2D (only when CurrentRenderTargetParams.NumSlices == 0, returns nullptr if CurrentRenderTargetParams.NumSlices > 0) */
	LANDSCAPE_API UTextureRenderTarget2D* TryGetRenderTarget2D() const;
	/**  @return the internal UTextureRenderTarget2DArray (only when CurrentRenderTargetParams.NumSlices > 0, asserts if CurrentRenderTargetParams.NumSlices == 0)  */
	LANDSCAPE_API UTextureRenderTarget2DArray* GetRenderTarget2DArray() const;
	/** @return the internal UTextureRenderTarget2DArray (only when CurrentRenderTargetParams.NumSlices > 0, returns nullptr if CurrentRenderTargetParams.NumSlices == 0) */
	LANDSCAPE_API UTextureRenderTarget2DArray* TryGetRenderTarget2DArray() const;
	/** @return the internal render target's true resolution (can be different than the effective resolution if CurrentRenderTargetParams.bExactDimensions is false) */
	LANDSCAPE_API FIntPoint GetResolution() const;
	/**@return the internal render target's effective resolution (can be different than the actual resolution if CurrentRenderTargetParams.bExactDimensions is false) while this render target is in use */
	LANDSCAPE_API FIntPoint GetEffectiveResolution() const;
	/** @return the internal render target's number of slices (can be different than the actual number of slices if CurrentRenderTargetParams.bExactDimensions is false) */
	LANDSCAPE_API int32 GetNumSlices() const;
	/** @return the internal render target's effective number of slices (can be different than the actual number of slices if CurrentRenderTargetParams.bExactDimensions is false) while this render target is in use */
	LANDSCAPE_API int32 GetEffectiveNumSlices() const;
	bool IsTexture2D() const { return (TryGetRenderTarget2D() != nullptr); }
	bool IsTexture2DArray() const { return (TryGetRenderTarget2DArray() != nullptr); }
	LANDSCAPE_API ETextureRenderTargetFormat GetFormat() const;
	LANDSCAPE_API FLinearColor GetClearColor() const;
	ERHIAccess GetCurrentState() const { return CurrentState; }
	const UE::Landscape::FScratchRenderTargetParams& GetCurrentRenderTargetParams() { return CurrentRenderTargetParams; }

	class FTransitionInfo
	{
	public:
		FTransitionInfo() = default;
		FTransitionInfo(FTextureResource* InResource, ERHIAccess InStateBefore, ERHIAccess InStateAfter)
			: Resource(InResource)
			, StateBefore(InStateBefore)
			, StateAfter(InStateAfter)
		{}

		FRHITransitionInfo ToRHITransitionInfo() const;

	private:
		FTextureResource* Resource = nullptr;
		ERHIAccess StateBefore = ERHIAccess::None;
		ERHIAccess StateAfter = ERHIAccess::None;
	};

	class FTransitionBatcherScope : public FNoncopyable
	{
	public:
		FTransitionBatcherScope(UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder);
		~FTransitionBatcherScope();

		void TransitionTo(ULandscapeScratchRenderTarget* InScratchRenderTarget, ERHIAccess InStateAfter);

	private:
		UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder;
		TArray<FTransitionInfo> PendingTransitions;
	};

	struct FCopyFromParams
	{
		FIntPoint CopySize = FIntPoint(0, 0);
		FIntPoint SourcePosition = FIntPoint(0, 0);
		FIntPoint DestPosition = FIntPoint(0, 0);
		uint8 SourceMip = 0;
		uint8 DestMip = 0;
		uint32 SourceSliceIndex = 0;
		uint32 DestSliceIndex = 0;
	};

	struct FCopyFromTextureParams : public FCopyFromParams
	{
		explicit FCopyFromTextureParams(UTexture* InSourceTexture)
			: SourceTexture(InSourceTexture)
		{}

		UTexture* SourceTexture = nullptr;
	};

	struct FCopyFromScratchRenderTargetParams : public FCopyFromParams
	{
		explicit FCopyFromScratchRenderTargetParams(ULandscapeScratchRenderTarget* InSourceScratchRenderTarget)
			: SourceScratchRenderTarget(InSourceScratchRenderTarget)
		{
			// No need to copy anything beyond the effective resolution in the case of a scratch render target : 
			CopySize = InSourceScratchRenderTarget->GetEffectiveResolution();
		}

		ULandscapeScratchRenderTarget* SourceScratchRenderTarget = nullptr;
	};
	
	/**
	 * Copies the content of the texture in parameter to the scratch texture (assuming the input texture is in CopySrc state already). Transitions the scratch texture's RHIAccess
	 *  @param InCopyParams parameters for the texture to copy from 
	 *  @param RDGBuilderRecorder when the recorder is in "recording" mode, the copy's render command enqueuing will be delayed to the render thread, after a FRDGBuilder has been created
	 *   but the scratch render target's current state will still be mutated to reflect the state change on the game thread. In "immediate" mode, a render command will be enqueued immediately
	 */
	LANDSCAPE_API void CopyFrom(const FCopyFromTextureParams& InCopyParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder);

	/**
	 * Copies the content of the scratch texture in parameter to the scratch texture. Transitions both scratch textures' RHIAccess
	 * @param InCopyParams parameters for the scratch texture to copy from
	 * @param RDGBuilderRecorder when the recorder is in "recording" mode, the copy's render command enqueuing will be delayed to the render thread, after a FRDGBuilder has been created
	*  but the scratch render targets' current state will still be mutated to reflect the state changes on the game thread. In "immediate" mode, a render command will be enqueued immediately
	 */
	LANDSCAPE_API void CopyFrom(const FCopyFromScratchRenderTargetParams& InCopyParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder);

	/** 
	 * Perform a transition of the scratch texture's RHIAccess 
	 *  @param InStateAfter desired RHIAccess of the scratch render target
	*  @param RDGBuilderRecorder  when the recorder is in "recording" mode, the transition's render command enqueuing will not be executed (we assume the FRDGBuilder will take care of it)
	 *   but the scratch render target's current state will still be mutated to reflect the state change on the game thread. In "immediate" mode, a render command will be enqueued immediately
	 */
	LANDSCAPE_API void TransitionTo(ERHIAccess InStateAfter, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder);

	/** 
	 * Perform a clear of the scratch texture 
	 *  @param RDGBuilderRecorder  when the recorder is in "recording" mode, the clear's render command enqueuing will be delayed to the render thread, after a FRDGBuilder has been created
	 *   but the scratch render target's current state will still be mutated to reflect the state change on the game thread. In "immediate" mode, a render command will be enqueued immediately
	 */
	LANDSCAPE_API void Clear(UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder);

	/** @return true if this scratch texture is compatible (and can therefore be used) with the requested render target params */
	bool IsCompatibleWith(const UE::Landscape::FScratchRenderTargetParams& InParams) const;

	/** @return true if the scratch texture is currently in use (it has been requested but not released) */
	bool IsInUse() const { return bIsInUse; }

private:
	void OnRequested(const UE::Landscape::FScratchRenderTargetParams& InParams);
	void OnReleased();

private:
	UPROPERTY(NonTransactional)
	TObjectPtr<UTextureRenderTarget> RenderTarget;

	// BEGIN Un-mutable section
	// The following variables are un-mutable after RenderTarget is initialized : 
	
	// Format of the render target. Technically, we could infer it from RenderTarget->Format but since it's stored as a EPixelFormat, 
	//  end there's no easy conversion from EPixelFormat to ETextureRenderTargetFormat, we store a copy here instead: 
	ETextureRenderTargetFormat RenderTargetFormat = ETextureRenderTargetFormat::RTF_RGBA8;
	// END Un-mutable section

	// Current state of the scratch render target :
	bool bIsInUse = false;
	UE::Landscape::FScratchRenderTargetParams CurrentRenderTargetParams;
	ERHIAccess CurrentState = ERHIAccess::None;
};

/** ULandscapeEditResourcesSubsystem provides services to manage/pool render resources used by the landscape tools, across landscape actors, in order to minimize memory consumption */
UCLASS()
class ULandscapeEditResourcesSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

public:
	ULandscapeEditResourcesSubsystem();

	// UEngineSubsystem implementation
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;

	/**
	 * Returns an existing (recycled) or new ULandscapeScratchRenderTarget
	 * @param InParams the requested attributes of the render target (see FScratchRenderTargetParams comments)
	 */
	ULandscapeScratchRenderTarget* RequestScratchRenderTarget(const UE::Landscape::FScratchRenderTargetParams& InParams);

	/**
	 * Releases a previously requested ULandscapeScratchRenderTarget and returns it to the pool
	 * @param InScratchRenderTarget scratch render target to release
	 */
	void ReleaseScratchRenderTarget(ULandscapeScratchRenderTarget* InScratchRenderTarget);

#if WITH_EDITOR
	UMaterialInterface* GetLayerDebugColorMaterial() const { return LayerDebugColorMaterial; }
	UMaterialInterface* GetSelectionColorMaterial() const { return SelectionColorMaterial; }
	UMaterialInterface* GetSelectionRegionMaterial() const { return SelectionRegionMaterial; }
	UMaterialInterface* GetMaskRegionMaterial() const { return MaskRegionMaterial; }
	UMaterialInterface* GetColorMaskRegionMaterial() const { return ColorMaskRegionMaterial; }
	UTexture2D* GetLandscapeBlackTexture() const { return LandscapeBlackTexture; }
	UMaterialInterface* GetLandscapeLayerUsageMaterial() const { return LandscapeLayerUsageMaterial; }
	UMaterialInterface* GetLandscapeDirtyMaterial() const { return LandscapeDirtyMaterial; }
#endif // WITH_EDITOR

private:
	UPROPERTY(Transient, NonTransactional)
	TArray<TObjectPtr<ULandscapeScratchRenderTarget>> ScratchRenderTargets;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient, NonTransactional) 
	TObjectPtr<UMaterialInterface> LayerDebugColorMaterial;

	UPROPERTY(Transient, NonTransactional) 
	TObjectPtr<UMaterialInterface> SelectionColorMaterial;

	UPROPERTY(Transient, NonTransactional) 
	TObjectPtr<UMaterialInterface> SelectionRegionMaterial;

	UPROPERTY(Transient, NonTransactional) 
	TObjectPtr<UMaterialInterface> MaskRegionMaterial;

	UPROPERTY(Transient, NonTransactional) 
	TObjectPtr<UMaterialInterface> ColorMaskRegionMaterial;

	UPROPERTY(Transient, NonTransactional) 
	TObjectPtr<UTexture2D> LandscapeBlackTexture;

	UPROPERTY(Transient, NonTransactional) 
	TObjectPtr<UMaterialInterface> LandscapeLayerUsageMaterial;

	UPROPERTY(Transient, NonTransactional) 
	TObjectPtr<UMaterialInterface> LandscapeDirtyMaterial;
#endif // WITH_EDITORONLY_DATA
	 
};
