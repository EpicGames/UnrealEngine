// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/EnumRange.h"

#include "LandscapeEditTypes.generated.h"

UENUM()
enum class ELandscapeToolTargetType : uint8
{
	Heightmap = 0,
	Weightmap = 1,
	Visibility = 2,
	Invalid = 3 UMETA(Hidden), // only valid for LandscapeEdMode->CurrentToolTarget.TargetType
	Count = Invalid UMETA(Hidden), // Only the elements above Invalid actually count as proper target types
};
ENUM_RANGE_BY_COUNT(ELandscapeToolTargetType, ELandscapeToolTargetType::Count);

enum class ELandscapeToolTargetTypeFlags : uint8
{
	None = 0,
	Heightmap = (1 << static_cast<uint8>(ELandscapeToolTargetType::Heightmap)),
	Weightmap = (1 << static_cast<uint8>(ELandscapeToolTargetType::Weightmap)),
	Visibility = (1 << static_cast<uint8>(ELandscapeToolTargetType::Visibility)),
	All = Heightmap | Weightmap | Visibility,
};
ENUM_CLASS_FLAGS(ELandscapeToolTargetTypeFlags);

UENUM()
enum class ELandscapeTargetLayerBlendMethod : uint8
{
	None = 0 UMETA(DisplayName = "No Weight Blending", Tooltip = "The target layer's weight is unaffected by other target layers."),
	FinalWeightBlending UMETA(DisplayName = "Weight Blending (Legacy)", Tooltip = "The target layer's weight is adjusted against the weights of the other (legacy) weight-blended target layers such that their sum is 0. Doesn't work well when combined with edit layers, as this (horizontal) blend is only applied at the end of the edit layers (vertical) blending."),
	PremultipliedAlphaBlending UMETA(DisplayName = "Advanced Weight Blending", Tooltip = "The target layer's weight is adjusted against the weights of the other advanced weight-blended target layers such that their sum is 0. Takes into account edit layers : the blending is applied at each edit layer, such that the target layer's weight will always be weight-blended but will also retain its relative weight against other edit layers underneath."),
	Count UMETA(Hidden), // Only the elements above actually count as proper target types
};

namespace UE::Landscape
{

enum class EOutdatedDataFlags : uint8
{
	None = 0,

	// Actual flags : 
	GrassMaps = (1 << 0),
	PhysicalMaterials = (1 << 1),
	NaniteMeshes = (1 << 2),
	PackageModified = (1 << 3),

	// Not real flags, only useful to loop through the actual flags : 
	LastPlusOne, 
	Last = LastPlusOne - 1,

	// Combined flags :
	All = (GrassMaps | PhysicalMaterials | NaniteMeshes | PackageModified)
};
ENUM_CLASS_FLAGS(EOutdatedDataFlags);

inline uint32 GetOutdatedDataFlagIndex(EOutdatedDataFlags InFlag)
{
	const uint32 InFlagAsUInt32 = static_cast<uint32>(InFlag);
	check((InFlagAsUInt32 > static_cast<uint32>(EOutdatedDataFlags::None)) && (InFlagAsUInt32 < static_cast<uint32>(EOutdatedDataFlags::LastPlusOne)) && (FMath::CountBits(InFlagAsUInt32) == 1u));
	return FMath::CountTrailingZeros(InFlagAsUInt32);
}

enum class EBuildFlags : uint8
{
	None = 0, 

	WriteFinalLog = (1 << 0), // Log a message at the end of the build operation
	ForceRebuild = (1 << 1), // Forces a rebuild even if not needed
};
ENUM_CLASS_FLAGS(EBuildFlags)

} // namespace UE::Landscape