// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Landscape.h"

class ILandscapeEditorServices
{
public:
	virtual ~ILandscapeEditorServices() = default;

	/**
	 * Attempts to find an edit layer named InEditLayerName in InTargetLandscape.
	 * Creates the layer if it does not exist.
	 * 
	 * @param InEditLayerName The name of the layer to search for and possibly create
	 * @param InTargetLandscape The landscape which should have a layer called InEditLayerName
	 * @param InEditLayerClass The class of the edit layer to create. If none specified, a standard edit layer (ULandscapeEditLayer) will be created
	 * @return The index at which the edit layer named InEditLayerName exists
	 */
	virtual int32 GetOrCreateEditLayer(FName InEditLayerName, ALandscape* InTargetLandscape, const TSubclassOf<ULandscapeEditLayerBase>& InEditLayerClass = TSubclassOf<ULandscapeEditLayerBase>()) = 0;

	/** Requires the landscape editor mode, if active, to refresh its detail panel */
	virtual void RefreshDetailPanel() = 0;

	/** Requires the landscape editor mode, if active, to invalidate all of its thumbnails */
	virtual void RegenerateLayerThumbnails() = 0;
};
