// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "WorldPartition/HLOD/HLODBuilder.h"
#include "LandscapeHLODBuilder.generated.h"

UCLASS(HideDropdown, MinimalAPI)
class ULandscapeHLODBuilder : public UHLODBuilder
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITOR
	LANDSCAPE_API virtual bool ComputeHLODHash(FHLODHashBuilder& HashBuilder, const UActorComponent* InSourceComponent) const override;

	/**
	 * Components created with this method need to be properly outered & assigned to your target actor.
	 */
	LANDSCAPE_API virtual TArray<UActorComponent*> Build(const FHLODBuildContext& InHLODBuildContext, const TArray<UActorComponent*>& InSourceComponents) const override;
#endif
};
