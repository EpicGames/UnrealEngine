// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
  LandscapeLight.h: Static lighting for LandscapeComponents
  =============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "PixelFormat.h"
#include "StaticLighting.h"
#include "RenderUtils.h"

class FLightmassExporter;
class FShadowMapData2D;
class ULandscapeComponent;
class ULevel;
class ULightComponent;
struct FQuantizedLightmapData;

/** A texture mapping for landscapes */
class FLandscapeStaticLightingTextureMapping : public FStaticLightingTextureMapping
{
public:
	/** Initialization constructor. */
	FLandscapeStaticLightingTextureMapping(ULandscapeComponent* InPrimitive,FStaticLightingMesh* InMesh,int32 InLightMapWidth,int32 InLightMapHeight,bool bPerformFullQualityRebuild);

	// FStaticLightingTextureMapping interface
	LANDSCAPE_API virtual void Apply(FQuantizedLightmapData* QuantizedData, const TMap<ULightComponent*,FShadowMapData2D*>& ShadowMapData, const FStaticLightingBuildContext* LightingContext);

#if WITH_EDITOR
	/** 
	* Export static lighting mapping instance data to an exporter 
	* @param Exporter - export interface to process static lighting data
	*/
	UNREALED_API virtual void ExportMapping(FLightmassExporter* Exporter);
	LANDSCAPE_API virtual void Serialize(FArchive& Ar);

#endif	//WITH_EDITOR

	LANDSCAPE_API FLandscapeStaticLightingTextureMapping(const FArchive& Ar);

	virtual FString GetDescription() const
	{
		return FString("LandscapeMapping");
	}
private:

	/** The primitive this mapping represents. */
	ULandscapeComponent* LandscapeComponent;
};

class FLandscapeStaticLightingGlobalVolumeMapping  : public FLandscapeStaticLightingTextureMapping 
{
public:
	LANDSCAPE_API FLandscapeStaticLightingGlobalVolumeMapping(ULandscapeComponent* InPrimitive,FStaticLightingMesh* InMesh,int32 InLightMapWidth,int32 InLightMapHeight,bool bPerformFullQualityRebuild);

	virtual void Apply(struct FQuantizedLightmapData* QuantizedData, const TMap<ULightComponent*,class FShadowMapData2D*>& ShadowMapData, const FStaticLightingBuildContext* LightingContext) override
	{
		// Should never be processed
		check(false);
	}

#if WITH_EDITOR
	virtual bool DebugThisMapping() const override
	{
		return false;
	}

	/** 
	 * Export static lighting mapping instance data to an exporter 
	 * @param Exporter - export interface to process static lighting data
	 */
	UNREALED_API virtual void ExportMapping(class FLightmassExporter* Exporter) override;
#endif	//WITH_EDITOR

	/** Whether or not this mapping should be processed or imported */
	virtual bool IsValidMapping() const override {return true;} 

	virtual FString GetDescription() const override
	{
		return FString("LandscapeVolumeMapping");
	}
};


/** Represents the triangles of a Landscape component to the static lighting system. */
class FLandscapeStaticLightingMesh : public FStaticLightingMesh
{
public:
	// tors
	FLandscapeStaticLightingMesh(ULandscapeComponent* InComponent, const TArray<ULightComponent*>& InRelevantLights, int32 InExpandQuadsX, int32 InExpandQuadsY, float LightMapRatio, int32 InLOD);
	virtual ~FLandscapeStaticLightingMesh();

	// FStaticLightingMesh interface
	virtual void GetTriangle(int32 TriangleIndex,FStaticLightingVertex& OutV0,FStaticLightingVertex& OutV1,FStaticLightingVertex& OutV2) const;
	virtual void GetTriangleIndices(int32 TriangleIndex,int32& OutI0,int32& OutI1,int32& OutI2) const;
	virtual FLightRayIntersection IntersectLightRay(const FVector& Start,const FVector& End,bool bFindNearestIntersection) const;

	virtual bool IsInstancedMesh() const override { return false; }

#if WITH_EDITOR
	/** 
	* Export static lighting mesh instance data to an exporter 
	* @param Exporter - export interface to process static lighting data
	**/
	UNREALED_API virtual void ExportMeshInstance(FLightmassExporter* Exporter) const;
#endif	//WITH_EDITOR

protected:
	void GetHeightmapData(int32 InLOD, int32 GeometryLOD);

	/** Fills in the static lighting vertex data for the Landscape vertex. */
	void GetStaticLightingVertex(int32 VertexIndex, FStaticLightingVertex& OutVertex) const;

	ULandscapeComponent* LandscapeComponent;

	// FLandscapeStaticLightingMeshData
	FTransform LocalToWorld;
	int32 ComponentSizeQuads;
	float LightMapRatio;
	int32 ExpandQuadsX;
	int32 ExpandQuadsY;

	TArray<FColor> HeightData;
	// Cache
	int32 NumVertices;
	int32 NumQuads;
	float UVFactor;
	bool bReverseWinding;

	friend class FLightmassExporter;

#if WITH_EDITOR
public:
	// Cache data for Landscape upscaling data
	LANDSCAPE_API static TMap<FIntPoint, FColor> LandscapeUpscaleHeightDataCache;
	UE_DEPRECATED(5.7, "RetopologizeTool/XYOffset deprecated with the removal of non-edit layer landscapes")
	LANDSCAPE_API static TMap<FIntPoint, FColor> LandscapeUpscaleXYOffsetDataCache;
#endif
};

namespace
{
	// LightmapRes: Multiplier of lightmap size relative to landscape size
	// X: (Output) PatchExpandCountX (at Lighting LOD)
	// Y: (Output) PatchExpandCountY (at Lighting LOD)
	// ComponentSize: Component size in patches (at LOD 0)
	// LigtmapSize: Size desired for lightmap (texels)
	// DesiredSize: (Output) Recommended lightmap size (texels)
	// return: LightMapRatio
	static float GetTerrainExpandPatchCount(float LightMapRes, int32& X, int32& Y, int32 ComponentSize, int32 LightmapSize, int32& DesiredSize, uint32 LightingLOD)
	{
		if (LightMapRes <= 0) return 0.f;

		// Assuming DXT_1 compression at the moment...
		const int32 PixelPaddingX = GPixelFormats[PF_DXT1].BlockSizeX; // "/2" ?
		const int32 PixelPaddingY = GPixelFormats[PF_DXT1].BlockSizeY;
		const int32 PatchExpandCountX = static_cast<int32>(LightMapRes >= 1.f ? PixelPaddingX / LightMapRes : PixelPaddingX);
		const int32 PatchExpandCountY = static_cast<int32>(LightMapRes >= 1.f ? PixelPaddingY / LightMapRes : PixelPaddingY);

		X = FMath::Max<int32>(1, PatchExpandCountX >> LightingLOD);
		Y = FMath::Max<int32>(1, PatchExpandCountY >> LightingLOD);

		DesiredSize = (LightMapRes >= 1.f) ? FMath::Min<int32>((int32)((ComponentSize + 1) * LightMapRes), 4096) : FMath::Min<int32>((int32)((LightmapSize)* LightMapRes), 4096);
		int32 CurrentSize = (LightMapRes >= 1.f) ? FMath::Min<int32>((int32)((2 * (X << LightingLOD) + ComponentSize + 1) * LightMapRes), 4096) : FMath::Min<int32>((int32)((2 * (X << LightingLOD) + LightmapSize) * LightMapRes), 4096);

		// Find proper Lightmap Size
		if (CurrentSize > DesiredSize)
		{
			// Find maximum bit
			int32 PriorSize = DesiredSize;
			while (DesiredSize > 0)
			{
				PriorSize = DesiredSize;
				DesiredSize = DesiredSize & ~(DesiredSize & ~(DesiredSize - 1));
			}

			DesiredSize = PriorSize << 1; // next bigger size
			if (CurrentSize * CurrentSize <= ((PriorSize * PriorSize) << 1))
			{
				DesiredSize = PriorSize;
			}
		}

		const float DestSize = FMath::Floor(static_cast<float>(DesiredSize) / CurrentSize * (ComponentSize*LightMapRes));
		const float LightMapRatio = DestSize / (ComponentSize*LightMapRes) * CurrentSize / DesiredSize;
		return LightMapRatio;
	}
}
