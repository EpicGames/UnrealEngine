// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "LandscapeEditorServices.h"

class FPrimitiveSceneProxy;
class FLandscapeSceneViewExtension;
class ULandscapeComponent;

namespace UE::Landscape
{
	DECLARE_DELEGATE_RetVal_OneParam(FPrimitiveSceneProxy*, FCreateLandscapeComponentSceneProxyDelegate, ULandscapeComponent* /*InLandscapeComponent*/);
	DECLARE_DELEGATE_TwoParams(FSetupLandscapePSOPrecacheVertexFactoryDataListDelegate, ULandscapeComponent* /*InLandscapeComponent*/, FPSOPrecacheVertexFactoryDataList& /*VertexFactoryDataList*/);
} // namespace UE::Landscape

/**
* Landscape module interface
*/
class ILandscapeModule : public IModuleInterface
{
public:
	virtual TSharedPtr<FLandscapeSceneViewExtension, ESPMode::ThreadSafe> GetLandscapeSceneViewExtension() const = 0;
	
	virtual void SetLandscapeEditorServices(ILandscapeEditorServices* InLandscapeEditorServices) PURE_VIRTUAL(ILandscapeModule::SetLandscapeEditorServices, );
	virtual ILandscapeEditorServices* GetLandscapeEditorServices() const PURE_VIRTUAL(ILandscapeModule::GetLandscapeEditorServices, return nullptr; );
	virtual void SetCreateLandscapeComponentSceneProxyDelegate(const UE::Landscape::FCreateLandscapeComponentSceneProxyDelegate& InDelegate) PURE_VIRTUAL(ILandscapeModule::SetCreateLandscapeComponentSceneProxyDelegate, );
	virtual const UE::Landscape::FCreateLandscapeComponentSceneProxyDelegate& GetCreateLandscapeComponentSceneProxyDelegate() const 
		PURE_VIRTUAL(ILandscapeModule::GetCreateLandscapeComponentSceneProxyDelegate, static UE::Landscape::FCreateLandscapeComponentSceneProxyDelegate Dummy; return Dummy; );
	virtual void SetSetupLandscapePSOPrecacheVertexFactoryDataListDelegate(const UE::Landscape::FSetupLandscapePSOPrecacheVertexFactoryDataListDelegate& InDelegate) PURE_VIRTUAL(ILandscapeModule::SetSetupLandscapePSOPrecacheVertexFactoryDataListDelegate, );
	virtual const UE::Landscape::FSetupLandscapePSOPrecacheVertexFactoryDataListDelegate& GetSetupLandscapePSOPrecacheVertexFactoryDataListDelegate() const 
		PURE_VIRTUAL(ILandscapeModule::GetSetupLandscapePSOPrecacheVertexFactoryDataListDelegate, static UE::Landscape::FSetupLandscapePSOPrecacheVertexFactoryDataListDelegate Dummy; return Dummy; );
};
