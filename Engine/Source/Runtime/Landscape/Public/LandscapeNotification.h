// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "CoreMinimal.h"
#include "Landscape.h"

/** The various types of landscape notifications, ordered by priority (a single notification is displayed at a time). */
enum class ELandscapeNotificationType
{
	ShadingModelInvalid = 0,
	LandscapeTextureResourcesNotReady,
	LandscapeEditLayerResourcesNotReady,
};

struct FLandscapeNotification
{
	using FConditionCallback = TFunction<bool(void)>;
	using FUpdateTextCallback = TFunction<void(FText&)>;

	/** 
	* Constructs a notification which is expected to have a longer lifetime and behave in a dynamic way.
	* 
	* @param InLandscape - The landscape actor associated with this notification.
	* @param InNotificationType - The type of notification which determines its display priority.
	* @param InConditionCallback - Lambda used to conditionally determine whether this notification should be displayed per tick.
	* @param InUpdateTextCallback - Lambda used to construct notification text when the text can change (e.g. displaying the value of a variable).
	*/ 
	FLandscapeNotification(const TWeakObjectPtr<ALandscape>& InLandscape, ELandscapeNotificationType InNotificationType, FConditionCallback InConditionCallback = FConditionCallback([]() { return true; }), FUpdateTextCallback InUpdateTextCallback = [](FText& InText) {});
	~FLandscapeNotification() = default;

	bool operator == (const FLandscapeNotification& Other) const;
	bool operator < (const FLandscapeNotification& Other) const;

	TWeakObjectPtr<ALandscape> GetLandscape() const { return Landscape; }
	ELandscapeNotificationType GetNotificationType() const { return NotificationType; }

	// Public wrappers for the callback functions.
	bool ShouldShowNotification() const;
	void SetNotificationText();

private:
	FLandscapeNotification() = delete;

public:
	/** The text to be displayed.Can either be set by user directly or by setting SetTextCallback at notification construction. */ 
	FText NotificationText;

	/** Determines when a notification should be displayed, ignored if less than 0. */ 
	double NotificationStartTime = -1.0;

private:
	TWeakObjectPtr<ALandscape> Landscape;
	ELandscapeNotificationType NotificationType;

	/** Determines whether or not a notification should be shown. Defaults to return true. */
	FConditionCallback ConditionCallback;

	/** Sets the notification text per frame by manipulating InText&. Defaults to do nothing, allowing direct manipulation of NotificationText. */
	FUpdateTextCallback UpdateTextCallback;
};

/**
 * FLandscapeNotificationManager : centralizes landscape-related user notifications so that the user doesn't get flooded by toasts when multiple messages from possibly multiple landscapes are tossed around
 */
class FLandscapeNotificationManager
{
public:
	~FLandscapeNotificationManager();
	
	void Tick();

	void RegisterNotification(const TWeakPtr<FLandscapeNotification>& InNotification);
	void UnregisterNotification(const TWeakPtr<FLandscapeNotification>& InNotification);

private:
	void ShowNotificationItem(const FText& InText);
	void HideNotificationItem();

private:
	TArray<TWeakPtr<FLandscapeNotification>> LandscapeNotifications;
	TWeakPtr<FLandscapeNotification> ActiveNotification;
	TWeakPtr<SNotificationItem> NotificationItem;
};

#endif // WITH_EDITOR