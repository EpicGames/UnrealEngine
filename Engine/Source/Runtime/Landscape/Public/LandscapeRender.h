// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
LandscapeRender.h: New terrain rendering
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Engine/EngineTypes.h"
#include "Templates/RefCounting.h"
#include "Containers/ArrayView.h"
#include "ShaderParameters.h"
#include "RenderResource.h"
#include "UniformBuffer.h"
#include "VertexFactory.h"
#include "MaterialShared.h"
#include "Materials/MaterialRenderProxy.h"
#include "LandscapeProxy.h"
#include "RendererInterface.h"
#include "MeshBatch.h"
#include "SceneManagement.h"
#include "Engine/MapBuildDataRegistry.h"
#include "LandscapeComponent.h"
#include "Materials/MaterialInterface.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "StaticMeshResources.h"
#include "StaticMeshSceneProxy.h"
#include "SceneViewExtension.h"
#include "Rendering/CustomRenderPass.h"
#include "Tasks/Task.h"

#define LANDSCAPE_LOD_LEVELS 8
#define LANDSCAPE_MAX_SUBSECTION_NUM 2

class FLandscapeComponentSceneProxy;
enum EShaderPlatform : uint16;

#if WITH_EDITOR
class ULandscapeEditResourcesSubsystem;
#endif

namespace UE::Renderer::Private { class IShadowInvalidatingInstances; }

#if RHI_RAYTRACING
struct FLandscapeRayTracingImpl;
#endif

#if WITH_EDITOR
namespace ELandscapeViewMode
{
	enum Type: int32
	{
		Invalid = -1,
		/** Color only */
		Normal = 0,
		EditLayer,
		/** Layer debug only */
		DebugLayer,
		LayerDensity,
		LayerUsage,
		LOD,
		WireframeOnTop,
		LayerContribution
	};
}

extern LANDSCAPE_API int32 GLandscapeViewMode;

namespace ELandscapeEditRenderMode
{
	enum Type
	{
		None = 0x0,
		Gizmo = 0x1,
		SelectRegion = 0x2,
		SelectComponent = 0x4,
		Select = SelectRegion | SelectComponent,
		Mask = 0x8,
		InvertedMask = 0x10, // Should not be overlapped with other bits 
		BitMaskForMask = Mask | InvertedMask,

	};
}

LANDSCAPE_API extern bool GLandscapeEditModeActive;
LANDSCAPE_API extern int32 GLandscapeEditRenderMode;
#endif

namespace UE::Landscape
{
	bool NeedsFixedGridVertexFactory(EShaderPlatform InShaderPlatform);
	bool ShouldBuildGrassMapRenderingResources();
	bool UseLandscapeVertexMask();
	EPSOPrecacheProxyCreationStrategy GetLandscapePSOPrecacheProxyCreationStrategy();
} // namespace UE::Landscape

/** The uniform shader parameters for a landscape draw call. */
BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FLandscapeUniformShaderParameters, LANDSCAPE_API)
	SHADER_PARAMETER(int32, ComponentBaseX)
	SHADER_PARAMETER(int32, ComponentBaseY)
	SHADER_PARAMETER(int32, SubsectionSizeVerts)
	SHADER_PARAMETER(int32, NumSubsections)
	SHADER_PARAMETER(int32, RayTracingSectionSizeVerts)
	SHADER_PARAMETER(int32, NumRayTracingSections)
	SHADER_PARAMETER(int32, RayTracingLODBias)
	SHADER_PARAMETER(int32, LastLOD)
	SHADER_PARAMETER(uint32, VirtualTexturePerPixelHeight)
	SHADER_PARAMETER(float, InvLODBlendRange)
	SHADER_PARAMETER(float, NonNaniteVirtualShadowMapConstantDepthBias)
	SHADER_PARAMETER(FVector4f, HeightmapTextureSize)
	SHADER_PARAMETER(FVector4f, HeightmapUVScaleBias)
    SHADER_PARAMETER(FVector4f, WeightmapUVScaleBias)
    SHADER_PARAMETER(FVector4f, LandscapeLightmapScaleBias)
    SHADER_PARAMETER(FVector4f, SubsectionSizeVertsLayerUVPan)
    SHADER_PARAMETER(FVector4f, SubsectionOffsetParams)
    SHADER_PARAMETER(FVector4f, LightmapSubsectionOffsetParams)
	SHADER_PARAMETER(FMatrix44f, LocalToWorldNoScaling)
	SHADER_PARAMETER_TEXTURE(Texture2D, HeightmapTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, HeightmapTextureSampler)
	SHADER_PARAMETER_TEXTURE(Texture2D, NormalmapTexture)
	SHADER_PARAMETER_SAMPLER(SamplerState, NormalmapTextureSampler)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FLandscapeVertexFactoryMVFParameters, LANDSCAPE_API)
	SHADER_PARAMETER(FIntPoint, SubXY)
	SHADER_PARAMETER(FIntPoint, RayTracingSectionXY)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

typedef TUniformBufferRef<FLandscapeVertexFactoryMVFParameters> FLandscapeVertexFactoryMVFUniformBufferRef;

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FLandscapeSectionLODUniformParameters, LANDSCAPE_API)
	SHADER_PARAMETER(int32, LandscapeIndex)
	SHADER_PARAMETER(FIntPoint, Min)
	SHADER_PARAMETER(FIntPoint, Size)
	SHADER_PARAMETER_SRV(Buffer<float>, SectionLODBias)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

BEGIN_GLOBAL_SHADER_PARAMETER_STRUCT(FLandscapeFixedGridUniformShaderParameters, LANDSCAPE_API)
	SHADER_PARAMETER(FVector4f, LodValues)
END_GLOBAL_SHADER_PARAMETER_STRUCT()

/* Data needed for the landscape vertex factory to set the render state for an individual batch element */
struct FLandscapeBatchElementParams
{
#if RHI_RAYTRACING
	FRHIUniformBuffer* LandscapeVertexFactoryMVFUniformBuffer;
#endif
	const TUniformBuffer<FLandscapeUniformShaderParameters>* LandscapeUniformShaderParametersResource;
	const TArray<TUniformBuffer<FLandscapeFixedGridUniformShaderParameters>>* FixedGridUniformShaderParameters;
	FUniformBufferRHIRef LandscapeSectionLODUniformParameters;
	const FLandscapeComponentSceneProxy* SceneProxy;
	int32 CurrentLOD;
};

class FLandscapeElementParamArray : public FOneFrameResource
{
public:
	TArray<FLandscapeBatchElementParams, SceneRenderingAllocator> ElementParams;
};

class FLandscapeVertexFactoryVertexShaderParameters : public FVertexFactoryShaderParameters
{
	DECLARE_TYPE_LAYOUT(FLandscapeVertexFactoryVertexShaderParameters, NonVirtual);
public:
	/**
	* Bind shader constants by name
	* @param	ParameterMap - mapping of named shader constants to indices
	*/
	void Bind(const FShaderParameterMap& ParameterMap)
	{
	}

	void GetElementShaderBindings(
		const class FSceneInterface* Scene,
		const FSceneView* InView,
		const class FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FVertexFactory* VertexFactory,
		const FMeshBatchElement& BatchElement,
		class FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams
	) const;
};

/** Pixel shader parameters for use with FLandscapeVertexFactory */
class FLandscapeVertexFactoryPixelShaderParameters : public FVertexFactoryShaderParameters
{
	DECLARE_TYPE_LAYOUT(FLandscapeVertexFactoryPixelShaderParameters, NonVirtual);
public:
	void GetElementShaderBindings(
		const class FSceneInterface* Scene,
		const FSceneView* InView,
		const class FMeshMaterialShader* Shader,
		const EVertexInputStreamType InputStreamType,
		ERHIFeatureLevel::Type FeatureLevel,
		const FVertexFactory* VertexFactory,
		const FMeshBatchElement& BatchElement,
		class FMeshDrawSingleShaderBindings& ShaderBindings,
		FVertexInputStreamArray& VertexStreams
	) const;

	
	
};

/** vertex factory for VTF-heightmap terrain  */
class FLandscapeVertexFactory : public FVertexFactory
{
	DECLARE_VERTEX_FACTORY_TYPE_API(FLandscapeVertexFactory, LANDSCAPE_API);

public:

	LANDSCAPE_API FLandscapeVertexFactory(ERHIFeatureLevel::Type InFeatureLevel);

	virtual ~FLandscapeVertexFactory()
	{
		// can only be destroyed from the render thread
		ReleaseResource();
	}

	struct FDataType
	{
		/** The stream to read the vertex position from. */
		FVertexStreamComponent PositionComponent;
	};

	/**
	* Should we cache the material's shadertype on this platform with this vertex factory?
	*/
	static LANDSCAPE_API bool ShouldCompilePermutation(const FVertexFactoryShaderPermutationParameters& Parameters);

	/**
	* Can be overridden by FVertexFactory subclasses to modify their compile environment just before compilation occurs.
	*/
	static LANDSCAPE_API void ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
	
	/**
	* Get vertex elements used when during PSO precaching materials using this vertex factory type
	*/
	static LANDSCAPE_API void GetPSOPrecacheVertexFetchElements(EVertexInputStreamType VertexInputStreamType, FVertexDeclarationElementList& Elements);

	/**
	* Copy the data from another vertex factory
	* @param Other - factory to copy from
	*/
	LANDSCAPE_API void Copy(const FLandscapeVertexFactory& Other);

	// FRenderResource interface.
	LANDSCAPE_API virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseResource() override final { FVertexFactory::ReleaseResource(); }

	/**
	 * An implementation of the interface used by TSynchronizedResource to update the resource with new data from the game thread.
	 */
	void SetData(const FDataType& InData)
	{
		Data = InData;
		UpdateRHI(FRHICommandListImmediate::Get());
	}

	/** stream component data bound to this vertex factory */
	FDataType Data;
};


/** vertex factory for VTF-heightmap terrain  */
class UE_DEPRECATED(5.7, "XYOffset deprecated with the removal of non-edit layer landscapes") FLandscapeXYOffsetVertexFactory : public FLandscapeVertexFactory
{
	DECLARE_VERTEX_FACTORY_TYPE_API(FLandscapeXYOffsetVertexFactory, LANDSCAPE_API);

public:
	FLandscapeXYOffsetVertexFactory(ERHIFeatureLevel::Type InFeatureLevel)
		: FLandscapeVertexFactory(InFeatureLevel)
	{
	}
};


/** Vertex factory for fixed grid runtime virtual texture lod  */
class FLandscapeFixedGridVertexFactory : public FLandscapeVertexFactory
{
	DECLARE_VERTEX_FACTORY_TYPE_API(FLandscapeFixedGridVertexFactory, LANDSCAPE_API);

public:
	FLandscapeFixedGridVertexFactory(ERHIFeatureLevel::Type InFeatureLevel)
		: FLandscapeVertexFactory(InFeatureLevel)
	{
	}

	static LANDSCAPE_API void ModifyCompilationEnvironment(const FVertexFactoryShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
};


struct FLandscapeVertex
{
	uint8 VertexX;
	uint8 VertexY;
	uint8 SubX;
	uint8 SubY;
};

//
// FLandscapeVertexBuffer
//
class FLandscapeVertexBuffer final : public FVertexBuffer
{
	ERHIFeatureLevel::Type FeatureLevel;
	int32 NumVertices;
	int32 SubsectionSizeVerts;
	int32 NumSubsections;
public:

	/** Constructor. */
	FLandscapeVertexBuffer(FRHICommandListBase& RHICmdList, ERHIFeatureLevel::Type InFeatureLevel, int32 InNumVertices, int32 InSubsectionSizeVerts, int32 InNumSubsections, const FName& InOwnerName)
		: FeatureLevel(InFeatureLevel)
		, NumVertices(InNumVertices)
		, SubsectionSizeVerts(InSubsectionSizeVerts)
		, NumSubsections(InNumSubsections)
	{
		SetOwnerName(InOwnerName);
		InitResource(RHICmdList);
	}

	/** Destructor. */
	virtual ~FLandscapeVertexBuffer()
	{
		ReleaseResource();
	}

	/**
	* Initialize the RHI for this rendering resource
	*/
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
};

//
// FLandscapeSharedBuffers
//
class FLandscapeSharedBuffers : public FRefCountedObject
{
public:
	struct FLandscapeIndexRanges
	{
		int32 MinIndex[LANDSCAPE_MAX_SUBSECTION_NUM][LANDSCAPE_MAX_SUBSECTION_NUM];
		int32 MaxIndex[LANDSCAPE_MAX_SUBSECTION_NUM][LANDSCAPE_MAX_SUBSECTION_NUM];
		int32 MinIndexFull;
		int32 MaxIndexFull;
	};

	int32 NumVertices;
	int32 SharedBuffersKey;
	int32 NumIndexBuffers;
	int32 SubsectionSizeVerts;
	int32 NumSubsections;
	int32 NumRayTracingSections;

	FLandscapeVertexFactory* VertexFactory;
	FLandscapeVertexFactory* FixedGridVertexFactory;
	FLandscapeVertexBuffer* VertexBuffer;

	TArray<TUniformBuffer<FLandscapeFixedGridUniformShaderParameters>> LandscapeFixedGridUniformShaderParameters;
	
	FRenderResource* TileMesh;
	FLandscapeVertexFactory* TileVertexFactory;
	FVertexBuffer* TileDataBuffer;
	
	// array per mip level, storing FIndexBuffer pointers
	FIndexBuffer** IndexBuffers;

	// array per mip level, storing index buffer ranges per subssection, and for the entire component
	FLandscapeIndexRanges* IndexRanges;

	bool bUse32BitIndices;
	FIndexBuffer* GrassIndexBuffer;
	TArray<int32, TInlineAllocator<8>> GrassIndexMipOffsets;

#if RHI_RAYTRACING
	TArray<FIndexBuffer*> ZeroOffsetIndexBuffers;
#endif
	
	LANDSCAPE_API FLandscapeSharedBuffers(FRHICommandListBase& RHICmdList, int32 SharedBuffersKey, int32 SubsectionSizeQuads, int32 NumSubsections, int32 NumRayTracingSections, ERHIFeatureLevel::Type FeatureLevel, const FName& OwnerName = NAME_None);

	template <typename INDEX_TYPE>
	void CreateIndexBuffers(FRHICommandListBase& RHICmdList, const FName& OwnerName);
	
	template <typename INDEX_TYPE>
	void CreateGrassIndexBuffer(FRHICommandListBase& RHICmdList, const FName& InOwnerName);

	LANDSCAPE_API virtual ~FLandscapeSharedBuffers();
};

//
// FLandscapeSectionInfo
//

class FLandscapeSectionInfo : public TIntrusiveLinkedList<FLandscapeSectionInfo>
{
public:
	FLandscapeSectionInfo(const UWorld* InWorld, const FGuid& InLandscapeGuid, const FIntPoint& InComponentBase, uint32 LODGroupKey, uint32 InLandscapeKey);
	virtual ~FLandscapeSectionInfo() = default;

	virtual float ComputeLODForView(const FSceneView& InView) const = 0;
	virtual float ComputeLODBias() const = 0;
	virtual int32 GetSectionPriority() const { return INDEX_NONE; }
	virtual const FPrimitiveSceneInfo* GetPrimitiveSceneInfo() const = 0;

	/** Computes the worldspace units per vertex of the landscape section. */
	virtual double ComputeSectionResolution() const { return -1.0; }
	
	virtual void GetSectionBoundsAndLocalToWorld(FBoxSphereBounds& LocalBounds, FMatrix& LocalToWorld) const = 0;
	virtual void GetSectionCenterAndVectors(FVector& OutSectionCenterWorldSpace, FVector& OutSectionXVectorWorldSpace, FVector& OutSectionYVectorWorldSpace) const = 0;

	/* return the resolution of a component, in vertices (-1 for any sections that are not grid based, i.e. mesh sections) */
	virtual int32 GetComponentResolution() const { return -1; }

	/* Used to notify derived classes when render coords are calculated */
	virtual void OnRenderCoordsChanged(FRHICommandListBase& RHICmdList) = 0;

	virtual bool ShouldInvalidateShadows(const FSceneView& InView, float InLODValue, float InLastShadowInvalidationLODValue) const { return false; }

#if RHI_RAYTRACING
	virtual void RayTracingNotifyViewDeleted(uint32 ViewKey) = 0;
#endif

public:
	// A hash of the world and (LandscapeGUID or LOD Group Key)
	uint32 LandscapeKey = 0;
	// LOD Group Key (0 if no group)
	uint32 LODGroupKey = 0;
	// Coordinate in the RenderSystem
	FIntPoint RenderCoord = FIntPoint(INT32_MIN, INT32_MIN);
	// Component base coordinate (relative to the ALandscape actor)
	FIntPoint ComponentBase = FIntPoint(ForceInit);
	// Scene that this landscape section belongs to
	FSceneInterface* Scene = nullptr;

	bool bResourcesCreated = false;
	bool bRegistered = false;
};

//
// FLandscapeRenderSystem
//
struct FLandscapeRenderSystem
{
	typedef uint32 FViewKey;

	struct LODSettingsComponent
	{
		float LOD0ScreenSizeSquared;
		float LOD1ScreenSizeSquared;
		float LODOnePlusDistributionScalarSquared;
		float LastLODScreenSizeSquared;
		float VirtualShadowMapInvalidationLimitLOD;
		int8 LastLODIndex;
		int8 ForcedLOD;
		int8 DrawCollisionPawnLOD;
		int8 DrawCollisionVisibilityLOD;
	};

	static float ComputeLODFromScreenSize(const LODSettingsComponent& InLODSettings, float InScreenSizeSquared);

	static TBitArray<> LandscapeIndexAllocator;

	int32 LandscapeIndex;

	FIntPoint Min;
	FIntPoint Size;

	TResourceArray<float> SectionLODBiases;
	TArray<FLandscapeSectionInfo*> SectionInfos;
	// Number of sections with resources created
	int32 ReferenceCount;			
	// Number of sections registered
	int32 RegisteredCount;

	FBufferRHIRef SectionLODBiasBuffer;
	FShaderResourceViewRHIRef SectionLODBiasSRV;

	FUniformBufferRHIRef SectionLODUniformBuffer;

	// For a given view, we use those 2 maps to store, for each FLandscapeSectionInfo in SectionInfos, the current LOD Values so that it can be passed down to the renderer
	//  These are only really valid for a given render since the list of sections can change in the next frame. 
	//  The first map is for views that don't have a persistent view state
	TMap<const FSceneView*, TResourceArray<float>> PerViewCachedSectionLODValues; // Key = view to render, Value = linear list of LOD values for each FLandscapeSectionInfo
	TMap<uint32, TResourceArray<float>> PerViewStateCachedSectionLODValues; // Key = view state key corresponding to the view to render, Value = linear list of LOD values for each FLandscapeSectionInfo

	// This map allows to track, for each FLandscapeSectionInfo that triggered a shadow invalidation, the LOD value that was used when the invalidation occurred
	//  Key = SectionInfo's RenderCoord (we cannot use a linear index like the TMaps above because these might change as new sections get added/removed dynamically and this needs to be tracked across frames)
	//  Value = LOD value of the section when the last shadow invalidation occurred
	using SectionKeyToLODValueMap = TMap<FIntPoint, float>; 
	TMap<uint32, SectionKeyToLODValueMap> PerViewStateLastShadowInvalidationSectionLODValues; // Key = view state key corresponding to the view to render, Value = see above

	// List of shadow invalidations to perform this frame (because IShadowInvalidatingInstances is not parallel-render thread-safe, we have to store them temporarily and issue them on the render thread later on) :
	TMap<UE::Renderer::Private::IShadowInvalidatingInstances*, TArray<const FPrimitiveSceneInfo*>> ShadowInvalidationRequests;

	/** Forced LOD level which overrides the ForcedLOD level of all the sections under this LandscapeRenderSystem. */
	int8 ForcedLODOverride;

	//  Resolution, Origin and Size, for use in LOD Groups to verify that all landscapes are of matching resolutions, orientation and scale
	int32 ComponentResolution = -1;
	FVector ComponentOrigin = FVector::ZeroVector;		// world space position of the center of the origin component (render coord 0,0)
	FVector ComponentXVector = FVector::ZeroVector;		// world space vector in the direction of component local X
	FVector ComponentYVector = FVector::ZeroVector;		// world space vector in the direction of component local Y

	// Counter used to reduce how often we call compact on the map when removing sections
	int32 SectionsRemovedSinceLastCompact;

	uint32 LandscapeKey = 0; 
	FSceneInterface* Scene = nullptr;

	FLandscapeRenderSystem(uint32 InLandscapeKey, FSceneInterface* InScene);
	~FLandscapeRenderSystem();

	static void CreateResources(FRHICommandListBase& RHICmdList, FLandscapeSectionInfo* SectionInfo);
	static void DestroyResources(FLandscapeSectionInfo* SectionInfo);

	static void RegisterSection(FLandscapeSectionInfo* SectionInfo);
	static void UnregisterSection(FLandscapeSectionInfo* SectionInfo);

	bool IsValidCoord(FIntPoint InRenderCoord) const
	{
		return	InRenderCoord.X >= Min.X && InRenderCoord.X < Min.X + Size.X &&
				InRenderCoord.Y >= Min.Y && InRenderCoord.Y < Min.Y + Size.Y;
	}

	int32 GetSectionLinearIndex(FIntPoint InRenderCoord) const
	{
		check(IsValidCoord(InRenderCoord));
		int32 LinearIndex = (InRenderCoord.Y - Min.Y) * Size.X + InRenderCoord.X - Min.X;
		return LinearIndex;
	}
	
	void ResizeAndMoveTo(FIntPoint NewMin, FIntPoint NewMax);
	void ResizeToInclude(const FIntPoint& NewCoord);
	void CompactMap();
	bool AnySectionsInRangeInclusive(FIntPoint RangeMin, FIntPoint RangeMax);

	void SetSectionInfo(FIntPoint InRenderCoord, FLandscapeSectionInfo* InSectionInfo)
	{
		if (IsValidCoord(InRenderCoord))
		{
			SectionInfos[GetSectionLinearIndex(InRenderCoord)] = InSectionInfo;
		}
	}

	FLandscapeSectionInfo* GetSectionInfo(FIntPoint InRenderCoord)
	{
		if (IsValidCoord(InRenderCoord))
		{
			return SectionInfos[GetSectionLinearIndex(InRenderCoord)];
		}
		return nullptr;
	}

	float GetSectionLODValue(const FSceneView& InView, FIntPoint InRenderCoord) const;
	const TResourceArray<float>& GetCachedSectionLODValues(const FSceneView& InView) const;

	float GetSectionLODBias(FIntPoint InRenderCoord) const;

	const TResourceArray<float>& ComputeSectionsLODForView(const FSceneView& InView, UE::Renderer::Private::IShadowInvalidatingInstances* InShadowInvalidatingInstances);
	void PerformShadowInvalidations(UE::Renderer::Private::IShadowInvalidatingInstances& InShadowInvalidatingInstances);
	void FetchHeightmapLODBiases();
	void UpdateBuffers(FRHICommandListBase& RHICmdList);

private:
	void CreateResources_Internal(FRHICommandListBase& RHICmdList, FLandscapeSectionInfo* InSectionInfo);
	void DestroyResources_Internal(FLandscapeSectionInfo* InSectionInfo);
};

//
// FLandscapeLODOverridesCustomRenderPassUserData
//
class FLandscapeLODOverridesCustomRenderPassUserData : public ICustomRenderPassUserData
{
public:
	IMPLEMENT_CUSTOM_RENDER_PASS_USER_DATA(FLandscapeLODOverridesCustomRenderPassUserData);

	FLandscapeLODOverridesCustomRenderPassUserData(const TMap<uint32, int32>& InLandscapeLODOverrides)
		: LandscapeLODOverrides(InLandscapeLODOverrides)
	{}


	const TMap<uint32, int32>& GetLandscapeLODOverrides() const { return LandscapeLODOverrides; }

private:
	TMap<uint32, int32> LandscapeLODOverrides;
};

//
// FLandscapeSceneViewExtension
//
class FLandscapeSceneViewExtension : public FSceneViewExtensionBase
{
public:
	FLandscapeSceneViewExtension(const FAutoRegister& AutoReg);
	virtual ~FLandscapeSceneViewExtension();

	virtual void BeginRenderViewFamily(FSceneViewFamily& InViewFamily) override;
	
	virtual void PreRenderViewFamily_RenderThread(FRDGBuilder& GraphBuilder, FSceneViewFamily& InViewFamily) override;
	virtual void PreRenderView_RenderThread(FRDGBuilder& GraphBuilder, FSceneView& InView) override;
	virtual void PreInitViews_RenderThread(FRDGBuilder& GraphBuilder) override;

	LANDSCAPE_API static const TMap<uint32, FLandscapeRenderSystem*>& GetLandscapeRenderSystems();
	static TArray<FLandscapeRenderSystem*> GetLandscapeRenderSystems(const class FSceneInterface* InScene);
	static FLandscapeRenderSystem* GetLandscapeRenderSystem(const class FSceneInterface* InScene, uint32 InLandscapeKey);
	int32 GetNumViewsWithShowCollision() const { return NumViewsWithShowCollision; }

private:
	void EndFrame_GameThread();
	void EndFrame_RenderThread();

private:

	struct FLandscapeViewData
	{
		FLandscapeViewData() = default;
		FLandscapeViewData(FSceneView& InView);

		FSceneView* View = nullptr;
		// Optional interface to use if the view needs to invalidate the shadow cache : 
		UE::Renderer::Private::IShadowInvalidatingInstances* ShadowInvalidatingInstances = nullptr;
		TResourceArray<uint32> LandscapeIndirection;
		TResourceArray<float> LandscapeLODData;
	};

	TArray<FLandscapeViewData> LandscapeViews;
	UE::Tasks::FTask LandscapeSetupTask;
	int32 NumViewsWithShowCollision = 0;    // Last frame number of views with collision enabled.
	int32 NumViewsWithShowCollisionAcc = 0; // Accumulate the number of views with collision
};

//
// FLandscapeDebugOptions
//
struct FLandscapeDebugOptions
{
	LANDSCAPE_API FLandscapeDebugOptions();

	bool bShowPatches;
	bool bDisableStatic;

private:
	FAutoConsoleCommand PatchesConsoleCommand;
	FAutoConsoleCommand StaticConsoleCommand;

	void Patches();
	void Static();
};

LANDSCAPE_API extern FLandscapeDebugOptions GLandscapeDebugOptions;

//
// FLandscapeMeshProxySceneProxy
//
class FLandscapeMeshProxySceneProxy final : public FStaticMeshSceneProxy
{
public:
	SIZE_T GetTypeHash() const override;

	FLandscapeMeshProxySceneProxy(UStaticMeshComponent* InComponent, const FGuid& InLandscapeGuid, const TArray<FIntPoint>& InProxySectionsBases, const TArray<FVector>& InProxySectionsCentersLocalSpace, const FVector& InComponentXVector, const FVector& InComponentYVector, const FTransform& LocalToWorld, int32 ComponentResolution, int8 InProxyLOD, uint32 InLODGroupKey, uint32 LandscapeKey);
	virtual void CreateRenderThreadResources(FRHICommandListBase& RHICmdList) override;
	virtual void DestroyRenderThreadResources() override;

protected:
	virtual void OnForceHiddenChanged() override;

private:
	void RegisterSections();
	void UnregisterSections();

	TArray<TUniquePtr<FLandscapeSectionInfo>> ProxySectionsInfos;
};

//
// FLandscapeComponentSceneProxy
//
class FLandscapeComponentSceneProxy : public FPrimitiveSceneProxy, public FLandscapeSectionInfo
{
	friend class FLandscapeSharedBuffers;

	LANDSCAPE_API SIZE_T GetTypeHash() const override;
	class FLandscapeLCI final : public FLightCacheInterface
	{
	public:
		/** Initialization constructor. */
		FLandscapeLCI(const ULandscapeComponent* InComponent, ERHIFeatureLevel::Type FeatureLevel, bool bVFRequiresPrimitiveUniformBuffer)
			: FLightCacheInterface()
		{
			if (InComponent->GetLightmapType() == ELightmapType::ForceVolumetric)
			{
				SetGlobalVolumeLightmap(true);
			}

			const FMeshMapBuildData* MapBuildData = InComponent->GetMeshMapBuildData();

			if (MapBuildData)
			{
				SetLightMap(MapBuildData->LightMap);
				SetShadowMap(MapBuildData->ShadowMap);
				SetResourceCluster(MapBuildData->ResourceCluster);
				// If landscape uses VF that requires primitive UB that means it does not use GPUScene therefore it may need precomputed lighting buffer as well
				if (FeatureLevel >= ERHIFeatureLevel::SM5 && !bVFRequiresPrimitiveUniformBuffer)
				{
					// Landscape does not support GPUScene on mobile
					// TODO: enable this when GPUScene support is implemented
					bCanUsePrecomputedLightingParametersFromGPUScene = true;
				}
				IrrelevantLights = MapBuildData->IrrelevantLights;
			}
		}

		// FLightCacheInterface
		virtual FLightInteraction GetInteraction(const FLightSceneProxy* LightSceneProxy) const override;

	private:
		TArray<FGuid> IrrelevantLights;
	};

public:
	static const int8 MAX_SUBSECTION_COUNT = 2*2;

#if RHI_RAYTRACING
	TPimplPtr<FLandscapeRayTracingImpl> RayTracingImpl;
#endif

	friend FLandscapeRenderSystem;

	// Reference counted vertex and index buffer shared among all landscape scene proxies of the same component size
	// Key is the component size and number of subsections.
	// Also being reused by GPULightmass currently to save mem
	static LANDSCAPE_API TMap<uint32, FLandscapeSharedBuffers*> SharedBuffersMap;

protected:
	// Maximum LOD level, user override possible
	int8 MaxLOD;
	int8 NumWeightmapLayerAllocations;
	uint8 StaticLightingLOD;
	uint8 VirtualTexturePerPixelHeight;
	float WeightmapSubsectionOffset;
	// Table of valid screen size -> LOD index
	TArray<float> LODScreenRatioSquared;		
	// First LOD we have batch elements for
	int32 FirstLOD;	
	// Last LOD we have batch elements for
	int32 LastLOD;					
	int32 FirstVirtualTextureLOD;
	int32 LastVirtualTextureLOD;
	// The max extend value in any axis
	float ComponentMaxExtend; 
	// 1.0 / LODBlendRange
	float InvLODBlendRange;

	FLandscapeRenderSystem::LODSettingsComponent LODSettings;

	/** 
	 * Number of subsections within the component in each dimension, this can be 1 or 2.
	 * Subsections exist to improve the speed at which LOD transitions can take place over distance.
	 */
	int32 NumSubsections;
	/** Number of unique heights in the subsection. */
	int32 SubsectionSizeQuads;
	/** Number of heightmap heights in the subsection. This includes the duplicate row at the end. */
	int32 SubsectionSizeVerts;
	/** Size of the component in unique heights. */
	int32 ComponentSizeQuads;
	/** 
	 * ComponentSizeQuads + 1.
	 * Note: in the case of multiple subsections, this is not very useful, as there will be an internal duplicate row of heights in addition to the row at the end.
	 */
	int32 ComponentSizeVerts;
	/** Number of ray tracing sections in a landscape section for better culling and optimized BLAS builds */
	int32 NumRayTracingSections;

	float StaticLightingResolution;
	/** Address of the component within the parent Landscape in unique height texels. */
	FIntPoint SectionBase;

	FMatrix LocalToWorldNoScaling;

	// Storage for static draw list batch params
	TArray<FLandscapeBatchElementParams> StaticBatchParamArray;

	bool bNaniteActive;
	bool bUsesLandscapeCulling;

	// Precomputed grass rendering MeshBatch and per-LOD params
	FMeshBatch GrassMeshBatch;
	TArray<FLandscapeBatchElementParams> GrassBatchParams;

	FVector4f WeightmapScaleBias;
	TArray<UTexture2D*> WeightmapTextures;

	UTexture2D* VisibilityWeightmapTexture;
	int32 VisibilityWeightmapChannel;

#if WITH_EDITOR
	TArray<FLinearColor> LayerColors;
	static ULandscapeEditResourcesSubsystem* LandscapeEditResourcesSubsystem;
#endif
	// Heightmap in RG and Normalmap in BA
	UTexture2D* HeightmapTexture; 
	FVector4f HeightmapScaleBias;
	float HeightmapSubsectionOffsetU;
	float HeightmapSubsectionOffsetV;

	uint32 SharedBuffersKey;
	FLandscapeSharedBuffers* SharedBuffers;
	FLandscapeVertexFactory* VertexFactory;
	FLandscapeVertexFactory* FixedGridVertexFactory;

	/** All available materials, including LOD Material, Tessellation generated materials*/
	TArray<FMaterialRenderProxy*> AvailableMaterials;

	// FLightCacheInterface
	TUniquePtr<FLandscapeLCI> ComponentLightInfo;

	/** Mapping between LOD and Material Index*/
	TArray<int8> LODIndexToMaterialIndex;
	
	/** Mapping between Material Index to Static Mesh Batch */
	TArray<int8> MaterialIndexToStaticMeshBatchLOD;

	/** Material Relevance for each material in AvailableMaterials */
	TArray<FMaterialRelevance> MaterialRelevances;

	/** Number of mips that are actually usable (that have more than 1 vertex) */
	int32 NumRelevantMips = 0;

	/** Maximum deltas between vertices and their counterparts from other mips (see additional details in ULandscapeComponent)
	*   Stored in world space to avoid useless runtime computation :
	*/
	TArray<double> WorldSpaceMipToMipMaxDeltas;

	/** Constant bias to handle the worst artifacts of the continuous LOD morphing when rendering to VSM. */
	float VirtualShadowMapConstantDepthBias;

	/** Height threshold to invalidate VSM pages when using non-Nanite landscape. */
	float VirtualShadowMapInvalidationHeightErrorThreshold;

#if WITH_EDITORONLY_DATA
	FLandscapeEditToolRenderData EditToolRenderData;
#endif

	// data used in editor or visualisers
#if WITH_EDITOR || !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	int32 CollisionMipLevel;
	int32 SimpleCollisionMipLevel;

	FCollisionResponseContainer CollisionResponse;
#endif

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	/** LightMap resolution used for VMI_LightmapDensity */
	int32 LightMapResolution;
#endif

	TUniformBuffer<FLandscapeUniformShaderParameters> LandscapeUniformShaderParameters;
	
	// Cached versions of these
	FMatrix WorldToLocal;

#if !UE_BUILD_SHIPPING
	FName DebugName;
#endif // !UE_BUILD_SHIPPING

protected:
	LANDSCAPE_API virtual ~FLandscapeComponentSceneProxy();
	
	LANDSCAPE_API bool GetMeshElementForFixedGrid(int32 InLodIndex, FMaterialRenderProxy* InMaterialInterface, FMeshBatch& OutMeshBatch, TArray<FLandscapeBatchElementParams>& OutStaticBatchParamArray) const;
	template<class ArrayType> bool GetStaticMeshElement(int32 LODIndex, bool bForToolMesh, FMeshBatch& MeshBatch, ArrayType& OutStaticBatchParamArray) const;

	LANDSCAPE_API virtual void OnForceHiddenChanged() override;

public:
	// constructor
	LANDSCAPE_API FLandscapeComponentSceneProxy(ULandscapeComponent* InComponent);

	// FPrimitiveSceneProxy interface.
	LANDSCAPE_API virtual void DrawStaticElements(FStaticPrimitiveDrawInterface* PDI) override;
	LANDSCAPE_API virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;
	LANDSCAPE_API virtual void ApplyViewDependentMeshArguments(const FSceneView& View, FMeshBatch& ViewDependentMeshBatch) const override;
	virtual uint32 GetMemoryFootprint() const override { return(sizeof(*this) + GetAllocatedSize()); }
	LANDSCAPE_API virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;
	LANDSCAPE_API virtual bool CanBeOccluded() const override;
	LANDSCAPE_API virtual void GetLightRelevance(const FLightSceneProxy* LightSceneProxy, bool& bDynamic, bool& bRelevant, bool& bLightMapped, bool& bShadowMapped) const override;
	LANDSCAPE_API virtual void OnTransformChanged(FRHICommandListBase& RHICmdList) override;
	LANDSCAPE_API virtual void CreateRenderThreadResources(FRHICommandListBase& RHICmdList) override;
	LANDSCAPE_API virtual void DestroyRenderThreadResources() override;

	friend class ULandscapeComponent;
	friend class FLandscapeVertexFactoryVertexShaderParameters;
	friend class FLandscapeXYOffsetVertexFactoryVertexShaderParameters;
	friend class FLandscapeVertexFactoryPixelShaderParameters;
	friend struct FLandscapeBatchElementParams;

	const FMeshBatch& GetGrassMeshBatch() const { return GrassMeshBatch; }

	// FLandcapeSceneProxy
	LANDSCAPE_API virtual bool HeightfieldHasPendingStreaming() const override;

	LANDSCAPE_API virtual void GetHeightfieldRepresentation(UTexture2D*& OutHeightmapTexture, UTexture2D*& OutVisibilityTexture, FHeightfieldComponentDescription& OutDescription) const override;

	LANDSCAPE_API virtual void GetLCIs(FLCIArray& LCIs) override;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	virtual int32 GetLightMapResolution() const override { return LightMapResolution; }
#endif

#if RHI_RAYTRACING
	LANDSCAPE_API virtual void GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector) override final;
	virtual bool HasRayTracingRepresentation() const override { return true; }
	virtual bool IsRayTracingRelevant() const override { return true; }
#endif

	// FLandscapeSectionInfo interface
	LANDSCAPE_API virtual float ComputeLODForView(const FSceneView& InView) const override;
	LANDSCAPE_API virtual float ComputeLODBias() const override;
	LANDSCAPE_API virtual void OnRenderCoordsChanged(FRHICommandListBase& RHICmdList) override;
	LANDSCAPE_API virtual int32 GetComponentResolution() const override;
	LANDSCAPE_API virtual double ComputeSectionResolution() const override;
	LANDSCAPE_API virtual void GetSectionBoundsAndLocalToWorld(FBoxSphereBounds& LocalBounds, FMatrix& LocalToWorld) const override;
	LANDSCAPE_API virtual void GetSectionCenterAndVectors(FVector& OutSectionCenterWorldSpace, FVector& OutSectionXVectorWorldSpace, FVector& OutSectionYVectorWorldSpace) const override;
	LANDSCAPE_API virtual const FPrimitiveSceneInfo* GetPrimitiveSceneInfo() const override;
	LANDSCAPE_API virtual bool ShouldInvalidateShadows(const FSceneView& InView, float InLODValue, float InLastShadowInvalidationLODValue) const override;

#if RHI_RAYTRACING
	LANDSCAPE_API virtual void RayTracingNotifyViewDeleted(uint32 ViewKey) override;
#endif

};

class FLandscapeDebugMaterialRenderProxy : public FMaterialRenderProxy
{
public:
	const FMaterialRenderProxy* const Parent;
	const UTexture2D* RedTexture;
	const UTexture2D* GreenTexture;
	const UTexture2D* BlueTexture;
	const FLinearColor R;
	const FLinearColor G;
	const FLinearColor B;

	/** Initialization constructor. */
	FLandscapeDebugMaterialRenderProxy(const FMaterialRenderProxy* InParent, const UTexture2D* TexR, const UTexture2D* TexG, const UTexture2D* TexB,
		const FLinearColor& InR, const FLinearColor& InG, const FLinearColor& InB) :
		FMaterialRenderProxy(InParent->GetMaterialName()),
		Parent(InParent),
		RedTexture(TexR),
		GreenTexture(TexG),
		BlueTexture(TexB),
		R(InR),
		G(InG),
		B(InB)
	{}

	// FMaterialRenderProxy interface.
	virtual const FMaterial* GetMaterialNoFallback(ERHIFeatureLevel::Type InFeatureLevel) const override
	{
		return Parent->GetMaterialNoFallback(InFeatureLevel);
	}

	virtual const FMaterialRenderProxy* GetFallback(ERHIFeatureLevel::Type InFeatureLevel) const override
	{
		return Parent->GetFallback(InFeatureLevel);
	}

	virtual bool GetParameterValue(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, FMaterialParameterValue& OutValue, const FMaterialRenderContext& Context) const
	{
		switch (Type)
		{
		case EMaterialParameterType::Vector:
			if (ParameterInfo.Name == FName("Landscape_RedMask"))
			{
				OutValue = R;
				return true;
			}
			else if (ParameterInfo.Name == FName("Landscape_GreenMask"))
			{
				OutValue = G;
				return true;
			}
			else if (ParameterInfo.Name == FName("Landscape_BlueMask"))
			{
				OutValue = B;
				return true;
			}
			break;
		case EMaterialParameterType::Texture:
			if (ParameterInfo.Name == FName("Landscape_RedTexture"))
			{
				OutValue = RedTexture;
				return true;
			}
			else if (ParameterInfo.Name == FName("Landscape_GreenTexture"))
			{
				OutValue = GreenTexture;
				return true;
			}
			else if (ParameterInfo.Name == FName("Landscape_BlueTexture"))
			{
				OutValue = BlueTexture;
				return true;
			}
			break;
		default:
			break;
		}
		return Parent->GetParameterValue(Type, ParameterInfo, OutValue, Context);
	}
};

class FLandscapeSelectMaterialRenderProxy : public FMaterialRenderProxy
{
public:
	const FMaterialRenderProxy* const Parent;
	const UTexture2D* SelectTexture;

	/** Initialization constructor. */
	FLandscapeSelectMaterialRenderProxy(const FMaterialRenderProxy* InParent, const UTexture2D* InTexture) :
		FMaterialRenderProxy(InParent->GetMaterialName()),
		Parent(InParent),
		SelectTexture(InTexture)
	{}

	// FMaterialRenderProxy interface.
	virtual const FMaterial* GetMaterialNoFallback(ERHIFeatureLevel::Type InFeatureLevel) const override
	{
		return Parent->GetMaterialNoFallback(InFeatureLevel);
	}
	virtual const FMaterialRenderProxy* GetFallback(ERHIFeatureLevel::Type InFeatureLevel) const override
	{
		return Parent->GetFallback(InFeatureLevel);
	}

	virtual bool GetParameterValue(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, FMaterialParameterValue& OutValue, const FMaterialRenderContext& Context) const
	{
		switch (Type)
		{
		case EMaterialParameterType::Vector:
			if (ParameterInfo.Name == FName("HighlightColor"))
			{
				OutValue = FLinearColor(1.f, 0.5f, 0.5f);
				return true;
			}
			break;
		case EMaterialParameterType::Texture:
			if (ParameterInfo.Name == FName("SelectedData"))
			{
				OutValue = SelectTexture;
				return true;
			}
			break;
		default:
			break;
		}
		return Parent->GetParameterValue(Type, ParameterInfo, OutValue, Context);
	}
};

class FLandscapeMaskMaterialRenderProxy : public FMaterialRenderProxy
{
public:
	const FMaterialRenderProxy* const Parent;
	const UTexture2D* SelectTexture;
	const bool bInverted;

	/** Initialization constructor. */
	FLandscapeMaskMaterialRenderProxy(const FMaterialRenderProxy* InParent, const UTexture2D* InTexture, const bool InbInverted) :
		FMaterialRenderProxy(InParent->GetMaterialName()),
		Parent(InParent),
		SelectTexture(InTexture),
		bInverted(InbInverted)
	{}

	// FMaterialRenderProxy interface.
	virtual const FMaterial* GetMaterialNoFallback(ERHIFeatureLevel::Type InFeatureLevel) const override
	{
		return Parent->GetMaterialNoFallback(InFeatureLevel);
	}
	virtual const FMaterialRenderProxy* GetFallback(ERHIFeatureLevel::Type InFeatureLevel) const override
	{
		return Parent->GetFallback(InFeatureLevel);
	}
	
	virtual bool GetParameterValue(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, FMaterialParameterValue& OutValue, const FMaterialRenderContext& Context) const
	{
		switch (Type)
		{
		case EMaterialParameterType::Scalar:
			if (ParameterInfo.Name == FName("bInverted"))
			{
				OutValue = (float)bInverted;
				return true;
			}
			break;
		case EMaterialParameterType::Texture:
			if (ParameterInfo.Name == FName("SelectedData"))
			{
				OutValue = SelectTexture;
				return true;
			}
			break;
		default:
			break;
		}
		return Parent->GetParameterValue(Type, ParameterInfo, OutValue, Context);
	}
};

class FLandscapeLayerUsageRenderProxy : public FMaterialRenderProxy
{
	const FMaterialRenderProxy* const Parent;

	int32 ComponentSizeVerts;
	TArray<FLinearColor> LayerColors;
	float Rotation;
public:
	FLandscapeLayerUsageRenderProxy(const FMaterialRenderProxy* InParent, int32 InComponentSizeVerts, const TArray<FLinearColor>& InLayerColors, float InRotation)
	: FMaterialRenderProxy(InParent->GetMaterialName())
	, Parent(InParent)
	, ComponentSizeVerts(InComponentSizeVerts)
	, LayerColors(InLayerColors)
	, Rotation(InRotation)
	{}

	// FMaterialRenderProxy interface.
	virtual const FMaterial* GetMaterialNoFallback(ERHIFeatureLevel::Type InFeatureLevel) const override
	{
		return Parent->GetMaterialNoFallback(InFeatureLevel);
	}
	virtual const FMaterialRenderProxy* GetFallback(ERHIFeatureLevel::Type InFeatureLevel) const override
	{
		return Parent->GetFallback(InFeatureLevel);
	}
	
	virtual bool GetParameterValue(EMaterialParameterType Type, const FHashedMaterialParameterInfo& ParameterInfo, FMaterialParameterValue& OutValue, const FMaterialRenderContext& Context) const
	{
		static FName ColorNames[] =
		{
			FName("Color0"),
			FName("Color1"),
			FName("Color2"),
			FName("Color3"),
			FName("Color4"),
			FName("Color5"),
			FName("Color6"),
			FName("Color7"),
			FName("Color8"),
			FName("Color9"),
			FName("Color10"),
			FName("Color11"),
			FName("Color12"),
			FName("Color13"),
			FName("Color14"),
			FName("Color15")
		};

		switch (Type)
		{
		case EMaterialParameterType::Vector:
			for (int32 i = 0; i < UE_ARRAY_COUNT(ColorNames) && i < LayerColors.Num(); i++)
			{
				if (ParameterInfo.Name == ColorNames[i])
				{
					OutValue = LayerColors[i];
					return true;
				}
			}
			break;
		case EMaterialParameterType::Scalar:
			if (ParameterInfo.Name == FName("Rotation"))
			{
				OutValue = Rotation;
				return true;
			}
			else if (ParameterInfo.Name == FName("NumStripes"))
			{
				OutValue = (float)LayerColors.Num();
				return true;
			}
			else if (ParameterInfo.Name == FName("ComponentSizeVerts"))
			{
				OutValue = (float)ComponentSizeVerts;
				return true;
			}
			break;
		default:
			break;
		}

		return Parent->GetParameterValue(Type, ParameterInfo, OutValue, Context);
	}
};
