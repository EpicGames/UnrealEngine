// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "Delegates/Delegate.h"
#include "LandscapeEditTypes.h"
#include "LandscapeSettings.generated.h"

class UMaterialInterface;
class ULandscapeLayerInfoObject;

UENUM()
enum class ELandscapeDirtyingMode : uint8
{
	/** Auto : 
	 Landscapes that are marked as needing to be resaved will appear in the Choose files to save dialog.
	 Changes are saved whenever the Landscape requires it.*/
	Auto,
	/** In Landscape Mode Only : 
	 Landscapes that are marked as needing to be resaved will not appear in the Choose files to save dialog.
	 This is a manual saving mode that puts the responsibility on the user to avoid file contention with other team members.
	 The viewport will display an error message indicating that landscape actors are not up-to-date and need to be resaved. This is done using Build > Save Modified Landscapes (or Build > Build Landscape). */
	InLandscapeModeOnly,
	/** In Landscape Mode And User Triggered Changes : 
	 Landscapes that are marked as needing to be resaved will not appear in the Choose files to save dialog.
	 However, any user-triggered changes (direct or indirect) will require the Landscape to be resaved.
	 This mode is recommended for team collaboration as it provides the best features of the other two modes while ensuring that modified landscape actors are still saved and properly submitted to source control. */
	InLandscapeModeAndUserTriggeredChanges
};

UENUM()
enum class EFoliageLandscapeSnappingMode : uint8
{
	/** Foliage will not ever be snapped to the landscape */
	DontSnap,
	/** Foliage will only be snapped on areas where the landscape sculpt brush tool was used. This doesn't include procedural changes (e.g. water brush) or global updates (e.g. deactivating a layer) */
	SnapUnderBrushOnly,
	/** Foliage will be snapped onto the entire landscape upon update. This includes procedural changes (e.g. water brush) or global updates (e.g. deactivating a layer) */
	SnapEverywhere
};


UCLASS(config = Engine, defaultconfig, meta = (DisplayName = "Landscape"), MinimalAPI)
class ULandscapeSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	/** Returns true if landscape resolution should be constrained. */
	bool IsLandscapeResolutionRestricted() const { return InRestrictiveMode(); }

	/** Returns true if blueprint landscape tools usage is allowed */
	bool AreBlueprintToolsAllowed() const { return !InRestrictiveMode(); }
	
	/** Returns the current landscape resolution limit. */
	int32 GetTotalResolutionLimit() const { return SideResolutionLimit * SideResolutionLimit; }

	bool InRestrictiveMode() const { return bRestrictiveMode; }
	void SetRestrictiveMode(bool bEnabled) { bRestrictiveMode = bEnabled; }

	int32 GetSideResolutionLimit() const { return SideResolutionLimit; }

	float GetBrushSizeUIMax() const { return BrushSizeUIMax; }
	float GetBrushSizeClampMax() const { return BrushSizeClampMax; }

	int32 GetHLODMaxTextureSize() const { return HLODMaxTextureSize; }

	float GetSplineIconWorldZOffset() const { return SplineIconWorldZOffset; }
	float GetSplineIconScale() const { return SplineIconScale;  }

	/** Returns the default landscape material that should be used when creating a new landscape. */
	TSoftObjectPtr<UMaterialInterface> GetDefaultLandscapeMaterial() const { return DefaultLandscapeMaterial; }

	/** Returns the default landscape layer info object that will be assigned to unset layers when creating a new landscape. */
	TSoftObjectPtr<ULandscapeLayerInfoObject> GetDefaultLayerInfoObject() const { return DefaultLayerInfoObject; }

	bool GetShouldUpdateEditLayersDuringInteractiveChanges() const { return bShouldUpdateEditLayersDuringInteractiveChanges; }

	EFoliageLandscapeSnappingMode GetFoliageLandscapeSnappingMode() const { return FoliageLandscapeSnappingMode; }

	bool ShouldDisplayTargetLayerThumbnails() const { return bDisplayTargetLayerThumbnails; }

	bool ShouldDisableTemporalAntiAliasingInLandscapeMode() const { return bDisableTemporalAntiAliasingInLandscapeMode; }

	ELandscapeTargetLayerBlendMethod GetTargetLayerDefaultBlendMethod() const { return TargetLayerDefaultBlendMethod; }

#if WITH_EDITOR
	//~ Begin UObject Interface.
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PreEditUndo() override;
	virtual void PostEditUndo() override;
#endif // WITH_EDITOR

public:
	UPROPERTY(config, EditAnywhere, Category = "Edit Layers", meta=(UIMin = "1", UIMax = "32", ClampMin = "1", ClampMax = "32", ToolTip = "This option controls the maximum editing layers that can be added to a Landscape"))
	int32 MaxNumberOfLayers = 8;

	UPROPERTY(EditAnywhere, config, Category = "Edit Layers", meta = (ToolTip = 
		"When true, automatic edit layer creation pops up a dialog where the new layer can be reordered relative to other layers."))
	bool bShowDialogForAutomaticLayerCreation = true;

	UPROPERTY(config, EditAnywhere, Category = "Configuration", meta=(ToolTip = "Maximum Dimension of Landscape in Components"))
	int32 MaxComponents = 256;

	UPROPERTY(config, EditAnywhere, Category = "Configuration", meta = (UIMin = "1", UIMax = "1024", ClampMin = "1", ClampMax = "1024", ToolTip = "Maximum Size of Import Image Cache in MB"))
	uint32 MaxImageImportCacheSizeMegaBytes = 256;

	UPROPERTY(config, EditAnywhere, Category = "Configuration", meta = (UIMin = "0.0", UIMax = "10.0", ClampMin = "0.0", ClampMax = "10.0", ToolTip = "Exponent for the Paint Tool Strength"))
	float PaintStrengthGamma = 2.2f;

	UPROPERTY(config, EditAnywhere, Category = "Configuration", meta = (ToolTip = "Disable Painting Startup Slowdown"))
	bool bDisablePaintingStartupSlowdown = true;
	
	/** Defines when the engine requires the landscape actors to be resaved */
	UPROPERTY(Config, Category = "Configuration", EditAnywhere)
	ELandscapeDirtyingMode LandscapeDirtyingMode = ELandscapeDirtyingMode::InLandscapeModeAndUserTriggeredChanges;

protected:
	UPROPERTY(config)
	int32 SideResolutionLimit = 2048;

	UPROPERTY(EditAnywhere, config, Category = "Materials", meta = (ToolTip = "Default Landscape Material will be prefilled when creating a new landscape."))
	TSoftObjectPtr<UMaterialInterface> DefaultLandscapeMaterial;

	UPROPERTY(EditAnywhere, config, Category = "Target Layers", meta = (ToolTip = "Default Layer Info Object"))
	TSoftObjectPtr<ULandscapeLayerInfoObject> DefaultLayerInfoObject;

	UPROPERTY(EditAnywhere, config, Category = "Configuration", meta = (ToolTip = "Maximum size that can be set via the slider for the landscape sculpt/paint brushes"))
	float BrushSizeUIMax = 8192;

	UPROPERTY(EditAnywhere, config, Category = "Configuration", meta = (ToolTip = "Maximum size that can be set manually for the landscape sculpt/paint brushes"))
	float BrushSizeClampMax = 65536;

	UPROPERTY(EditAnywhere, config, Category = "HLOD", meta = (DisplayName = "HLOD Max Texture Size", ClampMin = "64", ClampMax = "8192", ToolTip = "Maximum size of the textures generated for landscape HLODs"))
	int32 HLODMaxTextureSize = 1024;

	/** For landscape layers-affecting changes, allows the landscape to be updated when performing an interactive change (e.g. when changing an edit layer's alpha). Set to false if the performance when editing gets too bad (the landscape will be properly updated when the dragging operation is done). */
	UPROPERTY(EditAnywhere, config, Category = "Edit Layers")
	bool bShouldUpdateEditLayersDuringInteractiveChanges = false;

	UPROPERTY(EditAnywhere, config, Category = "Edit Layers")
	EFoliageLandscapeSnappingMode FoliageLandscapeSnappingMode = EFoliageLandscapeSnappingMode::DontSnap;

	UPROPERTY(transient)
	bool bRestrictiveMode = false;

	/** Offset in Z for the landscape spline point icon in world-space. */
	UPROPERTY(EditAnywhere, config, Category = "Spline", meta = (ClampMin = "0", UIMax = "4096"))
	float SplineIconWorldZOffset = 100.0f;

	UPROPERTY(EditAnywhere, config, Category = "Spline", meta = (ClampMin = "32", UIMax = "2048"))
	/** Size of the landscape spline control point icon in the viewport. */
	float SplineIconScale = 125.0f;

	UPROPERTY(EditAnywhere, config, Category = "Target Layers", meta = (ToolTip = 
		"When true, each target layer will have a representative thumbnail in landscape mode. However, the thumbnail system can be slow. Plus, certain landscape materials (e.g. using RVT) are not able to properly render thumbnails. Thus, setting this to false will skip needlessly rendering landscape layer thumbnails, which can improve the editing experience"))
	bool bDisplayTargetLayerThumbnails = true;

	UPROPERTY(EditAnywhere, config, Category = "Configuration", meta = (ToolTip = 
		"When true, temporal anti-aliasing will be inactive while in landscape mode. This avoids the ghosting effect on the landscape brush but can lead to aliasing or shimmering on other parts of the image."))
	bool bDisableTemporalAntiAliasingInLandscapeMode = true;

	/** Target layer blend method to use for newly created Landscape Layer Info assets. Changing this setting will not affect existing Landscape Layer Info assets. This is only used when DefaultLayerInfoObject isn't set. */
	UPROPERTY(Config, Category = "Target Layers", EditAnywhere)
	ELandscapeTargetLayerBlendMethod TargetLayerDefaultBlendMethod = ELandscapeTargetLayerBlendMethod::None;

private:
	TOptional<bool> DisplayTargetLayerThumbnailsBeforeUndo;
};
