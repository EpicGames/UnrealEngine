// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EngineUtils.h"

class ULandscapeSplineControlPoint;
class ULandscapeSplinesComponent;
class ULandscapeSplineSegment;

//////////////////////////////////////////////////////////////////////////
// LANDSCAPE SPLINES HIT PROXY

struct HLandscapeSplineProxy : public HActor
{
	DECLARE_HIT_PROXY( LANDSCAPE_API );

	HLandscapeSplineProxy(ULandscapeSplinesComponent* SplineComponent, EHitProxyPriority InPriority = HPP_Wireframe);
	virtual EMouseCursor::Type GetMouseCursor() override;
};

struct HLandscapeSplineProxy_Segment : public HLandscapeSplineProxy
{
	DECLARE_HIT_PROXY( LANDSCAPE_API );

	TObjectPtr<ULandscapeSplineSegment> SplineSegment;

	HLandscapeSplineProxy_Segment(ULandscapeSplineSegment* InSplineSegment);
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
};

struct HLandscapeSplineProxy_ControlPoint : public HLandscapeSplineProxy
{
	DECLARE_HIT_PROXY( LANDSCAPE_API );

	TObjectPtr<ULandscapeSplineControlPoint> ControlPoint;

	HLandscapeSplineProxy_ControlPoint(ULandscapeSplineControlPoint* InControlPoint);
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
};

struct HLandscapeSplineProxy_Tangent : public HLandscapeSplineProxy
{
	DECLARE_HIT_PROXY( LANDSCAPE_API );

	TObjectPtr<ULandscapeSplineSegment> SplineSegment;
	uint32 End:1;

	LANDSCAPE_API HLandscapeSplineProxy_Tangent(ULandscapeSplineSegment* InSplineSegment, bool InEnd);
	LANDSCAPE_API virtual void Serialize(FArchive& Ar);

	virtual EMouseCursor::Type GetMouseCursor() override;
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
};
