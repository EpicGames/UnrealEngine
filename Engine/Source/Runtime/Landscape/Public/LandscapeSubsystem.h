// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Subsystems/WorldSubsystem.h"
#include "LandscapeProxy.h"
#include "LandscapeSubsystem.generated.h"

class ALandscapeProxy;
class ALandscape;
class ALandscapeStreamingProxy;
class AWorldSettings;
class IConsoleVariable;
class ULandscapeInfo;
class FLandscapeNotificationManager;
class ULandscapeComponent;
class FLandscapeGrassMapsBuilder;
class FLandscapeTextureStreamingManager;
struct FActionableMessage;
struct FDateTime;
struct FScopedSlowTask;
class ULandscapeHeightmapTextureEdgeFixup;
struct FLandscapeGroup;
enum class EUpdateTransformFlags : int32;

namespace UE::Landscape
{
	enum class EOutdatedDataFlags : uint8;
	enum class EBuildFlags : uint8;

#if WITH_EDITOR
	/** Returns true if there are some landscapes in the editor world that have been automatically modified and are in need of being saved (see LandscapeDirtyingMode) */
	bool LANDSCAPE_API HasModifiedLandscapes();

	/** Dirties and saves the landscapes in the editor world that have been automatically modified and are in need of being saved (see LandscapeDirtyingMode) */
	void LANDSCAPE_API SaveModifiedLandscapes(EBuildFlags InBuildFlags);
	void LANDSCAPE_API MarkModifiedLandscapesAsDirty(EBuildFlags InBuildFlags);
	void LANDSCAPE_API BuildGrassMaps(EBuildFlags InBuildFlags);
	void LANDSCAPE_API BuildPhysicalMaterial(EBuildFlags InBuildFlags);
	void LANDSCAPE_API BuildNanite(EBuildFlags InBuildFlags);
	void LANDSCAPE_API BuildAll(EBuildFlags InBuildFlags);
#endif // WITH_EDITOR

	namespace Nanite
	{
		struct FAsyncBuildData;
	}
} // end of namespace UE::Landscape

#if WITH_EDITOR
struct FOnHeightmapStreamedContext
{
private:
	const ALandscape* Landscape = nullptr;
	const FBox2D& UpdateRegion;
	const TSet<class ULandscapeComponent*>& LandscapeComponentsInvolved;

public:
	const ALandscape* GetLandscape() const 
	{ 
		return Landscape; 
	}

	const FBox2D& GetUpdateRegion() const 
	{ 
		return UpdateRegion; 
	}

	const TSet<class ULandscapeComponent*>& GetLandscapeComponentsInvolved() const 
	{ 
		return LandscapeComponentsInvolved; 
	}

	FOnHeightmapStreamedContext(const ALandscape* InLandscape, const FBox2D& InUpdateRegion, const TSet<class ULandscapeComponent*>& InLandscapeComponentsInvolved)
		: Landscape(InLandscape)
		, UpdateRegion(InUpdateRegion)
		, LandscapeComponentsInvolved(InLandscapeComponentsInvolved)
	{}
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnHeightmapStreamedDelegate, const FOnHeightmapStreamedContext& context);
#endif // WITH_EDITOR

UCLASS(MinimalAPI)
class ULandscapeSubsystem : public UTickableWorldSubsystem
{
	GENERATED_BODY()

public:
	ULandscapeSubsystem();
	virtual ~ULandscapeSubsystem();

	void RegisterActor(ALandscapeProxy* Proxy);
	void UnregisterActor(ALandscapeProxy* Proxy);

	// Begin FTickableGameObject overrides
	virtual void Tick(float DeltaTime) override;
	virtual bool IsTickableInEditor() const override { return true; }
	virtual ETickableTickType GetTickableTickType() const override;
	virtual bool DoesSupportWorldType(const EWorldType::Type WorldType) const override;
	virtual TStatId GetStatId() const override;
	// End FTickableGameObject overrides

	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	// setting this to true causes grass instance generation to go wider (multiplies the limits by GGrassCreationPrioritizedMultipler)
	void PrioritizeGrassCreation(bool bPrioritizeGrassCreation) { bIsGrassCreationPrioritized = bPrioritizeGrassCreation; }
	bool IsGrassCreationPrioritized() const { return bIsGrassCreationPrioritized; }
	FLandscapeGrassMapsBuilder* GetGrassMapBuilder() { return GrassMapsBuilder; }
	FLandscapeTextureStreamingManager* GetTextureStreamingManager() { return TextureStreamingManager; }

	/**
	 * Can be called at runtime : (optionally) flushes grass on all landscape components and updates them
	 * @param bInFlushGrass : flushes all grass from landscape components prior to updating them
	 * @param bInForceSync : synchronously updates grass on all landscape components 
	 * @param InOptionalCameraLocations : (optional) camera locations that should be used when updating the grass. If not specified, the usual (streaming manager-based) view locations will be used
	 */
	LANDSCAPE_API void RegenerateGrass(bool bInFlushGrass, bool bInForceSync, TOptional<TArrayView<FVector>> InOptionalCameraLocations = TOptional<TArrayView<FVector>>());

	// Remove all grass instances from the specified components.  If passed null, removes all grass instances from all proxies.
	void RemoveGrassInstances(const TSet<ULandscapeComponent*>* ComponentsToRemoveGrassInstances = nullptr);

	// called when components are registered to the world	
	void RegisterComponent(ULandscapeComponent* Component);
	void UnregisterComponent(ULandscapeComponent* Component);

	void OnLevelRemovedFromWorld(ULevel* InLevel, UWorld* InWorld);

#if UE_ENABLE_DEBUG_DRAWING
	bool AnyViewShowCollisions() const { return bAnyViewShowCollisions; }  //! Returns true if any view has view collisions enabled.
#endif // UE_ENABLE_DEBUG_DRAWING

#if WITH_EDITOR
	LANDSCAPE_API void BuildAll(UE::Landscape::EBuildFlags InBuildFlags);

	// Synchronously build grass maps for all components
	LANDSCAPE_API void BuildGrassMaps(UE::Landscape::EBuildFlags InBuildFlags);

	LANDSCAPE_API void BuildPhysicalMaterial(UE::Landscape::EBuildFlags InBuildFlags);

	/**
	 * Updates the Nanite mesh on all landscape actors whose mesh is not up to date.
	 * @param InProxiesToBuild - If specified, only the Nanite meshes of the specified landscape actors (recursively for all streaming proxies, in the case of a 1 ALandscape / N ALandscapeStreamingProxy setup) will be built
	 * @param InBuildFlags - Various flags for driving the build
	 */
	LANDSCAPE_API void BuildNanite(UE::Landscape::EBuildFlags InBuildFlags, TArrayView<ALandscapeProxy*> InProxiesToBuild = TArrayView<ALandscapeProxy*>());
	
	LANDSCAPE_API TArray<TTuple<ALandscapeProxy*, UE::Landscape::EOutdatedDataFlags>> GetOutdatedProxyDetails(UE::Landscape::EOutdatedDataFlags InMatchingOutdatedDataFlags, bool bInMustMatchAllFlags) const;
	
	LANDSCAPE_API bool IsGridBased() const;
	LANDSCAPE_API void ChangeGridSize(ULandscapeInfo* LandscapeInfo, uint32 NewGridSizeInComponents);
	LANDSCAPE_API ALandscapeProxy* FindOrAddLandscapeProxy(ULandscapeInfo* LandscapeInfo, const FIntPoint& SectionBase);

	LANDSCAPE_API bool GetActionableMessage(FActionableMessage& OutActionableMessage);
	LANDSCAPE_API void MarkModifiedLandscapesAsDirty(UE::Landscape::EBuildFlags InBuildFlags);
	/** Dirties and saves the landscapes in the editor world that have been automatically modified and are in need of being saved (see LandscapeDirtyingMode) */
	LANDSCAPE_API void SaveModifiedLandscapes(UE::Landscape::EBuildFlags InBuildFlags);
	/** Returns true if there are some landscapes in the editor world that have been automatically modified and are in need of being saved (see LandscapeDirtyingMode) */
	LANDSCAPE_API bool HasModifiedLandscapes() const;
	LANDSCAPE_API bool GetDirtyOnlyInMode() const;
	
	FLandscapeNotificationManager* GetNotificationManager() 
	{ 
		return NotificationManager; 
	}

	FOnHeightmapStreamedDelegate::RegistrationType& OnHeightmapStreamed()
	{ 
		return OnHeightmapStreamedDelegate;
	}

	FOnLandscapeProxyComponentDataChanged::RegistrationType& OnLandscapeProxyComponentDataChanged() const
	{
		return OnLandscapeProxyComponentDataChangedDelegate;
	}

	FOnLandscapeProxyMaterialChanged::RegistrationType& OnLandscapeProxyMaterialChanged() const
	{
		return OnLandscapeProxyMaterialChangedDelegate;
	}

	FDateTime GetAppCurrentDateTime();

	TSharedRef<UE::Landscape::Nanite::FAsyncBuildData> CreateTrackedNaniteBuildState(ALandscapeProxy* LandscapeProxy, int32 InLODToExport, const TArray<ULandscapeComponent*>& InComponentsToExport);

	enum class EFinishAllNaniteBuildsInFlightFlags
	{
		Default = 0x00,
		DisplaySlowTaskDialog = 0x01,
		AllowCancel = 0x02,
	};

	void AddNaniteFinalizeBuildEvent(FGraphEventRef InNaniteFinalizeBuildEvent);

	// returns true if all nanite builds were completed (false if cancelled, or failed to complete)
	bool FinishAllNaniteBuildsInFlightNow(EFinishAllNaniteBuildsInFlightFlags FinishFlags);

	// Returns true if we should build nanite meshes in parallel asynchronously. 
	bool IsMultithreadedNaniteBuildEnabled();

	// Returns true if the user has requested Nanite Meshes to be generated on landscape edit. If we return false then the nanite build will happen either on map save or explicit build 
	bool IsLiveNaniteRebuildEnabled();

	bool AreNaniteBuildsInProgress() const;
	void IncNaniteBuild();
	void DecNaniteBuild();

	// Wait unit we're able to continue a landscape export task (Max concurrent nanite mesh builds is defined by  landscape.Nanite.MaxSimultaneousMultithreadBuilds and landscape.Nanite.MultithreadBuild CVars)
	void WaitLaunchNaniteBuild(); 

	// Helper class reserved for friends that are allowed to fire the subsystem's inner callbacks
	class FDelegateAccess
	{
		friend ULandscapeSubsystem;
		friend ALandscape;
		friend ALandscapeProxy;

		FDelegateAccess(FOnHeightmapStreamedDelegate& InOnHeightmapStreamed, FOnLandscapeProxyComponentDataChanged& InOnLandscapeProxyComponentDataChanged, FOnLandscapeProxyMaterialChanged& InOnLandscapeProxyMaterialChanged);

		FOnHeightmapStreamedDelegate& OnHeightmapStreamedDelegate;
		FOnLandscapeProxyComponentDataChanged& OnLandscapeProxyComponentDataChangedDelegate;
		FOnLandscapeProxyMaterialChanged& OnLandscapeProxyMaterialChangedDelegate;
	};
	FDelegateAccess GetDelegateAccess() const;
#endif // WITH_EDITOR

	// runs per-tick edge fixup on ALL landscape groups in the subsystem
	void TickEdgeFixup();

	FLandscapeGroup* GetLandscapeGroupForProxy(TObjectPtr<ALandscapeProxy> Proxy);
	FLandscapeGroup* GetLandscapeGroupForComponent(ULandscapeComponent* Component);

	LANDSCAPE_API void ForEachLandscapeInfo(TFunctionRef<bool(ULandscapeInfo*)> ForEachLandscapeInfoFunc) const;

private:

	// Begin USubsystem
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	// End USubsystem

	void OnNaniteWorldSettingsChanged(AWorldSettings* WorldSettings) { RegenerateGrass(true, true); }
	void OnNaniteEnabledChanged(IConsoleVariable*);

	void HandlePostGarbageCollect();

	TSet<UPackage*> GetDirtyLandscapeProxyPackages() const;

	// When proxies move, we may need to update their position in the landscape group
	void OnProxyMoved(USceneComponent*, EUpdateTransformFlags, ETeleportType);

#if WITH_EDITOR
	void TickNaniteFinalizeBuildEvents();
#endif // WITH_EDITOR

private:
	// LODGroupKey --> Landscape Group
	TMap<uint32, FLandscapeGroup*> Groups;

	// list of streaming proxies that need to re-register with their group because they moved, or changed their LODGroupKey
	UPROPERTY(Transient, DuplicateTransient, NonTransactional)
	TSet<TObjectPtr<ALandscapeStreamingProxy>> StreamingProxiesNeedingReregister;

	bool bIsGrassCreationPrioritized = false;

	UPROPERTY(Transient, DuplicateTransient, NonTransactional)
	TArray<TObjectPtr<ALandscape>> LandscapeActors;

	// UPROPERTY ensures these objects are not deleted before being unregistered
	// (technically not necessary, as actors should always unregister prior to deletion)
	UPROPERTY(Transient, DuplicateTransient, NonTransactional)
	TArray<TObjectPtr<ALandscapeProxy>> Proxies;

	FDelegateHandle OnNaniteWorldSettingsChangedHandle;

	FLandscapeTextureStreamingManager* TextureStreamingManager = nullptr;
	FLandscapeGrassMapsBuilder* GrassMapsBuilder = nullptr;

	int32 LastVisualTickFrameNumber = -1;

#if WITH_EDITORONLY_DATA
	mutable FOnHeightmapStreamedDelegate OnHeightmapStreamedDelegate;
	mutable FOnLandscapeProxyComponentDataChanged OnLandscapeProxyComponentDataChangedDelegate;
	mutable FOnLandscapeProxyMaterialChanged OnLandscapeProxyMaterialChangedDelegate;

	class FLandscapePhysicalMaterialBuilder* PhysicalMaterialBuilder = nullptr;
	FLandscapeNotificationManager* NotificationManager = nullptr;
	FDateTime AppCurrentDateTime; // Represents FDateTime::Now(), at the beginning of the frame (useful to get a human-readable date/time that is fixed during the frame)

	// A list of graph events that track the status of 
	TArray<FGraphEventRef> NaniteFinalizeBuildEvents;
	TArray<TSharedRef<UE::Landscape::Nanite::FAsyncBuildData>> NaniteMeshBuildStates;
	float NumNaniteMeshUpdatesAvailable = 0.0f;

	std::atomic<int32> NaniteBuildsInFlight;
	std::atomic<int32> NaniteStaticMeshesInFlight;

#endif // WITH_EDITORONLY_DATA
	
#if UE_ENABLE_DEBUG_DRAWING
	bool bAnyViewShowCollisions = false;
#endif // UE_ENABLE_DEBUG_DRAWING

	FDelegateHandle OnScalabilityChangedHandle;
};
