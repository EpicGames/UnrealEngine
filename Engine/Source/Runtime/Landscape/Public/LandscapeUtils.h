// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "UObject/NameTypes.h"
#include "AssetRegistry/AssetData.h"
#include "LandscapeTextureHash.h"
#include "Materials/Material.h"
#include "RenderGraphDefinitions.h"
#include "RenderGraphEvent.h"
#include "RenderGraphBuilder.h"

enum EShaderPlatform : uint16;
class AActor;
class ALandscapeProxy;
class FRDGBuilder;
class FRDGEventName;
class ULandscapeComponent;
class ULandscapeGrassType;
class ULandscapeLayerInfoObject;
class ULandscapeMaterialInstanceConstant;
class ULevel;
class UTexture;
class UTexture2D;
class UWorld;
class FTextureResource;
class FMaterialUpdateContext;
enum class ELandscapeToolTargetType : uint8;
enum class ELandscapeToolTargetTypeFlags : uint8;
struct FShaderCompilerEnvironment;

namespace UE::Landscape
{
enum class EBuildFlags : uint8;

/**
* Returns true if edit layers (GPU landscape tools) are enabled on this platform :
* Note: this is intended for the editor but is in runtime code since global shaders need to exist in runtime modules
*/
LANDSCAPE_API bool DoesPlatformSupportEditLayers(EShaderPlatform InShaderPlatform);

/** Provides an opportunity to modify the shader compiler environment (for landscape debugging purposes) */
LANDSCAPE_API void ModifyShaderCompilerEnvironmentForDebug(FShaderCompilerEnvironment& OutEnvironment);

LANDSCAPE_API ELandscapeToolTargetTypeFlags GetLandscapeToolTargetTypeAsFlags(ELandscapeToolTargetType InTargetType);
LANDSCAPE_API ELandscapeToolTargetType GetLandscapeToolTargetTypeSingleFlagAsType(ELandscapeToolTargetTypeFlags InSingleFlag);
LANDSCAPE_API FString GetLandscapeToolTargetTypeFlagsAsString(ELandscapeToolTargetTypeFlags InTargetTypeFlags);

LANDSCAPE_API bool SupportsRuntimeGrassMapGeneration();

// ----------------------------------------------------------------------------------

/** This struct is usually meant to be allocated on the game thread (where there's no FRDGBuilder, which is Render Thread-only) and allows to queue successive operations (lambdas) onto a single render command,
 *   sharing the same FRDGBuilder. This allows to sequence a list of RDG passes from the game thread and makes it possible to interleave render thread operations (in a single render command) with game thread-initiated
 *   render commands.
 *   Here's an extensive use case :
 *
 *   FRDGBuilderRecorder Recorder;
 *
 *   // Start recording commands to a single graph builder :
 *   Recorder.StartRecording();
 *
 *   Recorder.EnqueueRDGCommand([](FRDGBuilder* GraphBuilder) { GraphBuilder->AddPass(); }); // Append a Pass_A
 *   ENQUEUE_RENDER_COMMAND(...)  // Push Render_Command_A (immediately)
 *   Recorder.EnqueueRDGCommand([](FRDGBuilder* GraphBuilder) { GraphBuilder->AddPass(); }, { { SomeTexture, ERHIAccess::RTV } }); // Append a Pass_B and inform of the final state of a given texture to prevent the RDG from auto-transitioning to SRVMask at the end
 *   Recorder.EnqueueRenderCommand([](FRHICommandListImmediate& InRHICmdList) mutable { [...]; }); // Append a Render_Command_B to the graph builder
 *   Recorder.EnqueueRDGCommand([](FRDGBuilder* GraphBuilder) { GraphBuilder->AddPass(); }, { { }SomeTexture, ERHIAccess::CopySrc } }); // Append a Pass_C and inform of the final state of a given texture to prevent the RDG from auto-transitioning to SRVMask at the end
 *
 *   // Stop recording and issue a render command with all that's been recorded so far
 *   Recorder.StopRecordingAndFlush(RDG_EVENT_NAME("Pass ABC"));
 *
 *   // Enqueue some game-thread render commands (e.g. BP render) either directly or via the recorder in immediate mode :
 *   ENQUEUE_RENDER_COMMAND(...)  // Render_Command_C
 *   Recorder.EnqueueRDGCommand([](FRDGBuilder* GraphBuilder) { GraphBuilder->AddPass(); }, { { SomeTexture, ERHIAccess::CopyDst } }); // Append a Pass_D and inform of the final state of a given texture to prevent the RDG from auto-transitioning to SRVMask at the end
 *   Recorder.EnqueueRenderCommand([RHIOperation](FRHICommandListImmediate& InRHICmdList) mutable { [...]; }); // Append a Render_Command_D
 *
 *   // Start recording commands again to a new single graph builder :
 *   Recorder.StartRecording();
 *   Recorder.EnqueueRDGCommand([](FRDGBuilder* GraphBuilder) { GraphBuilder->AddPass(); }); // Append a Pass_E
 *
 *   // Stop recording and issue a render command with all that's been recorded so far
 *   Recorder.StopRecordingAndFlush(RDG_EVENT_NAME("Pass E"));
 *
 *   --> Will yield the following sequence on the render thread:
 *   + Render_Command_A
 *   + Render Command "Pass ABC"
 *     +- (RDGBuilder_0)
 *     +- RDGBuilder_0.Pass_A
 *     +- RDGBuilder_0.Pass_B
 *     +- RDGBuilder_0.LambdaPass (Render_Command_B)
 *     +- RDGBuilder_0.Pass_C
 *     +- RDGBuilder_0.SetTextureAccessFinal(OutputTexture, ERHIAccess::CopySrc); // Only the final state recorded for a given texture is set
 *     +- RDGBuilder_0.Execute
 *   + Render_Command_C
 *   + Render Command
 *     +- (RDGBuilder_1)
 *     +- RDGBuilder_1.Pass_D
 *     +- RDGBuilder_1.SetTextureAccessFinal(OutputTexture, ERHIAccess::CopyDst);
 *     +- RDGBuilder_1.Execute
 *   + Render_Command_D
 *   + Render Command "Pass E"
 *     +- (RDGBuilder_2)
 *       +- RDGBuilder_2.Pass_E
 *       +- RDGBuilder_2.Execute
 */
class FRDGBuilderRecorder final
{
public:
	using FRDGRecorderRDGCommand = TFunction<void(FRDGBuilder& /*GraphBuilder*/)>;
	using FRDGRecorderRenderCommand = TFunction<void(FRHICommandListImmediate& /*InRHICmdList*/)>;
	struct FRDGExternalTextureAccessFinal
	{
		FTextureResource* TextureResource = nullptr;
		ERHIAccess Access = ERHIAccess::None;
	};

	FRDGBuilderRecorder() = default;
	LANDSCAPE_API ~FRDGBuilderRecorder();

	enum class EState
	{
		Immediate, // In immediate mode, any command that is enqueued will be pushed to the render thread immediately (effectively acting like a ENQUEUE_RENDER_COMMAND) 
		Recording, // In recording mode, any command that is enqueued will be deferred to the render thread (effectively acting like a ENQUEUE_RENDER_COMMAND) 
	};

	inline EState GetState() const { return State; }
	inline bool IsRecording() const { return State == EState::Recording; }

	/**
	* Starts recording commands
	 */
	LANDSCAPE_API void StartRecording();

	/**
	* Stops recording commands. A call to Flush is needed to ensure any pending command is flushed to the render thread (use StopRecordingAndFlush to do both)
	*/
	LANDSCAPE_API void StopRecording();

	/**
	 * Stops recording commands and flushes them to the render thread. Expects the recorder to be in Recording mode and changes it to Immediate mode.
	 * @param EventName RDG event name to use on the render command's FRDGBuilder
	*/
	LANDSCAPE_API void StopRecordingAndFlush(FRDGEventName&& EventName);

	/**
	 * Flushes any pending command to the render thread
	 * @param EventName RDG event name to use on the render command's FRDGBuilder
	*/
	LANDSCAPE_API void Flush(FRDGEventName&& EventName);

	/**
	 * Records a FRDGRecorderRDGCommand to execute when registering passes to the single FRDGBuilder when in Recording mode or pushes it immediately to the render thread when in Immediate mode
	 *
	 * @param InRDGCommand Lambda to execute on the render thread
	 * @param InRDGTextureAccessFinalList (optional) : list of external texture with the RHIAccess they should have when executing the FRDGBuilder. This is to prevent the RDG from auto-transitioning to SRVMask at the end
	 */
	LANDSCAPE_API void EnqueueRDGCommand(FRDGRecorderRDGCommand InRDGCommand, TConstArrayView<FRDGExternalTextureAccessFinal> InRDGExternalTextureAccessFinalList = {});

	/**
	 * Records a FRDGRecorderRenderCommand to execute when registering passes to the single FRDGBuilder when in Recording mode or pushes it immediately to the render thread when in Immediate mode
	 *
	 * @param InRenderCommand Lambda to execute on the render thread
	 */
	LANDSCAPE_API void EnqueueRenderCommand(FRDGRecorderRenderCommand InRenderCommand);

	/**
	 * @return true if there's no command currently recorded
	 */
	LANDSCAPE_API bool IsEmpty() const;

	/**
	 * Cancels all recorder operations. This must be used if the FRDGEventRenderRecorder is "cancelled" (i.e. its sequence of operations is not flushed to a render command).
	 *  Otherwise, there will be an assert in ~FRDGBuilderRecorder().
	 */
	LANDSCAPE_API void Clear();

private:
	EState State = EState::Immediate;

	// List of callbacks to call on the render thread after the render command was initiated
	TArray<FRDGRecorderRDGCommand> RDGCommands;

	// Map of textures and the RHI access they should have when leaving the FRDGBuilder :
	TMap<FTextureResource*, ERHIAccess> RDGExternalTextureAccessFinal;

#if RDG_EVENTS
public:
	/**
	 * Scope object meant to insert a RDG event in the RDG operations, as if it was inserted on the render thread on a FRDGBuilder.
	 *  Use RDG_RENDER_COMMAND_RECORDER_BREADCRUMB_EVENT to create one.
	 */
	class FScopedBreadcrumbEvent final
	{
	public:
		template <size_t N>
		FScopedBreadcrumbEvent(FRDGBuilderRecorder& InRecorder, TCHAR const(&StaticName)[N], FRDGEventName&& EventName)
			: Recorder(&InRecorder)
			, RDGEvent(MakeShared<TOptional<TRDGEventScopeGuard<FRDGScope_RHI>>>())
		{
			//  We use a shared ptr to create a TOptional<TRDGEventScopeGuard> immediately, then capture this shared ptr in these additional operations' lambdas (so that the object
			//  continues to live until the closing tag operation)
			Recorder->EnqueueRDGCommand(
			[
				RDGEvent = RDGEvent, 
				&StaticName = StaticName,
				EventName = MoveTemp(EventName)
			](FRDGBuilder& GraphBuilder) mutable
			{
				// Allocate the TOptional<TRDGEventScopeGuard> now in order to insert the tag : 
				RDG_EVENT_SCOPE_CONSTRUCT(*RDGEvent, GraphBuilder, true, ERDGScopeFlags::None, RHI_GPU_STAT_ARGS_NONE, StaticName, MoveTemp(EventName));
			});
		}

		~FScopedBreadcrumbEvent()
		{
			Recorder->EnqueueRDGCommand([RDGEvent = RDGEvent](FRDGBuilder& GraphBuilder)
			{
				// Reset the TOptional in order to delete the TRDGEventScopeGuard, which will remove the tag : 
				RDGEvent->Reset();
			});
		}

	private:
		FRDGBuilderRecorder* Recorder = nullptr;
		TSharedPtr<TOptional<TRDGEventScopeGuard<FRDGScope_RHI>>> RDGEvent;
	};
#endif // RDG_EVENTS
};

#if RDG_EVENTS
#define RDG_RENDER_COMMAND_RECORDER_BREADCRUMB_EVENT(Recorder, Format, ...) FRDGBuilderRecorder::FScopedBreadcrumbEvent ANONYMOUS_VARIABLE(BreadcrumbEvent)(Recorder, TEXT(Format), RDG_EVENT_NAME(Format, ##__VA_ARGS__))
#else // RDG_EVENTS
#define RDG_RENDER_COMMAND_RECORDER_BREADCRUMB_EVENT(Recorder, Format, ...) do { } while(0)
#endif // !RDG_EVENTS


#if WITH_EDITOR

// ----------------------------------------------------------------------------------

struct FTextureCopyRequest
{
	UTexture2D* Source = nullptr;
	UTexture* Destination = nullptr;
	int8 DestinationSlice = 0;
	ELandscapeTextureUsage TextureUsage = ELandscapeTextureUsage::Unknown;
	ELandscapeTextureType TextureType = ELandscapeTextureType::Unknown;
};

uint32 GetTypeHash(const FTextureCopyRequest& InKey);
bool operator==(const FTextureCopyRequest& InEntryA, const FTextureCopyRequest& InEntryB);

/** Represents the DestinationChannel->SourceChannel binding.DestinationChannel is used as index.
 *  For example if the source channel is 1 and the destination channel is 2, then Mappings[2] == 1.
 */
struct FTextureCopyChannelMapping
{
	FTextureCopyChannelMapping()
		: Mappings{ INDEX_NONE, INDEX_NONE, INDEX_NONE, INDEX_NONE }
	{}

	int8& operator[](int32 Index) { return Mappings[Index]; }
	const int8 operator[](int32 Index) const { return Mappings[Index]; }

	int8 Mappings[4];
};

class FBatchTextureCopy
{
public:
	/**
	* Uses the provided arguments to add proper source/destination entries to internal copy requests.
	* @param	InDestination	The texture used as a destination for the copy.
	* @param	InDestinationSlice	The Texture array slice to write to (use 0 for a Texture2D)
	* @param	InDestinationChannel	The channel used as a destination for the copy.
	* @param	InComponent		The component containing the wanted source weightmap.
	* @param	InLayerInfo		The layer info used to retrieve the proper source weightmap and channel.
	* @return True if the copy has been successfully added.
	*/
	LANDSCAPE_API bool AddWeightmapCopy(UTexture* InDestination, int8 InDestinationSlice, int8 InDestinationChannel, const ULandscapeComponent* InComponent, ULandscapeLayerInfoObject* InLayerInfo);

	/** Process pending internal copy requests. */
	LANDSCAPE_API bool ProcessTextureCopies();

private:
	using FTextureCopyChannelMappingMap = TMap<FTextureCopyRequest, FTextureCopyChannelMapping>;

	FTextureCopyChannelMappingMap CopyRequests;
};

/**
 * Returns a generated path used for Landscape Shared Assets
 * @param	InPath	Path used as a basis to generate shared assets path. If /Temp/, it will be replaced by the last valid path used for level.
 * @return Path used for Landscape Shared Assets
*/
LANDSCAPE_API FString GetSharedAssetsPath(const FString& InPath);

/**
 * Returns a generated path used for Landscape Shared Assets
 * @param	InLevel		Level's Path will be used as a basis to generate shared assets path. If /Temp/, it will be replaced by the last valid path used for level.
 * @return Path used for Landscape Shared Assets
*/
LANDSCAPE_API FString GetSharedAssetsPath(const ULevel* InLevel);

/**
 * Returns a generated package name for a Layer Info Object
 * @param	InLayerName	The LayerName of the Layer Info Object
 * @param	InPackagePath	Base package path that will be used, should be Current Level SharedAssetPath or the LandscapeEditorObject->TargetLayerAssetFilePath
 * @param	OutLayerObjectName	The generated object name for Layer Info Object
*/
LANDSCAPE_API FString GetLayerInfoObjectPackageName(const FName& InLayerName, const FString& InPackagePath, FName& OutLayerObjectName);

/**
* Creates a new layer info object, using the default template if available, or a new empty one. Sets asset file name as LayerName_LayerInfo_%d
* @param	InLayerName	The layer name of the created asset info
* @param	InFilePath	New asset file path, typically is CurrentLevel/SharedAssetPath or the LandscapeEditorObject->TargetLayerAssetFilePath->DirectoryPath
* @return a new LandscapeLayerInfoObject.
*/
LANDSCAPE_API ULandscapeLayerInfoObject* CreateTargetLayerInfo(const FName& InLayerName, const FString& InFilePath);

/** 
* Creates a new layer info object, using the default template if available, or a new empty one at the path: InFilePath/InFilename
* @param	InLayerName	The layer name of the created asset info
* @param	InFilePath	New asset file path, typically is CurrentLevel/SharedAssetPath or the LandscapeEditorObject->TargetLayerAssetFilePath->DirectoryPath
* @param	InFileName 	The unique file name of the created asset
* @return a new LandscapeLayerInfoObject.
*/
LANDSCAPE_API ULandscapeLayerInfoObject* CreateTargetLayerInfo(const FName& InLayerName, const FString& InFilePath, const FString& InFileName);

/** Returns true if the provided layer info object is the current visibility layer. */
LANDSCAPE_API bool IsVisibilityLayer(const ULandscapeLayerInfoObject* InLayerInfoObject);

/**
 * Returns a pointer to a newly created ULandscapeMaterialInstanceConstant
 * @param	BaseMaterial	The base material appended to the debug name and set as parent of MaterialInstance
 * @return  Pointer to ULandscapeMaterialInstanceConstant
 */
LANDSCAPE_API UMaterialInstance* CreateToolLandscapeMaterialInstanceConstant(UMaterialInterface* BaseMaterial);


/** Create a thumbnail material for a given layer. Can return nullptr if the option to disable landscape thumbnails has been turned on */
LANDSCAPE_API ULandscapeMaterialInstanceConstant* CreateLandscapeLayerThumbnailMIC(FMaterialUpdateContext& MaterialUpdateContext, UMaterialInterface* LandscapeMaterial, FName LayerName);

/** Concatenates the target layer names in parameter into a string */
LANDSCAPE_API FString ConvertTargetLayerNamesToString(const TArrayView<const FName>& InTargetLayerNames);

/**
 * Helper to delete one or multiple actors. 
 * @param InActorsToDelete the list of actors to delete. Cannot contain null entries and all actors should be part of the world passed in InWorld
 * @param InWorld world to which all actors to be deleted belong
 * @param bInAllowUI allows the standard delete actors UX to be displayed, allowing the user to remove lingering reference to these actors, etc.
 * 
 * @return true if all actors could be properly deleted
 */
LANDSCAPE_API bool DeleteActors(const TArray<AActor*>& InActorsToDelete, UWorld* InWorld, bool bInAllowUI);


/**
 * Analyzes the material's material expressions and return the landscape target layer names declared in them
 * @param InMaterial the material to analyze
 * @param bIncludeVisibilityLayer when false, removes the landscape visibility layer from the list of names returned by the material
 * 
 * @return the list of unique target layer names declared in that material
 */
LANDSCAPE_API TArray<FName> RetrieveTargetLayerNamesFromMaterial(UMaterialInterface* InMaterial, bool bIncludeVisibilityLayer = true);

#endif // WITH_EDITOR

/** Helper to sort ALandscapeProxy actors by their section base. Compares X axis values by default and Y values if needed */
LANDSCAPE_API bool LandscapeProxySortPredicate(const TWeakObjectPtr<ALandscapeProxy> APtr, const TWeakObjectPtr<ALandscapeProxy> BPtr);

} // end namespace UE::Landscape


// ----------------------------------------------------------------------------------

namespace UE::Landscape::Grass
{

LANDSCAPE_API TArray<TPair<FName, TObjectPtr<ULandscapeGrassType>>> ConvertToNamedGrassTypesList(const TMap<FName, TObjectPtr<ULandscapeGrassType>>& InNamedGrassTypes);
/** From the list of named grass types, only retain those with a valid name and grass type. */
LANDSCAPE_API TMap<FName, TObjectPtr<ULandscapeGrassType>> FilterValidNamedGrassTypes(const TMap<FName, TObjectPtr<ULandscapeGrassType>>& InNamedGrassTypes);

} // namespace UE::Landscape::Grass
